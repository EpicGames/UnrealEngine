[CoreRedirects]

+ClassRedirects=(OldName="GameplayAbilityBlueprintGeneratedClass",NewName="/Script/Engine.BlueprintGeneratedClass")

+FunctionRedirects=(OldName="AbilitySystemComponent.TryActivateAbilityByTag",NewName="AbilitySystemComponent.TryActivateAbilitiesByTag")
+FunctionRedirects=(OldName="AbilitySystemComponent.GetEffectContext",NewName="AbilitySystemComponent.MakeEffectContext")

+PropertyRedirects=(OldName="GameplayEffect.InheritableClearTagsContainer",NewName="GameplayEffect.RemoveGameplayEffectsWithTags")
+PropertyRedirects=(OldName="GameplayEffect.IsInhibited",NewName="GameplayEffect.bIsInhibited")

+EnumRedirects=(OldName="EGameplayAbilityNetExecutionPolicy",ValueChanges=(("EGameplayAbilityNetExecutionPolicy::Client","EGameplayAbilityNetExecutionPolicy::LocalOnly"),("EGameplayAbilityNetExecutionPolicy::Predictive","EGameplayAbilityNetExecutionPolicy::LocalPredicted"),("EGameplayAbilityNetExecutionPolicy::Server","EGameplayAbilityNetExecutionPolicy::ServerInitiated")) )
