[/Script/EngineSettings.ConsoleSettings]
+ManualAutoCompleteList=(Command="ShowDebug AbilitySystem",Desc="Toggles debug information about the Gameplay Ability System")