{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Gameplay Abilities",
	"Description" : "Adds GameplayEffect and GameplayAbility classes to handle complicated gameplay interactions.",
	"Category" : "Gameplay",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "GameplayAbilities",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "GameplayAbilitiesEditor",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "EngineAssetDefinitions",
			"Enabled": true
		},
		{
			"Name": "GameplayTagsEditor",
			"Enabled": true
		},
		{
			"Name": "Niagara",
			"Enabled": true
		},
		{
			"Name": "DataRegistry",
			"Enabled": true
		}
	]

}