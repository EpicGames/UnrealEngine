// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameplayAbilities : ModuleRules
	{
		public GameplayAbilities(ReadOnlyTargetRules Target) : base(Target)
		{
			NumIncludedBytesPerUnityCPPOverride = 491520; // best unity size found from using UBT ProfileUnitySizes mode

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"NetCore",
					"Engine",
					"GameplayTags",
					"GameplayTasks",
					"MovieScene",
					"PhysicsCore",
					"DeveloperSettings",
					"DataRegistry"
				}
				);

			// Niagara support for gameplay cue notifies.
			{
				PrivateDependencyModuleNames.Add("Niagara");
			}

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
				PrivateDependencyModuleNames.Add("Slate");
				PrivateDependencyModuleNames.Add("SequenceRecorder");
			}

			SetupGameplayDebuggerSupport(Target);

			SetupIrisSupport(Target);
		}
	}
}
