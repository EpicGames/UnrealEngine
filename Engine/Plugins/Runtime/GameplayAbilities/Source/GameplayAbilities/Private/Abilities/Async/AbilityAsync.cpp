// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Async/AbilityAsync.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityAsync)

void UAbilityAsync::Cancel()
{
	EndAction();
	Super::Cancel();
}

void UAbilityAsync::EndAction()
{
	// Child classes should override this if they need to explicitly unbind delegates that aren't just using weak pointers

	// Clear our ASC so it won't broadcast delegates
	SetAbilitySystemComponent(nullptr);
	SetReadyToDestroy();
}

bool UAbilityAsync::ShouldBroadcastDelegates() const
{
	// By default, broadcast if our ASC is valid
	if (GetAbilitySystemComponent() != nullptr)
	{
		return true;
	}

	return false;
}

UAbilitySystemComponent* UAbilityAsync::GetAbilitySystemComponent() const
{
	return AbilitySystemComponent.Get();
}

void UAbilityAsync::SetAbilitySystemComponent(UAbilitySystemComponent* InAbilitySystemComponent)
{
	AbilitySystemComponent = InAbilitySystemComponent;
}

void UAbilityAsync::SetAbilityActor(AActor* InActor)
{
	AbilitySystemComponent = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(InActor);
}

