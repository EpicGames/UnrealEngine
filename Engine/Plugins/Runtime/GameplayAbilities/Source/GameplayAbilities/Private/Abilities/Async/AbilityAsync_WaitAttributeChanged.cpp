// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Async/AbilityAsync_WaitAttributeChanged.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityAsync_WaitAttributeChanged)

UAbilityAsync_WaitAttributeChanged* UAbilityAsync_WaitAttributeChanged::WaitForAttributeChanged(AActor* TargetActor, FGameplayAttribute Attribute, bool OnlyTriggerOnce)
{
	UAbilityAsync_WaitAttributeChanged* MyObj = NewObject<UAbilityAsync_WaitAttributeChanged>();
	MyObj->SetAbilityActor(TargetActor);
	MyObj->Attribute = Attribute;
	MyObj->OnlyTriggerOnce = OnlyTriggerOnce;
	return MyObj;
}

void UAbilityAsync_WaitAttributeChanged::Activate()
{
	Super::Activate();

	if (UAbilitySystemComponent* ASC = GetAbilitySystemComponent())
	{
		MyHandle = ASC->GetGameplayAttributeValueChangeDelegate(Attribute).AddUObject(this, &ThisClass::OnAttributeChanged);
	}
	else
	{
		EndAction();
	}
}

void UAbilityAsync_WaitAttributeChanged::OnAttributeChanged(const FOnAttributeChangeData& ChangeData)
{
	if (ShouldBroadcastDelegates())
	{
		Changed.Broadcast(ChangeData.Attribute, ChangeData.NewValue, ChangeData.OldValue);

		if (OnlyTriggerOnce)
		{
			EndAction();
		}
	}
	else
	{
		EndAction();
	}
}

void UAbilityAsync_WaitAttributeChanged::EndAction()
{
	if (UAbilitySystemComponent* ASC = GetAbilitySystemComponent())
	{
		ASC->GetGameplayAttributeValueChangeDelegate(Attribute).Remove(MyHandle);
	}
	Super::EndAction();
}

