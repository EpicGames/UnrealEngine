// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Async/AbilityAsync_WaitGameplayEffectApplied.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityAsync_WaitGameplayEffectApplied)

UAbilityAsync_WaitGameplayEffectApplied* UAbilityAsync_WaitGameplayEffectApplied::WaitGameplayEffectAppliedToActor(AActor* TargetActor, const FGameplayTargetDataFilterHandle SourceFilter, FGameplayTagRequirements SourceTagRequirements, FGameplayTagRequirements TargetTagRequirements, FGameplayTagRequirements AssetTagRequirements, FGameplayTagRequirements GrantedTagRequirements, bool TriggerOnce, bool ListenForPeriodicEffect)
{
	UAbilityAsync_WaitGameplayEffectApplied* MyObj = NewObject<UAbilityAsync_WaitGameplayEffectApplied>();
	MyObj->SetAbilityActor(TargetActor);
	MyObj->Filter = SourceFilter;
	MyObj->SourceTagRequirements = SourceTagRequirements;
	MyObj->TargetTagRequirements = TargetTagRequirements;
	MyObj->AssetTagRequirements = AssetTagRequirements;
	MyObj->GrantedTagRequirements = GrantedTagRequirements;
	MyObj->TriggerOnce = TriggerOnce;
	MyObj->ListenForPeriodicEffects = ListenForPeriodicEffect;
	return MyObj;
}

void UAbilityAsync_WaitGameplayEffectApplied::Activate()
{
	Super::Activate();

	if (UAbilitySystemComponent* ASC = GetAbilitySystemComponent())
	{
		OnApplyGameplayEffectCallbackDelegateHandle = ASC->OnGameplayEffectAppliedDelegateToSelf.AddUObject(this, &UAbilityAsync_WaitGameplayEffectApplied::OnApplyGameplayEffectCallback);
		if (ListenForPeriodicEffects)
		{
			OnPeriodicGameplayEffectExecuteCallbackDelegateHandle = ASC->OnPeriodicGameplayEffectExecuteDelegateOnSelf.AddUObject(this, &UAbilityAsync_WaitGameplayEffectApplied::OnApplyGameplayEffectCallback);
		}
	}
	else
	{
		EndAction();
	}
}

void UAbilityAsync_WaitGameplayEffectApplied::OnApplyGameplayEffectCallback(UAbilitySystemComponent* Target, const FGameplayEffectSpec& SpecApplied, FActiveGameplayEffectHandle ActiveHandle)
{
	AActor* AvatarActor = Target ? Target->GetAvatarActor_Direct() : nullptr;

	if (!Filter.FilterPassesForActor(AvatarActor))
	{
		return;
	}
	if (!SourceTagRequirements.RequirementsMet(*SpecApplied.CapturedSourceTags.GetAggregatedTags()))
	{
		return;
	}
	if (!TargetTagRequirements.RequirementsMet(*SpecApplied.CapturedTargetTags.GetAggregatedTags()))
	{
		return;
	}
	FGameplayTagContainer AllAssetTags;
	SpecApplied.GetAllAssetTags(AllAssetTags);
	if (!AssetTagRequirements.RequirementsMet(AllAssetTags))
	{
		ABILITY_LOG(Verbose, TEXT("AbilityAsync_WaitGameplayEffectApplied: Not triggering for Applied Spec: %s, AssetTagRequirements not passed: %s"), *SpecApplied.ToSimpleString(), *AssetTagRequirements.ToString());
		return;
	}

	FGameplayTagContainer AllGrantedTags;
	SpecApplied.GetAllGrantedTags(AllGrantedTags);
	if (!GrantedTagRequirements.RequirementsMet(AllGrantedTags))
	{
		ABILITY_LOG(Verbose, TEXT("AbilityAsync_WaitGameplayEffectApplied: Not triggering for Applied Spec: %s, GrantedTagRequirements not passed: %s"), *SpecApplied.ToSimpleString(), *GrantedTagRequirements.ToString());
		return;
	}

	// We allow GameplayEffect application to trigger other GameplayEffect application (as of UE 5.7). However, in order to prevent infinite recursions, 
	// we don't allow any effect class to be applied recursively. Keep track of effect classes as this function is entered recursively and abort if the
	// current applied effect class is already being applied higher in the callstack.
	const TSubclassOf<UGameplayEffect> EffectClass = SpecApplied.Def ? SpecApplied.Def->GetClass() : nullptr;
	if (BroadcastingEffectStack.Contains(EffectClass))
	{
		ABILITY_LOG(Error, TEXT("AbilityAsync_WaitGameplayEffectApplied: recursive application of '%s' detected. This could cause an infinite loop, ignoring broadcast. Current stack: %s"), *GetNameSafe(EffectClass), *GetBroadcastingEffectStackString());
		return;
	}
	else if (UE_LOG_ACTIVE(LogAbilitySystem, VeryVerbose) && !BroadcastingEffectStack.IsEmpty())
	{
		ABILITY_LOG(VeryVerbose, TEXT("AbilityAsync_WaitGameplayEffectApplied: application of '%s' causes no recursion. Current stack: %s"), *GetNameSafe(EffectClass), *GetBroadcastingEffectStackString());
	}

	FGameplayEffectSpecHandle SpecHandle(new FGameplayEffectSpec(SpecApplied));

	if (ShouldBroadcastDelegates())
	{
		BroadcastingEffectStack.Add(EffectClass);
		OnApplied.Broadcast(AvatarActor, SpecHandle, ActiveHandle);
		BroadcastingEffectStack.RemoveSingle(EffectClass);

		if (TriggerOnce)
		{
			EndAction();
		}
	}
	else
	{
		EndAction();
	}
}

void UAbilityAsync_WaitGameplayEffectApplied::EndAction()
{
	if (UAbilitySystemComponent* ASC = GetAbilitySystemComponent())
	{
		if (OnPeriodicGameplayEffectExecuteCallbackDelegateHandle.IsValid())
		{
			ASC->OnPeriodicGameplayEffectExecuteDelegateOnSelf.Remove(OnPeriodicGameplayEffectExecuteCallbackDelegateHandle);
		}
		ASC->OnGameplayEffectAppliedDelegateToSelf.Remove(OnApplyGameplayEffectCallbackDelegateHandle);
	}
	Super::EndAction();
}

FString UAbilityAsync_WaitGameplayEffectApplied::GetBroadcastingEffectStackString() const
{
	TArray<FString> AllEffectNames;
	for (const TSubclassOf<UGameplayEffect>& ExistingEffectClass : BroadcastingEffectStack)
	{
		AllEffectNames.Add(GetNameSafe(ExistingEffectClass));

	}
	return FString::Join(AllEffectNames, TEXT(","));
}
