// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Async/AbilityAsync_WaitGameplayEvent.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityAsync_WaitGameplayEvent)

UAbilityAsync_WaitGameplayEvent* UAbilityAsync_WaitGameplayEvent::WaitGameplayEventToActor(AActor* TargetActor, FGameplayTag EventTag, bool OnlyTriggerOnce, bool OnlyMatchExact)
{
	UAbilityAsync_WaitGameplayEvent* MyObj = NewObject<UAbilityAsync_WaitGameplayEvent>();
	MyObj->SetAbilityActor(TargetActor);
	MyObj->Tag = EventTag;
	MyObj->OnlyTriggerOnce = OnlyTriggerOnce;
	MyObj->OnlyMatchExact = OnlyMatchExact;
	return MyObj;
}

void UAbilityAsync_WaitGameplayEvent::Activate()
{
	Super::Activate();

	UAbilitySystemComponent* ASC = GetAbilitySystemComponent();
	if (ASC)
	{
		if (OnlyMatchExact)
		{
			MyHandle = ASC->GenericGameplayEventCallbacks.FindOrAdd(Tag).AddUObject(this, &UAbilityAsync_WaitGameplayEvent::GameplayEventCallback);
		}
		else
		{
			MyHandle = ASC->AddGameplayEventTagContainerDelegate(FGameplayTagContainer(Tag), FGameplayEventTagMulticastDelegate::FDelegate::CreateUObject(this, &UAbilityAsync_WaitGameplayEvent::GameplayEventContainerCallback));
		}
	}
	else
	{
		EndAction();
	}
}

void UAbilityAsync_WaitGameplayEvent::GameplayEventCallback(const FGameplayEventData* Payload)
{
	GameplayEventContainerCallback(Tag, Payload);
}

void UAbilityAsync_WaitGameplayEvent::GameplayEventContainerCallback(FGameplayTag MatchingTag, const FGameplayEventData* Payload)
{
	if (ShouldBroadcastDelegates())
	{
		FGameplayEventData TempPayload = *Payload;
		TempPayload.EventTag = MatchingTag;
		EventReceived.Broadcast(MoveTemp(TempPayload));
		
		if (OnlyTriggerOnce)
		{
			EndAction();
		}
	}
	else
	{
		EndAction();
	}
}

void UAbilityAsync_WaitGameplayEvent::EndAction()
{
	UAbilitySystemComponent* ASC = GetAbilitySystemComponent();
	if (ASC && MyHandle.IsValid())
	{
		if (OnlyMatchExact)
		{
			ASC->GenericGameplayEventCallbacks.FindOrAdd(Tag).Remove(MyHandle);
		}
		else
		{
			ASC->RemoveGameplayEventTagContainerDelegate(FGameplayTagContainer(Tag), MyHandle);
		}

	}
	Super::EndAction();
}

