// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Async/AbilityAsync_WaitGameplayTag.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityAsync_WaitGameplayTag)

void UAbilityAsync_WaitGameplayTag::Activate()
{
	Super::Activate();

	check(TargetCount >= 0);
	UAbilitySystemComponent* ASC = GetAbilitySystemComponent();
	if (ASC && ShouldBroadcastDelegates())
	{
		MyHandle = ASC->RegisterGameplayTagEvent(Tag).AddUObject(this, &UAbilityAsync_WaitGameplayTag::GameplayTagCallback);

		if ((TargetCount == 0 && ASC->GetTagCount(Tag) == 0) || (TargetCount > 0 && ASC->GetTagCount(Tag) >= TargetCount))
		{
			BroadcastDelegate();
			if (OnlyTriggerOnce)
			{
				EndAction();
			}
		}
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("%s: AbilitySystemComponent is nullptr! Could not register for gameplay tag event with Tag = %s."), *GetName(), *Tag.GetTagName().ToString());
		EndAction();
	}
}

void UAbilityAsync_WaitGameplayTag::GameplayTagCallback(const FGameplayTag InTag, int32 NewCount)
{
	if (NewCount == TargetCount)
	{
		if (ShouldBroadcastDelegates())
		{
			BroadcastDelegate();
			if (OnlyTriggerOnce)
			{
				EndAction();
			}
		}
		else
		{
			EndAction();
		}
	}
}

void UAbilityAsync_WaitGameplayTag::BroadcastDelegate()
{
}

void UAbilityAsync_WaitGameplayTag::EndAction()
{
	UAbilitySystemComponent* ASC = GetAbilitySystemComponent();
	if (ASC && MyHandle.IsValid())
	{
		ASC->RegisterGameplayTagEvent(Tag).Remove(MyHandle);
	}
	Super::EndAction();
}


UAbilityAsync_WaitGameplayTagAdded* UAbilityAsync_WaitGameplayTagAdded::WaitGameplayTagAddToActor(AActor* TargetActor, FGameplayTag Tag, bool OnlyTriggerOnce/*=false*/)
{
	UAbilityAsync_WaitGameplayTagAdded* MyObj = NewObject<UAbilityAsync_WaitGameplayTagAdded>();
	MyObj->SetAbilityActor(TargetActor);
	MyObj->Tag = Tag;
	MyObj->OnlyTriggerOnce = OnlyTriggerOnce;
	MyObj->TargetCount = 1;

	return MyObj;
}

void UAbilityAsync_WaitGameplayTagAdded::BroadcastDelegate()
{
	Added.Broadcast();
}


UAbilityAsync_WaitGameplayTagRemoved* UAbilityAsync_WaitGameplayTagRemoved::WaitGameplayTagRemoveFromActor(AActor* TargetActor, FGameplayTag Tag, bool OnlyTriggerOnce/*=false*/)
{
	UAbilityAsync_WaitGameplayTagRemoved* MyObj = NewObject<UAbilityAsync_WaitGameplayTagRemoved>();
	MyObj->SetAbilityActor(TargetActor);
	MyObj->Tag = Tag;
	MyObj->OnlyTriggerOnce = OnlyTriggerOnce;
	MyObj->TargetCount = 0;

	return MyObj;
}

void UAbilityAsync_WaitGameplayTagRemoved::BroadcastDelegate()
{
	Removed.Broadcast();
}

