// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Async/AbilityAsync_WaitGameplayTagCountChanged.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityAsync_WaitGameplayTagCountChanged)

void UAbilityAsync_WaitGameplayTagCountChanged::Activate()
{
	Super::Activate();

	UAbilitySystemComponent* ASC = GetAbilitySystemComponent();
	if (ASC && ShouldBroadcastDelegates())
	{
		GameplayTagCountChangedHandle = ASC->RegisterGameplayTagEvent(Tag, EGameplayTagEventType::AnyCountChange).AddUObject(this, &UAbilityAsync_WaitGameplayTagCountChanged::GameplayTagCallback);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("%s: AbilitySystemComponent is nullptr! Could not register for gameplay tag event with Tag = %s."), *GetName(), *Tag.GetTagName().ToString());
		EndAction();
	}
}

void UAbilityAsync_WaitGameplayTagCountChanged::EndAction()
{
	UAbilitySystemComponent* ASC = GetAbilitySystemComponent();
	if (ASC && GameplayTagCountChangedHandle.IsValid())
	{
		ASC->RegisterGameplayTagEvent(Tag).Remove(GameplayTagCountChangedHandle);
		GameplayTagCountChangedHandle.Reset();
	}
	Super::EndAction();
}

void UAbilityAsync_WaitGameplayTagCountChanged::GameplayTagCallback(const FGameplayTag InTag, int32 NewCount)
{
	if (ShouldBroadcastDelegates())
	{
		TagCountChanged.Broadcast(NewCount);
	}
	else
	{
		EndAction();
	}
}

UAbilityAsync_WaitGameplayTagCountChanged* UAbilityAsync_WaitGameplayTagCountChanged::WaitGameplayTagCountChangedOnActor(AActor* TargetActor, FGameplayTag Tag)
{	
	UPackage* TransientPackage = GetTransientPackage();
	FName UniqueActionName = NAME_None;

	// Use the name of the passed in TargetActor to generate a more unique basename for this Action 
	// by concatenating that AActor's name with our classname.  This is to avoid spinning inside MakeUniqueObjectName 
	// probing for a valid name since they're all just ClassName_x, where x can get pretty large
	if (TargetActor != nullptr)
	{
		const UClass* ThisClass = UAbilityAsync_WaitGameplayTagCountChanged::StaticClass();
		const FName ActionNameConcat = FName(*FString::Printf(TEXT("%s_%s"), *TargetActor->GetName(), *ThisClass->GetName()));
		UniqueActionName = ::MakeUniqueObjectName(TransientPackage, ThisClass, ActionNameConcat);
	}

	UAbilityAsync_WaitGameplayTagCountChanged* MyObj = NewObject<UAbilityAsync_WaitGameplayTagCountChanged>(TransientPackage, UniqueActionName);
	MyObj->SetAbilityActor(TargetActor);
	MyObj->Tag = Tag;

	return MyObj;
}
