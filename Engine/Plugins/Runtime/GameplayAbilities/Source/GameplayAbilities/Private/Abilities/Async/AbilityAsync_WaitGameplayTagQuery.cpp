// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Async/AbilityAsync_WaitGameplayTagQuery.h"

#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityAsync_WaitGameplayTagQuery)

UAbilityAsync_WaitGameplayTagQuery* UAbilityAsync_WaitGameplayTagQuery::WaitGameplayTagQueryOnActor(AActor* TargetActor,
	const FGameplayTagQuery TagQuery,
	const EWaitGameplayTagQueryTriggerCondition TriggerCondition /*= EWaitGameplayTagQueryTriggerCondition::WhenTrue*/,
	const bool bOnlyTriggerOnce/*=false*/)
{
	UAbilityAsync_WaitGameplayTagQuery* MyObj = NewObject<UAbilityAsync_WaitGameplayTagQuery>();
	MyObj->SetAbilityActor(TargetActor);
	MyObj->TagQuery = TagQuery;
	MyObj->TriggerCondition = TriggerCondition;
	MyObj->bOnlyTriggerOnce = bOnlyTriggerOnce;

	return MyObj;
}

void UAbilityAsync_WaitGameplayTagQuery::Activate()
{
	Super::Activate();

	UAbilitySystemComponent* ASC = ShouldBroadcastDelegates() ? GetAbilitySystemComponent() : nullptr;
	if (ASC != nullptr)
	{
		//Start the query state to the opposite of the triggered condition so it can be triggered immediately
		bQueryState = TriggerCondition != EWaitGameplayTagQueryTriggerCondition::WhenTrue;

		TArray<FGameplayTag> QueryTags;
		TagQuery.GetGameplayTagArray(QueryTags);

		for (const FGameplayTag& Tag : QueryTags)
		{
			if (!TagHandleMap.Contains(Tag))
			{
				UpdateTargetTags(Tag, ASC->GetTagCount(Tag));
				TagHandleMap.Add(Tag, ASC->RegisterGameplayTagEvent(Tag).AddUObject(this, &UAbilityAsync_WaitGameplayTagQuery::UpdateTargetTags));
			}
		}

		EvaluateTagQuery();

		bRegisteredCallbacks = true;
	}
	else
	{
		EndAction();
	}
}

void UAbilityAsync_WaitGameplayTagQuery::EndAction()
{
	if (UAbilitySystemComponent* ASC = GetAbilitySystemComponent())
	{
		for (TPair<FGameplayTag, FDelegateHandle> Pair : TagHandleMap)
		{
			if (Pair.Value.IsValid())
			{
				ASC->UnregisterGameplayTagEvent(Pair.Value, Pair.Key);
			}
		}
	}

	TagHandleMap.Empty();
	TargetTags.Reset();

	Super::EndAction();
}

bool UAbilityAsync_WaitGameplayTagQuery::ShouldBroadcastDelegates() const
{
	return !TagQuery.IsEmpty() && Super::ShouldBroadcastDelegates();
}

void UAbilityAsync_WaitGameplayTagQuery::UpdateTargetTags(const FGameplayTag Tag, int32 NewCount)
{
	if (ShouldBroadcastDelegates())
	{
		if (NewCount <= 0)
		{
			TargetTags.RemoveTag(Tag);
		}
		else
		{
			TargetTags.AddTag(Tag);
		}

		if (bRegisteredCallbacks)
		{
			EvaluateTagQuery();
		}
	}
	else
	{
		EndAction();
	}
}

void UAbilityAsync_WaitGameplayTagQuery::EvaluateTagQuery()
{
	if (TagQuery.IsEmpty())
	{
		return;
	}

	const bool bMatchesQuery = TagQuery.Matches(TargetTags);
	const bool bStateChanged = bMatchesQuery != bQueryState;
	bQueryState = bMatchesQuery;

	bool bTriggerDelegate = false;
	if (bStateChanged)
	{
		if (TriggerCondition == EWaitGameplayTagQueryTriggerCondition::WhenTrue && bQueryState)
		{
			bTriggerDelegate = true;
		}
		else if (TriggerCondition == EWaitGameplayTagQueryTriggerCondition::WhenFalse && !bQueryState)
		{
			bTriggerDelegate = true;
		}
	}

	if (bTriggerDelegate)
	{
		Triggered.Broadcast();
		if (bOnlyTriggerOnce)
		{
			EndAction();
		}
	}
}
