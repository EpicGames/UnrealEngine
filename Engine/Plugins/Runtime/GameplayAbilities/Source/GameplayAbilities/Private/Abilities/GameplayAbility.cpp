// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbility.h"
#include "AbilitySystemLog.h"
#include "TimerManager.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/NetDriver.h"
#include "AbilitySystemStats.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemPrivate.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameplayAbilitiesDeveloperSettings.h"
#include "GameplayCue_Types.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "GameFramework/Character.h"
#include "Misc/DataValidation.h"
#include "UObject/Package.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/UObjectMigrationContext.h"
#endif

#define LOCTEXT_NAMESPACE "GameplayAbility"

#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbility)

namespace UE::AbilitySystem::Private
{
	int32 CVarAllowNonInstancedGAsValue = 0;
	FAutoConsoleVariableRef CVarAllowNonInstancedGAs(TEXT("AbilitySystem.Fix.AllowNonInstancedAbilities"), CVarAllowNonInstancedGAsValue, TEXT("Whether to allow the deprecated EGameplayAbilityInstancingPolicy::NonInstanced type (removed in UE5.5)"), ECVF_Default);

	int32 AbilitySystemShowMakeOutgoingGameplayEffectSpecs = 0;
	static FAutoConsoleVariableRef CVarAbilitySystemShowMakeOutgoingGameplayEffectSpecs(TEXT("AbilitySystem.ShowClientMakeOutgoingSpecs"), AbilitySystemShowMakeOutgoingGameplayEffectSpecs, TEXT("Displays all GameplayEffect specs created on non authority clients"), ECVF_Default );

	/** We want to deprecate the usage of Replicated Properties due to replication order issues (see the GameplayAbilities/README.md for more information).  Default to 2 to not invalidate existing data during unattended runs (e.g. automated tests). */
	int32 CVarDeprecateReplicatedPropertiesValue = 2;
	static FAutoConsoleVariableRef CVarDeprecateReplicatedProperties(TEXT("AbilitySystem.DeprecateReplicatedProperties"), CVarDeprecateReplicatedPropertiesValue,
		TEXT("Deprecate the use of Replicated Properties. ") TEXT("0: No. 1: Yes. 2: Yes, Except During Automation"), ECVF_Default );

	/** 
		By default, for a GameplayEffect to be usable as an ability's CooldownGameplayEffectClass class, that GameplayEffect must grant at least one GameplayTag.
		If the GE grants no tags, the ability doesn't consider itself on cooldown even if the GE is applied. Thus by default we want to warn developers at editor-time
		when a CooldownGameplayEffectClass is configured that grants no tags. These warnings can be disabled via this CVar, because UGameplayAbility::CheckCooldown
		can be overridden and thus the CooldownGameplayEffectClass may not require granted tags.
	*/
	int32 CVarWarnCooldownEffectWithoutTagsValue = 1;
	static FAutoConsoleVariableRef CVarWarnCooldownEffectWithoutTags(TEXT("AbilitySystem.WarnCooldownEffectWithoutTags"), CVarWarnCooldownEffectWithoutTagsValue, TEXT("Show an editor warning and runtime log when a GameplayAbility is configured with a CooldownGameplayEffectClass that grants no tags"), ECVF_Default);
}

namespace FAbilitySystemTweaks
{
	int ClearAbilityTimers = 1;
	FAutoConsoleVariableRef CVarClearAbilityTimers(TEXT("AbilitySystem.ClearAbilityTimers"), FAbilitySystemTweaks::ClearAbilityTimers, TEXT("Whether to call ClearAllTimersForObject as part of EndAbility call"), ECVF_Default);
}

int32 FScopedCanActivateAbilityLogEnabler::LogEnablerCounter = 0;


UGameplayAbility::UGameplayAbility(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	auto ImplementedInBlueprint = [](const UFunction* Func) -> bool
	{
		return Func && ensure(Func->GetOuter())
			&& Func->GetOuter()->IsA(UBlueprintGeneratedClass::StaticClass());
	};

	{
		static FName FuncName = FName(TEXT("K2_ShouldAbilityRespondToEvent"));
		UFunction* ShouldRespondFunction = GetClass()->FindFunctionByName(FuncName);
		bHasBlueprintShouldAbilityRespondToEvent = ImplementedInBlueprint(ShouldRespondFunction);
	}
	{
		static FName FuncName = FName(TEXT("K2_CanActivateAbility"));
		UFunction* CanActivateFunction = GetClass()->FindFunctionByName(FuncName);
		bHasBlueprintCanUse = ImplementedInBlueprint(CanActivateFunction);
	}
	{
		static FName FuncName = FName(TEXT("K2_ActivateAbility"));
		UFunction* ActivateFunction = GetClass()->FindFunctionByName(FuncName);
		// FIXME: temp to work around crash
		if (ActivateFunction && (HasAnyFlags(RF_ClassDefaultObject) || ActivateFunction->IsValidLowLevelFast()))
		{
			bHasBlueprintActivate = ImplementedInBlueprint(ActivateFunction);
		}
	}
	{
		static FName FuncName = FName(TEXT("K2_ActivateAbilityFromEvent"));
		UFunction* ActivateFunction = GetClass()->FindFunctionByName(FuncName);
		bHasBlueprintActivateFromEvent = ImplementedInBlueprint(ActivateFunction);
	}

	bServerRespectsRemoteAbilityCancellation = true;
	bReplicateInputDirectly = false;
	RemoteInstanceEnded = false;

	InstancingPolicy = EGameplayAbilityInstancingPolicy::InstancedPerExecution;

	NetSecurityPolicy = EGameplayAbilityNetSecurityPolicy::ClientOrServer;

	ScopeLockCount = 0;
}

EGameplayAbilityInstancingPolicy::Type UGameplayAbility::GetInstancingPolicy() const
{
	if (UE::AbilitySystem::Private::CVarAllowNonInstancedGAsValue)
	{
		return InstancingPolicy;
	}

	return (InstancingPolicy != EGameplayAbilityInstancingPolicy::Type::NonInstanced) ?
		InstancingPolicy.GetValue() : EGameplayAbilityInstancingPolicy::Type::InstancedPerActor;
}

UWorld* UGameplayAbility::GetWorld() const
{
	if (!IsInstantiated())
	{
		// If we are a CDO, we must return nullptr instead of calling Outer->GetWorld() to fool UObject::ImplementsGetWorld.
		return nullptr;
	}
	return GetOuter()->GetWorld();
}

int32 UGameplayAbility::GetFunctionCallspace(UFunction* Function, FFrame* Stack)
{
	if (HasAnyFlags(RF_ClassDefaultObject) || !IsSupportedForNetworking())
	{
		// This handles absorbing authority/cosmetic
		return GEngine->GetGlobalFunctionCallspace(Function, this, Stack);
	}
	check(GetOuter() != nullptr);
	return GetOuter()->GetFunctionCallspace(Function, Stack);
}

bool UGameplayAbility::CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack)
{
	// Make sure we are not invalid when being called
	if (!IsValidChecked(this))
	{
		ABILITY_LOG(Warning, TEXT("%s:CallRemoteFunction called while 'this' was invalid. Skipping the rest of the code."), *GetPathName());
		return false;
	}

	check(!HasAnyFlags(RF_ClassDefaultObject));
	check(GetOuter() != nullptr);

	AActor* Owner = CastChecked<AActor>(GetOuter());

	bool bProcessed = false;

	FWorldContext* const Context = GEngine->GetWorldContextFromWorld(GetWorld());
	if (Context != nullptr)
	{
		for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
		{
			if (Driver.NetDriver != nullptr && Driver.NetDriver->ShouldReplicateFunction(Owner, Function))
			{
				Driver.NetDriver->ProcessRemoteFunction(Owner, Function, Parameters, OutParms, Stack, this);
				bProcessed = true;
			}
		}
	}

	return bProcessed;
}

void UGameplayAbility::SendGameplayEvent(FGameplayTag EventTag, FGameplayEventData Payload)
{
	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		using namespace UE::AbilitySystem::Private;
		if (EnumHasAnyFlags(static_cast<EAllowPredictiveGEFlags>(CVarAllowPredictiveGEFlagsValue), EAllowPredictiveGEFlags::AllowGameplayEventToApplyGE))
		{
			FScopedPredictionWindow NewScopedWindow(AbilitySystemComponent, true);
			AbilitySystemComponent->HandleGameplayEvent(EventTag, &Payload);
		}
		else
		{
			AbilitySystemComponent->HandleGameplayEvent(EventTag, &Payload);
		}
	}
}

void UGameplayAbility::PostNetInit()
{
	/** We were dynamically spawned from replication - we need to init a currentactorinfo by looking at outer.
	 *  This may need to be updated further if we start having abilities live on different outers than player AbilitySystemComponents.
	 */
	
	if (CurrentActorInfo == nullptr)
	{
		AActor* OwnerActor = Cast<AActor>(GetOuter());
		if (ensure(OwnerActor))
		{
			UAbilitySystemComponent* AbilitySystemComponent = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(OwnerActor);
			if (ensure(AbilitySystemComponent))
			{
				CurrentActorInfo = AbilitySystemComponent->AbilityActorInfo.Get();
			}
		}
	}
}

bool UGameplayAbility::IsActive() const
{
	// Only Instanced-Per-Actor abilities persist between activations
	if (GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerActor)
	{
		return bIsActive;
	}

	// this should not be called on NonInstanced warn about it, Should call IsActive on the ability spec instead
	if (GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::NonInstanced)
	{
		ABILITY_LOG(Warning, TEXT("UGameplayAbility::IsActive() called on %s NonInstanced ability, call IsActive on the Ability Spec instead"), *GetName());
	}

	// NonInstanced and Instanced-Per-Execution abilities are by definition active unless they are pending kill
	return IsValidChecked(this);
}

bool UGameplayAbility::IsSupportedForNetworking() const
{
	/**
	 *	We can only replicate references to:
	 *		-CDOs and DataAssets (e.g., static, non-instanced gameplay abilities)
	 *		-Instanced abilities that are replicating (and will thus be created on clients).
	 *		
	 *	Otherwise it is not supported, and it will be recreated on the client
	 */

	bool Supported = GetReplicationPolicy() != EGameplayAbilityReplicationPolicy::ReplicateNo || GetOuter()->IsA(UPackage::StaticClass());

	return Supported;
}

void UGameplayAbility::PreDestroyFromReplication()
{
	// We're being destroyed, just cancel locally.
	if (IsActive())
	{
		CancelAbility(CurrentSpecHandle, CurrentActorInfo, CurrentActivationInfo, false);
	}
}

#if WITH_EDITOR
EDataValidationResult UGameplayAbility::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = EDataValidationResult::Valid;
	const bool bIsLikelyRunningAutomation = IsRunningCommandlet() || FApp::IsUnattended();

	if (!UE::AbilitySystem::Private::CVarAllowNonInstancedGAsValue)
	{
		if (InstancingPolicy == EGameplayAbilityInstancingPolicy::Type::NonInstanced)
		{
			Context.AddError(LOCTEXT("NonInstancedIsDeprecated", "Gameplay Ability Instancing Policy is NonInstanced which is deprecated. Use InstancedPerActor. Use CVar AbilitySystem.Fix.AllowNonInstancedAbilities to temporarily allow this during fixup."));
			Result = EDataValidationResult::Invalid;
		}
	}

	const bool bIsReplicated = GetReplicationPolicy() == EGameplayAbilityReplicationPolicy::ReplicateYes;
	if (bIsReplicated)
	{
		if (GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::InstancedPerActor)
		{
			Context.AddError(FText::Format(LOCTEXT("ReplicatedInstancePolicyNotSupported", "Instancing Policy '{0}' is not supported for Replication.  Either change the Replication Policy or the Instancing Policy"), UEnum::GetDisplayValueAsText(GetInstancingPolicy())));
			Result = EDataValidationResult::Invalid;
		}
	}

	// Find out which properties are replicated
	if (UE::AbilitySystem::Private::CVarDeprecateReplicatedPropertiesValue > 0)
	{
		const UClass* Class = GetClass();
		for (const FProperty* Property = Class->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			if (Property->RepIndex > 0 || Property->HasAnyPropertyFlags(CPF_RepNotify | CPF_Net))
			{
				Context.AddWarning(FText::Format(LOCTEXT("ReplicatedPropertiesAreDeprecated", "{0}: Replicated properties are deprecated in Gameplay Abilities (see GameplayAbilities/README.md). Use Reliable RPCs for variable replication."), Property->GetDisplayNameText()));
				const bool bAllowSuppressErrorsSoAutomationCanPass = (UE::AbilitySystem::Private::CVarDeprecateReplicatedPropertiesValue > 1);
				Result = (bAllowSuppressErrorsSoAutomationCanPass && bIsLikelyRunningAutomation) ? CombineDataValidationResults(Result, EDataValidationResult::Valid) : EDataValidationResult::Invalid;
			}
		}
	}
	else
	{
		if (GetReplicationPolicy() == EGameplayAbilityReplicationPolicy::ReplicateNo)
		{
			UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(GetClass());
			if (BPClass && BPClass->NumReplicatedProperties > 0)
			{
				Context.AddError(LOCTEXT("ReplicatedVariablesNeedReplicationYes", "Gameplay Ability Blueprint has replicated variables but Replication Policy is set to not replicate"));
				Result = EDataValidationResult::Invalid;
			}
		}
	}

	for (TFieldIterator<const UFunction> FuncIter(GetClass(), EFieldIterationFlags::IncludeSuper); FuncIter; ++FuncIter)
	{
		if (FuncIter->HasAnyFunctionFlags(EFunctionFlags::FUNC_NetMulticast))
		{
			FText ErrorText = FText::Format(LOCTEXT("MulticastFunctionDisallowed", "Gameplay Abilities are not replicated to Simulated Proxies and therefore NetMulticast Function {0} is meaningless"), FText::FromString(FuncIter->GetName()));
			Context.AddError(ErrorText);
			Result = EDataValidationResult::Invalid;
		}
		else if (FuncIter->HasAnyFunctionFlags(EFunctionFlags::FUNC_Net) && !bIsReplicated)
		{
			FText ErrorText = FText::Format(LOCTEXT("RpcRequiresReplicationYes", "{0}: RPC Functions require ReplicationPolicy to be ReplicateYes in order to actually work."), FuncIter->GetDisplayNameText());
			if (bIsLikelyRunningAutomation)
			{
				Context.AddWarning(ErrorText);
				Result = CombineDataValidationResults(Result, EDataValidationResult::Valid);
			}
			else
			{
				Context.AddError(ErrorText);
				Result = EDataValidationResult::Invalid;
			}
		}
	}

	// Generate a blueprint warning if a Cooldown GE class is configured that doesn't grant tags.
	// If your project overrides GetCooldownGameplayEffect() or CheckCooldown() so that cooldown
	// checking doesn't depend on a tag, set this CVar to 0.
	if (UE::AbilitySystem::Private::CVarWarnCooldownEffectWithoutTagsValue > 0 && CooldownGameplayEffectClass != nullptr)
	{
		if (const UGameplayEffect* CooldownGE = GetCooldownGameplayEffect())
		{
			// Detect misconfigured cooldown GE class. Ignore when class doesn't match due to overridden GetCooldownGameplayEffect().
			if (CooldownGE->GetClass() == CooldownGameplayEffectClass && CooldownGE->GetGrantedTags().IsEmpty())
			{
				FText ErrorText = FText::Format(LOCTEXT("CooldownEffectNoTags", "CooldownGameplayEffectClass '{0}' grants no tags. A GameplayEffect class must grant tags (Component: Grant Tags to Target Actor) to be used as cooldown."), CooldownGameplayEffectClass->GetDisplayNameText());
				Context.AddError(ErrorText);
				Result = EDataValidationResult::Invalid;
			}
		}
	}

	return Result;
}
#endif

bool UGameplayAbility::DoesAbilitySatisfyTagRequirements(const UAbilitySystemComponent& AbilitySystemComponent, const FGameplayTagContainer* SourceTags, const FGameplayTagContainer* TargetTags, OUT FGameplayTagContainer* OptionalRelevantTags) const
{
	// Define a common lambda to check for blocked tags
	bool bBlocked = false;
	auto CheckForBlocked = [&](const FGameplayTagContainer& ContainerA, const FGameplayTagContainer& ContainerB)
	{
		// Do we not have any tags in common?  Then we're not blocked
		if (ContainerA.IsEmpty() || ContainerB.IsEmpty() || !ContainerA.HasAny(ContainerB))
		{
			return;
		}

		if (OptionalRelevantTags)
		{
			// Ensure the global blocking tag is only added once
			if (!bBlocked)
			{
				UAbilitySystemGlobals& AbilitySystemGlobals = UAbilitySystemGlobals::Get();
				const FGameplayTag& BlockedTag = AbilitySystemGlobals.ActivateFailTagsBlockedTag;
				OptionalRelevantTags->AddTag(BlockedTag);
			}

			// Now append all the blocking tags
			OptionalRelevantTags->AppendMatchingTags(ContainerA, ContainerB);
		}

		bBlocked = true;
	};

	// Define a common lambda to check for missing required tags
	bool bMissing = false;
	auto CheckForRequired = [&](const FGameplayTagContainer& TagsToCheck, const FGameplayTagContainer& RequiredTags)
	{
		// Do we have no requirements, or have met all requirements?  Then nothing's missing
		if (RequiredTags.IsEmpty() || TagsToCheck.HasAll(RequiredTags))
		{
			return;
		}

		if (OptionalRelevantTags)
		{
			// Ensure the global missing tag is only added once
			if (!bMissing)
			{
				UAbilitySystemGlobals& AbilitySystemGlobals = UAbilitySystemGlobals::Get();
				const FGameplayTag& MissingTag = AbilitySystemGlobals.ActivateFailTagsMissingTag;
				OptionalRelevantTags->AddTag(MissingTag);
			}

			FGameplayTagContainer MissingTags = RequiredTags; 
			MissingTags.RemoveTags(TagsToCheck.GetGameplayTagParents());
			OptionalRelevantTags->AppendTags(MissingTags);
		}

		bMissing = true;
	};

	// Start by checking all of the blocked tags first (so OptionalRelevantTags will contain blocked tags first)
	CheckForBlocked(GetAssetTags(), AbilitySystemComponent.GetBlockedAbilityTags());
	CheckForBlocked(AbilitySystemComponent.GetOwnedGameplayTags(), ActivationBlockedTags);
	if (SourceTags != nullptr)
	{
		CheckForBlocked(*SourceTags, SourceBlockedTags);
	}
	if (TargetTags != nullptr)
	{
		CheckForBlocked(*TargetTags, TargetBlockedTags);
	}

	// Now check all required tags
	CheckForRequired(AbilitySystemComponent.GetOwnedGameplayTags(), ActivationRequiredTags);
	if (SourceTags != nullptr)
	{
		CheckForRequired(*SourceTags, SourceRequiredTags);
	}
	if (TargetTags != nullptr)
	{
		CheckForRequired(*TargetTags, TargetRequiredTags);
	}

	if (!bBlocked && !bMissing)
	{
		// If it's a custom implementation that blocks, we can't specify exactly which tag so just use the generic
		bBlocked = AbilitySystemComponent.AreAbilityTagsBlocked(GetAssetTags());
		if (bBlocked && OptionalRelevantTags)
		{
			UAbilitySystemGlobals& AbilitySystemGlobals = UAbilitySystemGlobals::Get();
			const FGameplayTag& BlockedTag = AbilitySystemGlobals.ActivateFailTagsBlockedTag;
			OptionalRelevantTags->AddTag(BlockedTag);
		}
	}

	// We succeeded if there were no blocked tags and no missing required tags	
	return !bBlocked && !bMissing;
}

bool UGameplayAbility::ShouldActivateAbility(ENetRole Role) const
{
	return Role != ROLE_SimulatedProxy && 		
		(Role == ROLE_Authority || (NetSecurityPolicy != EGameplayAbilityNetSecurityPolicy::ServerOnly && NetSecurityPolicy != EGameplayAbilityNetSecurityPolicy::ServerOnlyExecution));	// Don't violate security policy if we're not the server
}

void UGameplayAbility::K2_CancelAbility()
{
	ensure(CurrentActorInfo);
	CancelAbility(CurrentSpecHandle, CurrentActorInfo, CurrentActivationInfo, true);
}

bool UGameplayAbility::CanActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayTagContainer* SourceTags, const FGameplayTagContainer* TargetTags, OUT FGameplayTagContainer* OptionalRelevantTags) const
{
	// Don't set the actor info, CanActivate is called on the CDO

	// A valid AvatarActor is required. Simulated proxy check means only authority or autonomous proxies should be executing abilities.
	AActor* const AvatarActor = ActorInfo ? ActorInfo->AvatarActor.Get() : nullptr;
	if (AvatarActor == nullptr || !ShouldActivateAbility(AvatarActor->GetLocalRole()))
	{
		return false;
	}

	//make into a reference for simplicity
	static FGameplayTagContainer DummyContainer;
	DummyContainer.Reset();

	// make sure the ability system component is valid, if not bail out.
	UAbilitySystemComponent* const AbilitySystemComponent = ActorInfo->AbilitySystemComponent.Get();
	if (!AbilitySystemComponent)
	{
		return false;
	}

	FGameplayAbilitySpec* Spec = AbilitySystemComponent->FindAbilitySpecFromHandle(Handle);
	if (!Spec)
	{
		ABILITY_LOG(Warning, TEXT("CanActivateAbility %s failed, called with invalid Handle"), *GetName());
		return false;
	}

	if (AbilitySystemComponent->GetUserAbilityActivationInhibited())
	{
		/**
		 *	Input is inhibited (UI is pulled up, another ability may be blocking all other input, etc).
		 *	When we get into triggered abilities, we may need to better differentiate between CanActivate and CanUserActivate or something.
		 *	E.g., we would want LMB/RMB to be inhibited while the user is in the menu UI, but we wouldn't want to prevent a 'buff when I am low health'
		 *	ability to not trigger.
		 *	
		 *	Basically: CanActivateAbility is only used by user activated abilities now. If triggered abilities need to check costs/cooldowns, then we may
		 *	want to split this function up and change the calling API to distinguish between 'can I initiate an ability activation' and 'can this ability be activated'.
		 */ 

		if (FScopedCanActivateAbilityLogEnabler::IsLoggingEnabled())
		{
			UE_LOGF(LogAbilitySystem, Verbose, "%ls: %ls could not be activated due to GetUserAbilityActivationInhibited", *GetNameSafe(ActorInfo->OwnerActor.Get()), *GetNameSafe(Spec->Ability));
			UE_VLOG(ActorInfo->OwnerActor.Get(), VLogAbilitySystem, Verbose, TEXT("%s could not be activated due to GetUserAbilityActivationInhibited"), *GetNameSafe(Spec->Ability));
		}
		return false;
	}
	
	UAbilitySystemGlobals& AbilitySystemGlobals = UAbilitySystemGlobals::Get();

	if (!AbilitySystemGlobals.ShouldIgnoreCooldowns() && !CheckCooldown(Handle, ActorInfo, OptionalRelevantTags))
	{
		if (FScopedCanActivateAbilityLogEnabler::IsLoggingEnabled())
		{
			UE_LOGF(LogAbilitySystem, Verbose, "%ls: %ls could not be activated due to Cooldown (%ls)", *GetNameSafe(ActorInfo->OwnerActor.Get()), *GetNameSafe(Spec->Ability), OptionalRelevantTags ? *OptionalRelevantTags->ToStringSimple() : TEXT("Unknown"));
			UE_VLOG(ActorInfo->OwnerActor.Get(), VLogAbilitySystem, Verbose, TEXT("%s could not be activated due to Cooldown (%s)"), *GetNameSafe(Spec->Ability), OptionalRelevantTags ? *OptionalRelevantTags->ToStringSimple() : TEXT("Unknown"));
		}
		return false;
	}

	if (!AbilitySystemGlobals.ShouldIgnoreCosts() && !CheckCost(Handle, ActorInfo, OptionalRelevantTags))
	{
		if (FScopedCanActivateAbilityLogEnabler::IsLoggingEnabled())
		{
			UE_LOGF(LogAbilitySystem, Verbose, "%ls: %ls could not be activated due to Cost (%ls)", *GetNameSafe(ActorInfo->OwnerActor.Get()), *GetNameSafe(Spec->Ability), OptionalRelevantTags ? *OptionalRelevantTags->ToStringSimple() : TEXT("Unknown"));
			UE_VLOG(ActorInfo->OwnerActor.Get(), VLogAbilitySystem, Verbose, TEXT("%s could not be activated due to Cost (%s)"), *GetNameSafe(Spec->Ability), OptionalRelevantTags ? *OptionalRelevantTags->ToStringSimple() : TEXT("Unknown"));
		}
		return false;
	}

	if (!DoesAbilitySatisfyTagRequirements(*AbilitySystemComponent, SourceTags, TargetTags, OptionalRelevantTags))
	{	// If the ability's tags are blocked, or if it has a "Blocking" tag or is missing a "Required" tag, then it can't activate.
		if (FScopedCanActivateAbilityLogEnabler::IsLoggingEnabled())
		{
			UE_LOGF(LogAbilitySystem, Verbose, "%ls: %ls could not be activated due to Blocking Tags or Missing Required Tags (%ls)", *GetNameSafe(ActorInfo->OwnerActor.Get()), *GetNameSafe(Spec->Ability), OptionalRelevantTags ? *OptionalRelevantTags->ToStringSimple() : TEXT("Unknown"));
			UE_VLOG(ActorInfo->OwnerActor.Get(), VLogAbilitySystem, Verbose, TEXT("%s could not be activated due to Blocking Tags or Missing Required Tags (%s)"), *GetNameSafe(Spec->Ability), OptionalRelevantTags ? *OptionalRelevantTags->ToStringSimple() : TEXT("Unknown"));
		}
		return false;
	}

	// Check if this ability's input binding is currently blocked
	if (AbilitySystemComponent->IsAbilityInputBlocked(Spec->InputID))
	{
		if (FScopedCanActivateAbilityLogEnabler::IsLoggingEnabled())
		{
			UE_LOGF(LogAbilitySystem, Verbose, "%ls: %ls could not be activated due to blocked input ID %d", *GetNameSafe(ActorInfo->OwnerActor.Get()), *GetNameSafe(Spec->Ability), Spec->InputID);
			UE_VLOG(ActorInfo->OwnerActor.Get(), VLogAbilitySystem, Verbose, TEXT("%s could not be activated due to blocked input ID %d"), *GetNameSafe(Spec->Ability), Spec->InputID);
		}
		return false;
	}

	if (bHasBlueprintCanUse)
	{
		FGameplayTagContainer K2FailTags;
		if (K2_CanActivateAbility(*ActorInfo, Handle, K2FailTags) == false)
		{
			if (FScopedCanActivateAbilityLogEnabler::IsLoggingEnabled())
			{
				UE_LOGF(LogAbilitySystem, Verbose, "%ls: CanActivateAbility on %ls failed, Blueprint override returned false", *GetNameSafe(ActorInfo->OwnerActor.Get()), *GetNameSafe(Spec->Ability));
				UE_VLOG(ActorInfo->OwnerActor.Get(), VLogAbilitySystem, Verbose, TEXT("CanActivateAbility on %s failed, Blueprint override returned false"), *GetNameSafe(Spec->Ability));
			}

			if (OptionalRelevantTags)
			{
				const FGameplayTag& FailTag = GetDefault<UGameplayAbilitiesDeveloperSettings>()->ActivateFailCanActivateAbilityTag;
				if (FailTag.IsValid())
				{
					OptionalRelevantTags->AddTag(FailTag);
				}

				OptionalRelevantTags->AppendTags(K2FailTags);
			}

			return false;
		}
	}

	return true;
}

bool UGameplayAbility::ShouldAbilityRespondToEvent(const FGameplayAbilityActorInfo* ActorInfo, const FGameplayEventData* Payload) const
{
	if (bHasBlueprintShouldAbilityRespondToEvent)
	{
		if (K2_ShouldAbilityRespondToEvent(*ActorInfo, *Payload) == false)
		{
			ABILITY_LOG(Log, TEXT("ShouldAbilityRespondToEvent %s failed, blueprint refused"), *GetName());
			return false;
		}
	}

	return true;
}

bool UGameplayAbility::CommitAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, OUT FGameplayTagContainer* OptionalRelevantTags)
{
	// Last chance to fail (maybe we no longer have resources to commit since we after we started this ability activation)
	if (!CommitCheck(Handle, ActorInfo, ActivationInfo, OptionalRelevantTags))
	{
		return false;
	}

	CommitExecute(Handle, ActorInfo, ActivationInfo);

	// Fixme: Should we always call this or only if it is implemented? A noop may not hurt but could be bad for perf (storing a HasBlueprintCommit per instance isn't good either)
	K2_CommitExecute();

	// Broadcast this commitment
	ActorInfo->AbilitySystemComponent->NotifyAbilityCommit(this);

	return true;
}

bool UGameplayAbility::CommitAbilityCooldown(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const bool ForceCooldown, OUT FGameplayTagContainer* OptionalRelevantTags)
{
	if (UAbilitySystemGlobals::Get().ShouldIgnoreCooldowns())
	{
		return true;
	}

	if (!ForceCooldown)
	{
		// Last chance to fail (maybe we no longer have resources to commit since we after we started this ability activation)
		if (!CheckCooldown(Handle, ActorInfo, OptionalRelevantTags))
		{
			return false;
		}
	}

	ApplyCooldown(Handle, ActorInfo, ActivationInfo);
	return true;
}

bool UGameplayAbility::CommitAbilityCost(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, OUT FGameplayTagContainer* OptionalRelevantTags)
{
	if (UAbilitySystemGlobals::Get().ShouldIgnoreCosts())
	{
		return true;
	}

	// Last chance to fail (maybe we no longer have resources to commit since we after we started this ability activation)
	if (!CheckCost(Handle, ActorInfo, OptionalRelevantTags))
	{
		return false;
	}

	ApplyCost(Handle, ActorInfo, ActivationInfo);
	return true;
}

bool UGameplayAbility::CommitCheck(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, OUT FGameplayTagContainer* OptionalRelevantTags)
{
	/**
	 *	Checks if we can (still) commit this ability. There are some subtleties here.
	 *		-An ability can start activating, play an animation, wait for a user confirmation/target data, and then actually commit
	 *		-Commit = spend resources/cooldowns. It's possible the source has changed state since it started activation, so a commit may fail.
	 *		-We don't want to just call CanActivateAbility() since right now that also checks things like input inhibition.
	 *			-E.g., its possible the act of starting your ability makes it no longer activatable (CanActivateAbility() may be false if called here).
	 */

	const bool bValidHandle = Handle.IsValid();
	const bool bValidActorInfoPieces = (ActorInfo && (ActorInfo->AbilitySystemComponent != nullptr));
	const bool bValidSpecFound = bValidActorInfoPieces && (ActorInfo->AbilitySystemComponent->FindAbilitySpecFromHandle(Handle) != nullptr);

	// Ensure that the ability spec is even valid before trying to process the commit
	if (!bValidHandle || !bValidActorInfoPieces || !bValidSpecFound)
	{
		ABILITY_LOG(Warning, TEXT("UGameplayAbility::CommitCheck provided an invalid handle or actor info or couldn't find ability spec: %s Handle Valid: %d ActorInfo Valid: %d Spec Not Found: %d"), *GetName(), bValidHandle, bValidActorInfoPieces, bValidSpecFound);
		return false;
	}

	UAbilitySystemGlobals& AbilitySystemGlobals = UAbilitySystemGlobals::Get();

	if (!AbilitySystemGlobals.ShouldIgnoreCooldowns() && !CheckCooldown(Handle, ActorInfo, OptionalRelevantTags))
	{
		return false;
	}

	if (!AbilitySystemGlobals.ShouldIgnoreCosts() && !CheckCost(Handle, ActorInfo, OptionalRelevantTags))
	{
		return false;
	}

	return true;
}

void UGameplayAbility::CommitExecute(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo)
{
	ApplyCooldown(Handle, ActorInfo, ActivationInfo);

	ApplyCost(Handle, ActorInfo, ActivationInfo);
}

bool UGameplayAbility::CanBeCanceled() const
{
	if (GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced)
	{
		return bIsCancelable;
	}

	// Non instanced are always cancelable
	return true;
}

void UGameplayAbility::SetCanBeCanceled(bool bCanBeCanceled)
{
	if (GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced && bCanBeCanceled != bIsCancelable)
	{
		bIsCancelable = bCanBeCanceled;

		UAbilitySystemComponent* Comp = CurrentActorInfo->AbilitySystemComponent.Get();
		if (Comp)
		{
			Comp->HandleChangeAbilityCanBeCanceled(GetAssetTags(), this, bCanBeCanceled);
		}
	}
}

bool UGameplayAbility::IsBlockingOtherAbilities() const
{
	if (GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced)
	{
		return bIsBlockingOtherAbilities;
	}

	// Non instanced are always marked as blocking other abilities
	return true;
}

void UGameplayAbility::SetShouldBlockOtherAbilities(bool bShouldBlockAbilities)
{
	if (bIsActive && GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced && bShouldBlockAbilities != bIsBlockingOtherAbilities)
	{
		bIsBlockingOtherAbilities = bShouldBlockAbilities;

		UAbilitySystemComponent* Comp = CurrentActorInfo->AbilitySystemComponent.Get();
		if (Comp)
		{
			Comp->ApplyAbilityBlockAndCancelTags(GetAssetTags(), this, bIsBlockingOtherAbilities, BlockAbilitiesWithTag, false, CancelAbilitiesWithTag);
		}
	}
}

void UGameplayAbility::CancelAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, bool bReplicateCancelAbility)
{
	if (CanBeCanceled())
	{
		if (ScopeLockCount > 0)
		{
			UE_LOGF(LogAbilitySystem, Verbose, "Attempting to cancel Ability %ls but ScopeLockCount was greater than 0, adding cancel to the WaitingToExecute Array", *GetName());
			WaitingToExecute.Add(FPostLockDelegate::CreateUObject(this, &UGameplayAbility::CancelAbility, Handle, ActorInfo, ActivationInfo, bReplicateCancelAbility));
			return;
		}

		// Replicate the the server/client if needed
		if (bReplicateCancelAbility && ActorInfo && ActorInfo->AbilitySystemComponent.IsValid())
		{
			ActorInfo->AbilitySystemComponent->ReplicateEndOrCancelAbility(Handle, ActivationInfo, this, true);
		}

		// Gives the Ability BP a chance to perform custom logic/cleanup when any active ability states are active
		if (OnGameplayAbilityCancelled.IsBound())
		{
			OnGameplayAbilityCancelled.Broadcast();
		}

		// End the ability but don't replicate it, we replicate the CancelAbility call directly
		bool bReplicateEndAbility = false;
		bool bWasCancelled = true;
		EndAbility(Handle, ActorInfo, ActivationInfo, bReplicateEndAbility, bWasCancelled);
	}
}

bool UGameplayAbility::IsEndAbilityValid(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo) const
{
	// Protect against EndAbility being called multiple times
	// Ending an AbilityState may cause this to be invoked again
	if ((bIsActive == false || bIsAbilityEnding == true) && GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced)
	{
		UE_LOGF(LogAbilitySystem, Verbose, "IsEndAbilityValid returning false on Ability %ls due to EndAbility being called multiple times", *GetName());
		return false;
	}

	// check if ability has valid owner
	UAbilitySystemComponent* AbilityComp = ActorInfo ? ActorInfo->AbilitySystemComponent.Get() : nullptr;
	if (AbilityComp == nullptr)
	{
		UE_LOGF(LogAbilitySystem, Verbose, "IsEndAbilityValid returning false on Ability %ls due to AbilitySystemComponent being invalid", *GetName());
		return false;
	}

	// check to see if this is an NonInstanced or if the ability is active.
	const FGameplayAbilitySpec* Spec = AbilityComp->FindAbilitySpecFromHandle(Handle);
	const bool bIsSpecActive = Spec ? Spec->IsActive() : IsActive();

	if (!bIsSpecActive)
	{
		UE_LOGF(LogAbilitySystem, Verbose, "IsEndAbilityValid returning false on Ability %ls due spec not being active", *GetName());
		return false;
	}

	return true;
}

void UGameplayAbility::EndAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, bool bReplicateEndAbility, bool bWasCancelled)
{
	if (IsEndAbilityValid(Handle, ActorInfo))
	{
		if (ScopeLockCount > 0)
		{
			UE_LOGF(LogAbilitySystem, Verbose, "Attempting to end Ability %ls but ScopeLockCount was greater than 0, adding end to the WaitingToExecute Array", *GetName());
			WaitingToExecute.Add(FPostLockDelegate::CreateUObject(this, &UGameplayAbility::EndAbility, Handle, ActorInfo, ActivationInfo, bReplicateEndAbility, bWasCancelled));
			return;
		}
        
        if (GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced)
        {
            bIsAbilityEnding = true;
        }

		// Give blueprint a chance to react
		K2_OnEndAbility(bWasCancelled);

		// Protect against blueprint causing us to EndAbility already
		if (bIsActive == false && GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced)
		{
			return;
		}

		// Stop any timers or latent actions for the ability
		UWorld* MyWorld = GetWorld();
		if (MyWorld)
		{
			MyWorld->GetLatentActionManager().RemoveActionsForObject(this);
			if (FAbilitySystemTweaks::ClearAbilityTimers)
			{
				MyWorld->GetTimerManager().ClearAllTimersForObject(this);
			}
		}

		// Execute our delegate and unbind it, as we are no longer active and listeners can re-register when we become active again.
		OnGameplayAbilityEnded.Broadcast(this);
		OnGameplayAbilityEnded.Clear();

		OnGameplayAbilityEndedWithData.Broadcast(FAbilityEndedData(this, Handle, bReplicateEndAbility, bWasCancelled));
		OnGameplayAbilityEndedWithData.Clear();

		if (GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced)
		{
			bIsActive = false;
			bIsAbilityEnding = false;
		}

		// Tell all our tasks that we are finished and they should cleanup
		for (int32 TaskIdx = ActiveTasks.Num() - 1; TaskIdx >= 0 && ActiveTasks.Num() > 0; --TaskIdx)
		{
			UGameplayTask* Task = ActiveTasks[TaskIdx];
			if (Task)
			{
				Task->TaskOwnerEnded();
			}
		}
		ActiveTasks.Reset();	// Empty the array but don't resize memory, since this object is probably going to be destroyed very soon anyways.

		if (UAbilitySystemComponent* const AbilitySystemComponent = ActorInfo->AbilitySystemComponent.Get())
		{
			if (bReplicateEndAbility)
			{
				AbilitySystemComponent->ReplicateEndOrCancelAbility(Handle, ActivationInfo, this, false);
			}

			// Remove tags we added to owner
			AbilitySystemComponent->RemoveLooseGameplayTags(ActivationOwnedTags, 1, UAbilitySystemGlobals::Get().ShouldReplicateActivationOwnedTags() ? EGameplayTagReplicationState::CountToOwner : EGameplayTagReplicationState::None);

			// Remove tracked GameplayCues that we added
			for (FGameplayTag& GameplayCueTag : TrackedGameplayCues)
			{
				AbilitySystemComponent->RemoveGameplayCue(GameplayCueTag);
			}
			TrackedGameplayCues.Empty();

			if (CanBeCanceled())
			{
				// If we're still cancelable, cancel it now
				AbilitySystemComponent->HandleChangeAbilityCanBeCanceled(GetAssetTags(), this, false);
			}
			
			if (IsBlockingOtherAbilities())
			{
				// If we're still blocking other abilities, cancel now
				AbilitySystemComponent->ApplyAbilityBlockAndCancelTags(GetAssetTags(), this, false, BlockAbilitiesWithTag, false, CancelAbilitiesWithTag);
			}

			AbilitySystemComponent->ClearAbilityReplicatedDataCache(Handle, CurrentActivationInfo);

			// Tell owning AbilitySystemComponent that we ended so it can do stuff (including MarkPendingKill us)
			AbilitySystemComponent->NotifyAbilityEnded(Handle, this, bWasCancelled);
		}

		if (IsInstantiated())
		{
			CurrentEventData = FGameplayEventData{};
		}
	}
}

void UGameplayAbility::ActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayEventData* TriggerEventData)
{
	if (TriggerEventData && bHasBlueprintActivateFromEvent)
	{
		// A Blueprinted ActivateAbility function must call CommitAbility somewhere in its execution chain.
		K2_ActivateAbilityFromEvent(*TriggerEventData);
	}
	else if (bHasBlueprintActivate)
	{
		// A Blueprinted ActivateAbility function must call CommitAbility somewhere in its execution chain.
		K2_ActivateAbility();
	}
	else if (bHasBlueprintActivateFromEvent)
	{
		UE_LOGF(LogAbilitySystem, Warning, "Ability %ls expects event data but none is being supplied. Use 'Activate Ability' instead of 'Activate Ability From Event' in the Blueprint.", *GetName());
		constexpr bool bReplicateEndAbility = false;
		constexpr bool bWasCancelled = true;
		EndAbility(Handle, ActorInfo, ActivationInfo, bReplicateEndAbility, bWasCancelled);
	}
	else
	{
		// Native child classes should override ActivateAbility and call CommitAbility.
		// CommitAbility is used to do one last check for spending resources.
		// Previous versions of this function called CommitAbility but that prevents the callers
		// from knowing the result. Your override should call it and check the result.
		// Here is some starter code:
		
		//	if (!CommitAbility(Handle, ActorInfo, ActivationInfo))
		//	{			
		//		constexpr bool bReplicateEndAbility = true;
		//		constexpr bool bWasCancelled = true;
		//		EndAbility(Handle, ActorInfo, ActivationInfo, bReplicateEndAbility, bWasCancelled);
		//	}
	}
}

void UGameplayAbility::PreActivate(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, FOnGameplayAbilityEnded::FDelegate* OnGameplayAbilityEndedDelegate, const FGameplayEventData* TriggerEventData)
{
	UAbilitySystemComponent* Comp = ActorInfo->AbilitySystemComponent.Get();

	//Flush any remaining server moves before activating the ability.
	//	Flushing the server moves prevents situations where previously pending move's DeltaTimes are figured into montages that are about to play and update.
	//	When this happened, clients would have a smaller delta time than the server which meant the server would get ahead and receive their notifies before the client, etc.
	//	The system depends on the server not getting ahead, so it's important to send along any previously pending server moves here.
	AActor* const MyActor = ActorInfo->AvatarActor.Get();
	if (MyActor && !ActorInfo->IsNetAuthority())
	{
		ACharacter* MyCharacter = Cast<ACharacter>(MyActor);
		if (MyCharacter)
		{
			UCharacterMovementComponent* CharMoveComp = Cast<UCharacterMovementComponent>(MyCharacter->GetMovementComponent());

			if (CharMoveComp)
			{
				CharMoveComp->FlushServerMoves();
			}
		}
	}

	if (GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced)
	{
		bIsActive = true;
		bIsBlockingOtherAbilities = true;
		bIsCancelable = true;
	}

	RemoteInstanceEnded = false;

	// This must be called before we start applying tags and blocking or canceling other abilities.
	// We could set off a chain that results in calling functions on this ability that rely on the current info being set.
	SetCurrentInfo(Handle, ActorInfo, ActivationInfo);

	if (IsInstantiated())
	{
		if (TriggerEventData)
		{
			CurrentEventData = *TriggerEventData;
		}
		else
		{
			CurrentEventData = FGameplayEventData();
		}
	}

	Comp->HandleChangeAbilityCanBeCanceled(GetAssetTags(), this, true);

	Comp->AddLooseGameplayTags(ActivationOwnedTags, 1, UAbilitySystemGlobals::Get().ShouldReplicateActivationOwnedTags() ? EGameplayTagReplicationState::CountToOwner : EGameplayTagReplicationState::None);

	if (OnGameplayAbilityEndedDelegate)
	{
		OnGameplayAbilityEnded.Add(*OnGameplayAbilityEndedDelegate);
	}

	Comp->NotifyAbilityActivated(Handle, this);

	Comp->ApplyAbilityBlockAndCancelTags(GetAssetTags(), this, true, BlockAbilitiesWithTag, true, CancelAbilitiesWithTag);

	// Spec's active count must be incremented after applying blockor cancel tags, otherwise the ability runs the risk of cancelling itself inadvertantly before it completely activates.
	FGameplayAbilitySpec* Spec = Comp->FindAbilitySpecFromHandle(Handle);
	if (!Spec)
	{
		ABILITY_LOG(Warning, TEXT("PreActivate called with a valid handle but no matching ability spec was found. Handle: %s ASC: %s. AvatarActor: %s"), *Handle.ToString(), *(Comp->GetPathName()), *GetNameSafe(Comp->GetAvatarActor_Direct()));
		return;
	}

	// make sure we do not incur a roll over if we go over the uint8 max, this will need to be updated if the var size changes
	if (LIKELY(Spec->ActiveCount < UINT8_MAX))
	{
		Spec->ActiveCount++;
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("PreActivate %s called when the Spec->ActiveCount (%d) >= UINT8_MAX"), *GetName(), (int32)Spec->ActiveCount)
	}
}

void UGameplayAbility::CallActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, FOnGameplayAbilityEnded::FDelegate* OnGameplayAbilityEndedDelegate, const FGameplayEventData* TriggerEventData)
{
	PreActivate(Handle, ActorInfo, ActivationInfo, OnGameplayAbilityEndedDelegate, TriggerEventData);
	ActivateAbility(Handle, ActorInfo, ActivationInfo, TriggerEventData);
}

void UGameplayAbility::ConfirmActivateSucceed()
{
	// On instanced abilities, update CurrentActivationInfo and call any registered delegates.
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		PostNetInit();
		ensure(CurrentActorInfo);
		CurrentActivationInfo.SetActivationConfirmed();

		OnConfirmDelegate.Broadcast(this);
		OnConfirmDelegate.Clear();
	}
}

UGameplayEffect* UGameplayAbility::GetCooldownGameplayEffect() const
{
	if ( CooldownGameplayEffectClass )
	{
		return CooldownGameplayEffectClass->GetDefaultObject<UGameplayEffect>();
	}
	else
	{
		return nullptr;
	}
}

UGameplayEffect* UGameplayAbility::GetCostGameplayEffect() const
{
	if ( CostGameplayEffectClass )
	{
		return CostGameplayEffectClass->GetDefaultObject<UGameplayEffect>();
	}
	else
	{
		return nullptr;
	}
}

bool UGameplayAbility::CheckCooldown(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, OUT FGameplayTagContainer* OptionalRelevantTags) const
{
	if (!ensure(ActorInfo))
	{
		return true;
	}

	const FGameplayTagContainer* CooldownTags = GetCooldownTags();
	if (CooldownTags)
	{
		if (!CooldownTags->IsEmpty())
		{
			if (UAbilitySystemComponent* AbilitySystemComponent = ActorInfo->AbilitySystemComponent.Get())
			{
				if (AbilitySystemComponent->HasAnyMatchingGameplayTags(*CooldownTags))
				{
					if (OptionalRelevantTags)
					{
						const FGameplayTag& FailCooldownTag = UAbilitySystemGlobals::Get().ActivateFailCooldownTag;
						if (FailCooldownTag.IsValid())
						{
							OptionalRelevantTags->AddTag(FailCooldownTag);
						}

						// Let the caller know which tags were blocking
						OptionalRelevantTags->AppendMatchingTags(AbilitySystemComponent->GetOwnedGameplayTags(), *CooldownTags);
					}

					return false;
				}
			}
		}
		else if (UE::AbilitySystem::Private::CVarWarnCooldownEffectWithoutTagsValue > 0 && CooldownGameplayEffectClass != nullptr)
		{
			// Unless this function is overridden, a CooldownGameplayEffectClass that doesn't grant tags isn't effective as cooldown. 
			// Log a runtime warning when an ability's cooldown is checked while having this misconfiguration.
			UE_LOGF(LogAbilitySystem, Warning, "CooldownGameplayEffectClass '%ls' grants no tags. A GameplayEffect class must grant tags (Component: Grant Tags to Target Actor) to be used as cooldown.", *CooldownGameplayEffectClass->GetName());
		}
	}
	return true;
}

void UGameplayAbility::ApplyCooldown(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo) const
{
	UGameplayEffect* CooldownGE = GetCooldownGameplayEffect();
	if (CooldownGE)
	{
		ApplyGameplayEffectToOwner(Handle, ActorInfo, ActivationInfo, CooldownGE, GetAbilityLevel(Handle, ActorInfo));
	}
}

bool UGameplayAbility::CheckCost(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, OUT FGameplayTagContainer* OptionalRelevantTags) const
{
	UGameplayEffect* CostGE = GetCostGameplayEffect();
	if (CostGE)
	{
		UAbilitySystemComponent* AbilitySystemComponent = ActorInfo ? ActorInfo->AbilitySystemComponent.Get() : nullptr;
		if (ensure(AbilitySystemComponent))
		{
			if (!AbilitySystemComponent->CanApplyAttributeModifiers(CostGE, GetAbilityLevel(Handle, ActorInfo), MakeEffectContext(Handle, ActorInfo)))
			{
				const FGameplayTag& CostTag = UAbilitySystemGlobals::Get().ActivateFailCostTag;

				if (OptionalRelevantTags && CostTag.IsValid())
				{
					OptionalRelevantTags->AddTag(CostTag);
				}
				return false;
			}
		}
	}
	return true;
}

void UGameplayAbility::ApplyCost(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo) const
{
	UGameplayEffect* CostGE = GetCostGameplayEffect();
	if (CostGE)
	{
		ApplyGameplayEffectToOwner(Handle, ActorInfo, ActivationInfo, CostGE, GetAbilityLevel(Handle, ActorInfo));
	}
}

float UGameplayAbility::GetCooldownTimeRemaining(const FGameplayAbilityActorInfo* ActorInfo) const
{
	SCOPE_CYCLE_COUNTER(STAT_GameplayAbilityGetCooldownTimeRemaining);

	const UAbilitySystemComponent* const ASC = ActorInfo->AbilitySystemComponent.Get();
	if (ASC)
	{
		const FGameplayTagContainer* CooldownTags = GetCooldownTags();
		if (CooldownTags && CooldownTags->Num() > 0)
		{
			FGameplayEffectQuery const Query = FGameplayEffectQuery::MakeQuery_MatchAnyOwningTags(*CooldownTags);
			TArray< float > Durations = ASC->GetActiveEffectsTimeRemaining(Query);
			if (Durations.Num() > 0)
			{
				Durations.Sort();
				return Durations[Durations.Num() - 1];
			}
		}
	}

	return 0.f;
}

void UGameplayAbility::InvalidateClientPredictionKey() const
{
	if (CurrentActorInfo)
	{
		if (UAbilitySystemComponent* const AbilitySystemComponent = CurrentActorInfo->AbilitySystemComponent.Get())
		{
			AbilitySystemComponent->ScopedPredictionKey = FPredictionKey();
		}
	}
}

void UGameplayAbility::GetCooldownTimeRemainingAndDuration(FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, float& TimeRemaining, float& CooldownDuration) const
{
	SCOPE_CYCLE_COUNTER(STAT_GameplayAbilityGetCooldownTimeRemainingAndDuration);

	TimeRemaining = 0.f;
	CooldownDuration = 0.f;
	
	const FGameplayTagContainer* CooldownTags = GetCooldownTags();
	if (CooldownTags && CooldownTags->Num() > 0)
	{
		UAbilitySystemComponent* AbilitySystemComponent = ActorInfo ? ActorInfo->AbilitySystemComponent.Get() : nullptr;
		if (ensure(AbilitySystemComponent))
		{
			FGameplayEffectQuery const Query = FGameplayEffectQuery::MakeQuery_MatchAnyOwningTags(*CooldownTags);
			TArray<TPair<float, float>> DurationAndTimeRemaining = AbilitySystemComponent->GetActiveEffectsTimeRemainingAndDuration(Query);
			if (DurationAndTimeRemaining.Num() > 0)
			{
				int32 BestIdx = 0;
				float LongestTime = DurationAndTimeRemaining[0].Key;
				for (int32 Idx = 1; Idx < DurationAndTimeRemaining.Num(); ++Idx)
				{
					if (DurationAndTimeRemaining[Idx].Key > LongestTime)
					{
						LongestTime = DurationAndTimeRemaining[Idx].Key;
						BestIdx = Idx;
					}
				}

				TimeRemaining = DurationAndTimeRemaining[BestIdx].Key;
				CooldownDuration = DurationAndTimeRemaining[BestIdx].Value;
			}
		}
	}
}

const FGameplayTagContainer* UGameplayAbility::GetCooldownTags() const
{
	UGameplayEffect* CDGE = GetCooldownGameplayEffect();
	return CDGE ? &CDGE->GetGrantedTags() : nullptr;
}

FGameplayAbilityActorInfo UGameplayAbility::GetActorInfo() const
{
	if (!ensure(CurrentActorInfo))
	{
		return FGameplayAbilityActorInfo();
	}
	return *CurrentActorInfo;
}

AActor* UGameplayAbility::GetOwningActorFromActorInfo() const
{
	if (!ensureMsgf(UE::AbilitySystem::Private::CVarAllowNonInstancedGAsValue || IsInstantiated(),
		TEXT("GetOwningActorFromActorInfo() called on the CDO for %s.  NonInstanced abilities are deprecated, thus we always expect this to be called on an instanced object."), *GetNameSafe(this)))
	{
		return nullptr;
	}

	if (!ensure(CurrentActorInfo))
	{
		return nullptr;
	}
	return CurrentActorInfo->OwnerActor.Get();
}

AActor* UGameplayAbility::GetAvatarActorFromActorInfo() const
{
	ensureMsgf(UE::AbilitySystem::Private::CVarAllowNonInstancedGAsValue || IsInstantiated(),
		TEXT("GetAvatarActorFromActorInfo() called on the CDO for %s.  NonInstanced abilities are deprecated, thus we always expect this to be called on an instanced object."), *GetNameSafe(this));

	if (!ensure(CurrentActorInfo))
	{
		return nullptr;
	}
	return CurrentActorInfo->AvatarActor.Get();
}

USkeletalMeshComponent* UGameplayAbility::GetOwningComponentFromActorInfo() const
{
	ensureMsgf(UE::AbilitySystem::Private::CVarAllowNonInstancedGAsValue || IsInstantiated(),
		TEXT("GetOwningComponentFromActorInfo() called on the CDO for %s.  NonInstanced abilities are deprecated, thus we always expect this to be called on an instanced object."), *GetNameSafe(this));

	if (!ensure(CurrentActorInfo))
	{
		return nullptr;
	}

	return CurrentActorInfo->SkeletalMeshComponent.Get();
}

UAbilitySystemComponent* UGameplayAbility::GetAbilitySystemComponentFromActorInfo() const
{
	ensureMsgf(UE::AbilitySystem::Private::CVarAllowNonInstancedGAsValue || IsInstantiated(),
		TEXT("GetAbilitySystemComponentFromActorInfo() called on the CDO for %s.  NonInstanced abilities are deprecated, thus we always expect this to be called on an instanced object."), *GetNameSafe(this));

	if (!ensure(CurrentActorInfo))
	{
		return nullptr;
	}
	return CurrentActorInfo->AbilitySystemComponent.Get();
}

UAbilitySystemComponent* UGameplayAbility::GetAbilitySystemComponentFromActorInfo_Checked() const
{
	UAbilitySystemComponent* AbilitySystemComponent = CurrentActorInfo ? CurrentActorInfo->AbilitySystemComponent.Get() : nullptr;
	check(AbilitySystemComponent);

	return AbilitySystemComponent;
}

UAbilitySystemComponent* UGameplayAbility::GetAbilitySystemComponentFromActorInfo_Ensured() const
{
	UAbilitySystemComponent* AbilitySystemComponent = ensure(CurrentActorInfo) ? CurrentActorInfo->AbilitySystemComponent.Get() : nullptr;
	ensure(AbilitySystemComponent);

	return AbilitySystemComponent;
}

const FGameplayTagContainer& UGameplayAbility::GetAssetTags() const
{
	return AbilityTags;
}

void UGameplayAbility::SetAssetTags(const FGameplayTagContainer& InAssetTags)
{
	ensureMsgf(HasAnyFlags(RF_NeedInitialization), TEXT("%hs should only be used during construction as GetAbilityTags() are primarily read from the CDO"), __func__);
	AbilityTags = InAssetTags;
}

const FGameplayAbilityActorInfo* UGameplayAbility::GetCurrentActorInfo() const
{
	if (!ensureMsgf(UE::AbilitySystem::Private::CVarAllowNonInstancedGAsValue || IsInstantiated(),
		TEXT("GetCurrentActorInfo() called on the CDO for %s.  NonInstanced abilities are deprecated, thus we always expect this to be called on an instanced object."), *GetNameSafe(this)))
	{
		return nullptr;
	}

	return CurrentActorInfo;
}

FGameplayAbilityActivationInfo UGameplayAbility::GetCurrentActivationInfo() const
{
	if (!ensureMsgf(UE::AbilitySystem::Private::CVarAllowNonInstancedGAsValue || IsInstantiated(),
		TEXT("GetCurrentActivationInfo() called on the CDO for %s.  NonInstanced abilities are deprecated, thus we always expect this to be called on an instanced object."), *GetNameSafe(this)))
	{
		return FGameplayAbilityActivationInfo{};
	}

	return CurrentActivationInfo;
}

FGameplayAbilitySpecHandle UGameplayAbility::GetCurrentAbilitySpecHandle() const
{
	if (!ensureMsgf(UE::AbilitySystem::Private::CVarAllowNonInstancedGAsValue || IsInstantiated(),
		TEXT("GetCurrentAbilitySpecHandle() called on the CDO for %s.  NonInstanced abilities are deprecated, thus we always expect this to be called on an instanced object."), *GetNameSafe(this)))
	{
		return FGameplayAbilitySpecHandle{};
	}

	return CurrentSpecHandle;
}

FGameplayEffectSpecHandle UGameplayAbility::MakeOutgoingGameplayEffectSpec(TSubclassOf<UGameplayEffect> GameplayEffectClass, float Level) const
{
	ensure(CurrentActorInfo && CurrentActorInfo->AbilitySystemComponent.IsValid());
	return MakeOutgoingGameplayEffectSpec(CurrentSpecHandle, CurrentActorInfo, CurrentActivationInfo, GameplayEffectClass, Level);
}

FGameplayEffectSpecHandle UGameplayAbility::MakeOutgoingGameplayEffectSpec(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, TSubclassOf<UGameplayEffect> GameplayEffectClass, float Level) const
{
	if (!ensure(ActorInfo))
	{
		return FGameplayEffectSpecHandle{};
	}

	UAbilitySystemComponent* const AbilitySystemComponent = ActorInfo->AbilitySystemComponent.Get();
	
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (UE::AbilitySystem::Private::AbilitySystemShowMakeOutgoingGameplayEffectSpecs && HasAuthority(&ActivationInfo) == false)
	{
		ABILITY_LOG(Warning, TEXT("%s, MakeOutgoingGameplayEffectSpec: %s"), *AbilitySystemComponent->GetFullName(),  *GameplayEffectClass->GetName()); 
	}
#endif

	FGameplayEffectSpecHandle NewHandle = AbilitySystemComponent->MakeOutgoingSpec(GameplayEffectClass, Level, MakeEffectContext(Handle, ActorInfo));
	if (NewHandle.IsValid())
	{
		FGameplayAbilitySpec* AbilitySpec = AbilitySystemComponent->FindAbilitySpecFromHandle(Handle);
		ApplyAbilityTagsToGameplayEffectSpec(*NewHandle.Data.Get(), AbilitySpec);

		// Copy over set by caller magnitudes
		if (AbilitySpec)
		{
			NewHandle.Data->SetByCallerTagMagnitudes = AbilitySpec->SetByCallerTagMagnitudes;
		}
		BP_EditSpecValues(NewHandle);
	}
	return NewHandle;
}

void UGameplayAbility::ApplyAbilityTagsToGameplayEffectSpec(FGameplayEffectSpec& Spec, FGameplayAbilitySpec* AbilitySpec) const
{
	FGameplayTagContainer& CapturedSourceTags = Spec.CapturedSourceTags.GetSpecTags();

	CapturedSourceTags.AppendTags(GetAssetTags());

	// Allow the source object of the ability to propagate tags along as well
	if (AbilitySpec)
	{
		CapturedSourceTags.AppendTags(AbilitySpec->GetDynamicSpecSourceTags());

		const IGameplayTagAssetInterface* SourceObjAsTagInterface = Cast<IGameplayTagAssetInterface>(AbilitySpec->SourceObject);
		if (SourceObjAsTagInterface)
		{
			FGameplayTagContainer SourceObjTags;
			SourceObjAsTagInterface->GetOwnedGameplayTags(SourceObjTags);

			CapturedSourceTags.AppendTags(SourceObjTags);
		}

		// Copy SetByCallerMagnitudes 
		Spec.MergeSetByCallerMagnitudes(AbilitySpec->SetByCallerTagMagnitudes);
	}
}

/** Fixme: Naming is confusing here */

bool UGameplayAbility::K2_CommitAbility()
{
	ensure(CurrentActorInfo);
	return CommitAbility(CurrentSpecHandle, CurrentActorInfo, CurrentActivationInfo);
}

bool UGameplayAbility::K2_CommitAbilityCooldown(bool BroadcastCommitEvent, bool ForceCooldown)
{
	ensure(CurrentActorInfo);
	if (BroadcastCommitEvent)
	{
		if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
		{
			AbilitySystemComponent->NotifyAbilityCommit(this);
		}
	}
	return CommitAbilityCooldown(CurrentSpecHandle, CurrentActorInfo, CurrentActivationInfo, ForceCooldown);
}

bool UGameplayAbility::K2_CommitAbilityCost(bool BroadcastCommitEvent)
{
	ensure(CurrentActorInfo);
	if (BroadcastCommitEvent)
	{
		if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
		{
			AbilitySystemComponent->NotifyAbilityCommit(this);
		}
	}
	return CommitAbilityCost(CurrentSpecHandle, CurrentActorInfo, CurrentActivationInfo);
}

bool UGameplayAbility::K2_CheckAbilityCooldown()
{
	ensure(CurrentActorInfo);
	return UAbilitySystemGlobals::Get().ShouldIgnoreCooldowns() || CheckCooldown(CurrentSpecHandle, CurrentActorInfo);
}

bool UGameplayAbility::K2_CheckAbilityCost()
{
	ensure(CurrentActorInfo);
	return UAbilitySystemGlobals::Get().ShouldIgnoreCosts() || CheckCost(CurrentSpecHandle, CurrentActorInfo);
}

void UGameplayAbility::K2_EndAbility()
{
	ensure(CurrentActorInfo);

	bool bReplicateEndAbility = true;
	bool bWasCancelled = false;
	EndAbility(CurrentSpecHandle, CurrentActorInfo, CurrentActivationInfo, bReplicateEndAbility, bWasCancelled);
}

void UGameplayAbility::K2_EndAbilityLocally()
{
	ensure(CurrentActorInfo);

	bool bReplicateEndAbility = false;
	bool bWasCancelled = false;
	EndAbility(CurrentSpecHandle, CurrentActorInfo, CurrentActivationInfo, bReplicateEndAbility, bWasCancelled);
}

void UGameplayAbility::MontageJumpToSection(FName SectionName)
{
	ensure(CurrentActorInfo);

	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		if (AbilitySystemComponent->IsAnimatingAbility(this))
		{
			AbilitySystemComponent->CurrentMontageJumpToSection(SectionName);
		}
	}
}

void UGameplayAbility::MontageSetNextSectionName(FName FromSectionName, FName ToSectionName)
{
	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		if (AbilitySystemComponent->IsAnimatingAbility(this))
		{
			AbilitySystemComponent->CurrentMontageSetNextSectionName(FromSectionName, ToSectionName);
		}
	}
}

void UGameplayAbility::MontageStop(float OverrideBlendOutTime)
{
	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		// We should only stop the current montage if we are the animating ability
		if (AbilitySystemComponent->IsAnimatingAbility(this))
		{
			AbilitySystemComponent->CurrentMontageStop(OverrideBlendOutTime);
		}
	}
}

void UGameplayAbility::SetCurrentMontage(class UAnimMontage* InCurrentMontage)
{
	if (ensureMsgf(UE::AbilitySystem::Private::CVarAllowNonInstancedGAsValue || IsInstantiated(),
		TEXT("SetCurrentMontage() called on the CDO for %s.  NonInstanced abilities are deprecated, thus we always expect this to be called on an instanced object."), *GetNameSafe(this)))
	{
		CurrentMontage = InCurrentMontage;
	}
}

UAnimMontage* UGameplayAbility::GetCurrentMontage() const
{
	ensureMsgf(UE::AbilitySystem::Private::CVarAllowNonInstancedGAsValue || IsInstantiated(),
		TEXT("GetCurrentMontage() called on the CDO for %s.  NonInstanced abilities are deprecated, thus we always expect this to be called on an instanced object."), *GetNameSafe(this));
	return CurrentMontage;
}

FGameplayAbilityTargetingLocationInfo UGameplayAbility::MakeTargetLocationInfoFromOwnerActor()
{
	FGameplayAbilityTargetingLocationInfo ReturnLocation;
	ReturnLocation.LocationType = EGameplayAbilityTargetingLocationType::ActorTransform;
	ReturnLocation.SourceActor = GetActorInfo().AvatarActor.Get();
	ReturnLocation.SourceAbility = this;
	return ReturnLocation;
}

FGameplayAbilityTargetingLocationInfo UGameplayAbility::MakeTargetLocationInfoFromOwnerSkeletalMeshComponent(FName SocketName)
{
	FGameplayAbilityTargetingLocationInfo ReturnLocation;
	ReturnLocation.LocationType = EGameplayAbilityTargetingLocationType::SocketTransform;
	ReturnLocation.SourceComponent = GetActorInfo().SkeletalMeshComponent.Get();
	ReturnLocation.SourceAbility = this;
	ReturnLocation.SourceSocketName = SocketName;
	return ReturnLocation;
}

UGameplayTasksComponent* UGameplayAbility::GetGameplayTasksComponent(const UGameplayTask& Task) const
{
	return GetCurrentActorInfo() ? GetCurrentActorInfo()->AbilitySystemComponent.Get() : nullptr;
}

AActor* UGameplayAbility::GetGameplayTaskOwner(const UGameplayTask* Task) const
{
	const FGameplayAbilityActorInfo* Info = GetCurrentActorInfo();
	return Info ? Info->OwnerActor.Get() : nullptr;
}

AActor* UGameplayAbility::GetGameplayTaskAvatar(const UGameplayTask* Task) const
{
	const FGameplayAbilityActorInfo* Info = GetCurrentActorInfo();
	return Info ? Info->AvatarActor.Get() : nullptr;
}

void UGameplayAbility::OnGameplayTaskInitialized(UGameplayTask& Task)
{
	UAbilityTask* AbilityTask = Cast<UAbilityTask>(&Task);
	const FGameplayAbilityActorInfo* ActorInfo = GetCurrentActorInfo();

	if (AbilityTask && ActorInfo)
	{
		AbilityTask->SetAbilitySystemComponent(ActorInfo->AbilitySystemComponent.Get());
		AbilityTask->Ability = this;
	}
}

void UGameplayAbility::OnGameplayTaskActivated(UGameplayTask& Task)
{
	UE_VLOG(CastChecked<AActor>(GetOuter()), LogGameplayTasks, VeryVerbose, TEXT("GameplayAbility Task Started %s"), *Task.GetName());
	UE_LOGF(LogGameplayTasks, VeryVerbose, "GameplayAbility Task Started %ls", *Task.GetName());

	ActiveTasks.Add(&Task);
}

void UGameplayAbility::OnGameplayTaskDeactivated(UGameplayTask& Task)
{
	UE_VLOG(CastChecked<AActor>(GetOuter()), LogGameplayTasks, VeryVerbose, TEXT("GameplayAbility Task Ended %s"), *Task.GetName());
	UE_LOGF(LogGameplayTasks, VeryVerbose, "GameplayAbility Task Ended %ls", *Task.GetName());

	ActiveTasks.Remove(&Task);

	if constexpr (ENABLE_ABILITYTASK_DEBUGMSG)
	{
		AddAbilityTaskDebugMessage(&Task, TEXT("Ended."));
	}
}

void UGameplayAbility::ConfirmTaskByInstanceName(FName InstanceName, bool bEndTask)
{
	TArray<UGameplayTask*, TInlineAllocator<8> > NamedTasks;

	for (UGameplayTask* Task : ActiveTasks)
	{
		if (Task && Task->GetInstanceName() == InstanceName)
		{
			NamedTasks.Add(Task);
		}
	}
	
	for (int32 i = NamedTasks.Num() - 1; i >= 0; --i)
	{
		UGameplayTask* CurrentTask = NamedTasks[i];
		if (IsValid(CurrentTask))
		{
			CurrentTask->ExternalConfirm(bEndTask);
		}
	}
}

void UGameplayAbility::EndOrCancelTasksByInstanceName()
{
	// Static array for avoiding memory allocations
	TArray<UGameplayTask*, TInlineAllocator<8> > NamedTasks;

	// Call Endtask on everything in EndTaskInstanceNames list
	for (int32 j = 0; j < EndTaskInstanceNames.Num(); ++j)
	{
		FName InstanceName = EndTaskInstanceNames[j];
		NamedTasks.Reset();

		// Find every current task that needs to end before ending any
		for (UGameplayTask* Task : ActiveTasks)
		{
			if (Task && Task->GetInstanceName() == InstanceName)
			{
				NamedTasks.Add(Task);
			}
		}
		
		// End each one individually. Not ending a task may do "anything" including killing other tasks or the ability itself
		for (int32 i = NamedTasks.Num() - 1; i >= 0; --i)
		{
			UGameplayTask* CurrentTask = NamedTasks[i];
			if (IsValid(CurrentTask))
			{
				CurrentTask->EndTask();
			}
		}
	}
	EndTaskInstanceNames.Empty();


	// Call ExternalCancel on everything in CancelTaskInstanceNames list
	for (int32 j = 0; j < CancelTaskInstanceNames.Num(); ++j)
	{
		FName InstanceName = CancelTaskInstanceNames[j];
		NamedTasks.Reset();
		
		// Find every current task that needs to cancel before cancelling any
		for (UGameplayTask* Task : ActiveTasks)
		{
			if (Task && Task->GetInstanceName() == InstanceName)
			{
				NamedTasks.Add(Task);
			}
		}

		// Cancel each one individually.  Not canceling a task may do "anything" including killing other tasks or the ability itself
		for (int32 i = NamedTasks.Num() - 1; i >= 0; --i)
		{
			UGameplayTask* CurrentTask = NamedTasks[i];
			if (IsValid(CurrentTask))
			{
				CurrentTask->ExternalCancel();
			}
		}
	}
	CancelTaskInstanceNames.Empty();
}

void UGameplayAbility::EndTaskByInstanceName(FName InstanceName)
{
	//Avoid race condition by delaying for one frame
	EndTaskInstanceNames.AddUnique(InstanceName);
	GetWorld()->GetTimerManager().SetTimerForNextTick(this, &UGameplayAbility::EndOrCancelTasksByInstanceName);
}

void UGameplayAbility::CancelTaskByInstanceName(FName InstanceName)
{
	//Avoid race condition by delaying for one frame
	CancelTaskInstanceNames.AddUnique(InstanceName);
	GetWorld()->GetTimerManager().SetTimerForNextTick(this, &UGameplayAbility::EndOrCancelTasksByInstanceName);
}

void UGameplayAbility::EndAbilityState(FName OptionalStateNameToEnd)
{
	ensure(CurrentActorInfo);

	if (OnGameplayAbilityStateEnded.IsBound())
	{
		OnGameplayAbilityStateEnded.Broadcast(OptionalStateNameToEnd);
	}
}

void UGameplayAbility::AddAbilityTaskDebugMessage(UGameplayTask* AbilityTask, FString DebugMessage)
{
	TaskDebugMessages.AddDefaulted();
	FAbilityTaskDebugMessage& Msg = TaskDebugMessages.Last();
	Msg.FromTask = AbilityTask;
	Msg.Message = FString::Printf(TEXT("{%s} %s"), AbilityTask ? *AbilityTask->GetDebugString() : TEXT(""), *DebugMessage);
}

#if WITH_EDITOR
FGameplayTagContainer& UGameplayAbility::EditorGetAssetTags()
{
	return AbilityTags;
}
#endif //WITH_EDITOR

/**
 *	Helper methods for adding GameplayCues without having to go through GameplayEffects.
 *	For now, none of these will happen predictively. We can eventually build this out more to 
 *	work with the PredictionKey system.
 */

void UGameplayAbility::K2_ExecuteGameplayCue(FGameplayTag GameplayCueTag, FGameplayEffectContextHandle Context)
{
	ensure(CurrentActorInfo);

	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		AbilitySystemComponent->ExecuteGameplayCue(GameplayCueTag, Context);
	}
}

void UGameplayAbility::K2_ExecuteGameplayCueWithParams(FGameplayTag GameplayCueTag, const FGameplayCueParameters& GameplayCueParameters)
{
	ensure(CurrentActorInfo);
	const_cast<FGameplayCueParameters&>(GameplayCueParameters).AbilityLevel = GetAbilityLevel();

	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		AbilitySystemComponent->ExecuteGameplayCue(GameplayCueTag, GameplayCueParameters);
	}
}

void UGameplayAbility::K2_AddGameplayCue(FGameplayTag GameplayCueTag, FGameplayEffectContextHandle Context, bool bRemoveOnAbilityEnd)
{
	ensure(CurrentActorInfo);

	// Make default context if nothing is passed in
	if (Context.IsValid() == false)
	{
		Context = MakeEffectContext(CurrentSpecHandle, CurrentActorInfo);
	}

	Context.SetAbility(this);

	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		AbilitySystemComponent->AddGameplayCue(GameplayCueTag, Context);
	}

	if (bRemoveOnAbilityEnd)
	{
		TrackedGameplayCues.Add(GameplayCueTag);
	}
}

void UGameplayAbility::K2_AddGameplayCueWithParams(FGameplayTag GameplayCueTag, const FGameplayCueParameters& GameplayCueParameter, bool bRemoveOnAbilityEnd)
{
	ensure(CurrentActorInfo);

	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		AbilitySystemComponent->AddGameplayCue(GameplayCueTag, GameplayCueParameter);
	}

	if (bRemoveOnAbilityEnd)
	{
		TrackedGameplayCues.Add(GameplayCueTag);
	}
}


void UGameplayAbility::K2_RemoveGameplayCue(FGameplayTag GameplayCueTag)
{
	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		AbilitySystemComponent->RemoveGameplayCue(GameplayCueTag);
	}

	TrackedGameplayCues.Remove(GameplayCueTag);
}

FGameplayEffectContextHandle UGameplayAbility::GetContextFromOwner(FGameplayAbilityTargetDataHandle OptionalTargetData) const
{
	ensure(CurrentActorInfo);
	FGameplayEffectContextHandle Context = MakeEffectContext(CurrentSpecHandle, CurrentActorInfo);
	
	for (auto Data : OptionalTargetData.Data)
	{
		if (Data.IsValid())
		{
			Data->AddTargetDataToContext(Context, true);
		}
	}

	return Context;
}

int32 UGameplayAbility::GetAbilityLevel() const
{
	if (IsInstantiated() == false || CurrentActorInfo == nullptr)
	{
		return 1;
	}
	
	return GetAbilityLevel(CurrentSpecHandle, CurrentActorInfo);
}

/** Returns current ability level for non instanced abilities. You must call this version in these contexts! */
int32 UGameplayAbility::GetAbilityLevel(FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo) const
{
	ensure(ActorInfo);
	const UAbilitySystemComponent* AbilitySystemComponent = ActorInfo ? ActorInfo->AbilitySystemComponent.Get() : nullptr;
	const FGameplayAbilitySpec* Spec = AbilitySystemComponent ? AbilitySystemComponent->FindAbilitySpecFromHandle(Handle) : nullptr;
	
	if (Spec)
	{
		return Spec->Level;
	}

	// An ability was probably removed/ungranted then something came in and ask about its level. Caller should have detected this earlier.
	ABILITY_LOG(Warning, TEXT("UGameplayAbility::GetAbilityLevel. Invalid AbilitySpecHandle %s for Ability %s. Returning level 1."), *Handle.ToString(), *GetNameSafe(this));
	return 1;
}

int32 UGameplayAbility::GetAbilityLevel_BP(FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo& ActorInfo) const
{
	return GetAbilityLevel(Handle, &ActorInfo);
}

FGameplayAbilitySpec* UGameplayAbility::GetCurrentAbilitySpec() const
{
	ensureMsgf(IsInstantiated(), TEXT("%hs called on the CDO.  This function uses instance variables and therefore is invalid on the CDO."), __func__);

	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		return AbilitySystemComponent->FindAbilitySpecFromHandle(CurrentSpecHandle);
	}

	return nullptr;
}

FGameplayEffectContextHandle UGameplayAbility::GetGrantedByEffectContext() const
{
	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		FActiveGameplayEffectHandle ActiveHandle = AbilitySystemComponent->FindActiveGameplayEffectHandle(GetCurrentAbilitySpecHandle());
		if (ActiveHandle.IsValid())
		{
			return AbilitySystemComponent->GetEffectContextFromActiveGEHandle(ActiveHandle);
		}
	}

	return FGameplayEffectContextHandle();
}

void UGameplayAbility::RemoveGrantedByEffect()
{
	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		FActiveGameplayEffectHandle ActiveHandle = AbilitySystemComponent->FindActiveGameplayEffectHandle(GetCurrentAbilitySpecHandle());
		if (ActiveHandle.IsValid())
		{
			AbilitySystemComponent->RemoveActiveGameplayEffect(ActiveHandle);
		}
	}
}

UObject* UGameplayAbility::GetSourceObject(FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo) const
{
	if (ActorInfo != nullptr)
	{
		UAbilitySystemComponent* const AbilitySystemComponent = ActorInfo->AbilitySystemComponent.Get();
		if (AbilitySystemComponent != nullptr)
		{
			FGameplayAbilitySpec* AbilitySpec = AbilitySystemComponent->FindAbilitySpecFromHandle(Handle, EConsiderPending::All);
			if (AbilitySpec)
			{
				return AbilitySpec->SourceObject.Get();
			}
		}
	}
	return nullptr;
}

UObject* UGameplayAbility::GetSourceObject_BP(FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo& ActorInfo) const
{
	return GetSourceObject(Handle, &ActorInfo);
}

UObject* UGameplayAbility::GetCurrentSourceObject() const
{
	FGameplayAbilitySpec* AbilitySpec = GetCurrentAbilitySpec();
	if (AbilitySpec)
	{
		return AbilitySpec->SourceObject.Get();
	}
	return nullptr;
}

FGameplayEffectContextHandle UGameplayAbility::MakeEffectContext(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo *ActorInfo) const
{
	FGameplayEffectContextHandle Context = FGameplayEffectContextHandle(UAbilitySystemGlobals::Get().AllocGameplayEffectContext());
	Context.SetAbility(this);

	if (ensure(ActorInfo))
	{
		// By default use the owner and avatar as the instigator and causer
		Context.AddInstigator(ActorInfo->OwnerActor.Get(), ActorInfo->AvatarActor.Get());

		// Pass along the source object to the effect
		if (UAbilitySystemComponent* AbilitySystemComponent = ActorInfo->AbilitySystemComponent.Get())
		{
			if (FGameplayAbilitySpec* AbilitySpec = AbilitySystemComponent->FindAbilitySpecFromHandle(Handle))
			{
				Context.AddSourceObject(AbilitySpec->SourceObject.Get());
			}
		}
	}

	return Context;
}

bool UGameplayAbility::IsTriggered() const
{
	// Assume that if there is triggered data, then we are triggered. 
	// If we need to support abilities that can be both, this will need to be expanded.
	if (AbilityTriggers.IsEmpty())
	{
		const FGameplayAbilitySpec* AbilitySpec = GetCurrentAbilitySpec();
		return AbilitySpec != nullptr && !AbilitySpec->DynamicAbilityTriggers.IsEmpty();
	}

	return true;
}

bool UGameplayAbility::IsPredictingClient() const
{
	const FGameplayAbilityActorInfo* const CurrentActorInfoPtr = GetCurrentActorInfo();
	if (CurrentActorInfoPtr->OwnerActor.IsValid())
	{
		bool bIsLocallyControlled = CurrentActorInfoPtr->IsLocallyControlled();
		bool bIsAuthority = CurrentActorInfoPtr->IsNetAuthority();

		// LocalPredicted and ServerInitiated are both valid because in both those modes the ability also runs on the client
		if (!bIsAuthority && bIsLocallyControlled && (GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalPredicted || GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::ServerInitiated))
		{
			return true;
		}
	}

	return false;
}

bool UGameplayAbility::IsForRemoteClient() const
{
	const FGameplayAbilityActorInfo* const CurrentActorInfoPtr = GetCurrentActorInfo();
	if (CurrentActorInfoPtr->OwnerActor.IsValid())
	{
		bool bIsLocallyControlled = CurrentActorInfoPtr->IsLocallyControlled();
		bool bIsAuthority = CurrentActorInfoPtr->IsNetAuthority();

		if (bIsAuthority && !bIsLocallyControlled)
		{
			return true;
		}
	}

	return false;
}

bool UGameplayAbility::IsLocallyControlled() const
{
	const FGameplayAbilityActorInfo* const CurrentActorInfoPtr = GetCurrentActorInfo();
	if (CurrentActorInfoPtr->OwnerActor.IsValid())
	{
		return CurrentActorInfoPtr->IsLocallyControlled();
	}

	return false;
}

bool UGameplayAbility::HasAuthority(const FGameplayAbilityActivationInfo* ActivationInfo) const
{
	return (ActivationInfo->ActivationMode == EGameplayAbilityActivationMode::Authority);
}

bool UGameplayAbility::K2_HasAuthority() const
{
	return HasAuthority(&CurrentActivationInfo);
}

bool UGameplayAbility::HasAuthorityOrPredictionKey(const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo* ActivationInfo) const
{
	return ActorInfo->AbilitySystemComponent->HasAuthorityOrPredictionKey(ActivationInfo);
}

bool UGameplayAbility::IsInstantiated() const
{
	return !HasAllFlags(RF_ClassDefaultObject);
}

void UGameplayAbility::OnGiveAbility(const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilitySpec& Spec)
{
	SetCurrentActorInfo(Spec.Handle, ActorInfo);

	// If we already have an avatar set, call the OnAvatarSet event as well
	if (ActorInfo && ActorInfo->AvatarActor.IsValid())
	{
		OnAvatarSet(ActorInfo, Spec);
	}
}

void UGameplayAbility::OnAvatarSet(const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilitySpec& Spec)
{
	// Projects may want to initiate passives or do other "BeginPlay" type of logic here.
}

FActiveGameplayEffectHandle UGameplayAbility::BP_ApplyGameplayEffectToOwner(TSubclassOf<UGameplayEffect> GameplayEffectClass, int32 GameplayEffectLevel, int32 Stacks)
{
	ensureMsgf(CurrentActorInfo, TEXT("ability %s called BP_ApplyGameplayEffectToOwner but current actor info is null"), *GetNameSafe(this));
	ensureMsgf(CurrentSpecHandle.IsValid(), TEXT("ability %s called BP_ApplyGameplayEffectToOwner but current spec handle is invalid"), *GetNameSafe(this));

	if ( GameplayEffectClass )
	{
		const UGameplayEffect* GameplayEffect = GameplayEffectClass->GetDefaultObject<UGameplayEffect>();
		return ApplyGameplayEffectToOwner(CurrentSpecHandle, CurrentActorInfo, CurrentActivationInfo, GameplayEffect, GameplayEffectLevel, Stacks);
	}

	ABILITY_LOG(Error, TEXT("BP_ApplyGameplayEffectToOwner called on ability %s with no GameplayEffectClass."), *GetName());
	return FActiveGameplayEffectHandle();
}

FActiveGameplayEffectHandle UGameplayAbility::ApplyGameplayEffectToOwner(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const UGameplayEffect* GameplayEffect, float GameplayEffectLevel, int32 Stacks) const
{
	if (GameplayEffect && (HasAuthorityOrPredictionKey(ActorInfo, &ActivationInfo)))
	{
		FGameplayEffectSpecHandle SpecHandle = MakeOutgoingGameplayEffectSpec(Handle, ActorInfo, ActivationInfo, GameplayEffect->GetClass(), GameplayEffectLevel);
		if (SpecHandle.IsValid())
		{
			SpecHandle.Data->SetStackCount(Stacks);
			return ApplyGameplayEffectSpecToOwner(Handle, ActorInfo, ActivationInfo, SpecHandle);
		}
	}

	// We cannot apply GameplayEffects in this context. Return an empty handle.
	return FActiveGameplayEffectHandle();
}

FActiveGameplayEffectHandle UGameplayAbility::K2_ApplyGameplayEffectSpecToOwner(const FGameplayEffectSpecHandle EffectSpecHandle)
{
	return ApplyGameplayEffectSpecToOwner(CurrentSpecHandle, CurrentActorInfo, CurrentActivationInfo, EffectSpecHandle);
}

FActiveGameplayEffectHandle UGameplayAbility::ApplyGameplayEffectSpecToOwner(const FGameplayAbilitySpecHandle AbilityHandle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayEffectSpecHandle SpecHandle) const
{
	// This batches all created cues together
	FScopedGameplayCueSendContext GameplayCueSendContext;

	if (SpecHandle.IsValid() && ActorInfo && (HasAuthorityOrPredictionKey(ActorInfo, &ActivationInfo)))
	{
		UAbilitySystemComponent* const AbilitySystemComponent = ActorInfo->AbilitySystemComponent.Get();
		FActiveGameplayEffectHandle ReturnHandle = AbilitySystemComponent->ApplyGameplayEffectSpecToSelf(*SpecHandle.Data.Get(), AbilitySystemComponent->GetPredictionKeyForNewAction());

		// Force a net update for predicted effects so that short durations are replicated correctly
		if (ReturnHandle.IsValid() && ActorInfo->OwnerActor.IsValid() && ActorInfo->OwnerActor->HasAuthority())
		{
			AbilitySystemComponent->GetOwner()->ForceNetUpdate();
		}
		return ReturnHandle;
	}
	return FActiveGameplayEffectHandle();
}

TArray<FActiveGameplayEffectHandle> UGameplayAbility::BP_ApplyGameplayEffectToTarget(FGameplayAbilityTargetDataHandle Target, TSubclassOf<UGameplayEffect> GameplayEffectClass, int32 GameplayEffectLevel, int32 Stacks)
{
	return ApplyGameplayEffectToTarget(CurrentSpecHandle, CurrentActorInfo, CurrentActivationInfo, Target, GameplayEffectClass, GameplayEffectLevel, Stacks);
}

TArray<FActiveGameplayEffectHandle> UGameplayAbility::ApplyGameplayEffectToTarget(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayAbilityTargetDataHandle& Target, TSubclassOf<UGameplayEffect> GameplayEffectClass, float GameplayEffectLevel, int32 Stacks) const
{
	SCOPE_CYCLE_COUNTER(STAT_ApplyGameplayEffectToTarget);
	SCOPE_CYCLE_UOBJECT(This, this);
	SCOPE_CYCLE_UOBJECT(Effect, GameplayEffectClass);

	TArray<FActiveGameplayEffectHandle> EffectHandles;

	if (HasAuthority(&ActivationInfo) == false && UAbilitySystemGlobals::Get().ShouldPredictTargetGameplayEffects() == false)
	{
		// Early out to avoid making effect specs that we can't apply
		return EffectHandles;
	}

	// This batches all created cues together
	FScopedGameplayCueSendContext GameplayCueSendContext;

	if (GameplayEffectClass == nullptr)
	{
		ABILITY_LOG(Error, TEXT("ApplyGameplayEffectToTarget called on ability %s with no GameplayEffect."), *GetName());
	}
	else if (HasAuthorityOrPredictionKey(ActorInfo, &ActivationInfo))
	{
		FGameplayEffectSpecHandle SpecHandle = MakeOutgoingGameplayEffectSpec(Handle, ActorInfo, ActivationInfo, GameplayEffectClass, GameplayEffectLevel);
		if (SpecHandle.Data.IsValid())
		{
			SpecHandle.Data->SetStackCount(Stacks);

			SCOPE_CYCLE_UOBJECT(Source, SpecHandle.Data->GetContext().GetSourceObject());
			EffectHandles.Append(ApplyGameplayEffectSpecToTarget(Handle, ActorInfo, ActivationInfo, SpecHandle, Target));
		}
		else
		{
			ABILITY_LOG(Warning, TEXT("UGameplayAbility::ApplyGameplayEffectToTarget failed to create valid spec handle. Ability: %s"), *GetPathName());
		}
	}

	return EffectHandles;
}

TArray<FActiveGameplayEffectHandle> UGameplayAbility::K2_ApplyGameplayEffectSpecToTarget(const FGameplayEffectSpecHandle SpecHandle, FGameplayAbilityTargetDataHandle TargetData)
{
	return ApplyGameplayEffectSpecToTarget(CurrentSpecHandle, CurrentActorInfo, CurrentActivationInfo, SpecHandle, TargetData);
}

TArray<FActiveGameplayEffectHandle> UGameplayAbility::ApplyGameplayEffectSpecToTarget(const FGameplayAbilitySpecHandle AbilityHandle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayEffectSpecHandle SpecHandle, const FGameplayAbilityTargetDataHandle& TargetData) const
{
	TArray<FActiveGameplayEffectHandle> EffectHandles;
	
	if (SpecHandle.IsValid() && HasAuthorityOrPredictionKey(ActorInfo, &ActivationInfo))
	{
		TARGETLIST_SCOPE_LOCK(*ActorInfo->AbilitySystemComponent);
		for (TSharedPtr<FGameplayAbilityTargetData> Data : TargetData.Data)
		{
			if (Data.IsValid())
			{
				EffectHandles.Append(Data->ApplyGameplayEffectSpec(*SpecHandle.Data.Get(), ActorInfo->AbilitySystemComponent->GetPredictionKeyForNewAction()));
			}
			else
			{
				ABILITY_LOG(Warning, TEXT("UGameplayAbility::ApplyGameplayEffectSpecToTarget invalid target data passed in. Ability: %s"), *GetPathName());
			}
		}
	}
	return EffectHandles;
}

void UGameplayAbility::SetCurrentActorInfo(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo) const
{
	if (IsInstantiated())
	{
		CurrentActorInfo = ActorInfo;
		CurrentSpecHandle = Handle;
	}
}

void UGameplayAbility::SetCurrentActivationInfo(const FGameplayAbilityActivationInfo ActivationInfo)
{
	if (IsInstantiated())
	{
		CurrentActivationInfo = ActivationInfo;
	}
}

void UGameplayAbility::SetCurrentInfo(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo)
{
	SetCurrentActorInfo(Handle, ActorInfo);
	SetCurrentActivationInfo(ActivationInfo);
}

void UGameplayAbility::IncrementListLock() const
{
	++ScopeLockCount;
}
void UGameplayAbility::DecrementListLock() const
{
	if (--ScopeLockCount == 0)
	{
		// execute delayed functions in the order they came in
		// These may end or cancel this ability
		for (int32 Idx = 0; Idx < WaitingToExecute.Num(); ++Idx)
		{
			WaitingToExecute[Idx].ExecuteIfBound();
		}

		WaitingToExecute.Empty();
	}
}

void UGameplayAbility::BP_RemoveGameplayEffectFromOwnerWithAssetTags(FGameplayTagContainer WithTags, int32 StacksToRemove)
{
	if (HasAuthority(&CurrentActivationInfo) == false)
	{
		return;
	}

	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		FGameplayEffectQuery const Query = FGameplayEffectQuery::MakeQuery_MatchAnyEffectTags(WithTags);
		AbilitySystemComponent->RemoveActiveEffects(Query, StacksToRemove);
	}
}

void UGameplayAbility::BP_RemoveGameplayEffectFromOwnerWithGrantedTags(FGameplayTagContainer WithGrantedTags, int32 StacksToRemove)
{
	if (HasAuthority(&CurrentActivationInfo) == false)
	{
		return;
	}

	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		FGameplayEffectQuery const Query = FGameplayEffectQuery::MakeQuery_MatchAnyOwningTags(WithGrantedTags);
		AbilitySystemComponent->RemoveActiveEffects(Query, StacksToRemove);
	}
}

void UGameplayAbility::BP_RemoveGameplayEffectFromOwnerWithHandle(FActiveGameplayEffectHandle Handle, int32 StacksToRemove)
{
	if (HasAuthority(&CurrentActivationInfo) == false)
	{
		return;
	}

	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		AbilitySystemComponent->RemoveActiveGameplayEffect(Handle, StacksToRemove);
	}
}

float UGameplayAbility::GetCooldownTimeRemaining() const
{
	return IsInstantiated() ? GetCooldownTimeRemaining(CurrentActorInfo) : 0.f;
}

void UGameplayAbility::SetRemoteInstanceHasEnded()
{
	// This could potentially happen in shutdown corner cases
	if (!IsValidChecked(this) || CurrentActorInfo == nullptr)
	{
		return;
	}

	UAbilitySystemComponent* const AbilitySystemComponent = CurrentActorInfo->AbilitySystemComponent.Get();
	if (AbilitySystemComponent == nullptr)
	{
		return;
	}

	RemoteInstanceEnded = true;
	for (UGameplayTask* Task : ActiveTasks)
	{
		if (IsValid(Task) && Task->IsWaitingOnRemotePlayerdata())
		{
			// We have a task that is waiting for player input, but the remote player has ended the ability, so it will not send the input.
			// Kill the ability to avoid getting stuck active.
			
			ABILITY_LOG(Log, TEXT("Ability %s is force cancelling because Task %s is waiting on remote player input and the  remote player has just ended the ability."), *GetName(), *Task->GetDebugString());
			AbilitySystemComponent->ForceCancelAbilityDueToReplication(this);
			break;
		}
	}
}

void UGameplayAbility::NotifyAvatarDestroyed()
{
	// This could potentially happen in shutdown corner cases
	if (!IsValidChecked(this) || CurrentActorInfo == nullptr)
	{
		return;
	}

	UAbilitySystemComponent* const AbilitySystemComponent = CurrentActorInfo->AbilitySystemComponent.Get();
	if (AbilitySystemComponent == nullptr)
	{
		return;
	}

	RemoteInstanceEnded = true;
	for (UGameplayTask* Task : ActiveTasks)
	{
		if (IsValid(Task) && Task->IsWaitingOnAvatar())
		{
			// We have a task waiting on some Avatar state but the avatar is destroyed, so force end the ability to avoid getting stuck on.
			
			ABILITY_LOG(Log, TEXT("Ability %s is force cancelling because Task %s is waiting on avatar data avatar has been destroyed."), *GetName(), *Task->GetDebugString());
			AbilitySystemComponent->ForceCancelAbilityDueToReplication(this);
			break;
		}
	}
}

void UGameplayAbility::NotifyAbilityTaskWaitingOnPlayerData(class UAbilityTask* AbilityTask)
{
	// This should only be called from actively running ability tasks
	if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
	{
		if (RemoteInstanceEnded)
		{
			ABILITY_LOG(Log, TEXT("Ability %s is force cancelling because Task %s has started after the remote player has ended the ability."), *GetName(), *AbilityTask->GetDebugString());
			AbilitySystemComponent->ForceCancelAbilityDueToReplication(this);
		}
	}
}

void UGameplayAbility::NotifyAbilityTaskWaitingOnAvatar(class UAbilityTask* AbilityTask)
{
	if (CurrentActorInfo && CurrentActorInfo->AvatarActor.IsValid() == false)
	{
		ABILITY_LOG(Log, TEXT("Ability %s is force cancelling because Task %s has started while there is no valid AvatarActor"), *GetName(), *AbilityTask->GetDebugString());

		if (UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponentFromActorInfo_Ensured())
		{
			AbilitySystemComponent->ForceCancelAbilityDueToReplication(this);
		}
	}
}

void UGameplayAbility::GetLifetimeReplicatedProps(TArray< class FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	if (GetReplicationPolicy() != EGameplayAbilityReplicationPolicy::ReplicateNo)
	{
		if (UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(GetClass()))
		{
			BPClass->GetLifetimeBlueprintReplicationList(OutLifetimeProps);
		}
	}
}

void UGameplayAbility::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	// Build descriptors and allocate PropertyReplicationFragments for this object
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UGameplayAbility::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsMigratingRemoteObjects())
	{
		Ar << bHasBlueprintShouldAbilityRespondToEvent;
		Ar << bHasBlueprintCanUse;
		Ar << bHasBlueprintActivate;
		Ar << bHasBlueprintActivateFromEvent;
		Ar << CurrentSpecHandle;
		Ar << TrackedGameplayCues;
	}
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
void UGameplayAbility::PostMigrate(const FUObjectMigrationContext& MigrationContext)
{
	Super::PostMigrate(MigrationContext);

	if (MigrationContext.MigrationSide == EObjectMigrationSide::Receive)
	{
		// Setup our CurrentActorInfo
		PostNetInit();
	}
}
#endif

#undef LOCTEXT_NAMESPACE