// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayAbilitySet.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilitySet)


// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	UGameplayAbilitySet
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------


UGameplayAbilitySet::UGameplayAbilitySet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

void UGameplayAbilitySet::GiveAbilities(UAbilitySystemComponent* AbilitySystemComponent) const
{
	for (const FGameplayAbilityBindInfo& BindInfo : Abilities)
	{
		if (BindInfo.GameplayAbilityClass)
		{
			AbilitySystemComponent->GiveAbility(FGameplayAbilitySpec(BindInfo.GameplayAbilityClass, 1, (int32)BindInfo.Command));
		}
	}
}

