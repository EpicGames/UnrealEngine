// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbilityTargetActor.h"
#include "GameplayAbilitySpec.h"
#include "GameFramework/PlayerController.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityTargetActor)

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	AGameplayAbilityTargetActor
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

AGameplayAbilityTargetActor::AGameplayAbilityTargetActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ShouldProduceTargetDataOnServer = false;
	bDebug = false;
	bDestroyOnConfirmation = true;
}

void AGameplayAbilityTargetActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (GenericDelegateBoundASC)
	{
		// We must remove ourselves from GenericLocalConfirmCallbacks/GenericLocalCancelCallbacks, since while these are bound they will inhibit any *other* abilities
		// that are bound to the same key.

		UAbilitySystemComponent* UnboundASC = nullptr;
		const FGameplayAbilityActorInfo* Info = (OwningAbility ? OwningAbility->GetCurrentActorInfo() : nullptr);
		if (Info && Info->IsLocallyControlled())
		{
			UAbilitySystemComponent* ASC = Info->AbilitySystemComponent.Get();
			if (ASC)
			{
				ASC->GenericLocalConfirmCallbacks.RemoveDynamic(this, &AGameplayAbilityTargetActor::ConfirmTargeting);
				ASC->GenericLocalCancelCallbacks.RemoveDynamic(this, &AGameplayAbilityTargetActor::CancelTargeting);

				UnboundASC = ASC;
			}
		}

		ensure(GenericDelegateBoundASC == UnboundASC); // Error checking that we have removed delegates from the same ASC we bound them to
	}

	Super::EndPlay(EndPlayReason);
}

void AGameplayAbilityTargetActor::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
	DOREPLIFETIME(AGameplayAbilityTargetActor, StartLocation);
	DOREPLIFETIME(AGameplayAbilityTargetActor, SourceActor);
	DOREPLIFETIME(AGameplayAbilityTargetActor, bDebug);
	DOREPLIFETIME(AGameplayAbilityTargetActor, bDestroyOnConfirmation);

	DISABLE_REPLICATED_PROPERTY(AGameplayAbilityTargetActor, Filter);
}

void AGameplayAbilityTargetActor::StartTargeting(UGameplayAbility* Ability)
{
	OwningAbility = Ability;
}

bool AGameplayAbilityTargetActor::IsConfirmTargetingAllowed()
{
	return true;
}

void AGameplayAbilityTargetActor::ConfirmTargetingAndContinue()
{
	check(ShouldProduceTargetData());
	if (IsConfirmTargetingAllowed())
	{
		TargetDataReadyDelegate.Broadcast(FGameplayAbilityTargetDataHandle());
	}
}

void AGameplayAbilityTargetActor::ConfirmTargeting()
{
	const FGameplayAbilityActorInfo* ActorInfo = (OwningAbility ? OwningAbility->GetCurrentActorInfo() : nullptr);
	UAbilitySystemComponent* ASC = (ActorInfo ? ActorInfo->AbilitySystemComponent.Get() : nullptr);
	if (ASC)
	{
		ASC->AbilityReplicatedEventDelegate(EAbilityGenericReplicatedEvent::GenericConfirm, OwningAbility->GetCurrentAbilitySpecHandle(), OwningAbility->GetCurrentActivationInfo().GetActivationPredictionKey() ).Remove(GenericConfirmHandle);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("AGameplayAbilityTargetActor::ConfirmTargeting called with null Ability/ASC! Actor %s"), *GetName());
	}

	if (IsConfirmTargetingAllowed())
	{
		ConfirmTargetingAndContinue();
		if (bDestroyOnConfirmation)
		{
			Destroy();
		}
	}
}

/** Outside code is saying 'stop everything and just forget about it' */
void AGameplayAbilityTargetActor::CancelTargeting()
{
	const FGameplayAbilityActorInfo* ActorInfo = (OwningAbility ? OwningAbility->GetCurrentActorInfo() : nullptr);
	UAbilitySystemComponent* ASC = (ActorInfo ? ActorInfo->AbilitySystemComponent.Get() : nullptr);
	if (ASC)
	{
		ASC->AbilityReplicatedEventDelegate(EAbilityGenericReplicatedEvent::GenericCancel, OwningAbility->GetCurrentAbilitySpecHandle(), OwningAbility->GetCurrentActivationInfo().GetActivationPredictionKey() ).Remove(GenericCancelHandle);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("AGameplayAbilityTargetActor::CancelTargeting called with null ASC! Actor %s"), *GetName());
	}

	CanceledDelegate.Broadcast(FGameplayAbilityTargetDataHandle());
	Destroy();
}

bool AGameplayAbilityTargetActor::IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const
{
	//The player who created the ability doesn't need to be updated about it - there should be local prediction in place.
	if (RealViewer == PrimaryPC)
	{
		return false;
	}

	const FGameplayAbilityActorInfo* ActorInfo = (OwningAbility ? OwningAbility->GetCurrentActorInfo() : NULL);
	AActor* Avatar = (ActorInfo ? ActorInfo->AvatarActor.Get() : NULL);

	if (Avatar)
	{
		return Avatar->IsNetRelevantFor(RealViewer, ViewTarget, SrcLocation);
	}

	return Super::IsNetRelevantFor(RealViewer, ViewTarget, SrcLocation);
}

bool AGameplayAbilityTargetActor::OnReplicatedTargetDataReceived(FGameplayAbilityTargetDataHandle& Data) const
{
	return true;
}

bool AGameplayAbilityTargetActor::ShouldProduceTargetData() const
{
	// return true if we are locally owned, or (we are the server and this is a gameplaytarget ability that can produce target data server side)
	return (PrimaryPC && PrimaryPC->IsLocalController()) || ShouldProduceTargetDataOnServer;
}

void AGameplayAbilityTargetActor::BindToConfirmCancelInputs()
{
	check(OwningAbility);

	const FGameplayAbilityActorInfo* const Info = OwningAbility->GetCurrentActorInfo();
	UAbilitySystemComponent* const ASC = Info->AbilitySystemComponent.Get();
	if (ASC)
	{
		if (Info->IsLocallyControlled())
		{
			// We have to wait for the callback from the AbilitySystemComponent. Which will always be instigated locally
			ASC->GenericLocalConfirmCallbacks.AddDynamic(this, &AGameplayAbilityTargetActor::ConfirmTargeting);	// Tell me if the confirm input is pressed
			ASC->GenericLocalCancelCallbacks.AddDynamic(this, &AGameplayAbilityTargetActor::CancelTargeting);	// Tell me if the cancel input is pressed

			// Save off which ASC we bound so that we can error check that we're removing them later
			GenericDelegateBoundASC = ASC;
		}
		else
		{	
			FGameplayAbilitySpecHandle Handle = OwningAbility->GetCurrentAbilitySpecHandle();
			FPredictionKey PredKey = OwningAbility->GetCurrentActivationInfo().GetActivationPredictionKey();

			GenericConfirmHandle = ASC->AbilityReplicatedEventDelegate(EAbilityGenericReplicatedEvent::GenericConfirm, Handle, PredKey ).AddUObject(this, &AGameplayAbilityTargetActor::ConfirmTargeting);
			GenericCancelHandle = ASC->AbilityReplicatedEventDelegate(EAbilityGenericReplicatedEvent::GenericCancel, Handle, PredKey ).AddUObject(this, &AGameplayAbilityTargetActor::CancelTargeting);
			
			if (ASC->CallReplicatedEventDelegateIfSet(EAbilityGenericReplicatedEvent::GenericConfirm, Handle, PredKey))
			{
				return;
			}
			
			if (ASC->CallReplicatedEventDelegateIfSet(EAbilityGenericReplicatedEvent::GenericCancel, Handle, PredKey))
			{
				return;
			}
		}
	}
}

