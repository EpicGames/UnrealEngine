// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbilityTargetActor_ActorPlacement.h"
#include "Engine/World.h"
#include "Abilities/GameplayAbilityWorldReticle_ActorVisualization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityTargetActor_ActorPlacement)

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	AGameplayAbilityTargetActor_ActorPlacement
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

AGameplayAbilityTargetActor_ActorPlacement::AGameplayAbilityTargetActor_ActorPlacement(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void AGameplayAbilityTargetActor_ActorPlacement::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (ActorVisualizationReticle.IsValid())
	{
		ActorVisualizationReticle.Get()->Destroy();
	}

	Super::EndPlay(EndPlayReason);
}

void AGameplayAbilityTargetActor_ActorPlacement::StartTargeting(UGameplayAbility* InAbility)
{
	Super::StartTargeting(InAbility);
	if (AActor *VisualizationActor = GetWorld()->SpawnActor(PlacedActorClass))
	{
		ActorVisualizationReticle = GetWorld()->SpawnActor<AGameplayAbilityWorldReticle_ActorVisualization>();
		ActorVisualizationReticle->InitializeReticleVisualizationInformation(this, VisualizationActor, PlacedActorMaterial);
		GetWorld()->DestroyActor(VisualizationActor);
	}
	if (AGameplayAbilityWorldReticle* CachedReticleActor = ReticleActor.Get())
	{
		ActorVisualizationReticle->AttachToActor(CachedReticleActor, FAttachmentTransformRules::KeepRelativeTransform);
	}
	else
	{
		ReticleActor = ActorVisualizationReticle;
		ActorVisualizationReticle = nullptr;
	}
}

//Might want to override this function to allow for a radius check against the ground, possibly including a height check. Or might want to do it in ground trace.
//FHitResult AGameplayAbilityTargetActor_ActorPlacement::PerformTrace(AActor* InSourceActor) const

