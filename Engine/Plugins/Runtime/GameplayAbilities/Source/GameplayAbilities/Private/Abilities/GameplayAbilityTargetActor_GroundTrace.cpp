// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbilityTargetActor_GroundTrace.h"
#include "GameFramework/Pawn.h"
#include "DrawDebugHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityTargetActor_GroundTrace)

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	AGameplayAbilityTargetActor_GroundTrace
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

AGameplayAbilityTargetActor_GroundTrace::AGameplayAbilityTargetActor_GroundTrace(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	CollisionHeightOffset = 0.0f;
}

void AGameplayAbilityTargetActor_GroundTrace::StartTargeting(UGameplayAbility* InAbility)
{
	//CollisionShape starts as "line," which is correct as a default for our purposes
	if (CollisionRadius > 0.0f)
	{
		//Check CollisionHeight here because the shape code doesn't. It is used as half-height
		if ((CollisionHeight * 0.5f) > CollisionRadius)
		{
			CollisionShape = FCollisionShape::MakeCapsule(CollisionRadius, CollisionHeight * 0.5f);
			CollisionHeightOffset = CollisionHeight * 0.5f;
		}
		else
		{
			CollisionShape = FCollisionShape::MakeSphere(CollisionRadius);
			CollisionHeight = 0.0f;
			CollisionHeightOffset = CollisionRadius;
		}
	}
	else
	{
		//Make sure these are clean.
		CollisionRadius = CollisionHeight = 0.0f;
	}
	Super::StartTargeting(InAbility);
}

bool AGameplayAbilityTargetActor_GroundTrace::AdjustCollisionResultForShape(const FVector OriginalStartPoint, const FVector OriginalEndPoint, const FCollisionQueryParams Params, FHitResult& OutHitResult) const
{
	UWorld *ThisWorld = GetWorld();
	//Pull back toward player to find a better spot, accounting for the width of our object
	FVector Movement = (OriginalEndPoint - OriginalStartPoint);	
	FVector MovementDirection = Movement.GetSafeNormal();
	float MovementMagnitude2D = Movement.Size2D();

#if ENABLE_DRAW_DEBUG
	if (bDebug)
	{
		if (CollisionShape.ShapeType == ECollisionShape::Capsule)
		{
			DrawDebugCapsule(ThisWorld, OriginalEndPoint, CollisionHeight * 0.5f, CollisionRadius, FQuat::Identity, FColor::Black);
		}
		else
		{
			DrawDebugSphere(ThisWorld, OriginalEndPoint, CollisionRadius, 8, FColor::Black);
		}
	}
#endif // ENABLE_DRAW_DEBUG

	if (MovementMagnitude2D <= (CollisionRadius * 2.0f))
	{
		return false;		//Bad case!
	}

	//TODO This increment value needs to ramp up - the first few increments should be small, then we should start moving in larger steps. A few ideas for this:
	//1. Use a curve! Even one defined by a hardcoded formula would be fine, this isn't something that should require user tuning, or that the user should really know/care about.
	//2. Use larger increments as the object is further from the player/camera, since the user can't really perceive precision at long range.
	float IncrementSize = FMath::Clamp<float>(CollisionRadius * 0.5f, 20.0f, 50.0f);
	float LerpIncrement = IncrementSize / MovementMagnitude2D;
	FHitResult LocalResult;
	FVector TraceStart;
	FVector TraceEnd;
	for (float LerpValue = CollisionRadius / MovementMagnitude2D; LerpValue < 1.0f; LerpValue += LerpIncrement)
	{
		TraceEnd = TraceStart = OriginalEndPoint - (LerpValue * Movement);
		TraceEnd.Z -= 99999.0f;
		SweepWithFilter(LocalResult, ThisWorld, Filter, TraceStart, TraceEnd, FQuat::Identity, CollisionShape, TraceProfile.Name, Params);
		if (!LocalResult.bStartPenetrating)
		{
			if (!LocalResult.bBlockingHit || (LocalResult.HitObjectHandle.IsValid() && LocalResult.HitObjectHandle.DoesRepresentClass(APawn::StaticClass())))
			{
				//Off the map, or hit an actor
#if ENABLE_DRAW_DEBUG
				if (bDebug)
				{
					if (CollisionShape.ShapeType == ECollisionShape::Capsule)
					{
						DrawDebugCapsule(ThisWorld, LocalResult.Location, CollisionHeight * 0.5f, CollisionRadius, FQuat::Identity, FColor::Yellow);
					}
					else
					{
						DrawDebugSphere(ThisWorld, LocalResult.Location, CollisionRadius, 8, FColor::Yellow);
					}
				}
#endif // ENABLE_DRAW_DEBUG
				continue;
			}
#if ENABLE_DRAW_DEBUG
			if (bDebug)
			{
				if (CollisionShape.ShapeType == ECollisionShape::Capsule)
				{
					DrawDebugCapsule(ThisWorld, LocalResult.Location, CollisionHeight * 0.5f, CollisionRadius, FQuat::Identity, FColor::Green);
				}
				else
				{
					DrawDebugSphere(ThisWorld, LocalResult.Location, CollisionRadius, 8, FColor::Green);
				}
			}
#endif // ENABLE_DRAW_DEBUG

			//TODO: Test for flat ground. Concept: Test four corners and the center, make triangles out of the center and adjacent corner points. Check normal.Z of triangles against a minimum Z value.

			OutHitResult = LocalResult;
			return true;
		}
#if ENABLE_DRAW_DEBUG
		if (bDebug)
		{
			if (CollisionShape.ShapeType == ECollisionShape::Capsule)
			{
				DrawDebugCapsule(ThisWorld, TraceStart, CollisionHeight * 0.5f, CollisionRadius, FQuat::Identity, FColor::Red);
			}
			else
			{
				DrawDebugSphere(ThisWorld, TraceStart, CollisionRadius, 8, FColor::Red);
			}
		}
#endif // ENABLE_DRAW_DEBUG
	}
	return false;
}

FHitResult AGameplayAbilityTargetActor_GroundTrace::PerformTrace(AActor* InSourceActor)
{
	bool bTraceComplex = false;

	FCollisionQueryParams Params(SCENE_QUERY_STAT(AGameplayAbilityTargetActor_GroundTrace), bTraceComplex);
	Params.bReturnPhysicalMaterial = true;
	Params.AddIgnoredActor(InSourceActor);

	FVector TraceStart = StartLocation.GetTargetingTransform().GetLocation();// InSourceActor->GetActorLocation();
	FVector TraceEnd;
	AimWithPlayerController(InSourceActor, Params, TraceStart, TraceEnd);		//Effective on server and launching client only

	// ------------------------------------------------------

	FHitResult ReturnHitResult;
	//Use a line trace initially to see where the player is actually pointing
	LineTraceWithFilter(ReturnHitResult, InSourceActor->GetWorld(), Filter, TraceStart, TraceEnd, TraceProfile.Name, Params);
	//Default to end of trace line if we don't hit anything.
	if (!ReturnHitResult.bBlockingHit)
	{
		ReturnHitResult.Location = TraceEnd;
	}

	//Second trace, straight down. Consider using InSourceActor->GetWorld()->NavigationSystem->ProjectPointToNavigation() instead of just going straight down in the case of movement abilities (flag/bool).
	TraceStart = ReturnHitResult.Location - (TraceEnd - TraceStart).GetSafeNormal();		//Pull back very slightly to avoid scraping down walls
	TraceEnd = TraceStart;
	TraceStart.Z += CollisionHeightOffset;
	TraceEnd.Z -= 99999.0f;
	LineTraceWithFilter(ReturnHitResult, InSourceActor->GetWorld(), Filter, TraceStart, TraceEnd, TraceProfile.Name, Params);
	//if (!ReturnHitResult.bBlockingHit) then our endpoint may be off the map. Hopefully this is only possible in debug maps.

	bLastTraceWasGood = true;		//So far, we're good. If we need a ground spot and can't find one, we'll come back.

	//Use collision shape to find a valid ground spot, if appropriate
	if (CollisionShape.ShapeType != ECollisionShape::Line)
	{
		ReturnHitResult.Location.Z += CollisionHeightOffset;		//Rise up out of the ground
		TraceStart = InSourceActor->GetActorLocation();
		TraceEnd = ReturnHitResult.Location;
		TraceStart.Z += CollisionHeightOffset;
		bLastTraceWasGood = AdjustCollisionResultForShape(TraceStart, TraceEnd, Params, ReturnHitResult);
		if (bLastTraceWasGood)
		{
			ReturnHitResult.Location.Z -= CollisionHeightOffset;	//Undo the artificial height adjustment
		}
	}

	if (AGameplayAbilityWorldReticle* LocalReticleActor = ReticleActor.Get())
	{
		LocalReticleActor->SetIsTargetValid(bLastTraceWasGood);
		LocalReticleActor->SetActorLocation(ReturnHitResult.Location);
	}

	// Reset the trace start so the target data uses the correct origin
	ReturnHitResult.TraceStart = StartLocation.GetTargetingTransform().GetLocation();

	return ReturnHitResult;
}

bool AGameplayAbilityTargetActor_GroundTrace::IsConfirmTargetingAllowed()
{
	return bLastTraceWasGood;
}

