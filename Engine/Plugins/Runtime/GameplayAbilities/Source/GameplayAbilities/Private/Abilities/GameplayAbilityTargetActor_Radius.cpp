// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbilityTargetActor_Radius.h"
#include "GameFramework/Pawn.h"
#include "WorldCollision.h"
#include "Abilities/GameplayAbility.h"
#include "AbilitySystemLog.h"
#include "DrawDebugHelpers.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityTargetActor_Radius)

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	AGameplayAbilityTargetActor_Radius
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

AGameplayAbilityTargetActor_Radius::AGameplayAbilityTargetActor_Radius(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.TickGroup = TG_PrePhysics;
	ShouldProduceTargetDataOnServer = true;
}

void AGameplayAbilityTargetActor_Radius::StartTargeting(UGameplayAbility* InAbility)
{
	Super::StartTargeting(InAbility);
	SourceActor = InAbility->GetCurrentActorInfo()->AvatarActor.Get();
}

void AGameplayAbilityTargetActor_Radius::ConfirmTargetingAndContinue()
{
	check(ShouldProduceTargetData());
	if (SourceActor)
	{
		FVector Origin = StartLocation.GetTargetingTransform().GetLocation();
		FGameplayAbilityTargetDataHandle Handle = MakeTargetData(PerformOverlap(Origin), Origin);
		TargetDataReadyDelegate.Broadcast(Handle);
	}
}

FGameplayAbilityTargetDataHandle AGameplayAbilityTargetActor_Radius::MakeTargetData(const TArray<TWeakObjectPtr<AActor>>& Actors, const FVector& Origin) const
{
	if (OwningAbility)
	{
		/** Use the source location instead of the literal origin */
		return StartLocation.MakeTargetDataHandleFromActors(Actors, false);
	}

	return FGameplayAbilityTargetDataHandle();
}

TArray<TWeakObjectPtr<AActor> >	AGameplayAbilityTargetActor_Radius::PerformOverlap(const FVector& Origin)
{
	bool bTraceComplex = false;
	
	FCollisionQueryParams Params(SCENE_QUERY_STAT(RadiusTargetingOverlap), bTraceComplex);
	Params.bReturnPhysicalMaterial = false;

	TArray<FOverlapResult> Overlaps;

	SourceActor->GetWorld()->OverlapMultiByObjectType(Overlaps, Origin, FQuat::Identity, FCollisionObjectQueryParams(ECC_Pawn), FCollisionShape::MakeSphere(Radius), Params);

	TArray<TWeakObjectPtr<AActor>>	HitActors;

	for (int32 i = 0; i < Overlaps.Num(); ++i)
	{
		//Should this check to see if these pawns are in the AimTarget list?
		APawn* PawnActor = Overlaps[i].OverlapObjectHandle.FetchActor<APawn>();
		if (PawnActor && !HitActors.Contains(PawnActor) && Filter.FilterPassesForActor(PawnActor))
		{
			HitActors.Add(PawnActor);
		}
	}

	if (bDebug)
	{
		const FColor& DebugColor = HitActors.Num() == 0 ? FColor::Red : FColor::Green;
		UE_VLOG_SPHERE(SourceActor, LogAbilitySystem, Log, Origin, Radius, DebugColor, TEXT("TargetActor Radius"));

#if UE_ENABLE_DEBUG_DRAWING
		constexpr bool bPersistent = false;
		constexpr float LifeTime = 2.0f;
		DrawDebugSphere(GetWorld(), Origin, Radius, 16, DebugColor, bPersistent, LifeTime);
#endif // UE_ENABLE_DEBUG_DRAWING
	}

	return HitActors;
}

