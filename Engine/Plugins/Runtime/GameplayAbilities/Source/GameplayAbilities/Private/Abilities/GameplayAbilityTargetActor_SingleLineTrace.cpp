// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbilityTargetActor_SingleLineTrace.h"
#include "DrawDebugHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityTargetActor_SingleLineTrace)

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	AGameplayAbilityTargetActor_SingleLineTrace
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

AGameplayAbilityTargetActor_SingleLineTrace::AGameplayAbilityTargetActor_SingleLineTrace(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FHitResult AGameplayAbilityTargetActor_SingleLineTrace::PerformTrace(AActor* InSourceActor)
{
	bool bTraceComplex = false;
	TArray<AActor*> ActorsToIgnore;

	ActorsToIgnore.Add(InSourceActor);

	FCollisionQueryParams Params(SCENE_QUERY_STAT(AGameplayAbilityTargetActor_SingleLineTrace), bTraceComplex);
	Params.bReturnPhysicalMaterial = true;
	Params.AddIgnoredActors(ActorsToIgnore);

	FVector TraceStart = StartLocation.GetTargetingTransform().GetLocation();// InSourceActor->GetActorLocation();
	FVector TraceEnd;
	AimWithPlayerController(InSourceActor, Params, TraceStart, TraceEnd);		//Effective on server and launching client only

	// ------------------------------------------------------

	FHitResult ReturnHitResult;
	LineTraceWithFilter(ReturnHitResult, InSourceActor->GetWorld(), Filter, TraceStart, TraceEnd, TraceProfile.Name, Params);
	//Default to end of trace line if we don't hit anything.
	if (!ReturnHitResult.bBlockingHit)
	{
		ReturnHitResult.Location = TraceEnd;
	}
	if (AGameplayAbilityWorldReticle* LocalReticleActor = ReticleActor.Get())
	{
		const bool bHitActor = (ReturnHitResult.bBlockingHit && (ReturnHitResult.HitObjectHandle.IsValid()));
		const FVector ReticleLocation = (bHitActor && LocalReticleActor->bSnapToTargetedActor) 
			? ReturnHitResult.HitObjectHandle.GetLocation()
			: ReturnHitResult.Location;

		LocalReticleActor->SetActorLocation(ReticleLocation);
		LocalReticleActor->SetIsTargetAnActor(bHitActor);
	}

#if ENABLE_DRAW_DEBUG
	if (bDebug)
	{
		DrawDebugLine(GetWorld(), TraceStart, TraceEnd, FColor::Green);
		DrawDebugSphere(GetWorld(), TraceEnd, 100.0f, 16, FColor::Green);
	}
#endif // ENABLE_DRAW_DEBUG
	return ReturnHitResult;
}

