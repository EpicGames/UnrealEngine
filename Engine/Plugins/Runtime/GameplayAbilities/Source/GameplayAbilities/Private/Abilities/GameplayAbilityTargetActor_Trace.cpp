// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbilityTargetActor_Trace.h"
#include "DrawDebugHelpers.h"
#include "GameFramework/PlayerController.h"
#include "Abilities/GameplayAbility.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityTargetActor_Trace)

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	AGameplayAbilityTargetActor_Trace
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

AGameplayAbilityTargetActor_Trace::AGameplayAbilityTargetActor_Trace(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.TickGroup = TG_PostUpdateWork;

	MaxRange = 999999.0f;
	bTraceAffectsAimPitch = true;
}

void AGameplayAbilityTargetActor_Trace::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (ReticleActor.IsValid())
	{
		ReticleActor.Get()->Destroy();
	}

	Super::EndPlay(EndPlayReason);
}

void AGameplayAbilityTargetActor_Trace::LineTraceWithFilter(FHitResult& OutHitResult, const UWorld* World, const FGameplayTargetDataFilterHandle FilterHandle, const FVector& Start, const FVector& End, FName ProfileName, const FCollisionQueryParams Params)
{
	check(World);

	TArray<FHitResult> HitResults;
	World->LineTraceMultiByProfile(HitResults, Start, End, ProfileName, Params);

	OutHitResult.TraceStart = Start;
	OutHitResult.TraceEnd = End;

	for (int32 HitIdx = 0; HitIdx < HitResults.Num(); ++HitIdx)
	{
		const FHitResult& Hit = HitResults[HitIdx];

		if (!Hit.HitObjectHandle.IsValid() || FilterHandle.FilterPassesForActor(Hit.HitObjectHandle.FetchActor()))
		{
			OutHitResult = Hit;
			OutHitResult.bBlockingHit = true; // treat it as a blocking hit
			return;
		}
	}
}

void AGameplayAbilityTargetActor_Trace::SweepWithFilter(FHitResult& OutHitResult, const UWorld* World, const FGameplayTargetDataFilterHandle FilterHandle, const FVector& Start, const FVector& End, const FQuat& Rotation, const FCollisionShape CollisionShape, FName ProfileName, const FCollisionQueryParams Params)
{
	check(World);

	TArray<FHitResult> HitResults;
	World->SweepMultiByProfile(HitResults, Start, End, Rotation, ProfileName, CollisionShape, Params);

	OutHitResult.TraceStart = Start;
	OutHitResult.TraceEnd = End;

	for (int32 HitIdx = 0; HitIdx < HitResults.Num(); ++HitIdx)
	{
		const FHitResult& Hit = HitResults[HitIdx];

		if (!Hit.HitObjectHandle.IsValid() || FilterHandle.FilterPassesForActor(Hit.HitObjectHandle.FetchActor()))
		{
			OutHitResult = Hit;
			OutHitResult.bBlockingHit = true; // treat it as a blocking hit
			return;
		}
	}
}

void AGameplayAbilityTargetActor_Trace::AimWithPlayerController(const AActor* InSourceActor, FCollisionQueryParams Params, const FVector& TraceStart, FVector& OutTraceEnd, bool bIgnorePitch) const
{
	if (!OwningAbility) // Server and launching client only
	{
		return;
	}

	APlayerController* PC = OwningAbility->GetCurrentActorInfo()->PlayerController.Get();
	check(PC);

	FVector ViewStart;
	FRotator ViewRot;
	PC->GetPlayerViewPoint(ViewStart, ViewRot);

	const FVector ViewDir = ViewRot.Vector();
	FVector ViewEnd = ViewStart + (ViewDir * MaxRange);

	ClipCameraRayToAbilityRange(ViewStart, ViewDir, TraceStart, MaxRange, ViewEnd);

	FHitResult HitResult;
	LineTraceWithFilter(HitResult, InSourceActor->GetWorld(), Filter, ViewStart, ViewEnd, TraceProfile.Name, Params);

	const bool bUseTraceResult = HitResult.bBlockingHit && (FVector::DistSquared(TraceStart, HitResult.Location) <= (MaxRange * MaxRange));

	const FVector AdjustedEnd = (bUseTraceResult) ? HitResult.Location : ViewEnd;

	FVector AdjustedAimDir = (AdjustedEnd - TraceStart).GetSafeNormal();
	if (AdjustedAimDir.IsZero())
	{
		AdjustedAimDir = ViewDir;
	}

	if (!bTraceAffectsAimPitch && bUseTraceResult)
	{
		FVector OriginalAimDir = (ViewEnd - TraceStart).GetSafeNormal();

		if (!OriginalAimDir.IsZero())
		{
			// Convert to angles and use original pitch
			const FRotator OriginalAimRot = OriginalAimDir.Rotation();

			FRotator AdjustedAimRot = AdjustedAimDir.Rotation();
			AdjustedAimRot.Pitch = OriginalAimRot.Pitch;

			AdjustedAimDir = AdjustedAimRot.Vector();
		}
	}

	OutTraceEnd = TraceStart + (AdjustedAimDir * MaxRange);
}

bool AGameplayAbilityTargetActor_Trace::ClipCameraRayToAbilityRange(FVector CameraLocation, FVector CameraDirection, FVector AbilityCenter, float AbilityRange, FVector& ClippedPosition)
{
	FVector CameraToCenter = AbilityCenter - CameraLocation;
	float DotToCenter = FVector::DotProduct(CameraToCenter, CameraDirection);
	if (DotToCenter >= 0)		//If this fails, we're pointed away from the center, but we might be inside the sphere and able to find a good exit point.
	{
		float DistanceSquared = CameraToCenter.SizeSquared() - (DotToCenter * DotToCenter);
		float RadiusSquared = (AbilityRange * AbilityRange);
		if (DistanceSquared <= RadiusSquared)
		{
			float DistanceFromCamera = FMath::Sqrt(RadiusSquared - DistanceSquared);
			float DistanceAlongRay = DotToCenter + DistanceFromCamera;						//Subtracting instead of adding will get the other intersection point
			ClippedPosition = CameraLocation + (DistanceAlongRay * CameraDirection);		//Cam aim point clipped to range sphere
			return true;
		}
	}
	return false;
}

void AGameplayAbilityTargetActor_Trace::StartTargeting(UGameplayAbility* InAbility)
{
	Super::StartTargeting(InAbility);
	SourceActor = InAbility->GetCurrentActorInfo()->AvatarActor.Get();

	if (ReticleClass)
	{
		AGameplayAbilityWorldReticle* SpawnedReticleActor = GetWorld()->SpawnActor<AGameplayAbilityWorldReticle>(ReticleClass, GetActorLocation(), GetActorRotation());
		if (SpawnedReticleActor)
		{
			SpawnedReticleActor->InitializeReticle(this, PrimaryPC, ReticleParams);
			ReticleActor = SpawnedReticleActor;

			// This is to catch cases of playing on a listen server where we are using a replicated reticle actor.
			// (In a client controlled player, this would only run on the client and therefor never replicate. If it runs
			// on a listen server, the reticle actor may replicate. We want consistancy between client/listen server players.
			// Just saying 'make the reticle actor non replicated' isnt a good answer, since we want to mix and match reticle
			// actors and there may be other targeting types that want to replicate the same reticle actor class).
			if (!ShouldProduceTargetDataOnServer)
			{
				SpawnedReticleActor->SetReplicates(false);
			}
		}
	}
}

void AGameplayAbilityTargetActor_Trace::Tick(float DeltaSeconds)
{
	// very temp - do a mostly hardcoded trace from the source actor
	if (SourceActor && SourceActor->GetLocalRole() != ENetRole::ROLE_SimulatedProxy)
	{
		FHitResult HitResult = PerformTrace(SourceActor);
		FVector EndPoint = HitResult.Component.IsValid() ? HitResult.ImpactPoint : HitResult.TraceEnd;

#if ENABLE_DRAW_DEBUG
		if (bDebug)
		{
			DrawDebugLine(GetWorld(), SourceActor->GetActorLocation(), EndPoint, FColor::Green, false);
			DrawDebugSphere(GetWorld(), EndPoint, 16, 10, FColor::Green, false);
		}
#endif // ENABLE_DRAW_DEBUG

		SetActorLocationAndRotation(EndPoint, SourceActor->GetActorRotation());
	}
}

void AGameplayAbilityTargetActor_Trace::ConfirmTargetingAndContinue()
{
	check(ShouldProduceTargetData());
	if (SourceActor)
	{
		bDebug = false;
		FGameplayAbilityTargetDataHandle Handle = MakeTargetData(PerformTrace(SourceActor));
		TargetDataReadyDelegate.Broadcast(Handle);
	}
}

FGameplayAbilityTargetDataHandle AGameplayAbilityTargetActor_Trace::MakeTargetData(const FHitResult& HitResult) const
{
	/** Note: This will be cleaned up by the FGameplayAbilityTargetDataHandle (via an internal TSharedPtr) */
	return StartLocation.MakeTargetDataHandleFromHitResult(OwningAbility, HitResult);
}

