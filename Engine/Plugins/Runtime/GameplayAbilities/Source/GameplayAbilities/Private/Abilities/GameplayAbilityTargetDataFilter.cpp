// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbilityTargetDataFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityTargetDataFilter)

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	FGameplayTargetDataFilter
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

void FGameplayTargetDataFilter::InitializeFilterContext(AActor* FilterActor)
{
	SelfActor = FilterActor;
}

