// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbilityWorldReticle.h"
#include "Abilities/GameplayAbilityTargetActor.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityWorldReticle)

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	AGameplayAbilityWorldReticle
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

AGameplayAbilityWorldReticle::AGameplayAbilityWorldReticle(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.TickGroup = TG_PrePhysics;
	bIsTargetValid = true;
	bIsTargetAnActor = false;
	bFaceOwnerFlat = true;
}


void AGameplayAbilityWorldReticle::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	FaceTowardSource(bFaceOwnerFlat);
}

void AGameplayAbilityWorldReticle::InitializeReticle(AGameplayAbilityTargetActor* InTargetingActor, APlayerController* PlayerController, FWorldReticleParameters InParameters)
{
	check(InTargetingActor);
	TargetingActor = InTargetingActor;
	PrimaryPC = PlayerController;
	AddTickPrerequisiteActor(TargetingActor);		//We want the reticle to tick after the targeting actor so that designers have the final say on the position
	Parameters = InParameters;
	OnParametersInitialized();
}

bool AGameplayAbilityWorldReticle::IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const
{
	//The player who created the ability doesn't need to be updated about it - there should be local prediction in place.
	if (RealViewer == PrimaryPC)
	{
		return false;
	}

	return Super::IsNetRelevantFor(RealViewer, ViewTarget, SrcLocation);
}

void AGameplayAbilityWorldReticle::SetIsTargetValid(bool bNewValue)
{
	if (bIsTargetValid != bNewValue)
	{
		bIsTargetValid = bNewValue;
		OnValidTargetChanged(bNewValue);
	}
}

void AGameplayAbilityWorldReticle::SetIsTargetAnActor(bool bNewValue)
{
	if (bIsTargetAnActor != bNewValue)
	{
		bIsTargetAnActor = bNewValue;
		OnTargetingAnActor(bNewValue);
	}
}

void AGameplayAbilityWorldReticle::FaceTowardSource(bool bFaceIn2D)
{
	AActor* FacingActor = TargetingActor.Get();

	if (TargetingActor.Get() && TargetingActor->SourceActor)
	{
		FacingActor = TargetingActor->SourceActor;
	}
	
	if (FacingActor)
	{
		if (bFaceIn2D)
		{
			FVector FacingVector = (FacingActor->GetActorLocation() - GetActorLocation()).GetSafeNormal2D();
			if (FacingVector.IsZero())
			{
				FacingVector = -GetActorForwardVector().GetSafeNormal2D();
			}
			if (!FacingVector.IsZero())
			{
				SetActorRotation(FacingVector.Rotation());
			}
		}
		else
		{
			FVector FacingVector = (FacingActor->GetActorLocation() - GetActorLocation()).GetSafeNormal();
			if (FacingVector.IsZero())
			{
				FacingVector = -GetActorForwardVector().GetSafeNormal();
			}
			SetActorRotation(FacingVector.Rotation());
		}
	}
}

