// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbilityWorldReticle_ActorVisualization.h"
#include "Abilities/GameplayAbilityTargetActor.h"
#include "Components/CapsuleComponent.h"
#include "Components/MeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityWorldReticle_ActorVisualization)

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	AGameplayAbilityWorldReticle_ActorVisualization
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

AGameplayAbilityWorldReticle_ActorVisualization::AGameplayAbilityWorldReticle_ActorVisualization(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	//PrimaryActorTick.bCanEverTick = true;
	//PrimaryActorTick.TickGroup = TG_PrePhysics;

	CollisionComponent = CreateDefaultSubobject<UCapsuleComponent>(TEXT("CollisionCapsule0"));
	CollisionComponent->InitCapsuleSize(0.f, 0.f);
	CollisionComponent->AlwaysLoadOnClient = true;
	CollisionComponent->SetUsingAbsoluteScale(true);
	//CollisionComponent->AlwaysLoadOnServer = true;
	CollisionComponent->SetCanEverAffectNavigation(false);
	CollisionComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);

	//USceneComponent* SceneComponent = ObjectInitializer.CreateDefaultSubobject<USceneComponent>(this, TEXT("RootComponent0"));
	RootComponent = CollisionComponent;
}


void AGameplayAbilityWorldReticle_ActorVisualization::InitializeReticleVisualizationInformation(AGameplayAbilityTargetActor* InTargetingActor, AActor* VisualizationActor, UMaterialInterface *VisualizationMaterial)
{
	if (VisualizationActor)
	{
		//Get components
		TInlineComponentArray<UMeshComponent*> MeshComps;
		USceneComponent* MyRoot = GetRootComponent();
		VisualizationActor->GetComponents(MeshComps);
		check(MyRoot);

		TargetingActor = InTargetingActor;
		AddTickPrerequisiteActor(TargetingActor);		//We want the reticle to tick after the targeting actor so that designers have the final say on the position

		for (UMeshComponent* MeshComp : MeshComps)
		{
			//Special case: If we don't clear the root component explicitly, the component will be destroyed along with the original visualization actor.
			if (MeshComp == VisualizationActor->GetRootComponent())
			{
				VisualizationActor->SetRootComponent(NULL);
			}

			//Disable collision on visualization mesh parts so it doesn't interfere with aiming or any other client-side collision/prediction/physics stuff
			MeshComp->SetCollisionEnabled(ECollisionEnabled::NoCollision);		//All mesh components are primitive components, so no cast is needed

			//Move components from one actor to the other, attaching as needed. Hierarchy should not be important, but we can do fixups if it becomes important later.
			MeshComp->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
			MeshComp->AttachToComponent(MyRoot, FAttachmentTransformRules::KeepRelativeTransform);
			MeshComp->Rename(nullptr, this);
			if (VisualizationMaterial)
			{
				MeshComp->SetMaterial(0, VisualizationMaterial);
			}
		}
	}
}

void AGameplayAbilityWorldReticle_ActorVisualization::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);
}

