// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbility_CharacterJump.h"
#include "GameFramework/Character.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbility_CharacterJump)

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	UGameplayAbility_CharacterJump
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

UGameplayAbility_CharacterJump::UGameplayAbility_CharacterJump(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NetExecutionPolicy = EGameplayAbilityNetExecutionPolicy::LocalPredicted;
	InstancingPolicy = EGameplayAbilityInstancingPolicy::NonInstanced;
}

void UGameplayAbility_CharacterJump::ActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayEventData* TriggerEventData)
{
	
	if (HasAuthorityOrPredictionKey(ActorInfo, &ActivationInfo))
	{
		if (!CommitAbility(Handle, ActorInfo, ActivationInfo))
		{
			return;
		}

		ACharacter * Character = CastChecked<ACharacter>(ActorInfo->AvatarActor.Get());
		Character->Jump();
	}
}

void UGameplayAbility_CharacterJump::InputReleased(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo)
{
	if (ActorInfo != NULL && ActorInfo->AvatarActor != NULL)
	{
		CancelAbility(Handle, ActorInfo, ActivationInfo, true);
	}
}

bool UGameplayAbility_CharacterJump::CanActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayTagContainer* SourceTags, const FGameplayTagContainer* TargetTags, OUT FGameplayTagContainer* OptionalRelevantTags) const
{
	if (!Super::CanActivateAbility(Handle, ActorInfo, SourceTags, TargetTags, OptionalRelevantTags))
	{
		return false;
	}

	const ACharacter* Character = CastChecked<ACharacter>(ActorInfo->AvatarActor.Get(), ECastCheckedType::NullAllowed);
	return (Character && Character->CanJump());
}

/**
 *	Canceling an non instanced ability is tricky. Right now this works for Jump since there is nothing that can go wrong by calling
 *	StopJumping() if you aren't already jumping. If we had a montage playing non instanced ability, it would need to make sure the
 *	Montage that *it* played was still playing, and if so, to cancel it. If this is something we need to support, we may need some
 *	light weight data structure to represent 'non intanced abilities in action' with a way to cancel/end them.
 */
void UGameplayAbility_CharacterJump::CancelAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, bool bReplicateCancelAbility)
{
	if (ScopeLockCount > 0)
	{
		WaitingToExecute.Add(FPostLockDelegate::CreateUObject(this, &UGameplayAbility_CharacterJump::CancelAbility, Handle, ActorInfo, ActivationInfo, bReplicateCancelAbility));
		return;
	}

	Super::CancelAbility(Handle, ActorInfo, ActivationInfo, bReplicateCancelAbility);
	
	ACharacter * Character = CastChecked<ACharacter>(ActorInfo->AvatarActor.Get());
	Character->StopJumping();
}

