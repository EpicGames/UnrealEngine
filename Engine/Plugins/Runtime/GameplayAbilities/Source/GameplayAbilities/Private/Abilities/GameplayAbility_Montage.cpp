// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbility_Montage.h"
#include "Animation/AnimInstance.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbility_Montage)

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	UGameplayAbility_Montage
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

UGameplayAbility_Montage::UGameplayAbility_Montage(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PlayRate = 1.f;
	NetExecutionPolicy = EGameplayAbilityNetExecutionPolicy::ServerInitiated;
}

void UGameplayAbility_Montage::ActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayEventData* TriggerEventData)
{
	if (!CommitAbility(Handle, ActorInfo, ActivationInfo))
	{
		return;
	}

	UAnimInstance* AnimInstance = ActorInfo->GetAnimInstance();

	if (MontageToPlay != nullptr && AnimInstance != nullptr && AnimInstance->GetActiveMontageInstance() == nullptr)
	{
		TArray<FActiveGameplayEffectHandle>	AppliedEffects;

		// Apply GameplayEffects
		TArray<const UGameplayEffect*> Effects;
		GetGameplayEffectsWhileAnimating(Effects);
		if (Effects.Num() > 0)
		{
			UAbilitySystemComponent* const AbilitySystemComponent = ActorInfo->AbilitySystemComponent.Get();
			for (const UGameplayEffect* Effect : Effects)
			{
				FActiveGameplayEffectHandle EffectHandle = AbilitySystemComponent->ApplyGameplayEffectToSelf(Effect, 1.f, MakeEffectContext(Handle, ActorInfo));
				if (EffectHandle.IsValid())
				{
					AppliedEffects.Add(EffectHandle);
				}
			}
		}

		float const Duration = AnimInstance->Montage_Play(MontageToPlay, PlayRate);

		FOnMontageEnded EndDelegate;
		EndDelegate.BindUObject(this, &UGameplayAbility_Montage::OnMontageEnded, ActorInfo->AbilitySystemComponent, AppliedEffects);
		AnimInstance->Montage_SetEndDelegate(EndDelegate);

		if (SectionName != NAME_None)
		{
			AnimInstance->Montage_JumpToSection(SectionName);
		}
	}
}

void UGameplayAbility_Montage::OnMontageEnded(UAnimMontage* Montage, bool bInterrupted, TWeakObjectPtr<UAbilitySystemComponent> AbilitySystemComponentPtr, TArray<FActiveGameplayEffectHandle> AppliedEffects)
{
	// Remove any GameplayEffects that we applied
	UAbilitySystemComponent* const AbilitySystemComponent = AbilitySystemComponentPtr.Get();
	if (AbilitySystemComponent)
	{
		for (FActiveGameplayEffectHandle Handle : AppliedEffects)
		{
			AbilitySystemComponent->RemoveActiveGameplayEffect(Handle);
		}
	}
}

void UGameplayAbility_Montage::GetGameplayEffectsWhileAnimating(TArray<const UGameplayEffect*>& OutEffects) const
{
	OutEffects.Append(GameplayEffectsWhileAnimating);

	for ( TSubclassOf<UGameplayEffect> EffectClass : GameplayEffectClassesWhileAnimating )
	{
		if ( EffectClass )
		{
			OutEffects.Add(EffectClass->GetDefaultObject<UGameplayEffect>());
		}
	}
}

