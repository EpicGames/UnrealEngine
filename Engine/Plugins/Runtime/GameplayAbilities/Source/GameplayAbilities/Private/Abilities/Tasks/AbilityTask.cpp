// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask.h"
#include "AbilitySystemLog.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemStats.h"
#include "Templates/Greater.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask)

static void DebugRecordAbilityTaskCreated(const UAbilityTask* NewTask);
static void DebugRecordAbilityTaskDestroyed(const UAbilityTask* NewTask);
static void DebugPrintAbilityTasksByClass();

enum class EDebugBuildRecordFlag : int32
{
	Disabled = 0,
	EnableForNonShippingBuilds = 1,
	EnableForAllBuilds = 2
};

namespace AbilityTaskConstants
{
#if UE_BUILD_SHIPPING
	constexpr int32 DebugMinValueToEnableRecording = static_cast<int32>(EDebugBuildRecordFlag::EnableForAllBuilds);
#else
	constexpr int32 DebugMinValueToEnableRecording = static_cast<int32>(EDebugBuildRecordFlag::EnableForNonShippingBuilds);
#endif
}

namespace AbilityTaskCVars
{
	static int32 AbilityTaskMaxCount = 1000;
	static FAutoConsoleVariableRef CVarMaxAbilityTaskCount(
		TEXT("AbilitySystem.AbilityTask.MaxCount"),
		AbilityTaskMaxCount,
		TEXT("Global limit on the number of extant AbilityTasks. Use 'AbilitySystem.AbilityTask.Debug.RecordingEnabled' and 'AbilitySystem.AbilityTask.Debug.PrintCounts' to debug why you are hitting this before raising the cap.")
	);

	// 0 - disabled, 1 - enabled in non-shipping builds, 2 - enabled in all builds (including shipping)
	static int32 AbilityTaskRecordingType = static_cast<int32>(EDebugBuildRecordFlag::EnableForNonShippingBuilds);
	static FAutoConsoleVariableRef CVarRecordAbilityTaskCounts(
		TEXT("AbilitySystem.AbilityTask.Debug.RecordingEnabled"),
		AbilityTaskRecordingType,
		TEXT("Set to 0 to disable, 1 to enable in non-shipping builds, and 2 to enable in all builds (including shipping). If this is enabled, all new AbilityTasks will be counted by type. Use 'AbilitySystem.AbilityTask.Debug.PrintCounts' to print out the current counts.")
	);

	static bool bRecordAbilityTaskSourceAbilityCounts = true;
	static FAutoConsoleVariableRef CVarRecordAbilityTaskSourceAbilityCounts(
		TEXT("AbilitySystem.AbilityTask.Debug.SourceRecordingEnabled"),
		bRecordAbilityTaskSourceAbilityCounts,
		TEXT("Requires AbilityTaskRecordingType to be set to enabled (1 for non-shipping builds, 2 for all builds) for this value to do anything.  If both are enabled, all new AbilityTasks (after InitTask is called in NewAbilityTask) will be counted by the class of the ability that created them.  Use 'AbilitySystem.AbilityTask.Debug.PrintCounts' to print out the current counts.")
	);

	static int32 AbilityTaskDebugPrintTopNResults = 5;
	static FAutoConsoleVariableRef CVarAbilityTaskDebugPrintTopNResults(
		TEXT("AbilitySystem.AbilityTask.Debug.AbilityTaskDebugPrintTopNResults"),
		AbilityTaskDebugPrintTopNResults,
		TEXT("Set N to only print the top N results when printing ability task counts (N = 5 by default, if N = 0 prints all).  Use 'AbilitySystem.AbilityTask.Debug.PrintCounts' to print out the current counts.")
	);

	static FAutoConsoleCommand AbilityTaskPrintAbilityTaskCountsCmd(
		TEXT("AbilitySystem.AbilityTask.Debug.PrintCounts"),
		TEXT("Print out the current AbilityTask counts by class. 'AbilitySystem.AbilityTask.Debug.RecordingEnabled' must be turned on for this to function."),
		FConsoleCommandDelegate::CreateStatic(DebugPrintAbilityTasksByClass)
	);
}

static int32 GlobalAbilityTaskCount = 0;

UAbilityTask::UAbilityTask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WaitStateBitMask = static_cast<uint8>(EAbilityTaskWaitState::WaitingOnGame);

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (AbilityTaskCVars::AbilityTaskRecordingType >= AbilityTaskConstants::DebugMinValueToEnableRecording)
		{
			DebugRecordAbilityTaskCreated(this);
		}

		bool bExceededAbilityTaskMaxCount = false;

		++GlobalAbilityTaskCount;
		SET_DWORD_STAT(STAT_AbilitySystem_TaskCount, GlobalAbilityTaskCount);
		if (!(GlobalAbilityTaskCount < AbilityTaskCVars::AbilityTaskMaxCount))
		{
			bExceededAbilityTaskMaxCount = true;

			ABILITY_LOG(Warning, TEXT("Way too many AbilityTasks are currently active! %d. %s"), GlobalAbilityTaskCount, *GetClass()->GetName());

			// Auto dump the counts if we hit the limit
			if (AbilityTaskCVars::AbilityTaskRecordingType >= AbilityTaskConstants::DebugMinValueToEnableRecording)
			{
				static bool bHasDumpedAbilityTasks = false;  // The dump is spammy, so we only want to auto-dump once

				if (!bHasDumpedAbilityTasks)
				{
					DebugPrintAbilityTasksByClass();
					bHasDumpedAbilityTasks = true;

					// If we don't flush here the ensure is hit without debug ability task info printed in log
					GLog->FlushThreadedLogs();
					GLog->Flush();
				}
			}
		}

		ensureMsgf(!bExceededAbilityTaskMaxCount, TEXT("Exceeded AbilityTaskMaxCount. For more information in log set AbilitySystem.AbilityTask.Debug.SourceRecordingEnabled to 1 for non-shipping builds, or 2 for all builds (including shipping)."));
	}
}

void UAbilityTask::OnDestroy(bool bInOwnerFinished)
{
	// If we have already been destroyed this is being called recursively so skip the tracking as well as the super call
	if (!bWasSuccessfullyDestroyed)
	{
		if (!HasAnyFlags(RF_ClassDefaultObject))
		{
			ensureMsgf(GlobalAbilityTaskCount > 0, TEXT("Mismatched AbilityTask counting"));
			--GlobalAbilityTaskCount;
			SET_DWORD_STAT(STAT_AbilitySystem_TaskCount, GlobalAbilityTaskCount);

			if (AbilityTaskCVars::AbilityTaskRecordingType >= AbilityTaskConstants::DebugMinValueToEnableRecording)
			{
				DebugRecordAbilityTaskDestroyed(this);
			}
		}

		bWasSuccessfullyDestroyed = true;

		// #KillPendingKill Clear ability reference so we don't hold onto it and GC can delete it.
		Ability = nullptr;

		Super::OnDestroy(bInOwnerFinished);
	}
	else
	{
		ensureMsgf(TaskState == EGameplayTaskState::Finished, TEXT("OnDestroy called twice on %s with invalid state %i"), *GetName(), EnumToUnderlyingType(TaskState));
	}
}

void UAbilityTask::BeginDestroy()
{
	Super::BeginDestroy();

	if (!bWasSuccessfullyDestroyed)
	{
		bWasSuccessfullyDestroyed = true;
		if (!HasAnyFlags(RF_ClassDefaultObject))
		{
			// this shouldn't happen, it means that ability was destroyed while being active, but we need to keep GlobalAbilityTaskCount in sync anyway
			ensureMsgf(GlobalAbilityTaskCount > 0, TEXT("Mismatched AbilityTask counting"));
			--GlobalAbilityTaskCount;
			SET_DWORD_STAT(STAT_AbilitySystem_TaskCount, GlobalAbilityTaskCount);

			if (AbilityTaskCVars::AbilityTaskRecordingType >= AbilityTaskConstants::DebugMinValueToEnableRecording)
			{
				DebugRecordAbilityTaskDestroyed(this);
			}
		}
	}
}

void UAbilityTask::PreDestroyFromReplication()
{
	// Clear the pointer without causing sideffects that a specialized EndTask() could cause.
	Ability = nullptr;
}

FGameplayAbilitySpecHandle UAbilityTask::GetAbilitySpecHandle() const
{
	return Ability ? Ability->GetCurrentAbilitySpecHandle() : FGameplayAbilitySpecHandle();
}

void UAbilityTask::SetAbilitySystemComponent(UAbilitySystemComponent* InAbilitySystemComponent)
{
	AbilitySystemComponent = InAbilitySystemComponent;
}

void UAbilityTask::InitSimulatedTask(UGameplayTasksComponent& InGameplayTasksComponent)
{
	UGameplayTask::InitSimulatedTask(InGameplayTasksComponent);

	SetAbilitySystemComponent(Cast<UAbilitySystemComponent>(TasksComponent.Get()));
}

FPredictionKey UAbilityTask::GetActivationPredictionKey() const
{
	return Ability ? Ability->GetCurrentActivationInfo().GetActivationPredictionKey() : FPredictionKey();
}

int32 AbilityTaskWarnIfBroadcastSuppress = 0;
static FAutoConsoleVariableRef CVarAbilityTaskWarnIfBroadcastSuppress(TEXT("AbilitySystem.AbilityTaskWarnIfBroadcastSuppress"), AbilityTaskWarnIfBroadcastSuppress, TEXT("Print warning if an ability task broadcast is suppressed because the ability has ended"), ECVF_Default );

bool UAbilityTask::ShouldBroadcastAbilityTaskDelegates() const
{
	bool ShouldBroadcast = (Ability && Ability->IsActive());

	if (!ShouldBroadcast && AbilityTaskWarnIfBroadcastSuppress)
	{
		ABILITY_LOG(Warning, TEXT("Suppressing ability task %s broadcast"), *GetDebugString());
	}

	return ShouldBroadcast;
}

bool UAbilityTask::IsPredictingClient() const
{
	return Ability && Ability->IsPredictingClient();
}

bool UAbilityTask::IsForRemoteClient() const
{
	return Ability && Ability->IsForRemoteClient();
}

bool UAbilityTask::IsLocallyControlled() const
{
	return Ability && Ability->IsLocallyControlled();
}

bool UAbilityTask::CallOrAddReplicatedDelegate(EAbilityGenericReplicatedEvent::Type Event, FSimpleMulticastDelegate::FDelegate Delegate)
{
	if (!AbilitySystemComponent->CallOrAddReplicatedDelegate(Event, GetAbilitySpecHandle(), GetActivationPredictionKey(), Delegate))
	{
		SetWaitingOnRemotePlayerData();
		return false;
	}
	return true;
}

void UAbilityTask::SetWaitingOnRemotePlayerData()
{
	if (IsValid(Ability) && AbilitySystemComponent.IsValid())
	{
		WaitStateBitMask |= (uint8)EAbilityTaskWaitState::WaitingOnUser;
		Ability->NotifyAbilityTaskWaitingOnPlayerData(this);
	}
}

void UAbilityTask::ClearWaitingOnRemotePlayerData()
{
	WaitStateBitMask &= ~((uint8)EAbilityTaskWaitState::WaitingOnUser);
}

bool UAbilityTask::IsWaitingOnRemotePlayerdata() const
{
	return (WaitStateBitMask & (uint8)EAbilityTaskWaitState::WaitingOnUser) != 0;
}

void UAbilityTask::SetWaitingOnAvatar()
{
	if (IsValid(Ability) && AbilitySystemComponent.IsValid())
	{
		WaitStateBitMask |= (uint8)EAbilityTaskWaitState::WaitingOnAvatar;
		Ability->NotifyAbilityTaskWaitingOnAvatar(this);
	}
}

void UAbilityTask::ClearWaitingOnAvatar()
{
	WaitStateBitMask &= ~((uint8)EAbilityTaskWaitState::WaitingOnAvatar);
}

bool UAbilityTask::IsWaitingOnAvatar() const
{
	return (WaitStateBitMask & (uint8)EAbilityTaskWaitState::WaitingOnAvatar) != 0;
}

static TMap<const UClass*, int32> StaticAbilityTasksByClass = {};
static TMap<const UClass*, int32> StaticAbilityTasksByAbilityClass = {};

void DebugRecordAbilityTaskCreated(const UAbilityTask* NewTask)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AbilityTaskDebugRecording);

	const UClass* ClassPtr = (NewTask != nullptr) ? NewTask->GetClass() : nullptr;
	if (ClassPtr != nullptr)
	{
		if (StaticAbilityTasksByClass.Contains(ClassPtr))
		{
			StaticAbilityTasksByClass[ClassPtr]++;
		}
		else
		{
			StaticAbilityTasksByClass.Add(ClassPtr, 1);
		}
	}
}

void UAbilityTask::DebugRecordAbilityTaskCreatedByAbility(const UObject* Ability)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AbilityTaskDebugRecording);

	if (!AbilityTaskCVars::bRecordAbilityTaskSourceAbilityCounts || AbilityTaskCVars::AbilityTaskRecordingType < AbilityTaskConstants::DebugMinValueToEnableRecording)
	{	// Both the more detailed and the basic recording is required for the detailed recording to work properly.
		return;
	}

	const UClass* ClassPtr = (Ability != nullptr) ? Ability->GetClass() : nullptr;
	if (ClassPtr != nullptr)
	{
		if (StaticAbilityTasksByAbilityClass.Contains(ClassPtr))
		{
			StaticAbilityTasksByAbilityClass[ClassPtr]++;
		}
		else
		{
			StaticAbilityTasksByAbilityClass.Add(ClassPtr, 1);
		}
	}
}

static void DebugRecordAbilityTaskDestroyed(const UAbilityTask* DestroyedTask)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AbilityTaskDebugRecording);

	const UClass* ClassPtr = (DestroyedTask != nullptr) ? DestroyedTask->GetClass() : nullptr;
	if (ClassPtr != nullptr)
	{
		if (AbilityTaskCVars::bRecordAbilityTaskSourceAbilityCounts)
		{
			const UClass* AbilityClassPtr = (DestroyedTask->Ability != nullptr) ? DestroyedTask->Ability->GetClass() : nullptr;
			if (AbilityClassPtr != nullptr)
			{
				if (StaticAbilityTasksByAbilityClass.Contains(AbilityClassPtr))
				{
					StaticAbilityTasksByAbilityClass[AbilityClassPtr]--;

					if (StaticAbilityTasksByAbilityClass[AbilityClassPtr] <= 0)
					{
						StaticAbilityTasksByAbilityClass.Remove(AbilityClassPtr);
					}
				}
			}
		}

		if (StaticAbilityTasksByClass.Contains(ClassPtr))
		{
			StaticAbilityTasksByClass[ClassPtr]--;

			if (StaticAbilityTasksByClass[ClassPtr] <= 0)
			{
				StaticAbilityTasksByClass.Remove(ClassPtr);
			}
		}
	}
}

void DebugPrintAbilityTasksByClass()
{
	if (AbilityTaskCVars::AbilityTaskRecordingType >= AbilityTaskConstants::DebugMinValueToEnableRecording)
	{
		const int32 NumberOfTopResultsToShow = AbilityTaskCVars::AbilityTaskDebugPrintTopNResults;
		const int32 NumStaticAbilityTasks = StaticAbilityTasksByClass.Num();

		// If NumberOfTopResultsToShow == 0 print all, otherwise print the top N results
		const int32 LogElementMax = NumberOfTopResultsToShow > 0 ? NumberOfTopResultsToShow : NumStaticAbilityTasks;
		int32 LogElementCount = 0;

		int32 AccumulatedAbilityTasks = 0;
		ABILITY_LOG(Display, TEXT("Logging global UAbilityTask counts (showing top %d results):"), NumberOfTopResultsToShow);
		StaticAbilityTasksByClass.ValueSort(TGreater<int32>());
		for (const TPair<const UClass*, int32>& Pair : StaticAbilityTasksByClass)
		{
			// Only log top NumberOfTopResultsToShow entries
			if (LogElementCount < LogElementMax)
			{
				FString SafeName = GetNameSafe(Pair.Key);
				ABILITY_LOG(Display, TEXT("- Class '%s': %d"), *SafeName, Pair.Value);

				++LogElementCount;
			}
			
			AccumulatedAbilityTasks += Pair.Value;
		}

		const int32 UnaccountedAbilityTasks = GlobalAbilityTaskCount - AccumulatedAbilityTasks;
		if (UnaccountedAbilityTasks > 0)
		{
			// It's possible to allocate AbilityTasks before AbilityTaskCVars::AbilityTaskRecordingType was set to enabled, even if set via command line.
			// However, if this value increases during play, there is an issue.
			ABILITY_LOG(Display, TEXT("- Unknown (allocated before recording): %d"), UnaccountedAbilityTasks);
		}

		if (AbilityTaskCVars::bRecordAbilityTaskSourceAbilityCounts)
		{
			LogElementCount = 0;
			ABILITY_LOG(Display, TEXT("UAbilityTask counts per Ability Class (showing top %d results):"), NumberOfTopResultsToShow);
			StaticAbilityTasksByAbilityClass.ValueSort(TGreater<int32>());
			for (const TPair<const UClass*, int32>& Pair : StaticAbilityTasksByAbilityClass)
			{
				// Only log top NumberOfTopResultsToShow entries
				if (LogElementCount >= LogElementMax)
				{
					break;
				}

				FString SafeName = GetNameSafe(Pair.Key);
				ABILITY_LOG(Display, TEXT("- Ability Class '%s': %d"), *SafeName, Pair.Value);

				++LogElementCount;
			}
		}

		ABILITY_LOG(Display, TEXT("Total AbilityTask count: %d"), GlobalAbilityTaskCount);
	}
	else
	{
		ABILITY_LOG(Display, TEXT("Recording of UAbilityTask counts is disabled! Enable 'AbilitySystem.AbilityTask.Debug.RecordingEnabled' (1 for non-shipping builds, 2 for all builds) to turn on recording."))
	}
}

