// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_ApplyRootMotionConstantForce.h"
#include "GameFramework/RootMotionSource.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"
#include "Net/UnrealNetwork.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_ApplyRootMotionConstantForce)

UAbilityTask_ApplyRootMotionConstantForce::UAbilityTask_ApplyRootMotionConstantForce(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	StrengthOverTime = nullptr;
}

UAbilityTask_ApplyRootMotionConstantForce* UAbilityTask_ApplyRootMotionConstantForce::ApplyRootMotionConstantForce
(
	UGameplayAbility* OwningAbility, 
	FName TaskInstanceName, 
	FVector WorldDirection, 
	float Strength, 
	float Duration, 
	bool bIsAdditive, 
	UCurveFloat* StrengthOverTime,
	ERootMotionFinishVelocityMode VelocityOnFinishMode, 
	FVector SetVelocityOnFinish, 
	float ClampVelocityOnFinish,
	bool bEnableGravity
)
{
	UAbilitySystemGlobals::NonShipping_ApplyGlobalAbilityScaler_Duration(Duration);

	UAbilityTask_ApplyRootMotionConstantForce* MyTask = NewAbilityTask<UAbilityTask_ApplyRootMotionConstantForce>(OwningAbility, TaskInstanceName);

	MyTask->ForceName = TaskInstanceName;
	MyTask->WorldDirection = WorldDirection.GetSafeNormal();
	MyTask->Strength = Strength;
	MyTask->Duration = Duration;
	MyTask->bIsAdditive = bIsAdditive;
	MyTask->StrengthOverTime = StrengthOverTime;
	MyTask->FinishVelocityMode = VelocityOnFinishMode;
	MyTask->FinishSetVelocity = SetVelocityOnFinish;
	MyTask->FinishClampVelocity = ClampVelocityOnFinish;
	MyTask->bEnableGravity = bEnableGravity;
	MyTask->SharedInitAndApply();

	return MyTask;
}

void UAbilityTask_ApplyRootMotionConstantForce::SharedInitAndApply()
{
	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (ASC && ASC->AbilityActorInfo->MovementComponent.IsValid())
	{
		MovementComponent = Cast<UCharacterMovementComponent>(ASC->AbilityActorInfo->MovementComponent.Get());
		StartTime = GetWorld()->GetTimeSeconds();
		EndTime = StartTime + Duration;

		if (MovementComponent.IsValid())
		{
			ForceName = ForceName.IsNone() ? FName("AbilityTaskApplyRootMotionConstantForce"): ForceName;
			TSharedPtr<FRootMotionSource_ConstantForce> ConstantForce = MakeShared<FRootMotionSource_ConstantForce>();
			ConstantForce->InstanceName = ForceName;
			ConstantForce->AccumulateMode = bIsAdditive ? ERootMotionAccumulateMode::Additive : ERootMotionAccumulateMode::Override;
			ConstantForce->Priority = 5;
			ConstantForce->Force = WorldDirection * Strength;
			ConstantForce->Duration = Duration;
			ConstantForce->StrengthOverTime = StrengthOverTime;
			ConstantForce->FinishVelocityParams.Mode = FinishVelocityMode;
			ConstantForce->FinishVelocityParams.SetVelocity = FinishSetVelocity;
			ConstantForce->FinishVelocityParams.ClampVelocity = FinishClampVelocity;
			if (bEnableGravity)
			{
				ConstantForce->Settings.SetFlag(ERootMotionSourceSettingsFlags::IgnoreZAccumulate);
			}
			RootMotionSourceID = MovementComponent->ApplyRootMotionSource(ConstantForce);
		}
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilityTask_ApplyRootMotionConstantForce called in Ability %s with null MovementComponent; Task Instance Name %s."), 
			Ability ? *Ability->GetName() : TEXT("NULL"), 
			*InstanceName.ToString());
	}
}

void UAbilityTask_ApplyRootMotionConstantForce::TickTask(float DeltaTime)
{
	if (bIsFinished)
	{
		return;
	}

	Super::TickTask(DeltaTime);

	AActor* MyActor = GetAvatarActor();
	if (MyActor)
	{
		const bool bTimedOut = HasTimedOut();
		const bool bIsInfiniteDuration = Duration < 0.f;

		if (!bIsInfiniteDuration && bTimedOut)
		{
			// Task has finished
			bIsFinished = true;
			if (!bIsSimulating)
			{
				MyActor->ForceNetUpdate();
				if (ShouldBroadcastAbilityTaskDelegates())
				{
					OnFinish.Broadcast();
				}
				EndTask();
			}
		}
	}
	else
	{
		bIsFinished = true;
		EndTask();
	}
}

void UAbilityTask_ApplyRootMotionConstantForce::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(UAbilityTask_ApplyRootMotionConstantForce, WorldDirection);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionConstantForce, Strength);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionConstantForce, Duration);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionConstantForce, bIsAdditive);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionConstantForce, StrengthOverTime);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionConstantForce, bEnableGravity);
}

void UAbilityTask_ApplyRootMotionConstantForce::PreDestroyFromReplication()
{
	bIsFinished = true;
	EndTask();
}

void UAbilityTask_ApplyRootMotionConstantForce::OnDestroy(bool AbilityIsEnding)
{
	if (MovementComponent.IsValid())
	{
		MovementComponent->RemoveRootMotionSourceByID(RootMotionSourceID);
	}

	Super::OnDestroy(AbilityIsEnding);
}

