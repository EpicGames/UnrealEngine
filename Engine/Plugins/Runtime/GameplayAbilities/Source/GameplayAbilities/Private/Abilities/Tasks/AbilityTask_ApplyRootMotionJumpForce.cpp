// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_ApplyRootMotionJumpForce.h"
#include "GameFramework/RootMotionSource.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"
#include "Net/UnrealNetwork.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_ApplyRootMotionJumpForce)

UAbilityTask_ApplyRootMotionJumpForce::UAbilityTask_ApplyRootMotionJumpForce(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	PathOffsetCurve = nullptr;
	TimeMappingCurve = nullptr;
	bHasLanded = false;
}

UAbilityTask_ApplyRootMotionJumpForce* UAbilityTask_ApplyRootMotionJumpForce::ApplyRootMotionJumpForce(UGameplayAbility* OwningAbility, FName TaskInstanceName, FRotator Rotation, float Distance, float Height, float Duration, float MinimumLandedTriggerTime,
	bool bFinishOnLanded, ERootMotionFinishVelocityMode VelocityOnFinishMode, FVector SetVelocityOnFinish, float ClampVelocityOnFinish, UCurveVector* PathOffsetCurve, UCurveFloat* TimeMappingCurve)
{
	UAbilitySystemGlobals::NonShipping_ApplyGlobalAbilityScaler_Duration(Duration);

	UAbilityTask_ApplyRootMotionJumpForce* MyTask = NewAbilityTask<UAbilityTask_ApplyRootMotionJumpForce>(OwningAbility, TaskInstanceName);

	MyTask->ForceName = TaskInstanceName;
	MyTask->Rotation = Rotation;
	MyTask->Distance = Distance;
	MyTask->Height = Height;
	MyTask->Duration = FMath::Max(Duration, KINDA_SMALL_NUMBER); // No zero duration
	MyTask->MinimumLandedTriggerTime = MinimumLandedTriggerTime * Duration; // MinimumLandedTriggerTime is normalized
	MyTask->bFinishOnLanded = bFinishOnLanded;
	MyTask->FinishVelocityMode = VelocityOnFinishMode;
	MyTask->FinishSetVelocity = SetVelocityOnFinish;
	MyTask->FinishClampVelocity = ClampVelocityOnFinish;
	MyTask->PathOffsetCurve = PathOffsetCurve;
	MyTask->TimeMappingCurve = TimeMappingCurve;
	MyTask->SharedInitAndApply();

	return MyTask;
}

void UAbilityTask_ApplyRootMotionJumpForce::Activate()
{
	ACharacter* Character = Cast<ACharacter>(GetAvatarActor());
	if (Character)
	{
		Character->LandedDelegate.AddDynamic(this, &UAbilityTask_ApplyRootMotionJumpForce::OnLandedCallback);
	}
	SetWaitingOnAvatar();
}

void UAbilityTask_ApplyRootMotionJumpForce::OnLandedCallback(const FHitResult& Hit)
{
	bHasLanded = true;

	ACharacter* Character = Cast<ACharacter>(GetAvatarActor());
	if (Character && Character->bClientUpdating)
	{
		// If in a move replay, we just mark that we landed so that next tick we trigger landed
	}
	else
	{
		// TriggerLanded immediately if we're past time allowed, otherwise it'll get caught next valid tick
		const float CurrentTime = GetWorld()->GetTimeSeconds();
		if (CurrentTime >= (StartTime+MinimumLandedTriggerTime))
		{
			TriggerLanded();
		}
	}
}

void UAbilityTask_ApplyRootMotionJumpForce::TriggerLanded()
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnLanded.Broadcast();
	}

	if (bFinishOnLanded)
	{
		Finish();
	}
}

void UAbilityTask_ApplyRootMotionJumpForce::SharedInitAndApply()
{
	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (ASC && ASC->AbilityActorInfo->MovementComponent.IsValid())
	{
		MovementComponent = Cast<UCharacterMovementComponent>(ASC->AbilityActorInfo->MovementComponent.Get());
		StartTime = GetWorld()->GetTimeSeconds();
		EndTime = StartTime + Duration;

		if (MovementComponent.IsValid())
		{
			ForceName = ForceName.IsNone() ? FName("AbilityTaskApplyRootMotionJumpForce") : ForceName;
			TSharedPtr<FRootMotionSource_JumpForce> JumpForce = MakeShared<FRootMotionSource_JumpForce>();
			JumpForce->InstanceName = ForceName;
			JumpForce->AccumulateMode = ERootMotionAccumulateMode::Override;
			JumpForce->Priority = 500;
			JumpForce->Duration = Duration;
			JumpForce->Rotation = Rotation;
			JumpForce->Distance = Distance;
			JumpForce->Height = Height;
			JumpForce->Duration = Duration;
			JumpForce->bDisableTimeout = bFinishOnLanded; // If we finish on landed, we need to disable force's timeout
			JumpForce->PathOffsetCurve = PathOffsetCurve;
			JumpForce->TimeMappingCurve = TimeMappingCurve;
			JumpForce->FinishVelocityParams.Mode = FinishVelocityMode;
			JumpForce->FinishVelocityParams.SetVelocity = FinishSetVelocity;
			JumpForce->FinishVelocityParams.ClampVelocity = FinishClampVelocity;
			RootMotionSourceID = MovementComponent->ApplyRootMotionSource(JumpForce);
		}
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilityTask_ApplyRootMotionJumpForce called in Ability %s with null MovementComponent; Task Instance Name %s."), 
			Ability ? *Ability->GetName() : TEXT("NULL"), 
			*InstanceName.ToString());
	}
}

void UAbilityTask_ApplyRootMotionJumpForce::Finish()
{
	bIsFinished = true;

	if (!bIsSimulating)
	{
		AActor* MyActor = GetAvatarActor();
		if (MyActor)
		{
			MyActor->ForceNetUpdate();
			if (ShouldBroadcastAbilityTaskDelegates())
			{
				OnFinish.Broadcast();
			}
		}
		EndTask();
	}
}

void UAbilityTask_ApplyRootMotionJumpForce::TickTask(float DeltaTime)
{
	if (bIsFinished)
	{
		return;
	}

	const float CurrentTime = GetWorld()->GetTimeSeconds();

	if (bHasLanded && CurrentTime >= (StartTime+MinimumLandedTriggerTime))
	{
		TriggerLanded();
		return;
	}

	Super::TickTask(DeltaTime);

	AActor* MyActor = GetAvatarActor();
	if (MyActor)
	{
		const bool bTimedOut = HasTimedOut();

		if (!bFinishOnLanded && bTimedOut)
		{
			// Task has finished
			Finish();
		}
	}
	else
	{
		bIsFinished = true;
		EndTask();
	}
}

void UAbilityTask_ApplyRootMotionJumpForce::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(UAbilityTask_ApplyRootMotionJumpForce, Rotation);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionJumpForce, Distance);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionJumpForce, Height);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionJumpForce, Duration);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionJumpForce, MinimumLandedTriggerTime);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionJumpForce, bFinishOnLanded);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionJumpForce, PathOffsetCurve);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionJumpForce, TimeMappingCurve);
}

void UAbilityTask_ApplyRootMotionJumpForce::PreDestroyFromReplication()
{
	bIsFinished = true;
	EndTask();
}

void UAbilityTask_ApplyRootMotionJumpForce::OnDestroy(bool AbilityIsEnding)
{
	ACharacter* Character = Cast<ACharacter>(GetAvatarActor());
	if (Character)
	{
		Character->LandedDelegate.RemoveDynamic(this, &UAbilityTask_ApplyRootMotionJumpForce::OnLandedCallback);
	}

	if (MovementComponent.IsValid())
	{
		MovementComponent->RemoveRootMotionSourceByID(RootMotionSourceID);
	}

	Super::OnDestroy(AbilityIsEnding);
}

