// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_ApplyRootMotionMoveToActorForce.h"
#include "Curves/CurveFloat.h"
#include "GameFramework/RootMotionSource.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "DrawDebugHelpers.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"
#include "Net/UnrealNetwork.h"
#include "Abilities/GameplayAbilityTargetTypes.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_ApplyRootMotionMoveToActorForce)


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
int32 DebugMoveToActorForce = 0;
static FAutoConsoleVariableRef CVarDebugMoveToActorForce(
TEXT("AbilitySystem.DebugMoveToActorForce"),
	DebugMoveToActorForce,
	TEXT("Show debug info for MoveToActorForce"),
	ECVF_Default
	);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

UAbilityTask_ApplyRootMotionMoveToActorForce::UAbilityTask_ApplyRootMotionMoveToActorForce(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	bDisableDestinationReachedInterrupt = false;
	ReachedDestinationDistance = 50.0f;
	bSetNewMovementMode = false;
	NewMovementMode = EMovementMode::MOVE_Walking;
	PreviousMovementMode = EMovementMode::MOVE_None;
	PreviousCustomMode = 0;
	TargetLocationOffset = FVector::ZeroVector;
	OffsetAlignment = ERootMotionMoveToActorTargetOffsetType::AlignFromTargetToSource;
	bRestrictSpeedToExpected = false;
	PathOffsetCurve = nullptr;
	TimeMappingCurve = nullptr;
	TargetLerpSpeedHorizontalCurve = nullptr;
	TargetLerpSpeedVerticalCurve = nullptr;
}

void UAbilityTask_ApplyRootMotionMoveToActorForce::OnTargetActorSwapped(class AActor * OriginalTarget, class AActor* NewTarget)
{
	if (OriginalTarget && OriginalTarget == TargetActor)
	{
		TargetActor = NewTarget;

		//This is called from ApplyRootMotionMoveToTargetDataActorForce, which doesn't use TargetComponent for now, so just clearing it out for consistency
		TargetComponent = nullptr;
		TargetComponentRelativeLocation = FVector::ZeroVector;
	}
}

UAbilityTask_ApplyRootMotionMoveToActorForce* UAbilityTask_ApplyRootMotionMoveToActorForce::ApplyRootMotionMoveToActorForce(UGameplayAbility* OwningAbility, FName TaskInstanceName, AActor* TargetActor, FVector TargetLocationOffset, ERootMotionMoveToActorTargetOffsetType OffsetAlignment, float Duration, UCurveFloat* TargetLerpSpeedHorizontal, UCurveFloat* TargetLerpSpeedVertical, bool bSetNewMovementMode, EMovementMode MovementMode, bool bRestrictSpeedToExpected, UCurveVector* PathOffsetCurve, UCurveFloat* TimeMappingCurve, ERootMotionFinishVelocityMode VelocityOnFinishMode, FVector SetVelocityOnFinish, float ClampVelocityOnFinish, bool bDisableDestinationReachedInterrupt, float ReachedDestinationDistance/* = 50.0f*/)
{
	UAbilityTask_ApplyRootMotionMoveToActorForce* MyTask = NewAbilityTask<UAbilityTask_ApplyRootMotionMoveToActorForce>(OwningAbility, TaskInstanceName);

	UAbilitySystemGlobals::NonShipping_ApplyGlobalAbilityScaler_Duration(Duration);

	MyTask->ForceName = TaskInstanceName;
	MyTask->TargetActor = TargetActor;
	MyTask->TargetComponent = nullptr;
	MyTask->TargetComponentRelativeLocation = FVector::ZeroVector;
	MyTask->TargetLocationOffset = TargetLocationOffset;
	MyTask->OffsetAlignment = OffsetAlignment;
	MyTask->Duration = FMath::Max(Duration, KINDA_SMALL_NUMBER); // Avoid negative or divide-by-zero cases
	MyTask->bDisableDestinationReachedInterrupt = bDisableDestinationReachedInterrupt;
	MyTask->ReachedDestinationDistance = ReachedDestinationDistance;
	MyTask->TargetLerpSpeedHorizontalCurve = TargetLerpSpeedHorizontal;
	MyTask->TargetLerpSpeedVerticalCurve = TargetLerpSpeedVertical;
	MyTask->bSetNewMovementMode = bSetNewMovementMode;
	MyTask->NewMovementMode = MovementMode;
	MyTask->bRestrictSpeedToExpected = bRestrictSpeedToExpected;
	MyTask->PathOffsetCurve = PathOffsetCurve;
	MyTask->TimeMappingCurve = TimeMappingCurve;
	MyTask->FinishVelocityMode = VelocityOnFinishMode;
	MyTask->FinishSetVelocity = SetVelocityOnFinish;
	MyTask->FinishClampVelocity = ClampVelocityOnFinish;
	if (MyTask->GetAvatarActor() != nullptr)
	{
		MyTask->StartLocation = MyTask->GetAvatarActor()->GetActorLocation();
	}
	else
	{
		checkf(false, TEXT("UAbilityTask_ApplyRootMotionMoveToActorForce called without valid avatar actor to get start location from."));
		MyTask->StartLocation = TargetActor ? TargetActor->GetActorLocation() : FVector(0.f);
	}
	MyTask->SharedInitAndApply();

	return MyTask;
}

UAbilityTask_ApplyRootMotionMoveToActorForce* UAbilityTask_ApplyRootMotionMoveToActorForce::ApplyRootMotionMoveToComponentForce(UGameplayAbility* OwningAbility, FName TaskInstanceName, USceneComponent* TargetComponent, FVector TargetComponentRelativeLocation, FVector TargetLocationOffset, ERootMotionMoveToActorTargetOffsetType OffsetAlignment, float Duration, UCurveFloat* TargetLerpSpeedHorizontal, UCurveFloat* TargetLerpSpeedVertical, bool bSetNewMovementMode, EMovementMode MovementMode, bool bRestrictSpeedToExpected, UCurveVector* PathOffsetCurve, UCurveFloat* TimeMappingCurve, ERootMotionFinishVelocityMode VelocityOnFinishMode, FVector SetVelocityOnFinish, float ClampVelocityOnFinish, bool bDisableDestinationReachedInterrupt, float ReachedDestinationDistance)
{
	UAbilityTask_ApplyRootMotionMoveToActorForce* MyTask = NewAbilityTask<UAbilityTask_ApplyRootMotionMoveToActorForce>(OwningAbility, TaskInstanceName);

	UAbilitySystemGlobals::NonShipping_ApplyGlobalAbilityScaler_Duration(Duration);

	MyTask->ForceName = TaskInstanceName;
	MyTask->TargetComponent = TargetComponent;
	MyTask->TargetActor = TargetComponent ? TargetComponent->GetOwner() : nullptr;
	MyTask->TargetComponentRelativeLocation = TargetComponentRelativeLocation;
	MyTask->TargetLocationOffset = TargetLocationOffset;
	MyTask->OffsetAlignment = OffsetAlignment;
	MyTask->Duration = FMath::Max(Duration, KINDA_SMALL_NUMBER); // Avoid negative or divide-by-zero cases
	MyTask->bDisableDestinationReachedInterrupt = bDisableDestinationReachedInterrupt;
	MyTask->ReachedDestinationDistance = ReachedDestinationDistance;
	MyTask->TargetLerpSpeedHorizontalCurve = TargetLerpSpeedHorizontal;
	MyTask->TargetLerpSpeedVerticalCurve = TargetLerpSpeedVertical;
	MyTask->bSetNewMovementMode = bSetNewMovementMode;
	MyTask->NewMovementMode = MovementMode;
	MyTask->bRestrictSpeedToExpected = bRestrictSpeedToExpected;
	MyTask->PathOffsetCurve = PathOffsetCurve;
	MyTask->TimeMappingCurve = TimeMappingCurve;
	MyTask->FinishVelocityMode = VelocityOnFinishMode;
	MyTask->FinishSetVelocity = SetVelocityOnFinish;
	MyTask->FinishClampVelocity = ClampVelocityOnFinish;
	if (MyTask->GetAvatarActor() != nullptr)
	{
		MyTask->StartLocation = MyTask->GetAvatarActor()->GetActorLocation();
	}
	else
	{
		checkf(false, TEXT("UAbilityTask_ApplyRootMotionMoveToActorForce called without valid avatar actor to get start location from."));
		MyTask->StartLocation = TargetComponent ? TargetComponent->GetComponentTransform().TransformPosition(TargetComponentRelativeLocation) : FVector(0.f);
	}
	MyTask->SharedInitAndApply();

	return MyTask;
}

void UAbilityTask_ApplyRootMotionMoveToActorForce::OnRep_TargetLocation()
{
	if (bIsSimulating)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Draw debug
		if (DebugMoveToActorForce > 0)
		{
			DrawDebugSphere(GetWorld(), TargetLocation, 50.f, 10, FColor::Green, false, 15.f);
		}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

		SetRootMotionTargetLocation(TargetLocation);
	}
}

void UAbilityTask_ApplyRootMotionMoveToActorForce::SharedInitAndApply()
{
	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (ASC && ASC->AbilityActorInfo->MovementComponent.IsValid())
	{
		MovementComponent = Cast<UCharacterMovementComponent>(ASC->AbilityActorInfo->MovementComponent.Get());
		StartTime = GetWorld()->GetTimeSeconds();
		EndTime = StartTime + Duration;

		if (MovementComponent.IsValid())
		{
			if (bSetNewMovementMode)
			{
				PreviousMovementMode = MovementComponent->MovementMode;
				PreviousCustomMode = MovementComponent->CustomMovementMode;
				MovementComponent->SetMovementMode(NewMovementMode);
			}

			// Set initial target location
			if (TargetActor)
			{
				TargetLocation = CalculateTargetOffset();
			}

			ForceName = ForceName.IsNone() ? FName("AbilityTaskApplyRootMotionMoveToActorForce") : ForceName;
			TSharedPtr<FRootMotionSource_MoveToDynamicForce> MoveToActorForce = MakeShared<FRootMotionSource_MoveToDynamicForce>();
			MoveToActorForce->InstanceName = ForceName;
			MoveToActorForce->AccumulateMode = ERootMotionAccumulateMode::Override;
			MoveToActorForce->Settings.SetFlag(ERootMotionSourceSettingsFlags::UseSensitiveLiftoffCheck);
			MoveToActorForce->Priority = 900;
			MoveToActorForce->InitialTargetLocation = TargetLocation;
			MoveToActorForce->TargetLocation = TargetLocation;
			MoveToActorForce->StartLocation = StartLocation;
			MoveToActorForce->Duration = FMath::Max(Duration, KINDA_SMALL_NUMBER);
			MoveToActorForce->bRestrictSpeedToExpected = bRestrictSpeedToExpected;
			MoveToActorForce->PathOffsetCurve = PathOffsetCurve;
			MoveToActorForce->TimeMappingCurve = TimeMappingCurve;
			MoveToActorForce->FinishVelocityParams.Mode = FinishVelocityMode;
			MoveToActorForce->FinishVelocityParams.SetVelocity = FinishSetVelocity;
			MoveToActorForce->FinishVelocityParams.ClampVelocity = FinishClampVelocity;
			RootMotionSourceID = MovementComponent->ApplyRootMotionSource(MoveToActorForce);
		}
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilityTask_ApplyRootMotionMoveToActorForce called in Ability %s with null MovementComponent; Task Instance Name %s."), 
			Ability ? *Ability->GetName() : TEXT("NULL"), 
			*InstanceName.ToString());
	}
}

FVector UAbilityTask_ApplyRootMotionMoveToActorForce::CalculateTargetOffset() const
{
	check(TargetActor != nullptr);

	const FVector TargetActorLocation = TargetComponent ?
											TargetComponent->GetComponentTransform().TransformPosition(TargetComponentRelativeLocation) :
											TargetActor->GetActorLocation();
	FVector CalculatedTargetLocation = TargetActorLocation;
	
	if (OffsetAlignment == ERootMotionMoveToActorTargetOffsetType::AlignFromTargetToSource)
	{
		if (MovementComponent.IsValid())
		{
			FVector ToSource = MovementComponent->GetActorLocation() - TargetActorLocation;
			ToSource.Z = 0.f;
			CalculatedTargetLocation += ToSource.ToOrientationQuat().RotateVector(TargetLocationOffset);
		}

	}
	else if (OffsetAlignment == ERootMotionMoveToActorTargetOffsetType::AlignToTargetForward)
	{
		CalculatedTargetLocation += TargetActor->GetActorQuat().RotateVector(TargetLocationOffset);
	}
	else if (OffsetAlignment == ERootMotionMoveToActorTargetOffsetType::AlignToWorldSpace)
	{
		CalculatedTargetLocation += TargetLocationOffset;
	}
	
	return CalculatedTargetLocation;
}

bool UAbilityTask_ApplyRootMotionMoveToActorForce::UpdateTargetLocation(float DeltaTime)
{
	if (TargetActor && GetWorld())
	{
		const FVector PreviousTargetLocation = TargetLocation;
		FVector ExactTargetLocation = CalculateTargetOffset();

		const float CurrentTime = GetWorld()->GetTimeSeconds();
		const float CompletionPercent = (CurrentTime - StartTime) / Duration;

		const float TargetLerpSpeedHorizontal = TargetLerpSpeedHorizontalCurve ? TargetLerpSpeedHorizontalCurve->GetFloatValue(CompletionPercent) : 1000.f;
		const float TargetLerpSpeedVertical = TargetLerpSpeedVerticalCurve ? TargetLerpSpeedVerticalCurve->GetFloatValue(CompletionPercent) : 500.f;

		const float MaxHorizontalChange = FMath::Max(0.f, TargetLerpSpeedHorizontal * DeltaTime);
		const float MaxVerticalChange = FMath::Max(0.f, TargetLerpSpeedVertical * DeltaTime);

		FVector ToExactLocation = ExactTargetLocation - PreviousTargetLocation;
		FVector TargetLocationDelta = ToExactLocation;

		// Cap vertical lerp
		if (FMath::Abs(ToExactLocation.Z) > MaxVerticalChange)
		{
			if (ToExactLocation.Z >= 0.f)
			{
				TargetLocationDelta.Z = MaxVerticalChange;
			}
			else
			{
				TargetLocationDelta.Z = -MaxVerticalChange;
			}
		}

		// Cap horizontal lerp
		if (FMath::Abs(ToExactLocation.SizeSquared2D()) > MaxHorizontalChange*MaxHorizontalChange)
		{
			FVector ToExactLocationHorizontal(ToExactLocation.X, ToExactLocation.Y, 0.f);
			ToExactLocationHorizontal.Normalize();
			ToExactLocationHorizontal *= MaxHorizontalChange;

			TargetLocationDelta.X = ToExactLocationHorizontal.X;
			TargetLocationDelta.Y = ToExactLocationHorizontal.Y;
		}

		TargetLocation += TargetLocationDelta;

		return true;
	}

	return false;
}

void UAbilityTask_ApplyRootMotionMoveToActorForce::SetRootMotionTargetLocation(FVector NewTargetLocation)
{
	if (MovementComponent.IsValid())
	{
		TSharedPtr<FRootMotionSource> RMS = MovementComponent->GetRootMotionSourceByID(RootMotionSourceID);
		if (RMS.IsValid())
		{
			if (RMS->GetScriptStruct() == FRootMotionSource_MoveToDynamicForce::StaticStruct())
			{
				FRootMotionSource_MoveToDynamicForce* MoveToActorForce = static_cast<FRootMotionSource_MoveToDynamicForce*>(RMS.Get());
				if (MoveToActorForce)
				{
					MoveToActorForce->SetTargetLocation(TargetLocation);
				}
			}
		}
	}
}

void UAbilityTask_ApplyRootMotionMoveToActorForce::TickTask(float DeltaTime)
{
	if (bIsFinished)
	{
		return;
	}

	Super::TickTask(DeltaTime);

	AActor* MyActor = GetAvatarActor();
	if (MyActor)
	{
		const bool bTimedOut = HasTimedOut();

		// Update target location
		{
			const FVector PreviousTargetLocation = TargetLocation;
			if (UpdateTargetLocation(DeltaTime))
			{
				SetRootMotionTargetLocation(TargetLocation);
			}
			else
			{
				// TargetLocation not updated - TargetActor not around anymore, continue on to last set TargetLocation
			}
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Draw debug
		if (DebugMoveToActorForce > 0)
		{
			DrawDebugSphere(GetWorld(), TargetLocation, 50.f, 10, FColor::Green, false, 15.f);
		}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

		const float ReachedDestinationDistanceSqr = ReachedDestinationDistance * ReachedDestinationDistance;
		const bool bReachedDestination = FVector::DistSquared(TargetLocation, MyActor->GetActorLocation()) < ReachedDestinationDistanceSqr;

		if (bTimedOut || (bReachedDestination && !bDisableDestinationReachedInterrupt))
		{
			// Task has finished
			bIsFinished = true;
			if (!bIsSimulating)
			{
				MyActor->ForceNetUpdate();
				if (ShouldBroadcastAbilityTaskDelegates())
				{
					OnFinished.Broadcast(bReachedDestination, bTimedOut, TargetLocation);
				}
				EndTask();
			}
		}
	}
	else
	{
		bIsFinished = true;
		EndTask();
	}
}

void UAbilityTask_ApplyRootMotionMoveToActorForce::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, StartLocation);
	DOREPLIFETIME_CONDITION(UAbilityTask_ApplyRootMotionMoveToActorForce, TargetLocation, COND_SimulatedOnly); // Autonomous and server calculate target location independently
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, TargetActor);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, TargetLocationOffset);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, OffsetAlignment);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, Duration);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, bDisableDestinationReachedInterrupt);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, ReachedDestinationDistance);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, TargetLerpSpeedHorizontalCurve);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, TargetLerpSpeedVerticalCurve);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, bSetNewMovementMode);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, NewMovementMode);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, bRestrictSpeedToExpected);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, PathOffsetCurve);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToActorForce, TimeMappingCurve);
}

void UAbilityTask_ApplyRootMotionMoveToActorForce::PreDestroyFromReplication()
{
	bIsFinished = true;
	EndTask();
}

void UAbilityTask_ApplyRootMotionMoveToActorForce::OnDestroy(bool AbilityIsEnding)
{
	if (TargetActorSwapHandle.IsValid())
	{
		UAbilityTask_ApplyRootMotion_Base::OnTargetActorSwapped.Remove( TargetActorSwapHandle );
	}

	if (MovementComponent.IsValid())
	{
		MovementComponent->RemoveRootMotionSourceByID(RootMotionSourceID);

		if (bSetNewMovementMode)
		{
			MovementComponent->SetMovementMode(PreviousMovementMode, PreviousCustomMode);
		}
	}

	Super::OnDestroy(AbilityIsEnding);
}

UAbilityTask_ApplyRootMotionMoveToActorForce* UAbilityTask_ApplyRootMotionMoveToActorForce::ApplyRootMotionMoveToTargetDataActorForce(UGameplayAbility* OwningAbility, FName TaskInstanceName, FGameplayAbilityTargetDataHandle TargetDataHandle, int32 TargetDataIndex, int32 TargetActorIndex, FVector TargetLocationOffset, ERootMotionMoveToActorTargetOffsetType OffsetAlignment, float Duration, UCurveFloat* TargetLerpSpeedHorizontal, UCurveFloat* TargetLerpSpeedVertical, bool bSetNewMovementMode, EMovementMode MovementMode, bool bRestrictSpeedToExpected, UCurveVector* PathOffsetCurve, UCurveFloat* TimeMappingCurve, ERootMotionFinishVelocityMode VelocityOnFinishMode, FVector SetVelocityOnFinish, float ClampVelocityOnFinish, bool bDisableDestinationReachedInterrupt, float ReachedDestinationDistance/* = 50.0f*/)
{
	if (TargetDataIndex >= 0 && TargetDataIndex < TargetDataHandle.Num())
	{
		FGameplayAbilityTargetData* TargetData = TargetDataHandle.Get(TargetDataIndex);
		
		if (TargetData)
		{
			TArray<TWeakObjectPtr<AActor> > Actors = TargetData->GetActors();

			if (TargetActorIndex >= 0 && TargetActorIndex < Actors.Num())
			{
				if (Actors[TargetActorIndex].IsValid())
				{
					AActor* TargetActor = Actors[TargetActorIndex].Get();
					UAbilityTask_ApplyRootMotionMoveToActorForce* retVal = ApplyRootMotionMoveToActorForce(OwningAbility, TaskInstanceName, TargetActor, TargetLocationOffset, OffsetAlignment, Duration, TargetLerpSpeedHorizontal, TargetLerpSpeedVertical, bSetNewMovementMode, MovementMode, bRestrictSpeedToExpected, PathOffsetCurve, TimeMappingCurve, VelocityOnFinishMode, SetVelocityOnFinish, ClampVelocityOnFinish, bDisableDestinationReachedInterrupt, ReachedDestinationDistance);

					if (TargetData->ShouldCheckForTargetActorSwap())
					{
						retVal->TargetActorSwapHandle = UAbilityTask_ApplyRootMotion_Base::OnTargetActorSwapped.AddUObject( retVal, &ThisClass::OnTargetActorSwapped );
					}
					return retVal;
				}
			}
		}
	}

	return nullptr;
}


