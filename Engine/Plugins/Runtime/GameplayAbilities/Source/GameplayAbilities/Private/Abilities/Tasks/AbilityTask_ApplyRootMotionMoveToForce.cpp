// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_ApplyRootMotionMoveToForce.h"
#include "GameFramework/RootMotionSource.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"
#include "Net/UnrealNetwork.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_ApplyRootMotionMoveToForce)

UAbilityTask_ApplyRootMotionMoveToForce* UAbilityTask_ApplyRootMotionMoveToForce::ApplyRootMotionMoveToForce(UGameplayAbility* OwningAbility, FName TaskInstanceName, FVector TargetLocation, float Duration, bool bSetNewMovementMode, EMovementMode MovementMode, bool bRestrictSpeedToExpected, UCurveVector* PathOffsetCurve, ERootMotionFinishVelocityMode VelocityOnFinishMode, FVector SetVelocityOnFinish, float ClampVelocityOnFinish)
{
	UAbilitySystemGlobals::NonShipping_ApplyGlobalAbilityScaler_Duration(Duration);

	UAbilityTask_ApplyRootMotionMoveToForce* MyTask = NewAbilityTask<UAbilityTask_ApplyRootMotionMoveToForce>(OwningAbility, TaskInstanceName);

	MyTask->ForceName = TaskInstanceName;
	MyTask->TargetLocation = TargetLocation;
	MyTask->Duration = FMath::Max(Duration, KINDA_SMALL_NUMBER); // Avoid negative or divide-by-zero cases
	MyTask->bSetNewMovementMode = bSetNewMovementMode;
	MyTask->NewMovementMode = MovementMode;
	MyTask->bRestrictSpeedToExpected = bRestrictSpeedToExpected;
	MyTask->PathOffsetCurve = PathOffsetCurve;
	MyTask->FinishVelocityMode = VelocityOnFinishMode;
	MyTask->FinishSetVelocity = SetVelocityOnFinish;
	MyTask->FinishClampVelocity = ClampVelocityOnFinish;
	if (MyTask->GetAvatarActor() != nullptr)
	{
		MyTask->StartLocation = MyTask->GetAvatarActor()->GetActorLocation();
	}
	else
	{
		checkf(false, TEXT("UAbilityTask_ApplyRootMotionMoveToForce called without valid avatar actor to get start location from."));
		MyTask->StartLocation = TargetLocation;
	}
	MyTask->SharedInitAndApply();

	return MyTask;
}

void UAbilityTask_ApplyRootMotionMoveToForce::SharedInitAndApply()
{
	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (ASC && ASC->AbilityActorInfo->MovementComponent.IsValid())
	{
		MovementComponent = Cast<UCharacterMovementComponent>(ASC->AbilityActorInfo->MovementComponent.Get());
		StartTime = GetWorld()->GetTimeSeconds();
		EndTime = StartTime + Duration;

		if (MovementComponent.IsValid())
		{
			if (bSetNewMovementMode)
			{
				PreviousMovementMode = MovementComponent->MovementMode;
				PreviousCustomMovementMode = MovementComponent->CustomMovementMode;
				MovementComponent->SetMovementMode(NewMovementMode);
			}

			ForceName = ForceName.IsNone() ? FName("AbilityTaskApplyRootMotionMoveToForce") : ForceName;
			TSharedPtr<FRootMotionSource_MoveToForce> MoveToForce = MakeShared<FRootMotionSource_MoveToForce>();
			MoveToForce->InstanceName = ForceName;
			MoveToForce->AccumulateMode = ERootMotionAccumulateMode::Override;
			MoveToForce->Settings.SetFlag(ERootMotionSourceSettingsFlags::UseSensitiveLiftoffCheck);
			MoveToForce->Priority = 1000;
			MoveToForce->TargetLocation = TargetLocation;
			MoveToForce->StartLocation = StartLocation;
			MoveToForce->Duration = Duration;
			MoveToForce->bRestrictSpeedToExpected = bRestrictSpeedToExpected;
			MoveToForce->PathOffsetCurve = PathOffsetCurve;
			MoveToForce->FinishVelocityParams.Mode = FinishVelocityMode;
			MoveToForce->FinishVelocityParams.SetVelocity = FinishSetVelocity;
			MoveToForce->FinishVelocityParams.ClampVelocity = FinishClampVelocity;
			RootMotionSourceID = MovementComponent->ApplyRootMotionSource(MoveToForce);
		}
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilityTask_ApplyRootMotionMoveToForce called in Ability %s with null MovementComponent; Task Instance Name %s."), 
			Ability ? *Ability->GetName() : TEXT("NULL"), 
			*InstanceName.ToString());
	}
}

void UAbilityTask_ApplyRootMotionMoveToForce::TickTask(float DeltaTime)
{
	if (bIsFinished)
	{
		return;
	}

	Super::TickTask(DeltaTime);

	AActor* MyActor = GetAvatarActor();
	if (MyActor)
	{
		const bool bTimedOut = HasTimedOut();
		const float ReachedDestinationDistanceSqr = 50.f * 50.f;
		const bool bReachedDestination = FVector::DistSquared(TargetLocation, MyActor->GetActorLocation()) < ReachedDestinationDistanceSqr;

		if (bTimedOut)
		{
			// Task has finished
			bIsFinished = true;
			if (!bIsSimulating)
			{
				MyActor->ForceNetUpdate();
				if (bReachedDestination)
				{
					if (ShouldBroadcastAbilityTaskDelegates())
					{
						OnTimedOutAndDestinationReached.Broadcast();
					}
				}
				else
				{
					if (ShouldBroadcastAbilityTaskDelegates())
					{
						OnTimedOut.Broadcast();
					}
				}
				EndTask();
			}
		}
	}
	else
	{
		bIsFinished = true;
		EndTask();
	}
}

void UAbilityTask_ApplyRootMotionMoveToForce::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToForce, StartLocation);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToForce, TargetLocation);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToForce, Duration);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToForce, bSetNewMovementMode);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToForce, NewMovementMode);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToForce, bRestrictSpeedToExpected);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionMoveToForce, PathOffsetCurve);
}

void UAbilityTask_ApplyRootMotionMoveToForce::PreDestroyFromReplication()
{
	bIsFinished = true;
	EndTask();
}

void UAbilityTask_ApplyRootMotionMoveToForce::OnDestroy(bool AbilityIsEnding)
{
	if (MovementComponent.IsValid())
	{
		MovementComponent->RemoveRootMotionSourceByID(RootMotionSourceID);

		if (bSetNewMovementMode)
		{
 			MovementComponent->SetMovementMode(PreviousMovementMode, PreviousCustomMovementMode);
		}
	}

	Super::OnDestroy(AbilityIsEnding);
}

