// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_ApplyRootMotionRadialForce.h"
#include "GameFramework/RootMotionSource.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"
#include "Net/UnrealNetwork.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_ApplyRootMotionRadialForce)

UAbilityTask_ApplyRootMotionRadialForce::UAbilityTask_ApplyRootMotionRadialForce(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	StrengthDistanceFalloff = nullptr;
	StrengthOverTime = nullptr;
	bUseFixedWorldDirection = false;
}

UAbilityTask_ApplyRootMotionRadialForce* UAbilityTask_ApplyRootMotionRadialForce::ApplyRootMotionRadialForce(UGameplayAbility* OwningAbility, FName TaskInstanceName, FVector Location, AActor* LocationActor, float Strength, float Duration, float Radius, bool bIsPush, bool bIsAdditive, bool bNoZForce, UCurveFloat* StrengthDistanceFalloff, UCurveFloat* StrengthOverTime, bool bUseFixedWorldDirection, FRotator FixedWorldDirection, ERootMotionFinishVelocityMode VelocityOnFinishMode, FVector SetVelocityOnFinish, float ClampVelocityOnFinish)
{
	UAbilitySystemGlobals::NonShipping_ApplyGlobalAbilityScaler_Duration(Duration);

	UAbilityTask_ApplyRootMotionRadialForce* MyTask = NewAbilityTask<UAbilityTask_ApplyRootMotionRadialForce>(OwningAbility, TaskInstanceName);

	MyTask->ForceName = TaskInstanceName;
	MyTask->Location = Location;
	MyTask->LocationActor = LocationActor;
	MyTask->Strength = Strength;
	MyTask->Radius = FMath::Max(Radius, SMALL_NUMBER); // No zero radius
	MyTask->Duration = Duration;
	MyTask->bIsPush = bIsPush;
	MyTask->bIsAdditive = bIsAdditive;
	MyTask->bNoZForce = bNoZForce;
	MyTask->StrengthDistanceFalloff = StrengthDistanceFalloff;
	MyTask->StrengthOverTime = StrengthOverTime;
	MyTask->bUseFixedWorldDirection = bUseFixedWorldDirection;
	MyTask->FixedWorldDirection = FixedWorldDirection;
	MyTask->FinishVelocityMode = VelocityOnFinishMode;
	MyTask->FinishSetVelocity = SetVelocityOnFinish;
	MyTask->FinishClampVelocity = ClampVelocityOnFinish;
	MyTask->SharedInitAndApply();

	return MyTask;
}

void UAbilityTask_ApplyRootMotionRadialForce::SharedInitAndApply()
{
	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (ASC && ASC->AbilityActorInfo->MovementComponent.IsValid())
	{
		MovementComponent = Cast<UCharacterMovementComponent>(ASC->AbilityActorInfo->MovementComponent.Get());
		StartTime = GetWorld()->GetTimeSeconds();
		EndTime = StartTime + Duration;

		if (MovementComponent.IsValid())
		{
			ForceName = ForceName.IsNone() ? FName("AbilityTaskApplyRootMotionRadialForce") : ForceName;
			TSharedPtr<FRootMotionSource_RadialForce> RadialForce = MakeShared<FRootMotionSource_RadialForce>();
			RadialForce->InstanceName = ForceName;
			RadialForce->AccumulateMode = bIsAdditive ? ERootMotionAccumulateMode::Additive : ERootMotionAccumulateMode::Override;
			RadialForce->Priority = 5;
			RadialForce->Location = Location;
			RadialForce->LocationActor = LocationActor;
			RadialForce->Duration = Duration;
			RadialForce->Radius = Radius;
			RadialForce->Strength = Strength;
			RadialForce->bIsPush = bIsPush;
			RadialForce->bNoZForce = bNoZForce;
			RadialForce->StrengthDistanceFalloff = StrengthDistanceFalloff;
			RadialForce->StrengthOverTime = StrengthOverTime;
			RadialForce->bUseFixedWorldDirection = bUseFixedWorldDirection;
			RadialForce->FixedWorldDirection = FixedWorldDirection;
			RadialForce->FinishVelocityParams.Mode = FinishVelocityMode;
			RadialForce->FinishVelocityParams.SetVelocity = FinishSetVelocity;
			RadialForce->FinishVelocityParams.ClampVelocity = FinishClampVelocity;
			RootMotionSourceID = MovementComponent->ApplyRootMotionSource(RadialForce);
		}
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilityTask_ApplyRootMotionRadialForce called in Ability %s with null MovementComponent; Task Instance Name %s."), 
			Ability ? *Ability->GetName() : TEXT("NULL"), 
			*InstanceName.ToString());
	}
}

void UAbilityTask_ApplyRootMotionRadialForce::TickTask(float DeltaTime)
{
	if (bIsFinished)
	{
		return;
	}

	Super::TickTask(DeltaTime);

	AActor* MyActor = GetAvatarActor();
	if (MyActor)
	{
		const bool bTimedOut = HasTimedOut();
		const bool bIsInfiniteDuration = Duration < 0.f;

		if (!bIsInfiniteDuration && bTimedOut)
		{
			// Task has finished
			bIsFinished = true;
			if (!bIsSimulating)
			{
				MyActor->ForceNetUpdate();
				if (ShouldBroadcastAbilityTaskDelegates())
				{
					OnFinish.Broadcast();
				}
				EndTask();
			}
		}
	}
	else
	{
		bIsFinished = true;
		EndTask();
	}
}

void UAbilityTask_ApplyRootMotionRadialForce::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(UAbilityTask_ApplyRootMotionRadialForce, Location);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionRadialForce, LocationActor);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionRadialForce, Radius);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionRadialForce, Strength);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionRadialForce, Duration);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionRadialForce, bIsPush);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionRadialForce, bIsAdditive);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionRadialForce, bNoZForce);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionRadialForce, StrengthDistanceFalloff);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionRadialForce, StrengthOverTime);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionRadialForce, bUseFixedWorldDirection);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotionRadialForce, FixedWorldDirection);
}

void UAbilityTask_ApplyRootMotionRadialForce::PreDestroyFromReplication()
{
	bIsFinished = true;
	EndTask();
}

void UAbilityTask_ApplyRootMotionRadialForce::OnDestroy(bool AbilityIsEnding)
{
	if (MovementComponent.IsValid())
	{
		MovementComponent->RemoveRootMotionSourceByID(RootMotionSourceID);
	}

	Super::OnDestroy(AbilityIsEnding);
}

