// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_ApplyRootMotion_Base.h"
#include "AbilitySystemComponent.h"
#include "Net/UnrealNetwork.h"
#include "GameFramework/RootMotionSource.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "GameFramework/Character.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_ApplyRootMotion_Base)

FOnTargetActorSwapped UAbilityTask_ApplyRootMotion_Base::OnTargetActorSwapped;

UAbilityTask_ApplyRootMotion_Base::UAbilityTask_ApplyRootMotion_Base(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bTickingTask = true;
	bSimulatedTask = true;

	ForceName = NAME_None;
	FinishVelocityMode = ERootMotionFinishVelocityMode::MaintainLastRootMotionVelocity;
	FinishSetVelocity = FVector::ZeroVector;
	FinishClampVelocity = 0.0f;
	MovementComponent = nullptr;
	RootMotionSourceID = (uint16)ERootMotionSourceID::Invalid;
	bIsFinished = false;
	StartTime = 0.0f;
	EndTime = 0.0f;
}

void UAbilityTask_ApplyRootMotion_Base::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	DOREPLIFETIME(UAbilityTask_ApplyRootMotion_Base, ForceName);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotion_Base, FinishVelocityMode);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotion_Base, FinishSetVelocity);
	DOREPLIFETIME(UAbilityTask_ApplyRootMotion_Base, FinishClampVelocity);
}

void UAbilityTask_ApplyRootMotion_Base::InitSimulatedTask(UGameplayTasksComponent& InGameplayTasksComponent)
{
	Super::InitSimulatedTask(InGameplayTasksComponent);

	SharedInitAndApply();
}

bool UAbilityTask_ApplyRootMotion_Base::HasTimedOut() const
{
	const TSharedPtr<FRootMotionSource> RMS = (MovementComponent.IsValid() ? MovementComponent->GetRootMotionSourceByID(RootMotionSourceID) : nullptr);
	if (!RMS.IsValid())
	{
		return true;
	}

	return RMS->Status.HasFlag(ERootMotionSourceStatusFlags::Finished);
}


