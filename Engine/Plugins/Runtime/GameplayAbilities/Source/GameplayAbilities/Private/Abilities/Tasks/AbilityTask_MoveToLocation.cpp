// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_MoveToLocation.h"
#include "Curves/CurveFloat.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Curves/CurveVector.h"
#include "Net/UnrealNetwork.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_MoveToLocation)

UAbilityTask_MoveToLocation::UAbilityTask_MoveToLocation(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	bTickingTask = true;
	bSimulatedTask = true;
	bIsFinished = false;
}

UAbilityTask_MoveToLocation* UAbilityTask_MoveToLocation::MoveToLocation(UGameplayAbility* OwningAbility, FName TaskInstanceName, FVector Location, float Duration, UCurveFloat* OptionalInterpolationCurve, UCurveVector* OptionalVectorInterpolationCurve)
{
	UAbilityTask_MoveToLocation* MyObj = NewAbilityTask<UAbilityTask_MoveToLocation>(OwningAbility, TaskInstanceName);

	if (MyObj->GetAvatarActor() != nullptr)
	{
		MyObj->StartLocation = MyObj->GetAvatarActor()->GetActorLocation();
	}

	MyObj->TargetLocation = Location;
	MyObj->DurationOfMovement = FMath::Max(Duration, 0.001f);		// Avoid negative or divide-by-zero cases
	MyObj->TimeMoveStarted = MyObj->GetWorld()->GetTimeSeconds();
	MyObj->TimeMoveWillEnd = MyObj->TimeMoveStarted + MyObj->DurationOfMovement;
	MyObj->LerpCurve = OptionalInterpolationCurve;
	MyObj->LerpCurveVector = OptionalVectorInterpolationCurve;

	return MyObj;
}

void UAbilityTask_MoveToLocation::Activate()
{

}

void UAbilityTask_MoveToLocation::InitSimulatedTask(UGameplayTasksComponent& InGameplayTasksComponent)
{
	Super::InitSimulatedTask(InGameplayTasksComponent);

	TimeMoveStarted = GetWorld()->GetTimeSeconds();
	TimeMoveWillEnd = TimeMoveStarted + DurationOfMovement;
}

//TODO: This is still an awful way to do this and we should scrap this task or do it right.
void UAbilityTask_MoveToLocation::TickTask(float DeltaTime)
{
	if (bIsFinished)
	{
		return;
	}

	Super::TickTask(DeltaTime);
	AActor* MyActor = GetAvatarActor();
	if (MyActor)
	{
		ACharacter* MyCharacter = Cast<ACharacter>(MyActor);
		if (MyCharacter)
		{
			UCharacterMovementComponent* CharMoveComp = Cast<UCharacterMovementComponent>(MyCharacter->GetMovementComponent());
			if (CharMoveComp)
			{
				CharMoveComp->SetMovementMode(MOVE_Custom, 0);
			}
		}


		float CurrentTime = GetWorld()->GetTimeSeconds();

		if (CurrentTime >= TimeMoveWillEnd)
		{
			bIsFinished = true;

			// Teleport in attempt to find a valid collision spot
			MyActor->TeleportTo(TargetLocation, MyActor->GetActorRotation());
			if (!bIsSimulating)
			{
				MyActor->ForceNetUpdate();
				if (ShouldBroadcastAbilityTaskDelegates())
				{
					OnTargetLocationReached.Broadcast();
				}
				EndTask();
			}
		}
		else
		{
			FVector NewLocation;

			float MoveFraction = (CurrentTime - TimeMoveStarted) / DurationOfMovement;
			if (LerpCurveVector)
			{
				const FVector ComponentInterpolationFraction = LerpCurveVector->GetVectorValue(MoveFraction);
				NewLocation = FMath::Lerp<FVector, FVector>(StartLocation, TargetLocation, ComponentInterpolationFraction);
			}
			else
			{
				if (LerpCurve)
				{
					MoveFraction = LerpCurve->GetFloatValue(MoveFraction);
				}

				NewLocation = FMath::Lerp<FVector, float>(StartLocation, TargetLocation, MoveFraction);
			}

			MyActor->SetActorLocation(NewLocation);
		}
	}
	else
	{
		bIsFinished = true;
		EndTask();
	}
}

void UAbilityTask_MoveToLocation::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	DOREPLIFETIME(UAbilityTask_MoveToLocation, StartLocation);
	DOREPLIFETIME(UAbilityTask_MoveToLocation, TargetLocation);
	DOREPLIFETIME(UAbilityTask_MoveToLocation, DurationOfMovement);
	DOREPLIFETIME(UAbilityTask_MoveToLocation, LerpCurve);
	DOREPLIFETIME(UAbilityTask_MoveToLocation, LerpCurveVector);
}

void UAbilityTask_MoveToLocation::OnDestroy(bool AbilityIsEnding)
{
	AActor* MyActor = GetAvatarActor();
	if (MyActor)
	{
		ACharacter* MyCharacter = Cast<ACharacter>(MyActor);
		if (MyCharacter)
		{
			UCharacterMovementComponent* CharMoveComp = Cast<UCharacterMovementComponent>(MyCharacter->GetMovementComponent());
			if (CharMoveComp && CharMoveComp->MovementMode == MOVE_Custom)
			{
				CharMoveComp->SetMovementMode(MOVE_Falling);
			}
		}
	}

	Super::OnDestroy(AbilityIsEnding);
}

