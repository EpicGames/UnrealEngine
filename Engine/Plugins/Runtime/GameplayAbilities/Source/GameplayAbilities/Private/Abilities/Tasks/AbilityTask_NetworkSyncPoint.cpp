// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_NetworkSyncPoint.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_NetworkSyncPoint)

UAbilityTask_NetworkSyncPoint::UAbilityTask_NetworkSyncPoint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ReplicatedEventToListenFor = EAbilityGenericReplicatedEvent::MAX;
}

void UAbilityTask_NetworkSyncPoint::OnSignalCallback()
{
	if (AbilitySystemComponent.IsValid())
	{
		AbilitySystemComponent->ConsumeGenericReplicatedEvent(ReplicatedEventToListenFor, GetAbilitySpecHandle(), GetActivationPredictionKey());
	}
	SyncFinished();
}

UAbilityTask_NetworkSyncPoint* UAbilityTask_NetworkSyncPoint::WaitNetSync(class UGameplayAbility* OwningAbility, EAbilityTaskNetSyncType InSyncType)
{
	UAbilityTask_NetworkSyncPoint* MyObj = NewAbilityTask<UAbilityTask_NetworkSyncPoint>(OwningAbility);
	MyObj->SyncType = InSyncType;
	return MyObj;
}

void UAbilityTask_NetworkSyncPoint::Activate()
{
	FScopedPredictionWindow ScopedPrediction(AbilitySystemComponent.Get(), IsPredictingClient());

	if (AbilitySystemComponent.IsValid())
	{
		if (IsPredictingClient())
		{
			if (SyncType != EAbilityTaskNetSyncType::OnlyServerWait )
			{
				// As long as we are waiting (!= OnlyServerWait), listen for the GenericSignalFromServer event
				ReplicatedEventToListenFor = EAbilityGenericReplicatedEvent::GenericSignalFromServer;
			}
			if (SyncType != EAbilityTaskNetSyncType::OnlyClientWait)
			{
				// As long as the server is waiting (!= OnlyClientWait), send the Server and RPC for this signal
				AbilitySystemComponent->ServerSetReplicatedEvent(EAbilityGenericReplicatedEvent::GenericSignalFromClient, GetAbilitySpecHandle(), GetActivationPredictionKey(), AbilitySystemComponent->ScopedPredictionKey);
			}
			
		}
		else if (IsForRemoteClient())
		{
			if (SyncType != EAbilityTaskNetSyncType::OnlyClientWait )
			{
				// As long as we are waiting (!= OnlyClientWait), listen for the GenericSignalFromClient event
				ReplicatedEventToListenFor = EAbilityGenericReplicatedEvent::GenericSignalFromClient;
			}
			if (SyncType != EAbilityTaskNetSyncType::OnlyServerWait)
			{
				// As long as the client is waiting (!= OnlyServerWait), send the Server and RPC for this signal
				AbilitySystemComponent->ClientSetReplicatedEvent(EAbilityGenericReplicatedEvent::GenericSignalFromServer, GetAbilitySpecHandle(), GetActivationPredictionKey());
			}
		}

		if (ReplicatedEventToListenFor != EAbilityGenericReplicatedEvent::MAX)
		{
			CallOrAddReplicatedDelegate(ReplicatedEventToListenFor, FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UAbilityTask_NetworkSyncPoint::OnSignalCallback));
		}
		else
		{
			// We aren't waiting for a replicated event, so the sync is complete.
			SyncFinished();
		}
	}
}

void UAbilityTask_NetworkSyncPoint::SyncFinished()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (IsThisNotNull(this, "UAbilityTask_NetworkSyncPoint::SyncFinished") && IsValidChecked(this))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnSync.Broadcast();
		}
		EndTask();
	}
}

