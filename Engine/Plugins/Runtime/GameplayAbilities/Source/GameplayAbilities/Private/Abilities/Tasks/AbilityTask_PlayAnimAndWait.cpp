// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_PlayAnimAndWait.h"
#include "GameFramework/Character.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "AbilitySystemLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_PlayAnimAndWait)

void UAbilityTask_PlayAnimAndWait::OnMontageBlendingOut(UAnimMontage* Montage, bool bInterrupted)
{
	if (Montage && Ability && Ability->GetCurrentMontage() == Montage)
	{
		if (Montage->GetFirstAnimReference() == AnimSequenceToPlay)
		{
			if (UAbilitySystemComponent* ASC = AbilitySystemComponent.Get())
			{
				ASC->ClearAnimatingAbility(Ability);
			}

			// Reset AnimRootMotionTranslationScale
			ACharacter* Character = Cast<ACharacter>(GetAvatarActor());
			if (Character && (Character->GetLocalRole() == ROLE_Authority ||
							  (Character->GetLocalRole() == ROLE_AutonomousProxy && Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalPredicted)))
			{
				Character->SetAnimRootMotionTranslationScale(1.f);
			}

		}
	}

	if (bInterrupted)
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnInterrupted.Broadcast();
		}
	}
	else
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnBlendOut.Broadcast();
		}
	}
}

void UAbilityTask_PlayAnimAndWait::OnMontageBlendedIn(UAnimMontage* Montage)
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnBlendIn.Broadcast();
	}
}

void UAbilityTask_PlayAnimAndWait::OnMontageInterrupted()
{
	if (StopPlayingMontage())
	{
		// Let the BP handle the interrupt as well
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnInterrupted.Broadcast();
		}
	}
}

void UAbilityTask_PlayAnimAndWait::OnMontageEnded(UAnimMontage* Montage, bool bInterrupted)
{
	if (!bInterrupted)
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnCompleted.Broadcast();
		}
	}

	EndTask();
}

UAbilityTask_PlayAnimAndWait* UAbilityTask_PlayAnimAndWait::CreatePlayAnimAndWaitProxy(
	UGameplayAbility* OwningAbility,
	FName TaskInstanceName, 
	UAnimSequence* AnimSequence, 
	FName SlotName, 
	float BlendInTime /*= 0.25f*/,
	float BlendOutTime /*= 0.25f*/,
	float InPlayRate /*= 1.0f*/,
	float StartTimeSeconds /*= 0.0f*/, 
	bool bStopWhenAbilityEnds /*= true*/, 
	float AnimRootMotionTranslationScale /*= 1.f*/,
	float InPlayCount /*= 1.0f*/)
{
	UAbilityTask_PlayAnimAndWait* PlayAnimTask = NewAbilityTask<UAbilityTask_PlayAnimAndWait>(OwningAbility, TaskInstanceName);
	PlayAnimTask->AnimSequenceToPlay = AnimSequence;
	PlayAnimTask->SlotName = SlotName;
	PlayAnimTask->BlendInTime = BlendInTime;
	PlayAnimTask->BlendOutTime = BlendOutTime;
	PlayAnimTask->PlayRate = InPlayRate;
	PlayAnimTask->PlayCount = InPlayCount;
	PlayAnimTask->AnimRootMotionTranslationScale = AnimRootMotionTranslationScale;
	PlayAnimTask->bStopWhenAbilityEnds = bStopWhenAbilityEnds;
	PlayAnimTask->StartTimeSeconds = StartTimeSeconds;

	return PlayAnimTask;
}

void UAbilityTask_PlayAnimAndWait::Activate()
{
	if (Ability == nullptr)
	{
		return;
	}

	bool bPlayedMontage = false;

	if (UAbilitySystemComponent* ASC = AbilitySystemComponent.Get())
	{
		const FGameplayAbilityActorInfo* ActorInfo = Ability->GetCurrentActorInfo();
		UAnimInstance* AnimInstance = ActorInfo->GetAnimInstance();
		if (AnimInstance != nullptr)
		{
			UAnimMontage* NewDynamicMontage = ASC->PlaySlotAnimationAsDynamicMontage_WithFractionalLoops(Ability, Ability->GetCurrentActivationInfo(), AnimSequenceToPlay, SlotName, BlendInTime, BlendOutTime, PlayRate, StartTimeSeconds, PlayCount);
			if (NewDynamicMontage)
			{
				
				// Playing a montage could potentially fire off a callback into game code which could kill this ability! Early out if we are  pending kill.
				if (ShouldBroadcastAbilityTaskDelegates() == false)
				{
					return;
				}

				InterruptedHandle = Ability->OnGameplayAbilityCancelled.AddUObject(this, &UAbilityTask_PlayAnimAndWait::OnMontageInterrupted);

				BlendingOutDelegate.BindUObject(this, &UAbilityTask_PlayAnimAndWait::OnMontageBlendingOut);
				AnimInstance->Montage_SetBlendingOutDelegate(BlendingOutDelegate, NewDynamicMontage);

				BlendedInDelegate.BindUObject(this, &UAbilityTask_PlayAnimAndWait::OnMontageBlendedIn);
				AnimInstance->Montage_SetBlendedInDelegate(BlendedInDelegate, NewDynamicMontage);

				MontageEndedDelegate.BindUObject(this, &UAbilityTask_PlayAnimAndWait::OnMontageEnded);
				AnimInstance->Montage_SetEndDelegate(MontageEndedDelegate, NewDynamicMontage);

				ACharacter* Character = Cast<ACharacter>(GetAvatarActor());
				if (Character && (Character->GetLocalRole() == ROLE_Authority ||
								  (Character->GetLocalRole() == ROLE_AutonomousProxy && Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalPredicted)))
				{
					Character->SetAnimRootMotionTranslationScale(AnimRootMotionTranslationScale);
				}

				bPlayedMontage = true;
			}
		}
		else
		{
			ABILITY_LOG(Warning, TEXT("UAbilityTask_PlayAnimAndWait call to PlayMontage failed!"));
		}
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilityTask_PlayAnimAndWait called on invalid AbilitySystemComponent"));
	}

	if (!bPlayedMontage)
	{
		ABILITY_LOG(Warning, TEXT("UAbilityTask_PlayAnimAndWait called in Ability %s failed to play dynamic montage %s; Task Instance Name %s."), *Ability->GetName(), *GetNameSafe(AnimSequenceToPlay),*InstanceName.ToString());
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnCancelled.Broadcast();
		}
	}

	SetWaitingOnAvatar();
}

void UAbilityTask_PlayAnimAndWait::ExternalCancel()
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnCancelled.Broadcast();
	}
	Super::ExternalCancel();
}

void UAbilityTask_PlayAnimAndWait::OnDestroy(bool AbilityEnded)
{
	// Note: Clearing montage end delegate isn't necessary since its not a multicast and will be cleared when the next montage plays.
	// (If we are destroyed, it will detect this and not do anything)

	// This delegate, however, should be cleared as it is a multicast
	if (Ability)
	{
		Ability->OnGameplayAbilityCancelled.Remove(InterruptedHandle);
		if (AbilityEnded && bStopWhenAbilityEnds)
		{
			StopPlayingMontage();
		}
	}

	Super::OnDestroy(AbilityEnded);

}

bool UAbilityTask_PlayAnimAndWait::StopPlayingMontage()
{
	if (Ability == nullptr)
	{
		return false;
	}

	const FGameplayAbilityActorInfo* ActorInfo = Ability->GetCurrentActorInfo();
	if (ActorInfo == nullptr)
	{
		return false;
	}

	UAnimInstance* AnimInstance = ActorInfo->GetAnimInstance();
	if (AnimInstance == nullptr)
	{
		return false;
	}

	// Check if the montage is still playing
	// The ability would have been interrupted, in which case we should automatically stop the montage
	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (ASC && Ability)
	{
		const UAnimMontage* Montage = ASC->GetCurrentMontage();
		if (ASC->GetAnimatingAbility() == Ability && Montage && Montage->IsDynamicMontage() && Montage->GetFirstAnimReference() == AnimSequenceToPlay)
		{
			// Unbind delegates so they don't get called as well
			FAnimMontageInstance* MontageInstance = AnimInstance->GetActiveInstanceForMontage(Montage);
			if (MontageInstance)
			{
				MontageInstance->OnMontageBlendingOutStarted.Unbind();
				MontageInstance->OnMontageBlendedInEnded.Unbind();
				MontageInstance->OnMontageEnded.Unbind();
			}

			ASC->CurrentMontageStop();
			return true;
		}
	}

	return false;
}

FString UAbilityTask_PlayAnimAndWait::GetDebugString() const
{
	UAnimMontage* PlayingMontage = nullptr;
	if (Ability)
	{
		const FGameplayAbilityActorInfo* ActorInfo = Ability->GetCurrentActorInfo();
		UAnimInstance* AnimInstance = ActorInfo->GetAnimInstance();

		if (AnimInstance != nullptr)
		{
			PlayingMontage = AnimInstance->GetCurrentActiveMontage();
		}
	}

	if(PlayingMontage)
	{
		return FString::Printf(TEXT("PlayAnimAndWait. AnimSequenceToPlay: %s Currently Playing: %s IsDynamic: %d AnimSequence: %s"), 
			*GetNameSafe(AnimSequenceToPlay), *GetNameSafe(PlayingMontage), PlayingMontage->IsDynamicMontage(), *GetNameSafe(PlayingMontage->GetFirstAnimReference()));
	}
	else
	{
		return FString::Printf(TEXT("PlayAnimAndWait. AnimSequenceToPlay: %s  (Currently Playing): None"), *GetNameSafe(AnimSequenceToPlay));
	}
	
}
