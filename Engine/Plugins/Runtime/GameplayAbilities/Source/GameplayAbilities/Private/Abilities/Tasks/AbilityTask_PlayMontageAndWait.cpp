// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_PlayMontageAndWait.h"
#include "Animation/AnimMontage.h"
#include "GameFramework/Character.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"
#include "AbilitySystemGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_PlayMontageAndWait)

static bool GUseAggressivePlayMontageAndWaitEndTask = true;
static FAutoConsoleVariableRef CVarAggressivePlayMontageAndWaitEndTask(TEXT("AbilitySystem.PlayMontage.AggressiveEndTask"), GUseAggressivePlayMontageAndWaitEndTask, TEXT("This should be set to true in order to avoid multiple callbacks off an AbilityTask_PlayMontageAndWait node"));

static bool GPlayMontageAndWaitFireInterruptOnAnimEndInterrupt = true;
static FAutoConsoleVariableRef CVarPlayMontageAndWaitFireInterruptOnAnimEndInterrupt(TEXT("AbilitySystem.PlayMontage.FireInterruptOnAnimEndInterrupt"), GPlayMontageAndWaitFireInterruptOnAnimEndInterrupt, TEXT("This is a fix that will cause AbilityTask_PlayMontageAndWait to fire its Interrupt event if the underlying AnimInstance ends in an interrupted"));

void UAbilityTask_PlayMontageAndWait::OnMontageBlendingOut(UAnimMontage* Montage, bool bInterrupted)
{
	const bool bPlayingThisMontage = (Montage == MontageToPlay) && Ability && Ability->GetCurrentMontage() == MontageToPlay;
	if (bPlayingThisMontage)
	{
		// Reset AnimRootMotionTranslationScale
		ACharacter* Character = Cast<ACharacter>(GetAvatarActor());
		if (Character && (Character->GetLocalRole() == ROLE_Authority ||
							(Character->GetLocalRole() == ROLE_AutonomousProxy && Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalPredicted)))
		{
			Character->SetAnimRootMotionTranslationScale(1.f);
		}
	}

	if (bPlayingThisMontage && (bInterrupted || !bAllowInterruptAfterBlendOut))
	{
		if (UAbilitySystemComponent* ASC = AbilitySystemComponent.Get())
		{
			ASC->ClearAnimatingAbility(Ability);
		}
	}

	if (ShouldBroadcastAbilityTaskDelegates())
	{
		if (bInterrupted)
		{
            bAllowInterruptAfterBlendOut = false;
			OnInterrupted.Broadcast();

			if (GUseAggressivePlayMontageAndWaitEndTask)
			{
				EndTask();
			}
		}
		else
		{
			OnBlendOut.Broadcast();
		}
	}
}

void UAbilityTask_PlayMontageAndWait::OnMontageBlendedIn(UAnimMontage* Montage)
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnBlendedIn.Broadcast();
	}
}

void UAbilityTask_PlayMontageAndWait::OnGameplayAbilityCancelled()
{
	if (StopPlayingMontage() || bAllowInterruptAfterBlendOut)
	{
		// Let the BP handle the interrupt as well
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			bAllowInterruptAfterBlendOut = false;
			OnInterrupted.Broadcast();
		}
	}

	if (GUseAggressivePlayMontageAndWaitEndTask)
	{
		EndTask();
	}
}

void UAbilityTask_PlayMontageAndWait::OnMontageEnded(UAnimMontage* Montage, bool bInterrupted)
{
	if (!bInterrupted)
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnCompleted.Broadcast();
		}
	}
	else if(bAllowInterruptAfterBlendOut && GPlayMontageAndWaitFireInterruptOnAnimEndInterrupt)
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnInterrupted.Broadcast();
		}
	}

	EndTask();
}

UAbilityTask_PlayMontageAndWait* UAbilityTask_PlayMontageAndWait::CreatePlayMontageAndWaitProxy(UGameplayAbility* OwningAbility,
	FName TaskInstanceName, UAnimMontage *MontageToPlay, float Rate, FName StartSection, bool bStopWhenAbilityEnds, float AnimRootMotionTranslationScale, float StartTimeSeconds, bool bAllowInterruptAfterBlendOut)
{

	UAbilitySystemGlobals::NonShipping_ApplyGlobalAbilityScaler_Rate(Rate);

	UAbilityTask_PlayMontageAndWait* MyObj = NewAbilityTask<UAbilityTask_PlayMontageAndWait>(OwningAbility, TaskInstanceName);
	MyObj->MontageToPlay = MontageToPlay;
	MyObj->Rate = Rate;
	MyObj->StartSection = StartSection;
	MyObj->AnimRootMotionTranslationScale = AnimRootMotionTranslationScale;
	MyObj->bStopWhenAbilityEnds = bStopWhenAbilityEnds;
	MyObj->bAllowInterruptAfterBlendOut = bAllowInterruptAfterBlendOut;
	MyObj->StartTimeSeconds = StartTimeSeconds;
	
	return MyObj;
}

void UAbilityTask_PlayMontageAndWait::Activate()
{
	if (Ability == nullptr)
	{
		return;
	}

	bool bPlayedMontage = false;

	if (UAbilitySystemComponent* ASC = AbilitySystemComponent.Get())
	{
		const FGameplayAbilityActorInfo* ActorInfo = Ability->GetCurrentActorInfo();
		UAnimInstance* AnimInstance = ActorInfo->GetAnimInstance();
		if (AnimInstance != nullptr)
		{
			if (ASC->PlayMontage(Ability, Ability->GetCurrentActivationInfo(), MontageToPlay, Rate, StartSection, StartTimeSeconds) > 0.f)
			{
				// Playing a montage could potentially fire off a callback into game code which could kill this ability! Early out if we are  pending kill.
				if (ShouldBroadcastAbilityTaskDelegates() == false)
				{
					return;
				}

				InterruptedHandle = Ability->OnGameplayAbilityCancelled.AddUObject(this, &UAbilityTask_PlayMontageAndWait::OnGameplayAbilityCancelled);

				BlendedInDelegate.BindUObject(this, &UAbilityTask_PlayMontageAndWait::OnMontageBlendedIn);
				AnimInstance->Montage_SetBlendedInDelegate(BlendedInDelegate, MontageToPlay);

				BlendingOutDelegate.BindUObject(this, &UAbilityTask_PlayMontageAndWait::OnMontageBlendingOut);
				AnimInstance->Montage_SetBlendingOutDelegate(BlendingOutDelegate, MontageToPlay);

				MontageEndedDelegate.BindUObject(this, &UAbilityTask_PlayMontageAndWait::OnMontageEnded);
				AnimInstance->Montage_SetEndDelegate(MontageEndedDelegate, MontageToPlay);

				ACharacter* Character = Cast<ACharacter>(GetAvatarActor());
				if (Character && (Character->GetLocalRole() == ROLE_Authority ||
								  (Character->GetLocalRole() == ROLE_AutonomousProxy && Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalPredicted)))
				{
					Character->SetAnimRootMotionTranslationScale(AnimRootMotionTranslationScale);
				}

				bPlayedMontage = true;
			}
		}
		else
		{
			ABILITY_LOG(Warning, TEXT("UAbilityTask_PlayMontageAndWait call to PlayMontage failed!"));
		}
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilityTask_PlayMontageAndWait called on invalid AbilitySystemComponent"));
	}

	if (!bPlayedMontage)
	{
		ABILITY_LOG(Warning, TEXT("UAbilityTask_PlayMontageAndWait called in Ability %s failed to play montage %s; Task Instance Name %s."), *Ability->GetName(), *GetNameSafe(MontageToPlay),*InstanceName.ToString());
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnCancelled.Broadcast();
		}
	}

	SetWaitingOnAvatar();
}

void UAbilityTask_PlayMontageAndWait::ExternalCancel()
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnCancelled.Broadcast();
	}
	Super::ExternalCancel();
}

void UAbilityTask_PlayMontageAndWait::OnDestroy(bool AbilityEnded)
{
	// Note: Clearing montage end delegate isn't necessary since its not a multicast and will be cleared when the next montage plays.
	// (If we are destroyed, it will detect this and not do anything)

	// This delegate, however, should be cleared as it is a multicast
	if (Ability)
	{
		Ability->OnGameplayAbilityCancelled.Remove(InterruptedHandle);
		if (AbilityEnded && bStopWhenAbilityEnds)
		{
			StopPlayingMontage();
		}
	}

	Super::OnDestroy(AbilityEnded);

}

bool UAbilityTask_PlayMontageAndWait::StopPlayingMontage()
{
	if (Ability == nullptr)
	{
		return false;
	}

	const FGameplayAbilityActorInfo* ActorInfo = Ability->GetCurrentActorInfo();
	if (ActorInfo == nullptr)
	{
		return false;
	}

	UAnimInstance* AnimInstance = ActorInfo->GetAnimInstance();
	if (AnimInstance == nullptr)
	{
		return false;
	}

	// Check if the montage is still playing
	// The ability would have been interrupted, in which case we should automatically stop the montage
	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (ASC && Ability)
	{
		if (ASC->GetAnimatingAbility() == Ability
			&& ASC->GetCurrentMontage() == MontageToPlay)
		{
			// Unbind delegates so they don't get called as well
			FAnimMontageInstance* MontageInstance = AnimInstance->GetActiveInstanceForMontage(MontageToPlay);
			if (MontageInstance)
			{
				MontageInstance->OnMontageBlendedInEnded.Unbind();
				MontageInstance->OnMontageBlendingOutStarted.Unbind();
				MontageInstance->OnMontageEnded.Unbind();
			}

			ASC->CurrentMontageStop();
			return true;
		}
	}

	return false;
}

FString UAbilityTask_PlayMontageAndWait::GetDebugString() const
{
	UAnimMontage* PlayingMontage = nullptr;
	if (Ability)
	{
		const FGameplayAbilityActorInfo* ActorInfo = Ability->GetCurrentActorInfo();
		UAnimInstance* AnimInstance = ActorInfo->GetAnimInstance();

		if (AnimInstance != nullptr)
		{
			PlayingMontage = AnimInstance->Montage_IsActive(MontageToPlay) ? ToRawPtr(MontageToPlay) : AnimInstance->GetCurrentActiveMontage();
		}
	}

	return FString::Printf(TEXT("PlayMontageAndWait. MontageToPlay: %s  (Currently Playing): %s"), *GetNameSafe(MontageToPlay), *GetNameSafe(PlayingMontage));
}

