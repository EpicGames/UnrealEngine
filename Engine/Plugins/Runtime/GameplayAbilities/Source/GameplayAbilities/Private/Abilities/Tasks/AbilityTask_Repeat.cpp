// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_Repeat.h"
#include "TimerManager.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_Repeat)

UAbilityTask_Repeat::UAbilityTask_Repeat(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

void UAbilityTask_Repeat::PerformAction()
{		
	OnPerformAction.Broadcast(ActionCounter);
	++ActionCounter;
	if (ActionCounter >= ActionPerformancesDesired)		//Should we allow negative values of ActionPerformancesDesired to mean "unlimited"?
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnFinished.Broadcast(-1);
		}
		EndTask();
	}
}

UAbilityTask_Repeat* UAbilityTask_Repeat::RepeatAction(UGameplayAbility* OwningAbility, float InTimeBetweenActions, int32 TotalActionCount)
{
	UAbilityTask_Repeat* MyObj = NewAbilityTask<UAbilityTask_Repeat>(OwningAbility);

	//TODO Validate/fix TimeBetweenActions and TotalActionCount values as needed
	MyObj->ActionPerformancesDesired = TotalActionCount;
	MyObj->TimeBetweenActions = InTimeBetweenActions;
	MyObj->ActionCounter = 0;

	return MyObj;
}

void UAbilityTask_Repeat::Activate()
{
	if (ActionCounter < ActionPerformancesDesired)
	{
		PerformAction();
		if (ActionCounter < ActionPerformancesDesired)
		{
			UWorld* World = GetWorld();
			if (World)
			{
				World->GetTimerManager().SetTimer(TimerHandle_PerformAction, this, &UAbilityTask_Repeat::PerformAction, TimeBetweenActions, true);
			}
		}
	}
	else
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnFinished.Broadcast(-1);
		}
		EndTask();
	}
}

void UAbilityTask_Repeat::OnDestroy(bool AbilityIsEnding)
{
	UWorld* World = GetWorld();
	if (World)
	{
		World->GetTimerManager().ClearTimer(TimerHandle_PerformAction);
	}

	Super::OnDestroy(AbilityIsEnding);
}

FString UAbilityTask_Repeat::GetDebugString() const
{
	return FString::Printf(TEXT("RepeatAction. TimeBetweenActions: %.2f. ActionCounter: %d"), TimeBetweenActions, ActionCounter);
}

