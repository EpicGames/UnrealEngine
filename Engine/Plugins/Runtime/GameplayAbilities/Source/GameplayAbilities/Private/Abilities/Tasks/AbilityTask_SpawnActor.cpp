// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_SpawnActor.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_SpawnActor)


UAbilityTask_SpawnActor::UAbilityTask_SpawnActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	
}

UAbilityTask_SpawnActor* UAbilityTask_SpawnActor::SpawnActor(UGameplayAbility* OwningAbility, FGameplayAbilityTargetDataHandle TargetData, TSubclassOf<AActor> InClass)
{
	UAbilityTask_SpawnActor* MyObj = NewAbilityTask<UAbilityTask_SpawnActor>(OwningAbility);
	MyObj->CachedTargetDataHandle = MoveTemp(TargetData);
	return MyObj;
}

// ---------------------------------------------------------------------------------------

bool UAbilityTask_SpawnActor::BeginSpawningActor(UGameplayAbility* OwningAbility, FGameplayAbilityTargetDataHandle TargetData, TSubclassOf<AActor> InClass, AActor*& SpawnedActor)
{
	if (Ability && Ability->GetCurrentActorInfo()->IsNetAuthority() && ShouldBroadcastAbilityTaskDelegates())
	{
		UWorld* const World = GEngine->GetWorldFromContextObject(OwningAbility, EGetWorldErrorMode::LogAndReturnNull);
		if (World)
		{
			SpawnedActor = World->SpawnActorDeferred<AActor>(InClass, FTransform::Identity, NULL, NULL, ESpawnActorCollisionHandlingMethod::AlwaysSpawn);
		}
	}
	
	if (SpawnedActor == nullptr)
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			DidNotSpawn.Broadcast(nullptr);
		}
		return false;
	}

	return true;
}

void UAbilityTask_SpawnActor::FinishSpawningActor(UGameplayAbility* OwningAbility, FGameplayAbilityTargetDataHandle TargetData, AActor* SpawnedActor)
{
	if (SpawnedActor)
	{
		bool bTransformSet = false;
		FTransform SpawnTransform;
		if (FGameplayAbilityTargetData* LocationData = CachedTargetDataHandle.Get(0))		//Hardcode to use data 0. It's OK if data isn't useful/valid.
		{
			//Set location. Rotation is unaffected.
			if (LocationData->HasHitResult())
			{
				SpawnTransform.SetLocation(LocationData->GetHitResult()->Location);
				bTransformSet = true;
			}
			else if (LocationData->HasEndPoint())
			{
				SpawnTransform = LocationData->GetEndPointTransform();
				bTransformSet = true;
			}
			}
		if (!bTransformSet)
		{
			if (UAbilitySystemComponent* ASC = AbilitySystemComponent.Get())
			{
				SpawnTransform = ASC->GetOwner()->GetTransform();
			}
		}

		SpawnedActor->FinishSpawning(SpawnTransform);

		if (ShouldBroadcastAbilityTaskDelegates())
		{
			Success.Broadcast(SpawnedActor);
		}
	}

	EndTask();
}

// ---------------------------------------------------------------------------------------


