// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_StartAbilityState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_StartAbilityState)

namespace UE::AbilitySystem::AbilityTask_StartAbilityState
{
	// The default implementation of ShouldBroadcastAbilityTaskDelegates returns false if the Ability is not 'active'.
	// However, we want to execute anyway because we expect to be able to execute during EndAbility.
	bool CustomShouldBroadcastAbilityTaskDelegates(const UGameplayAbility* Ability)
	{
		return IsValid(Ability);
	}
}

UAbilityTask_StartAbilityState::UAbilityTask_StartAbilityState(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	bEndCurrentState = true;
	bWasEnded = false;
	bWasInterrupted = false;
}

UAbilityTask_StartAbilityState* UAbilityTask_StartAbilityState::StartAbilityState(UGameplayAbility* OwningAbility, FName StateName, bool bEndCurrentState)
{
	UAbilityTask_StartAbilityState* Task = NewAbilityTask<UAbilityTask_StartAbilityState>(OwningAbility, StateName);
	Task->bEndCurrentState = bEndCurrentState;
	return Task;
}

void UAbilityTask_StartAbilityState::Activate()
{
	if (Ability)
	{
		if (bEndCurrentState && Ability->OnGameplayAbilityStateEnded.IsBound())
		{
			Ability->OnGameplayAbilityStateEnded.Broadcast(NAME_None);
		}

		EndStateHandle = Ability->OnGameplayAbilityStateEnded.AddUObject(this, &UAbilityTask_StartAbilityState::OnEndState);
		InterruptStateHandle = Ability->OnGameplayAbilityCancelled.AddUObject(this, &UAbilityTask_StartAbilityState::OnInterruptState);
	}
}

void UAbilityTask_StartAbilityState::OnDestroy(bool AbilityEnded)
{
	using namespace UE::AbilitySystem::AbilityTask_StartAbilityState;

	// Unbind delegates so this doesn't get recursively called
	if (Ability)
	{
		Ability->OnGameplayAbilityCancelled.Remove(InterruptStateHandle);
		Ability->OnGameplayAbilityStateEnded.Remove(EndStateHandle);
	}

	if (bWasInterrupted && OnStateInterrupted.IsBound())
	{
		if (CustomShouldBroadcastAbilityTaskDelegates(Ability))
		{
			OnStateInterrupted.Broadcast();
		}
	}
	else if ((bWasEnded || AbilityEnded) && OnStateEnded.IsBound())
	{
		if (CustomShouldBroadcastAbilityTaskDelegates(Ability))
		{
			OnStateEnded.Broadcast();
		}
	}

	// This will invalidate the task so needs to happen after callbacks
	Super::OnDestroy(AbilityEnded);
}

void UAbilityTask_StartAbilityState::OnEndState(FName StateNameToEnd)
{
	// All states end if 'NAME_None' is passed to this delegate
	if (StateNameToEnd.IsNone() || StateNameToEnd == InstanceName)
	{
		bWasEnded = true;

		EndTask();
	}
}

void UAbilityTask_StartAbilityState::OnInterruptState()
{
	bWasInterrupted = true;
}

void UAbilityTask_StartAbilityState::ExternalCancel()
{
	bWasInterrupted = true;

	Super::ExternalCancel();
}

FString UAbilityTask_StartAbilityState::GetDebugString() const
{
	return FString::Printf(TEXT("%s (AbilityState)"), *InstanceName.ToString());
}

