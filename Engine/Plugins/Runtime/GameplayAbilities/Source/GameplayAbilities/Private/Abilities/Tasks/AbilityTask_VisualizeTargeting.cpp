// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_VisualizeTargeting.h"
#include "TimerManager.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_VisualizeTargeting)

UAbilityTask_VisualizeTargeting::UAbilityTask_VisualizeTargeting(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

UAbilityTask_VisualizeTargeting* UAbilityTask_VisualizeTargeting::VisualizeTargeting(UGameplayAbility* OwningAbility, TSubclassOf<AGameplayAbilityTargetActor> InTargetClass, FName TaskInstanceName, float Duration)
{
	UAbilityTask_VisualizeTargeting* MyObj = NewAbilityTask<UAbilityTask_VisualizeTargeting>(OwningAbility, TaskInstanceName);		//Register for task list here, providing a given FName as a key
	MyObj->TargetClass = InTargetClass;
	MyObj->TargetActor = NULL;
	MyObj->SetDuration(Duration);
	return MyObj;
}

UAbilityTask_VisualizeTargeting* UAbilityTask_VisualizeTargeting::VisualizeTargetingUsingActor(UGameplayAbility* OwningAbility, AGameplayAbilityTargetActor* InTargetActor, FName TaskInstanceName, float Duration)
{
	UAbilityTask_VisualizeTargeting* MyObj = NewAbilityTask<UAbilityTask_VisualizeTargeting>(OwningAbility, TaskInstanceName);		//Register for task list here, providing a given FName as a key
	MyObj->TargetClass = NULL;
	MyObj->TargetActor = InTargetActor;
	MyObj->SetDuration(Duration);
	return MyObj;
}

void UAbilityTask_VisualizeTargeting::Activate()
{
	// Need to handle case where target actor was passed into task
	if (Ability && (TargetClass == NULL))
	{
		if (TargetActor.IsValid())
		{
			AGameplayAbilityTargetActor* SpawnedActor = TargetActor.Get();

			TargetClass = SpawnedActor->GetClass();

			if (ShouldSpawnTargetActor())
			{
				InitializeTargetActor(SpawnedActor);
				FinalizeTargetActor(SpawnedActor);
			}
			else
			{
				TargetActor = NULL;

				// We may need a better solution here.  We don't know the target actor isn't needed till after it's already been spawned.
				SpawnedActor->Destroy();
				SpawnedActor = nullptr;
			}
		}
		else
		{
			EndTask();
		}
	}
}

bool UAbilityTask_VisualizeTargeting::BeginSpawningActor(UGameplayAbility* OwningAbility, TSubclassOf<AGameplayAbilityTargetActor> InTargetClass, AGameplayAbilityTargetActor*& SpawnedActor)
{
	SpawnedActor = nullptr;

	if (Ability)
	{
		if (ShouldSpawnTargetActor())
		{
			UClass* Class = *InTargetClass;
			if (Class != NULL)
			{
				UWorld* const World = GEngine->GetWorldFromContextObject(OwningAbility, EGetWorldErrorMode::LogAndReturnNull);
				if (World)
				{
					SpawnedActor = World->SpawnActorDeferred<AGameplayAbilityTargetActor>(Class, FTransform::Identity, NULL, NULL, ESpawnActorCollisionHandlingMethod::AlwaysSpawn);
				}
			}

			if (SpawnedActor)
			{
				TargetActor = SpawnedActor;
				InitializeTargetActor(SpawnedActor);
			}
		}
	}

	return (SpawnedActor != nullptr);
}

void UAbilityTask_VisualizeTargeting::FinishSpawningActor(UGameplayAbility* OwningAbility, AGameplayAbilityTargetActor* SpawnedActor)
{
	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (SpawnedActor && ASC)
	{
		check(TargetActor == SpawnedActor);

		const FTransform SpawnTransform = ASC->GetOwner()->GetTransform();

		SpawnedActor->FinishSpawning(SpawnTransform);

		FinalizeTargetActor(SpawnedActor);
	}
}

void UAbilityTask_VisualizeTargeting::SetDuration(const float Duration)
{
	if (Duration > 0.f)
	{
		GetWorld()->GetTimerManager().SetTimer(TimerHandle_OnTimeElapsed, this, &UAbilityTask_VisualizeTargeting::OnTimeElapsed, Duration, false);
	}
}

bool UAbilityTask_VisualizeTargeting::ShouldSpawnTargetActor() const
{
	check(TargetClass);
	check(Ability);

	// Spawn the actor if this is a locally controlled ability (always) or if this is a replicating targeting mode.
	// (E.g., server will spawn this target actor to replicate to all non owning clients)

	const AGameplayAbilityTargetActor* CDO = CastChecked<AGameplayAbilityTargetActor>(TargetClass->GetDefaultObject());

	const bool bReplicates = CDO->GetIsReplicated();
	const bool bIsLocallyControlled = Ability->GetCurrentActorInfo()->IsLocallyControlled();

	return (bReplicates || bIsLocallyControlled);
}

void UAbilityTask_VisualizeTargeting::InitializeTargetActor(AGameplayAbilityTargetActor* SpawnedActor) const
{
	check(SpawnedActor);
	check(Ability);

	SpawnedActor->PrimaryPC = Ability->GetCurrentActorInfo()->PlayerController.Get();
}

void UAbilityTask_VisualizeTargeting::FinalizeTargetActor(AGameplayAbilityTargetActor* SpawnedActor) const
{
	check(SpawnedActor);
	check(Ability);

	if (UAbilitySystemComponent* ASC = AbilitySystemComponent.Get())
	{
		ASC->SpawnedTargetActors.Push(SpawnedActor);
	}

	SpawnedActor->StartTargeting(Ability);
}

void UAbilityTask_VisualizeTargeting::OnDestroy(bool AbilityEnded)
{
	if (TargetActor.IsValid())
	{
		TargetActor->Destroy();
	}

	GetWorld()->GetTimerManager().ClearTimer(TimerHandle_OnTimeElapsed);

	Super::OnDestroy(AbilityEnded);
}

void UAbilityTask_VisualizeTargeting::OnTimeElapsed()
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		TimeElapsed.Broadcast();
	}
	EndTask();
}

