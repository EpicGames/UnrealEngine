// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitAbilityActivate.h"

#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitAbilityActivate)


UAbilityTask_WaitAbilityActivate::UAbilityTask_WaitAbilityActivate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	IncludeTriggeredAbilities = false;
}

UAbilityTask_WaitAbilityActivate* UAbilityTask_WaitAbilityActivate::WaitForAbilityActivate(UGameplayAbility* OwningAbility, FGameplayTag InWithTag, FGameplayTag InWithoutTag, bool InIncludeTriggeredAbilities, bool InTriggerOnce)
{
	UAbilityTask_WaitAbilityActivate* MyObj = NewAbilityTask<UAbilityTask_WaitAbilityActivate>(OwningAbility);
	MyObj->WithTag = InWithTag;
	MyObj->WithoutTag = InWithoutTag;
	MyObj->IncludeTriggeredAbilities = InIncludeTriggeredAbilities;
	MyObj->TriggerOnce = InTriggerOnce;
	return MyObj;
}

UAbilityTask_WaitAbilityActivate* UAbilityTask_WaitAbilityActivate::WaitForAbilityActivateWithTagRequirements(UGameplayAbility* OwningAbility, FGameplayTagRequirements TagRequirements, bool InIncludeTriggeredAbilities, bool InTriggerOnce)
{
	UAbilityTask_WaitAbilityActivate* MyObj = NewAbilityTask<UAbilityTask_WaitAbilityActivate>(OwningAbility);
	MyObj->TagRequirements = TagRequirements;
	MyObj->IncludeTriggeredAbilities = InIncludeTriggeredAbilities;
	MyObj->TriggerOnce = InTriggerOnce;
	return MyObj;
}

UAbilityTask_WaitAbilityActivate* UAbilityTask_WaitAbilityActivate::WaitForAbilityActivate_Query(UGameplayAbility* OwningAbility, FGameplayTagQuery Query, bool InIncludeTriggeredAbilities, bool InTriggerOnce)
{
	UAbilityTask_WaitAbilityActivate* MyObj = NewAbilityTask<UAbilityTask_WaitAbilityActivate>(OwningAbility);
	MyObj->Query = Query;
	MyObj->IncludeTriggeredAbilities = InIncludeTriggeredAbilities;
	MyObj->TriggerOnce = InTriggerOnce;
	return MyObj;

}

void UAbilityTask_WaitAbilityActivate::Activate()
{
	if (AbilitySystemComponent.IsValid())
	{
		OnAbilityActivateDelegateHandle = AbilitySystemComponent->AbilityActivatedCallbacks.AddUObject(this, &UAbilityTask_WaitAbilityActivate::OnAbilityActivate);
	}
}

void UAbilityTask_WaitAbilityActivate::OnAbilityActivate(UGameplayAbility* ActivatedAbility)
{
	if (!IncludeTriggeredAbilities && ActivatedAbility->IsTriggered())
	{
		return;
	}

	const FGameplayTagContainer& AbilityTags = ActivatedAbility->GetAssetTags();

	if (TagRequirements.IsEmpty())
	{
		if ((WithTag.IsValid() && !AbilityTags.HasTag(WithTag)) ||
			(WithoutTag.IsValid() && AbilityTags.HasTag(WithoutTag)))
		{
			// Failed tag check
			return;
		}
	}
	else
	{
		if (!TagRequirements.RequirementsMet(AbilityTags))
		{
			// Failed tag check
			return;
		}
	}

	if (Query.IsEmpty() == false)
	{
		if (Query.Matches(AbilityTags) == false)
		{
			// Failed query
			return;
		}
	}

	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnActivate.Broadcast(ActivatedAbility);
	}

	if (TriggerOnce)
	{
		EndTask();
	}
}

void UAbilityTask_WaitAbilityActivate::OnDestroy(bool AbilityEnded)
{
	if (AbilitySystemComponent.IsValid())
	{
		AbilitySystemComponent->AbilityActivatedCallbacks.Remove(OnAbilityActivateDelegateHandle);
	}

	Super::OnDestroy(AbilityEnded);
}

