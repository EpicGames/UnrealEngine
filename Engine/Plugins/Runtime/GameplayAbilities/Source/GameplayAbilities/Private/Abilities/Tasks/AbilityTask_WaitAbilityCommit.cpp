// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitAbilityCommit.h"

#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitAbilityCommit)



UAbilityTask_WaitAbilityCommit::UAbilityTask_WaitAbilityCommit(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAbilityTask_WaitAbilityCommit* UAbilityTask_WaitAbilityCommit::WaitForAbilityCommit(UGameplayAbility* OwningAbility, FGameplayTag InWithTag, FGameplayTag InWithoutTag, bool InTriggerOnce)
{
	UAbilityTask_WaitAbilityCommit* MyObj = NewAbilityTask<UAbilityTask_WaitAbilityCommit>(OwningAbility);
	MyObj->WithTag = InWithTag;
	MyObj->WithoutTag = InWithoutTag;
	MyObj->TriggerOnce = InTriggerOnce;

	return MyObj;
}

UAbilityTask_WaitAbilityCommit* UAbilityTask_WaitAbilityCommit::WaitForAbilityCommit_Query(UGameplayAbility* OwningAbility, FGameplayTagQuery Query, bool InTriggerOnce)
{
	UAbilityTask_WaitAbilityCommit* MyObj = NewAbilityTask<UAbilityTask_WaitAbilityCommit>(OwningAbility);
	MyObj->Query = Query;
	MyObj->TriggerOnce = InTriggerOnce;
	return MyObj;
}

void UAbilityTask_WaitAbilityCommit::Activate()
{
	if (AbilitySystemComponent.IsValid())	
	{		
		OnAbilityCommitDelegateHandle = AbilitySystemComponent->AbilityCommittedCallbacks.AddUObject(this, &UAbilityTask_WaitAbilityCommit::OnAbilityCommit);
	}
}

void UAbilityTask_WaitAbilityCommit::OnDestroy(bool AbilityEnded)
{
	if (AbilitySystemComponent.IsValid())
	{
		AbilitySystemComponent->AbilityCommittedCallbacks.Remove(OnAbilityCommitDelegateHandle);
	}

	Super::OnDestroy(AbilityEnded);
}

void UAbilityTask_WaitAbilityCommit::OnAbilityCommit(UGameplayAbility *ActivatedAbility)
{
	const FGameplayTagContainer& AbilityTags = ActivatedAbility->GetAssetTags();

	if ( (WithTag.IsValid() && !AbilityTags.HasTag(WithTag)) ||
		 (WithoutTag.IsValid() && AbilityTags.HasTag(WithoutTag)))
	{
		// Failed tag check
		return;
	}

	if (Query.IsEmpty() == false)
	{
		if (Query.Matches(AbilityTags) == false)
		{
			// Failed query
			return;
		}
	}

	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnCommit.Broadcast(ActivatedAbility);
	}

	if (TriggerOnce)
	{
		EndTask();
	}
}

