// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitAttributeChange.h"

#include "AbilitySystemComponent.h"

#include "GameplayEffectExtension.h"
#include "AbilitySystemGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitAttributeChange)

UAbilityTask_WaitAttributeChange::UAbilityTask_WaitAttributeChange(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bTriggerOnce = false;
}

UAbilityTask_WaitAttributeChange* UAbilityTask_WaitAttributeChange::WaitForAttributeChange(UGameplayAbility* OwningAbility, FGameplayAttribute InAttribute, FGameplayTag InWithTag, FGameplayTag InWithoutTag, bool TriggerOnce, AActor* OptionalExternalOwner)
{
	UAbilityTask_WaitAttributeChange* MyObj = NewAbilityTask<UAbilityTask_WaitAttributeChange>(OwningAbility);
	MyObj->WithTag = InWithTag;
	MyObj->WithoutTag = InWithoutTag;
	MyObj->Attribute = InAttribute;
	MyObj->ComparisonType = EWaitAttributeChangeComparison::None;
	MyObj->bTriggerOnce = TriggerOnce;
	MyObj->ExternalOwner = OptionalExternalOwner ? UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(OptionalExternalOwner) : nullptr;

	return MyObj;
}

UAbilityTask_WaitAttributeChange* UAbilityTask_WaitAttributeChange::WaitForAttributeChangeWithComparison(UGameplayAbility* OwningAbility, FGameplayAttribute InAttribute, FGameplayTag InWithTag, FGameplayTag InWithoutTag, TEnumAsByte<EWaitAttributeChangeComparison::Type> InComparisonType, float InComparisonValue, bool TriggerOnce, AActor* OptionalExternalOwner)
{
	UAbilityTask_WaitAttributeChange* MyObj = NewAbilityTask<UAbilityTask_WaitAttributeChange>(OwningAbility);
	MyObj->WithTag = InWithTag;
	MyObj->WithoutTag = InWithoutTag;
	MyObj->Attribute = InAttribute;
	MyObj->ComparisonType = InComparisonType;
	MyObj->ComparisonValue = InComparisonValue;
	MyObj->bTriggerOnce = TriggerOnce;
	MyObj->ExternalOwner = OptionalExternalOwner ? UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(OptionalExternalOwner) : nullptr;

	return MyObj;
}

void UAbilityTask_WaitAttributeChange::Activate()
{
	if (UAbilitySystemComponent* ASC = GetFocusedASC())
	{
		OnAttributeChangeDelegateHandle = ASC->GetGameplayAttributeValueChangeDelegate(Attribute).AddUObject(this, &UAbilityTask_WaitAttributeChange::OnAttributeChange);
	}
}

void UAbilityTask_WaitAttributeChange::OnAttributeChange(const FOnAttributeChangeData& CallbackData)
{
	float NewValue = CallbackData.NewValue;
	const FGameplayEffectModCallbackData* Data = CallbackData.GEModData;

	if (Data == nullptr)
	{
		// There may be no execution data associated with this change, for example a GE being removed. 
		// In this case, we auto fail any WithTag requirement and auto pass any WithoutTag requirement
		if (WithTag.IsValid())
		{
			return;
		}
	}
	else
	{
		if ((WithTag.IsValid() && !Data->EffectSpec.CapturedSourceTags.GetAggregatedTags()->HasTag(WithTag)) ||
			(WithoutTag.IsValid() && Data->EffectSpec.CapturedSourceTags.GetAggregatedTags()->HasTag(WithoutTag)))
		{
			// Failed tag check
			return;
		}
	}	

	bool PassedComparison = true;
	switch (ComparisonType)
	{
	case EWaitAttributeChangeComparison::ExactlyEqualTo:
		PassedComparison = (NewValue == ComparisonValue);
		break;		
	case EWaitAttributeChangeComparison::GreaterThan:
		PassedComparison = (NewValue > ComparisonValue);
		break;
	case EWaitAttributeChangeComparison::GreaterThanOrEqualTo:
		PassedComparison = (NewValue >= ComparisonValue);
		break;
	case EWaitAttributeChangeComparison::LessThan:
		PassedComparison = (NewValue < ComparisonValue);
		break;
	case EWaitAttributeChangeComparison::LessThanOrEqualTo:
		PassedComparison = (NewValue <= ComparisonValue);
		break;
	case EWaitAttributeChangeComparison::NotEqualTo:
		PassedComparison = (NewValue != ComparisonValue);
		break;
	default:
		break;
	}
	if (PassedComparison)
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnChange.Broadcast();
		}
		if (bTriggerOnce)
		{
			EndTask();
		}
	}
}

UAbilitySystemComponent* UAbilityTask_WaitAttributeChange::GetFocusedASC()
{
	return ExternalOwner ? ToRawPtr(ExternalOwner) : AbilitySystemComponent.Get();
}

void UAbilityTask_WaitAttributeChange::OnDestroy(bool AbilityEnded)
{
	if (UAbilitySystemComponent* ASC = GetFocusedASC())
	{
		ASC->GetGameplayAttributeValueChangeDelegate(Attribute).Remove(OnAttributeChangeDelegateHandle);
	}

	Super::OnDestroy(AbilityEnded);
}

