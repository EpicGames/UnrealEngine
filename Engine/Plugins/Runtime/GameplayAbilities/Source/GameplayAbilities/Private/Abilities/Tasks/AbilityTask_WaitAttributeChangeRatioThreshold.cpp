// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitAttributeChangeRatioThreshold.h"
#include "TimerManager.h"

#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitAttributeChangeRatioThreshold)

UAbilityTask_WaitAttributeChangeRatioThreshold::UAbilityTask_WaitAttributeChangeRatioThreshold(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bTriggerOnce = false;
	bMatchedComparisonLastAttributeChange = false;
	LastAttributeNumeratorValue = 1.f;
	LastAttributeDenominatorValue = 1.f;
}

UAbilityTask_WaitAttributeChangeRatioThreshold* UAbilityTask_WaitAttributeChangeRatioThreshold::WaitForAttributeChangeRatioThreshold(UGameplayAbility* OwningAbility, FGameplayAttribute AttributeNumerator, FGameplayAttribute AttributeDenominator, TEnumAsByte<EWaitAttributeChangeComparison::Type> ComparisonType, float ComparisonValue, bool bTriggerOnce, AActor* OptionalExternalOwner)
{
	UAbilityTask_WaitAttributeChangeRatioThreshold* MyTask = NewAbilityTask<UAbilityTask_WaitAttributeChangeRatioThreshold>(OwningAbility);
	MyTask->AttributeNumerator = AttributeNumerator;
	MyTask->AttributeDenominator = AttributeDenominator;
	MyTask->ComparisonType = ComparisonType;
	MyTask->ComparisonValue = ComparisonValue;
	MyTask->bTriggerOnce = bTriggerOnce;
	MyTask->ExternalOwner = OptionalExternalOwner ? UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(OptionalExternalOwner) : nullptr;

	return MyTask;
}

void UAbilityTask_WaitAttributeChangeRatioThreshold::Activate()
{
	if (UAbilitySystemComponent* ASC = GetFocusedASC())
	{
		LastAttributeNumeratorValue = ASC->GetNumericAttribute(AttributeNumerator);
		LastAttributeDenominatorValue = ASC->GetNumericAttribute(AttributeDenominator);
		bMatchedComparisonLastAttributeChange = DoesValuePassComparison(LastAttributeNumeratorValue, LastAttributeDenominatorValue);

		// Broadcast OnChange immediately with current value
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnChange.Broadcast(bMatchedComparisonLastAttributeChange, LastAttributeDenominatorValue != 0.f ? LastAttributeNumeratorValue/LastAttributeDenominatorValue : 0.f);
		}

		OnNumeratorAttributeChangeDelegateHandle = ASC->GetGameplayAttributeValueChangeDelegate(AttributeNumerator).AddUObject(this, &UAbilityTask_WaitAttributeChangeRatioThreshold::OnNumeratorAttributeChange);
		OnDenominatorAttributeChangeDelegateHandle = ASC->GetGameplayAttributeValueChangeDelegate(AttributeDenominator).AddUObject(this, &UAbilityTask_WaitAttributeChangeRatioThreshold::OnDenominatorAttributeChange);
	}
}

UAbilitySystemComponent* UAbilityTask_WaitAttributeChangeRatioThreshold::GetFocusedASC()
{
	return ExternalOwner ? ToRawPtr(ExternalOwner) : AbilitySystemComponent.Get();
}

void UAbilityTask_WaitAttributeChangeRatioThreshold::OnAttributeChange()
{
	UWorld* World = GetWorld();
	if (World && !CheckAttributeTimer.IsValid())
	{
		// Trigger OnRatioChange check at the end of this frame/next so that any individual attribute change
		// has time for the other attribute to update (in case they're linked)
		World->GetTimerManager().SetTimer(CheckAttributeTimer, this, &UAbilityTask_WaitAttributeChangeRatioThreshold::OnRatioChange, 0.001f, false);
	}
}

void UAbilityTask_WaitAttributeChangeRatioThreshold::OnRatioChange()
{
	CheckAttributeTimer.Invalidate();

	bool bPassedComparison = DoesValuePassComparison(LastAttributeNumeratorValue, LastAttributeDenominatorValue);
	if (bPassedComparison != bMatchedComparisonLastAttributeChange)
	{
		bMatchedComparisonLastAttributeChange = bPassedComparison;
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnChange.Broadcast(bMatchedComparisonLastAttributeChange, LastAttributeDenominatorValue != 0.f ? LastAttributeNumeratorValue/LastAttributeDenominatorValue : 0.f);
		}
		if (bTriggerOnce)
		{
			EndTask();
		}
	}
}

void UAbilityTask_WaitAttributeChangeRatioThreshold::OnNumeratorAttributeChange(const FOnAttributeChangeData& CallbackData)
{
	LastAttributeNumeratorValue = CallbackData.NewValue;
	OnAttributeChange();
}

void UAbilityTask_WaitAttributeChangeRatioThreshold::OnDenominatorAttributeChange(const FOnAttributeChangeData& CallbackData)
{
	LastAttributeDenominatorValue = CallbackData.NewValue;
	OnAttributeChange();
}

bool UAbilityTask_WaitAttributeChangeRatioThreshold::DoesValuePassComparison(float ValueNumerator, float ValueDenominator) const
{
	if (ValueDenominator == 0.f)
	{
		return bMatchedComparisonLastAttributeChange;
	}

	const float CurrentRatio = ValueNumerator / ValueDenominator;
	bool bPassedComparison = true;
	switch (ComparisonType)
	{
	case EWaitAttributeChangeComparison::ExactlyEqualTo:
		bPassedComparison = (CurrentRatio == ComparisonValue);
		break;		
	case EWaitAttributeChangeComparison::GreaterThan:
		bPassedComparison = (CurrentRatio > ComparisonValue);
		break;
	case EWaitAttributeChangeComparison::GreaterThanOrEqualTo:
		bPassedComparison = (CurrentRatio >= ComparisonValue);
		break;
	case EWaitAttributeChangeComparison::LessThan:
		bPassedComparison = (CurrentRatio < ComparisonValue);
		break;
	case EWaitAttributeChangeComparison::LessThanOrEqualTo:
		bPassedComparison = (CurrentRatio <= ComparisonValue);
		break;
	case EWaitAttributeChangeComparison::NotEqualTo:
		bPassedComparison = (CurrentRatio != ComparisonValue);
		break;
	default:
		break;
	}
	return bPassedComparison;
}

void UAbilityTask_WaitAttributeChangeRatioThreshold::OnDestroy(bool AbilityEnded)
{
	if (UAbilitySystemComponent* ASC = GetFocusedASC())
	{
		ASC->GetGameplayAttributeValueChangeDelegate(AttributeNumerator).Remove(OnNumeratorAttributeChangeDelegateHandle);
		ASC->GetGameplayAttributeValueChangeDelegate(AttributeDenominator).Remove(OnDenominatorAttributeChangeDelegateHandle);
	}

	Super::OnDestroy(AbilityEnded);
}

