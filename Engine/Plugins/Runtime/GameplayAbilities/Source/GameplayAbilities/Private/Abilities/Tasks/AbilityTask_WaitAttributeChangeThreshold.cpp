// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitAttributeChangeThreshold.h"

#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitAttributeChangeThreshold)

UAbilityTask_WaitAttributeChangeThreshold::UAbilityTask_WaitAttributeChangeThreshold(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bTriggerOnce = false;
	bMatchedComparisonLastAttributeChange = false;
}

UAbilityTask_WaitAttributeChangeThreshold* UAbilityTask_WaitAttributeChangeThreshold::WaitForAttributeChangeThreshold(UGameplayAbility* OwningAbility, FGameplayAttribute Attribute, TEnumAsByte<EWaitAttributeChangeComparison::Type> ComparisonType, float ComparisonValue, bool bTriggerOnce, AActor* OptionalExternalOwner)
{
	UAbilityTask_WaitAttributeChangeThreshold* MyTask = NewAbilityTask<UAbilityTask_WaitAttributeChangeThreshold>(OwningAbility);
	MyTask->Attribute = Attribute;
	MyTask->ComparisonType = ComparisonType;
	MyTask->ComparisonValue = ComparisonValue;
	MyTask->bTriggerOnce = bTriggerOnce;
	MyTask->ExternalOwner = OptionalExternalOwner ? UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(OptionalExternalOwner) : nullptr;

	return MyTask;
}

void UAbilityTask_WaitAttributeChangeThreshold::Activate()
{
	if (UAbilitySystemComponent* ASC = GetFocusedASC())
	{
		const float CurrentValue = ASC->GetNumericAttribute(Attribute);
		bMatchedComparisonLastAttributeChange = DoesValuePassComparison(CurrentValue);

		// Broadcast OnChange immediately with current value
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnChange.Broadcast(bMatchedComparisonLastAttributeChange, CurrentValue);
		}

		OnAttributeChangeDelegateHandle = ASC->GetGameplayAttributeValueChangeDelegate(Attribute).AddUObject(this, &UAbilityTask_WaitAttributeChangeThreshold::OnAttributeChange);
	}
}

void UAbilityTask_WaitAttributeChangeThreshold::OnAttributeChange(const FOnAttributeChangeData& CallbackData)
{
	float NewValue = CallbackData.NewValue;

	bool bPassedComparison = DoesValuePassComparison(NewValue);
	if (bPassedComparison != bMatchedComparisonLastAttributeChange)
	{
		bMatchedComparisonLastAttributeChange = bPassedComparison;
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnChange.Broadcast(bPassedComparison, NewValue);
		}
		if (bTriggerOnce)
		{
			EndTask();
		}
	}
}

bool UAbilityTask_WaitAttributeChangeThreshold::DoesValuePassComparison(float Value) const
{
	bool bPassedComparison = true;
	switch (ComparisonType)
	{
	case EWaitAttributeChangeComparison::ExactlyEqualTo:
		bPassedComparison = (Value == ComparisonValue);
		break;		
	case EWaitAttributeChangeComparison::GreaterThan:
		bPassedComparison = (Value > ComparisonValue);
		break;
	case EWaitAttributeChangeComparison::GreaterThanOrEqualTo:
		bPassedComparison = (Value >= ComparisonValue);
		break;
	case EWaitAttributeChangeComparison::LessThan:
		bPassedComparison = (Value < ComparisonValue);
		break;
	case EWaitAttributeChangeComparison::LessThanOrEqualTo:
		bPassedComparison = (Value <= ComparisonValue);
		break;
	case EWaitAttributeChangeComparison::NotEqualTo:
		bPassedComparison = (Value != ComparisonValue);
		break;
	default:
		break;
	}
	return bPassedComparison;
}

UAbilitySystemComponent* UAbilityTask_WaitAttributeChangeThreshold::GetFocusedASC()
{
	return ExternalOwner ? ToRawPtr(ExternalOwner) : AbilitySystemComponent.Get();
}

void UAbilityTask_WaitAttributeChangeThreshold::OnDestroy(bool AbilityEnded)
{
	if (UAbilitySystemComponent* ASC = GetFocusedASC())
	{
		ASC->GetGameplayAttributeValueChangeDelegate(Attribute).Remove(OnAttributeChangeDelegateHandle);
	}

	Super::OnDestroy(AbilityEnded);
}

