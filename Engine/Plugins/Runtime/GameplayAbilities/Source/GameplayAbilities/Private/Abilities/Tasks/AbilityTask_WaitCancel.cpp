// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitCancel.h"

#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitCancel)


UAbilityTask_WaitCancel::UAbilityTask_WaitCancel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RegisteredCallbacks = false;

}

void UAbilityTask_WaitCancel::OnCancelCallback()
{
	if (AbilitySystemComponent.IsValid())
	{
		AbilitySystemComponent->ConsumeGenericReplicatedEvent(EAbilityGenericReplicatedEvent::GenericCancel, GetAbilitySpecHandle(), GetActivationPredictionKey());
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnCancel.Broadcast();
		}
		EndTask();
	}
}

void UAbilityTask_WaitCancel::OnLocalCancelCallback()
{
	FScopedPredictionWindow ScopedPrediction(AbilitySystemComponent.Get(), IsPredictingClient());

	if (AbilitySystemComponent.IsValid() && IsPredictingClient())
	{
		AbilitySystemComponent->ServerSetReplicatedEvent(EAbilityGenericReplicatedEvent::GenericCancel, GetAbilitySpecHandle(), GetActivationPredictionKey() ,AbilitySystemComponent->ScopedPredictionKey);
	}
	OnCancelCallback();
}

UAbilityTask_WaitCancel* UAbilityTask_WaitCancel::WaitCancel(UGameplayAbility* OwningAbility)
{
	return NewAbilityTask<UAbilityTask_WaitCancel>(OwningAbility);
}

void UAbilityTask_WaitCancel::Activate()
{
	if (AbilitySystemComponent.IsValid())
	{
		const FGameplayAbilityActorInfo* Info = Ability->GetCurrentActorInfo();

		
		if (Info->IsLocallyControlled())
		{
			// We have to wait for the callback from the AbilitySystemComponent.
			AbilitySystemComponent->GenericLocalCancelCallbacks.AddDynamic(this, &UAbilityTask_WaitCancel::OnLocalCancelCallback);	// Tell me if the cancel input is pressed

			RegisteredCallbacks = true;
		}
		else
		{
			if (CallOrAddReplicatedDelegate(EAbilityGenericReplicatedEvent::GenericCancel,  FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UAbilityTask_WaitCancel::OnCancelCallback)) )
			{
				// GenericCancel was already received from the client and we just called OnCancelCallback. The task is done.
				return;
			}
		}
	}
}

void UAbilityTask_WaitCancel::OnDestroy(bool AbilityEnding)
{
	if (RegisteredCallbacks && AbilitySystemComponent.IsValid())
	{
		AbilitySystemComponent->GenericLocalCancelCallbacks.RemoveDynamic(this, &UAbilityTask_WaitCancel::OnLocalCancelCallback);
	}

	Super::OnDestroy(AbilityEnding);
}

