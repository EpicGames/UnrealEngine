// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitConfirm.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitConfirm)

UAbilityTask_WaitConfirm::UAbilityTask_WaitConfirm(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RegisteredCallback = false;
}

void UAbilityTask_WaitConfirm::OnConfirmCallback(UGameplayAbility* InAbility)
{
	ABILITYTASK_MSG("OnConfirmCallback");
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnConfirm.Broadcast();
	}

	// We are done. Kill us so we don't keep getting broadcast messages
	EndTask();
}

UAbilityTask_WaitConfirm* UAbilityTask_WaitConfirm::WaitConfirm(class UGameplayAbility* OwningAbility)
{
	return NewAbilityTask<UAbilityTask_WaitConfirm>(OwningAbility);
}

void UAbilityTask_WaitConfirm::Activate()
{
	if (Ability)
	{
		if (Ability->GetCurrentActivationInfo().ActivationMode == EGameplayAbilityActivationMode::Predicting)
		{
			// Register delegate so that when we are confirmed by the server, we call OnConfirmCallback	
			
			OnConfirmCallbackDelegateHandle = Ability->OnConfirmDelegate.AddUObject(this, &UAbilityTask_WaitConfirm::OnConfirmCallback);
			RegisteredCallback = true;
		}
		else
		{
			// We have already been confirmed, just call the OnConfirm callback
			OnConfirmCallback(Ability);
		}
	}
}

void UAbilityTask_WaitConfirm::OnDestroy(bool AbilityEnded)
{
	if (RegisteredCallback && Ability)
	{
		Ability->OnConfirmDelegate.Remove(OnConfirmCallbackDelegateHandle);
	}

	Super::OnDestroy(AbilityEnded);
}

