// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitConfirmCancel.h"

#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitConfirmCancel)


UAbilityTask_WaitConfirmCancel::UAbilityTask_WaitConfirmCancel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RegisteredCallbacks = false;

}

void UAbilityTask_WaitConfirmCancel::OnConfirmCallback()
{
	
	if (AbilitySystemComponent.IsValid())
	{
		AbilitySystemComponent->ConsumeGenericReplicatedEvent(EAbilityGenericReplicatedEvent::GenericConfirm, GetAbilitySpecHandle(), GetActivationPredictionKey());
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnConfirm.Broadcast();
		}
		EndTask();
	}
}

void UAbilityTask_WaitConfirmCancel::OnCancelCallback()
{
	if (AbilitySystemComponent.IsValid())
	{
		AbilitySystemComponent->ConsumeGenericReplicatedEvent(EAbilityGenericReplicatedEvent::GenericCancel, GetAbilitySpecHandle(), GetActivationPredictionKey());
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnCancel.Broadcast();
		}
		EndTask();
	}
}

void UAbilityTask_WaitConfirmCancel::OnLocalConfirmCallback()
{
	FScopedPredictionWindow ScopedPrediction(AbilitySystemComponent.Get(), IsPredictingClient());

	if (AbilitySystemComponent.IsValid() && IsPredictingClient())
	{
		AbilitySystemComponent->ServerSetReplicatedEvent(EAbilityGenericReplicatedEvent::GenericConfirm, GetAbilitySpecHandle(), GetActivationPredictionKey() ,AbilitySystemComponent->ScopedPredictionKey);
	}
	OnConfirmCallback();
}

void UAbilityTask_WaitConfirmCancel::OnLocalCancelCallback()
{
	FScopedPredictionWindow ScopedPrediction(AbilitySystemComponent.Get(), IsPredictingClient());

	if (AbilitySystemComponent.IsValid() && IsPredictingClient())
	{
		AbilitySystemComponent->ServerSetReplicatedEvent(EAbilityGenericReplicatedEvent::GenericCancel, GetAbilitySpecHandle(), GetActivationPredictionKey() ,AbilitySystemComponent->ScopedPredictionKey);
	}
	OnCancelCallback();
}

UAbilityTask_WaitConfirmCancel* UAbilityTask_WaitConfirmCancel::WaitConfirmCancel(UGameplayAbility* OwningAbility)
{
	return NewAbilityTask<UAbilityTask_WaitConfirmCancel>(OwningAbility);
}

void UAbilityTask_WaitConfirmCancel::Activate()
{
	if (AbilitySystemComponent.IsValid() && Ability)
	{
		const FGameplayAbilityActorInfo* Info = Ability->GetCurrentActorInfo();

		
		if (Info->IsLocallyControlled())
		{
			// We have to wait for the callback from the AbilitySystemComponent.
			AbilitySystemComponent->GenericLocalConfirmCallbacks.AddDynamic(this, &UAbilityTask_WaitConfirmCancel::OnLocalConfirmCallback);	// Tell me if the confirm input is pressed
			AbilitySystemComponent->GenericLocalCancelCallbacks.AddDynamic(this, &UAbilityTask_WaitConfirmCancel::OnLocalCancelCallback);	// Tell me if the cancel input is pressed

			Ability->OnWaitingForConfirmInputBegin();

			RegisteredCallbacks = true;
		}
		else
		{
			if (CallOrAddReplicatedDelegate(EAbilityGenericReplicatedEvent::GenericConfirm,  FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UAbilityTask_WaitConfirmCancel::OnConfirmCallback)) )
			{
				// GenericConfirm was already received from the client and we just called OnConfirmCallback. The task is done.
				return;
			}
			if (CallOrAddReplicatedDelegate(EAbilityGenericReplicatedEvent::GenericCancel,  FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UAbilityTask_WaitConfirmCancel::OnCancelCallback)) )
			{
				// GenericCancel was already received from the client and we just called OnCancelCallback. The task is done.
				return;
			}
		}
	}
}

void UAbilityTask_WaitConfirmCancel::OnDestroy(bool AbilityEnding)
{
	if (RegisteredCallbacks && AbilitySystemComponent.IsValid())
	{
		AbilitySystemComponent->GenericLocalConfirmCallbacks.RemoveDynamic(this, &UAbilityTask_WaitConfirmCancel::OnLocalConfirmCallback);
		AbilitySystemComponent->GenericLocalCancelCallbacks.RemoveDynamic(this, &UAbilityTask_WaitConfirmCancel::OnLocalCancelCallback);

		if (Ability)
		{
			Ability->OnWaitingForConfirmInputEnd();
		}
	}

	Super::OnDestroy(AbilityEnding);
}

