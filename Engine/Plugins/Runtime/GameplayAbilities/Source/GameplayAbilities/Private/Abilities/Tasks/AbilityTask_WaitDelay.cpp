// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitDelay.h"
#include "TimerManager.h"
#include "AbilitySystemGlobals.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitDelay)

UAbilityTask_WaitDelay::UAbilityTask_WaitDelay(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Time = 0.f;
	TimeStarted = 0.f;
}

UAbilityTask_WaitDelay* UAbilityTask_WaitDelay::WaitDelay(UGameplayAbility* OwningAbility, float Time)
{
	UAbilitySystemGlobals::NonShipping_ApplyGlobalAbilityScaler_Duration(Time);

	UAbilityTask_WaitDelay* MyObj = NewAbilityTask<UAbilityTask_WaitDelay>(OwningAbility);
	MyObj->Time = Time;
	return MyObj;
}

void UAbilityTask_WaitDelay::Activate()
{
	UWorld* World = GetWorld();
	TimeStarted = World->GetTimeSeconds();

	if (Time <= 0.0f)
	{
		World->GetTimerManager().SetTimerForNextTick(this, &UAbilityTask_WaitDelay::OnTimeFinish);
	}
	else
	{
		// Use a dummy timer handle as we don't need to store it for later but we don't need to look for something to clear
		FTimerHandle TimerHandle;
		World->GetTimerManager().SetTimer(TimerHandle, this, &UAbilityTask_WaitDelay::OnTimeFinish, Time, false);
	}
}

void UAbilityTask_WaitDelay::OnTimeFinish()
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnFinish.Broadcast();
	}
	EndTask();
}

FString UAbilityTask_WaitDelay::GetDebugString() const
{
	if (UWorld* World = GetWorld())
	{
		const float TimeLeft = Time - World->TimeSince(TimeStarted);
		return FString::Printf(TEXT("WaitDelay. Time: %.2f. TimeLeft: %.2f"), Time, TimeLeft);
	}
	else
	{
		return FString::Printf(TEXT("WaitDelay. Time: %.2f. Time Started: %.2f"), Time, TimeStarted);
	}
}
