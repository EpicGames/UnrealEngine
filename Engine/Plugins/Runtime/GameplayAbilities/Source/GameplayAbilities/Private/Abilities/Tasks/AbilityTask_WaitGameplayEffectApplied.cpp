// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitGameplayEffectApplied.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitGameplayEffectApplied)

UAbilityTask_WaitGameplayEffectApplied::UAbilityTask_WaitGameplayEffectApplied(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAbilityTask_WaitGameplayEffectApplied::Activate()
{
	if (GetASC())
	{
		RegisterDelegate();
	}
}

void UAbilityTask_WaitGameplayEffectApplied::OnApplyGameplayEffectCallback(UAbilitySystemComponent* Target, const FGameplayEffectSpec& SpecApplied, FActiveGameplayEffectHandle ActiveHandle)
{
	bool PassedComparison = false;

	AActor* AvatarActor = Target ? Target->GetAvatarActor_Direct() : nullptr;

	if (!Filter.FilterPassesForActor(AvatarActor))
	{
		return;
	}

	// Check tag queries if the ability task was created using with tag requirement params.
	if (!SourceTagRequirements.RequirementsMet(*SpecApplied.CapturedSourceTags.GetAggregatedTags()))
	{
		ABILITY_LOG(Verbose, TEXT("WaitGameplayEffectApplied in Ability '%s': Not triggering for Applied Spec: %s, SourceTagRequirements not passed: %s"), *GetNameSafe(Ability), *SpecApplied.ToSimpleString(), *SourceTagRequirements.ToString());
		return;
	}

	if (!TargetTagRequirements.RequirementsMet(*SpecApplied.CapturedTargetTags.GetAggregatedTags()))
	{
		ABILITY_LOG(Verbose, TEXT("WaitGameplayEffectApplied in Ability '%s': Not triggering for Applied Spec: %s, TargetTagRequirements not passed: %s"), *GetNameSafe(Ability), *SpecApplied.ToSimpleString(), *TargetTagRequirements.ToString());
		return;
	}

	FGameplayTagContainer AllAssetTags;
	SpecApplied.GetAllAssetTags(AllAssetTags);
	if (!AssetTagRequirements.RequirementsMet(AllAssetTags))
	{
		ABILITY_LOG(Verbose, TEXT("WaitGameplayEffectApplied in Ability '%s': Not triggering for Applied Spec: %s, AssetTagRequirements not passed: %s"), *GetNameSafe(Ability), *SpecApplied.ToSimpleString(), *AssetTagRequirements.ToString());
		return;
	}

	FGameplayTagContainer AllGrantedTags;
	SpecApplied.GetAllGrantedTags(AllGrantedTags);
	if (!GrantedTagRequirements.RequirementsMet(AllGrantedTags))
	{
		ABILITY_LOG(Verbose, TEXT("WaitGameplayEffectApplied in Ability '%s': Not triggering for Applied Spec: %s, GrantedTagRequirements not passed: %s"), *GetNameSafe(Ability), *SpecApplied.ToSimpleString(), *GrantedTagRequirements.ToString());
		return;
	}

	// Check tag queries if the ability task was created using with tag query params.
	if (!SourceTagQuery.IsEmpty() && !SourceTagQuery.Matches(*SpecApplied.CapturedSourceTags.GetAggregatedTags()))
	{
		ABILITY_LOG(Verbose, TEXT("WaitGameplayEffectApplied in Ability '%s': Not triggering for Applied Spec: %s, SourceTagQuery not passed: %s"), *GetNameSafe(Ability), *SpecApplied.ToSimpleString(), *SourceTagQuery.GetDescription());
		return;
	}

	if (!TargetTagQuery.IsEmpty() && !TargetTagQuery.Matches(*SpecApplied.CapturedTargetTags.GetAggregatedTags()))
	{
		ABILITY_LOG(Verbose, TEXT("WaitGameplayEffectApplied in Ability '%s': Not triggering for Applied Spec: %s, TargetTagQuery not passed: %s"), *GetNameSafe(Ability), *SpecApplied.ToSimpleString(), *TargetTagQuery.GetDescription());
		return;
	}

	if (!AssetTagQuery.IsEmpty() && !AssetTagQuery.Matches(AllAssetTags))
	{
		ABILITY_LOG(Verbose, TEXT("WaitGameplayEffectApplied in Ability '%s': Not triggering for Applied Spec: %s, AssetTagQuery not passed: %s"), *GetNameSafe(Ability), *SpecApplied.ToSimpleString(), *AssetTagQuery.GetDescription());
		return;
	}

	if (!GrantedTagQuery.IsEmpty() && !GrantedTagQuery.Matches(AllGrantedTags))
	{
		ABILITY_LOG(Verbose, TEXT("WaitGameplayEffectApplied in Ability '%s': Not triggering for Applied Spec: %s, GrantedTagQuery not passed: %s"), *GetNameSafe(Ability), *SpecApplied.ToSimpleString(), *GrantedTagQuery.GetDescription());
		return;
	}

	// We allow GameplayEffect application to trigger other GameplayEffect application (as of UE 5.7). However, in order to prevent infinite recursions, 
	// we don't allow any effect class to be applied recursively. Keep track of effect classes as this function is entered recursively and abort if the
	// current applied effect class is already being applied higher in the callstack.
	const TSubclassOf<UGameplayEffect> EffectClass = SpecApplied.Def ? SpecApplied.Def->GetClass() : nullptr;
	if (BroadcastingEffectStack.Contains(EffectClass))
	{
		ABILITY_LOG(Error, TEXT("WaitGameplayEffectApplied in Ability '%s': recursive application of '%s' detected. This could cause an infinite loop, ignoring broadcast. Current stack: %s"), *GetNameSafe(Ability), *GetNameSafe(EffectClass), *GetBroadcastingEffectStackString());
		return;
	}
	else if (UE_LOG_ACTIVE(LogAbilitySystem, VeryVerbose) && !BroadcastingEffectStack.IsEmpty())
	{
		ABILITY_LOG(VeryVerbose, TEXT("WaitGameplayEffectApplied in Ability '%s': application of '%s' causes no recursion. Current stack: %s"), *GetNameSafe(Ability), *GetNameSafe(EffectClass), *GetBroadcastingEffectStackString());
	}
	
	FGameplayEffectSpecHandle	SpecHandle(new FGameplayEffectSpec(SpecApplied));
	{
		BroadcastingEffectStack.Add(EffectClass);
		BroadcastDelegate(AvatarActor, SpecHandle, ActiveHandle);
		BroadcastingEffectStack.RemoveSingle(EffectClass);
	}

	if (TriggerOnce)
	{
		EndTask();
	}
}

FString UAbilityTask_WaitGameplayEffectApplied::GetBroadcastingEffectStackString() const
{
	TArray<FString> AllEffectNames;
	for (const TSubclassOf<UGameplayEffect>& ExistingEffectClass : BroadcastingEffectStack)
	{
		AllEffectNames.Add(GetNameSafe(ExistingEffectClass));

	}
	return FString::Join(AllEffectNames, TEXT(","));
}

void UAbilityTask_WaitGameplayEffectApplied::OnDestroy(bool AbilityEnded)
{
	if (GetASC())
	{
		RemoveDelegate();
	}

	Super::OnDestroy(AbilityEnded);
}

void UAbilityTask_WaitGameplayEffectApplied::SetExternalActor(AActor* InActor)
{
	if (InActor)
	{
		UseExternalOwner = true;
		ExternalOwner  = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(InActor);
	}
}

UAbilitySystemComponent* UAbilityTask_WaitGameplayEffectApplied::GetASC()
{
	if (UseExternalOwner)
	{
		return ExternalOwner;
	}
	return AbilitySystemComponent.Get();
}

