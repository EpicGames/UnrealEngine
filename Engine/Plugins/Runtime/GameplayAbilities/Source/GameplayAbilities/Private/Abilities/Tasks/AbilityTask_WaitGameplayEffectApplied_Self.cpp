// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitGameplayEffectApplied_Self.h"

#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitGameplayEffectApplied_Self)


UAbilityTask_WaitGameplayEffectApplied_Self::UAbilityTask_WaitGameplayEffectApplied_Self(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAbilityTask_WaitGameplayEffectApplied_Self* UAbilityTask_WaitGameplayEffectApplied_Self::WaitGameplayEffectAppliedToSelf(UGameplayAbility* OwningAbility, const FGameplayTargetDataFilterHandle InFilter, FGameplayTagRequirements InSourceTagRequirements, FGameplayTagRequirements InTargetTagRequirements, FGameplayTagRequirements InAssetTagRequirements, FGameplayTagRequirements InGrantedTagRequirements, bool InTriggerOnce, AActor* OptionalExternalOwner, bool InListenForPeriodicEffect)
{
	UAbilityTask_WaitGameplayEffectApplied_Self* MyObj = NewAbilityTask<UAbilityTask_WaitGameplayEffectApplied_Self>(OwningAbility);
	MyObj->Filter = InFilter;
	MyObj->SourceTagRequirements = InSourceTagRequirements;
	MyObj->TargetTagRequirements = InTargetTagRequirements;
	MyObj->AssetTagRequirements = InAssetTagRequirements;
	MyObj->GrantedTagRequirements = InGrantedTagRequirements;
	MyObj->TriggerOnce = InTriggerOnce;
	MyObj->SetExternalActor(OptionalExternalOwner);
	MyObj->ListenForPeriodicEffects = InListenForPeriodicEffect;
	return MyObj;
}

UAbilityTask_WaitGameplayEffectApplied_Self* UAbilityTask_WaitGameplayEffectApplied_Self::WaitGameplayEffectAppliedToSelf_Query(UGameplayAbility* OwningAbility, const FGameplayTargetDataFilterHandle InFilter, FGameplayTagQuery InSourceTagQuery, FGameplayTagQuery InTargetTagQuery, FGameplayTagQuery InAssetTagQuery, FGameplayTagQuery InGrantedTagQuery, bool InTriggerOnce, AActor* OptionalExternalOwner, bool InListenForPeriodicEffect)
{
	UAbilityTask_WaitGameplayEffectApplied_Self* MyObj = NewAbilityTask<UAbilityTask_WaitGameplayEffectApplied_Self>(OwningAbility);
	MyObj->Filter = InFilter;
	MyObj->SourceTagQuery = InSourceTagQuery;
	MyObj->TargetTagQuery = InTargetTagQuery;
	MyObj->AssetTagQuery = InAssetTagQuery;
	MyObj->GrantedTagQuery = InGrantedTagQuery;
	MyObj->TriggerOnce = InTriggerOnce;
	MyObj->SetExternalActor(OptionalExternalOwner);
	MyObj->ListenForPeriodicEffects = InListenForPeriodicEffect;
	return MyObj;
}

void UAbilityTask_WaitGameplayEffectApplied_Self::BroadcastDelegate(AActor* Avatar, FGameplayEffectSpecHandle SpecHandle, FActiveGameplayEffectHandle ActiveHandle)
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnApplied.Broadcast(Avatar, SpecHandle, ActiveHandle);
	}
}

void UAbilityTask_WaitGameplayEffectApplied_Self::RegisterDelegate()
{
	OnApplyGameplayEffectCallbackDelegateHandle = GetASC()->OnGameplayEffectAppliedDelegateToSelf.AddUObject(this, &UAbilityTask_WaitGameplayEffectApplied::OnApplyGameplayEffectCallback);
	if (ListenForPeriodicEffects)
	{
		OnPeriodicGameplayEffectExecuteCallbackDelegateHandle = GetASC()->OnPeriodicGameplayEffectExecuteDelegateOnSelf.AddUObject(this, &UAbilityTask_WaitGameplayEffectApplied::OnApplyGameplayEffectCallback);
	}
}

void UAbilityTask_WaitGameplayEffectApplied_Self::RemoveDelegate()
{
	GetASC()->OnGameplayEffectAppliedDelegateToSelf.Remove(OnApplyGameplayEffectCallbackDelegateHandle);
	if (OnPeriodicGameplayEffectExecuteCallbackDelegateHandle.IsValid())
	{
		GetASC()->OnPeriodicGameplayEffectExecuteDelegateOnSelf.Remove(OnPeriodicGameplayEffectExecuteCallbackDelegateHandle);
	}
}

