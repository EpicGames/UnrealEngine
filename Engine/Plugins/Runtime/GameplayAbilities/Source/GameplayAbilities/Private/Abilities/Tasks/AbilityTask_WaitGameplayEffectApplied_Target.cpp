// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitGameplayEffectApplied_Target.h"

#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitGameplayEffectApplied_Target)


UAbilityTask_WaitGameplayEffectApplied_Target::UAbilityTask_WaitGameplayEffectApplied_Target(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAbilityTask_WaitGameplayEffectApplied_Target* UAbilityTask_WaitGameplayEffectApplied_Target::WaitGameplayEffectAppliedToTarget(UGameplayAbility* OwningAbility, const FGameplayTargetDataFilterHandle InFilter, FGameplayTagRequirements InSourceTagRequirements, FGameplayTagRequirements InTargetTagRequirements, FGameplayTagRequirements InAssetTagRequirements, FGameplayTagRequirements InGrantedTagRequirements, bool InTriggerOnce, AActor* OptionalExternalOwner, bool InListenForPeriodicEffect)
{
	UAbilityTask_WaitGameplayEffectApplied_Target* MyObj = NewAbilityTask<UAbilityTask_WaitGameplayEffectApplied_Target>(OwningAbility);
	MyObj->Filter = InFilter;
	MyObj->SourceTagRequirements = InSourceTagRequirements;
	MyObj->TargetTagRequirements = InTargetTagRequirements;
	MyObj->AssetTagRequirements = InAssetTagRequirements;
	MyObj->GrantedTagRequirements = InGrantedTagRequirements;
	MyObj->TriggerOnce = InTriggerOnce;
	MyObj->SetExternalActor(OptionalExternalOwner);
	MyObj->ListenForPeriodicEffects = InListenForPeriodicEffect;
	return MyObj;
}

UAbilityTask_WaitGameplayEffectApplied_Target* UAbilityTask_WaitGameplayEffectApplied_Target::WaitGameplayEffectAppliedToTarget_Query(UGameplayAbility* OwningAbility, const FGameplayTargetDataFilterHandle InFilter, FGameplayTagQuery SourceTagQuery, FGameplayTagQuery TargetTagQuery, FGameplayTagQuery AssetTagQuery, FGameplayTagQuery GrantedTagQuery, bool InTriggerOnce, AActor* OptionalExternalOwner, bool InListenForPeriodicEffect)
{
	UAbilityTask_WaitGameplayEffectApplied_Target* MyObj = NewAbilityTask<UAbilityTask_WaitGameplayEffectApplied_Target>(OwningAbility);
	MyObj->Filter = InFilter;
	MyObj->SourceTagQuery = SourceTagQuery;
	MyObj->TargetTagQuery = TargetTagQuery;
	MyObj->AssetTagQuery = AssetTagQuery;
	MyObj->GrantedTagQuery = GrantedTagQuery;
	MyObj->TriggerOnce = InTriggerOnce;
	MyObj->SetExternalActor(OptionalExternalOwner);
	MyObj->ListenForPeriodicEffects = InListenForPeriodicEffect;
	return MyObj;
}

void UAbilityTask_WaitGameplayEffectApplied_Target::BroadcastDelegate(AActor* Avatar, FGameplayEffectSpecHandle SpecHandle, FActiveGameplayEffectHandle ActiveHandle)
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnApplied.Broadcast(Avatar, SpecHandle, ActiveHandle);
	}
}

void UAbilityTask_WaitGameplayEffectApplied_Target::RegisterDelegate()
{
	OnApplyGameplayEffectCallbackDelegateHandle = GetASC()->OnGameplayEffectAppliedDelegateToTarget.AddUObject(this, &UAbilityTask_WaitGameplayEffectApplied::OnApplyGameplayEffectCallback);
	if (ListenForPeriodicEffects)
	{
		OnPeriodicGameplayEffectExecuteCallbackDelegateHandle = GetASC()->OnPeriodicGameplayEffectExecuteDelegateOnTarget.AddUObject(this, &UAbilityTask_WaitGameplayEffectApplied::OnApplyGameplayEffectCallback);
	}
}

void UAbilityTask_WaitGameplayEffectApplied_Target::RemoveDelegate()
{
	GetASC()->OnGameplayEffectAppliedDelegateToTarget.Remove(OnApplyGameplayEffectCallbackDelegateHandle);
	if (OnPeriodicGameplayEffectExecuteCallbackDelegateHandle.IsValid())
	{
		GetASC()->OnPeriodicGameplayEffectExecuteDelegateOnTarget.Remove(OnPeriodicGameplayEffectExecuteCallbackDelegateHandle);
	}
}

