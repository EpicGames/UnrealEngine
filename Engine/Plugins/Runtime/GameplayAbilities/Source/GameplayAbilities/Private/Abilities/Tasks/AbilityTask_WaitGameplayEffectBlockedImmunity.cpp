// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitGameplayEffectBlockedImmunity.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitGameplayEffectBlockedImmunity)

UAbilityTask_WaitGameplayEffectBlockedImmunity::UAbilityTask_WaitGameplayEffectBlockedImmunity(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	UseExternalOwner = false;
	ExternalOwner = nullptr;
}

UAbilityTask_WaitGameplayEffectBlockedImmunity* UAbilityTask_WaitGameplayEffectBlockedImmunity::WaitGameplayEffectBlockedByImmunity(UGameplayAbility* OwningAbility, FGameplayTagRequirements InSourceTagRequirements, FGameplayTagRequirements InTargetTagRequirements, AActor* InOptionalExternalTarget, bool InTriggerOnce)
{
	UAbilityTask_WaitGameplayEffectBlockedImmunity* MyObj = NewAbilityTask<UAbilityTask_WaitGameplayEffectBlockedImmunity>(OwningAbility);
	MyObj->SourceTagRequirements = InSourceTagRequirements;
	MyObj->TargetTagRequirements = InTargetTagRequirements;
	MyObj->TriggerOnce = InTriggerOnce;
	MyObj->SetExternalActor(InOptionalExternalTarget);
	return MyObj;
}

void UAbilityTask_WaitGameplayEffectBlockedImmunity::Activate()
{
	if (GetASC())
	{
		RegisterDelegate();
	}
}

void UAbilityTask_WaitGameplayEffectBlockedImmunity::ImmunityCallback(const FGameplayEffectSpec& BlockedSpec, const FActiveGameplayEffect* ImmunityGE)
{
	bool PassedComparison = false;

	if (!SourceTagRequirements.RequirementsMet(*BlockedSpec.CapturedSourceTags.GetAggregatedTags()))
	{
		return;
	}
	if (!TargetTagRequirements.RequirementsMet(*BlockedSpec.CapturedTargetTags.GetAggregatedTags()))
	{
		return;
	}
	
	// We have to copy the spec, since the blocked spec is not ours
	FGameplayEffectSpecHandle SpecHandle(new FGameplayEffectSpec(BlockedSpec));

	if (ShouldBroadcastAbilityTaskDelegates())
	{
		Blocked.Broadcast(SpecHandle, ImmunityGE->Handle);
	}
	
	if (TriggerOnce)
	{
		EndTask();
	}
}

void UAbilityTask_WaitGameplayEffectBlockedImmunity::OnDestroy(bool AbilityEnded)
{
	if (GetASC())
	{
		RemoveDelegate();
	}

	Super::OnDestroy(AbilityEnded);
}

void UAbilityTask_WaitGameplayEffectBlockedImmunity::SetExternalActor(AActor* InActor)
{
	if (InActor)
	{
		UseExternalOwner = true;
		ExternalOwner  = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(InActor);
	}
}

UAbilitySystemComponent* UAbilityTask_WaitGameplayEffectBlockedImmunity::GetASC()
{
	if (UseExternalOwner)
	{
		return ExternalOwner;
	}
	return AbilitySystemComponent.Get();
}

void UAbilityTask_WaitGameplayEffectBlockedImmunity::RegisterDelegate()
{
	if (UAbilitySystemComponent* ASC =  GetASC())
	{
		// Only do this on the server. Clients could mispredict.
		if (ASC->IsNetSimulating() == false)
		{
			DelegateHandle = ASC->OnImmunityBlockGameplayEffectDelegate.AddUObject(this, &UAbilityTask_WaitGameplayEffectBlockedImmunity::ImmunityCallback);
		}
	}
}

void UAbilityTask_WaitGameplayEffectBlockedImmunity::RemoveDelegate()
{
	if (DelegateHandle.IsValid())
	{
		if (UAbilitySystemComponent* ASC =  GetASC())
		{
			ASC->OnImmunityBlockGameplayEffectDelegate.Remove(DelegateHandle);
			DelegateHandle.Reset();
		}
	}
}

