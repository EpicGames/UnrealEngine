// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitGameplayEffectRemoved.h"

#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitGameplayEffectRemoved)


UAbilityTask_WaitGameplayEffectRemoved::UAbilityTask_WaitGameplayEffectRemoved(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Registered = false;
}

UAbilityTask_WaitGameplayEffectRemoved* UAbilityTask_WaitGameplayEffectRemoved::WaitForGameplayEffectRemoved(UGameplayAbility* OwningAbility, FActiveGameplayEffectHandle InHandle)
{
	UAbilityTask_WaitGameplayEffectRemoved* MyObj = NewAbilityTask<UAbilityTask_WaitGameplayEffectRemoved>(OwningAbility);
	MyObj->Handle = InHandle;

	return MyObj;
}

void UAbilityTask_WaitGameplayEffectRemoved::Activate()
{
	FGameplayEffectRemovalInfo EmptyGameplayEffectRemovalInfo;

	if (Handle.IsValid() == false)
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			InvalidHandle.Broadcast(EmptyGameplayEffectRemovalInfo);
		}
		EndTask();
		return;
	}

	UAbilitySystemComponent* EffectOwningAbilitySystemComponent = Handle.GetOwningAbilitySystemComponent();

	if (EffectOwningAbilitySystemComponent)
	{
		FOnActiveGameplayEffectRemoved_Info* DelPtr = EffectOwningAbilitySystemComponent->OnGameplayEffectRemoved_InfoDelegate(Handle);
		if (DelPtr)
		{
			OnGameplayEffectRemovedDelegateHandle = DelPtr->AddUObject(this, &UAbilityTask_WaitGameplayEffectRemoved::OnGameplayEffectRemoved);
			Registered = true;
		}
	}

	if (!Registered)
	{
		// GameplayEffect was already removed, treat this as a warning? Could be cases of immunity or chained gameplay rules that would instant remove something
		OnGameplayEffectRemoved(EmptyGameplayEffectRemovalInfo);
	}
}

void UAbilityTask_WaitGameplayEffectRemoved::OnDestroy(bool AbilityIsEnding)
{
	UAbilitySystemComponent* EffectOwningAbilitySystemComponent = Handle.GetOwningAbilitySystemComponent();
	if (EffectOwningAbilitySystemComponent)
	{
		FOnActiveGameplayEffectRemoved_Info* DelPtr = EffectOwningAbilitySystemComponent->OnGameplayEffectRemoved_InfoDelegate(Handle);
		if (DelPtr)
		{
			DelPtr->Remove(OnGameplayEffectRemovedDelegateHandle);
		}
	}

	Super::OnDestroy(AbilityIsEnding);
}

void UAbilityTask_WaitGameplayEffectRemoved::OnGameplayEffectRemoved(const FGameplayEffectRemovalInfo& InGameplayEffectRemovalInfo)
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnRemoved.Broadcast(InGameplayEffectRemovalInfo);
	}
	EndTask();
}



