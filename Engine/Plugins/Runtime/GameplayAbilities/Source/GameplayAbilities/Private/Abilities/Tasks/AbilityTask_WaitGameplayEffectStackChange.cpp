// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitGameplayEffectStackChange.h"

#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitGameplayEffectStackChange)


UAbilityTask_WaitGameplayEffectStackChange::UAbilityTask_WaitGameplayEffectStackChange(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Registered = false;
}

UAbilityTask_WaitGameplayEffectStackChange* UAbilityTask_WaitGameplayEffectStackChange::WaitForGameplayEffectStackChange(UGameplayAbility* OwningAbility, FActiveGameplayEffectHandle InHandle)
{
	UAbilityTask_WaitGameplayEffectStackChange* MyObj = NewAbilityTask<UAbilityTask_WaitGameplayEffectStackChange>(OwningAbility);
	MyObj->Handle = InHandle;
	return MyObj;
}

void UAbilityTask_WaitGameplayEffectStackChange::Activate()
{
	if (Handle.IsValid() == false)
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			InvalidHandle.Broadcast(Handle, 0, 0);
		}
		EndTask();
		return;;
	}

	UAbilitySystemComponent* EffectOwningAbilitySystemComponent = Handle.GetOwningAbilitySystemComponent();

	if (EffectOwningAbilitySystemComponent)
	{
		FOnActiveGameplayEffectStackChange* DelPtr = EffectOwningAbilitySystemComponent->OnGameplayEffectStackChangeDelegate(Handle);
		if (DelPtr)
		{
			OnGameplayEffectStackChangeDelegateHandle = DelPtr->AddUObject(this, &UAbilityTask_WaitGameplayEffectStackChange::OnGameplayEffectStackChange);
			Registered = true;
		}
	}
}

void UAbilityTask_WaitGameplayEffectStackChange::OnDestroy(bool AbilityIsEnding)
{
	UAbilitySystemComponent* EffectOwningAbilitySystemComponent = Handle.GetOwningAbilitySystemComponent();
	if (EffectOwningAbilitySystemComponent && OnGameplayEffectStackChangeDelegateHandle.IsValid())
	{
		FOnActiveGameplayEffectStackChange* DelPtr = EffectOwningAbilitySystemComponent->OnGameplayEffectStackChangeDelegate(Handle);
		if (DelPtr)
		{
			DelPtr->Remove(OnGameplayEffectStackChangeDelegateHandle);
		}
	}

	Super::OnDestroy(AbilityIsEnding);
}

void UAbilityTask_WaitGameplayEffectStackChange::OnGameplayEffectStackChange(FActiveGameplayEffectHandle InHandle, int32 NewCount, int32 OldCount)
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnChange.Broadcast(InHandle, NewCount, OldCount);
	}
}

