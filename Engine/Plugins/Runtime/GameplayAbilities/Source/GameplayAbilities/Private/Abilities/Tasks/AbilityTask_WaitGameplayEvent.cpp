// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitGameplayEvent.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitGameplayEvent)

// ----------------------------------------------------------------

UAbilityTask_WaitGameplayEvent::UAbilityTask_WaitGameplayEvent(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{

}

UAbilityTask_WaitGameplayEvent* UAbilityTask_WaitGameplayEvent::WaitGameplayEvent(UGameplayAbility* OwningAbility, FGameplayTag Tag, AActor* OptionalExternalTarget, bool OnlyTriggerOnce, bool OnlyMatchExact)
{
	UAbilityTask_WaitGameplayEvent* MyObj = NewAbilityTask<UAbilityTask_WaitGameplayEvent>(OwningAbility);
	MyObj->Tag = Tag;
	MyObj->SetExternalTarget(OptionalExternalTarget);
	MyObj->OnlyTriggerOnce = OnlyTriggerOnce;
	MyObj->OnlyMatchExact = OnlyMatchExact;

	return MyObj;
}

void UAbilityTask_WaitGameplayEvent::Activate()
{
	UAbilitySystemComponent* ASC = GetTargetASC();
	if (ASC)
	{
		if (OnlyMatchExact)
		{
			MyHandle = ASC->GenericGameplayEventCallbacks.FindOrAdd(Tag).AddUObject(this, &UAbilityTask_WaitGameplayEvent::GameplayEventCallback);
		}
		else
		{
			MyHandle = ASC->AddGameplayEventTagContainerDelegate(FGameplayTagContainer(Tag), FGameplayEventTagMulticastDelegate::FDelegate::CreateUObject(this, &UAbilityTask_WaitGameplayEvent::GameplayEventContainerCallback));
		}	
	}

	Super::Activate();
}

void UAbilityTask_WaitGameplayEvent::GameplayEventCallback(const FGameplayEventData* Payload)
{
	GameplayEventContainerCallback(Tag, Payload);
}

void UAbilityTask_WaitGameplayEvent::GameplayEventContainerCallback(FGameplayTag MatchingTag, const FGameplayEventData* Payload)
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		ensureMsgf(Payload, TEXT("GameplayEventCallback expected non-null Payload"));
		FGameplayEventData TempPayload = Payload ? *Payload : FGameplayEventData{};
		TempPayload.EventTag = MatchingTag;
		EventReceived.Broadcast(MoveTemp(TempPayload));
	}
	if (OnlyTriggerOnce)
	{
		EndTask();
	}
}

void UAbilityTask_WaitGameplayEvent::SetExternalTarget(AActor* Actor)
{
	if (Actor)
	{
		UseExternalTarget = true;
		OptionalExternalTarget = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Actor);
	}
}

UAbilitySystemComponent* UAbilityTask_WaitGameplayEvent::GetTargetASC()
{
	if (UseExternalTarget)
	{
		return OptionalExternalTarget;
	}

	return AbilitySystemComponent.Get();
}

void UAbilityTask_WaitGameplayEvent::OnDestroy(bool AbilityEnding)
{
	UAbilitySystemComponent* ASC = GetTargetASC();
	if (ASC && MyHandle.IsValid())
	{
		if (OnlyMatchExact)
		{
			ASC->GenericGameplayEventCallbacks.FindOrAdd(Tag).Remove(MyHandle);
		}
		else
		{
			ASC->RemoveGameplayEventTagContainerDelegate(FGameplayTagContainer(Tag), MyHandle);
		}
		
	}

	Super::OnDestroy(AbilityEnding);
}

