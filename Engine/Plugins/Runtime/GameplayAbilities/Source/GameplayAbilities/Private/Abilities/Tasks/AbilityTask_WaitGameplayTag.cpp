// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitGameplayTag.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitGameplayTag)

// ----------------------------------------------------------------

UAbilityTask_WaitGameplayTagAdded::UAbilityTask_WaitGameplayTagAdded(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{

}

UAbilityTask_WaitGameplayTagAdded* UAbilityTask_WaitGameplayTagAdded::WaitGameplayTagAdd(UGameplayAbility* OwningAbility, FGameplayTag Tag, AActor* InOptionalExternalTarget, bool OnlyTriggerOnce)
{
	UAbilityTask_WaitGameplayTagAdded* MyObj = NewAbilityTask<UAbilityTask_WaitGameplayTagAdded>(OwningAbility);
	MyObj->Tag = Tag;
	MyObj->SetExternalTarget(InOptionalExternalTarget);
	MyObj->OnlyTriggerOnce = OnlyTriggerOnce;

	return MyObj;
}

void UAbilityTask_WaitGameplayTagAdded::Activate()
{
	UAbilitySystemComponent* ASC = GetTargetASC();
	if (ASC && ASC->HasMatchingGameplayTag(Tag))
	{	
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			Added.Broadcast();
		}
		if(OnlyTriggerOnce)
		{
			EndTask();
			return;
		}
	}

	Super::Activate();
}

void UAbilityTask_WaitGameplayTagAdded::GameplayTagCallback(const FGameplayTag InTag, int32 NewCount)
{
	if (NewCount==1)
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			Added.Broadcast();
		}
		if(OnlyTriggerOnce)
		{
			EndTask();
		}
	}
}

// ----------------------------------------------------------------

UAbilityTask_WaitGameplayTagRemoved::UAbilityTask_WaitGameplayTagRemoved(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{

}

UAbilityTask_WaitGameplayTagRemoved* UAbilityTask_WaitGameplayTagRemoved::WaitGameplayTagRemove(UGameplayAbility* OwningAbility, FGameplayTag Tag, AActor* InOptionalExternalTarget, bool OnlyTriggerOnce)
{
	UAbilityTask_WaitGameplayTagRemoved* MyObj = NewAbilityTask<UAbilityTask_WaitGameplayTagRemoved>(OwningAbility);
	MyObj->Tag = Tag;
	MyObj->SetExternalTarget(InOptionalExternalTarget);
	MyObj->OnlyTriggerOnce = OnlyTriggerOnce;

	return MyObj;
}

void UAbilityTask_WaitGameplayTagRemoved::Activate()
{
	UAbilitySystemComponent* ASC = GetTargetASC();
	if (ASC && ASC->HasMatchingGameplayTag(Tag) == false)
	{			
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			Removed.Broadcast();
		}
		if(OnlyTriggerOnce)
		{
			EndTask();
			return;
		}
	}

	Super::Activate();
}

void UAbilityTask_WaitGameplayTagRemoved::GameplayTagCallback(const FGameplayTag InTag, int32 NewCount)
{
	if (NewCount==0)
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			Removed.Broadcast();
		}
		if(OnlyTriggerOnce)
		{
			EndTask();
		}
	}
}

// ----------------------------------------------------------------

