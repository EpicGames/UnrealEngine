// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitGameplayTagBase.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitGameplayTagBase)

UAbilityTask_WaitGameplayTag::UAbilityTask_WaitGameplayTag(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RegisteredCallback = false;
	UseExternalTarget = false;
	OnlyTriggerOnce = false;
}

void UAbilityTask_WaitGameplayTag::Activate()
{
	UAbilitySystemComponent* ASC = GetTargetASC();
	if (ASC)
	{
		DelegateHandle = ASC->RegisterGameplayTagEvent(Tag).AddUObject(this, &UAbilityTask_WaitGameplayTag::GameplayTagCallback);
		RegisteredCallback = true;
	}
}

void UAbilityTask_WaitGameplayTag::GameplayTagCallback(const FGameplayTag InTag, int32 NewCount)
{
}

void UAbilityTask_WaitGameplayTag::OnDestroy(bool AbilityIsEnding)
{
	UAbilitySystemComponent* ASC = GetTargetASC();
	if (RegisteredCallback && ASC)
	{
		ASC->RegisterGameplayTagEvent(Tag).Remove(DelegateHandle);
	}

	Super::OnDestroy(AbilityIsEnding);
}

UAbilitySystemComponent* UAbilityTask_WaitGameplayTag::GetTargetASC()
{
	if (UseExternalTarget)
	{
		return OptionalExternalTarget;
	}

	return AbilitySystemComponent.Get();
}

void UAbilityTask_WaitGameplayTag::SetExternalTarget(AActor* Actor)
{
	if (Actor)
	{
		UseExternalTarget = true;
		OptionalExternalTarget = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Actor);
	}
}

