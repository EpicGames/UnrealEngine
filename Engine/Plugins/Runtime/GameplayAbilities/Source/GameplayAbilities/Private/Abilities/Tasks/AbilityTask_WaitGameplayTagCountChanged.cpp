// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitGameplayTagCountChanged.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitGameplayTagCountChanged)

UAbilityTask_WaitGameplayTagCountChanged* UAbilityTask_WaitGameplayTagCountChanged::WaitGameplayTagCountChange(UGameplayAbility* OwningAbility, FGameplayTag Tag, AActor* InOptionalExternalTarget /*= nullptr*/)
{
	UAbilityTask_WaitGameplayTagCountChanged* MyObj = NewAbilityTask<UAbilityTask_WaitGameplayTagCountChanged>(OwningAbility);
	MyObj->Tag = Tag;
	if (InOptionalExternalTarget)
	{
		MyObj->OptionalExternalTarget = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(InOptionalExternalTarget);
		if (!MyObj->OptionalExternalTarget)
		{
			ABILITY_LOG(Warning, TEXT("%s: Passed OptionalExternalTarget's AbilitySystemComponent is nullptr! Will not be able to register for gameplay tag event with Tag = %s. Will instead register event with the AbilitySystemComponent this task's owner (%s)."), *MyObj->GetName(), *Tag.GetTagName().ToString(), *GetNameSafe(MyObj->GetOwnerActor()));
		}
	}

	return MyObj;
}

void UAbilityTask_WaitGameplayTagCountChanged::Activate()
{
	Super::Activate();

	UAbilitySystemComponent* ASC = GetTargetASC();
	if (ASC)
	{
		GameplayTagCountChangedHandle = ASC->RegisterGameplayTagEvent(Tag, EGameplayTagEventType::AnyCountChange).AddUObject(this, &UAbilityTask_WaitGameplayTagCountChanged::GameplayTagCallback);
	}
}

void UAbilityTask_WaitGameplayTagCountChanged::GameplayTagCallback(const FGameplayTag InTag, int32 NewCount)
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		TagCountChanged.Broadcast(NewCount);
	}
}

UAbilitySystemComponent* UAbilityTask_WaitGameplayTagCountChanged::GetTargetASC()
{
	if (OptionalExternalTarget)
	{
		return OptionalExternalTarget;
	}

	return AbilitySystemComponent.Get();
}

void UAbilityTask_WaitGameplayTagCountChanged::OnDestroy(bool bAbilityIsEnding)
{
	UAbilitySystemComponent* ASC = GetTargetASC();
	if (GameplayTagCountChangedHandle.IsValid() && ASC)
	{
		ASC->RegisterGameplayTagEvent(Tag).Remove(GameplayTagCountChangedHandle);
		GameplayTagCountChangedHandle.Reset();
	}

	Super::OnDestroy(bAbilityIsEnding);
}
