// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitGameplayTagQuery.h"

#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitGameplayTagQuery)

void UAbilityTask_WaitGameplayTagQuery::Activate()
{
	Super::Activate();

	if (UAbilitySystemComponent* ASC = GetTargetASC())
	{
		//Start the query state to the opposite of the triggered condition so it can be triggered immediately
		bQueryState = TriggerCondition != EWaitGameplayTagQueryTriggerCondition::WhenTrue;

		TArray<FGameplayTag> QueryTags;
		TagQuery.GetGameplayTagArray(QueryTags);

		for (const FGameplayTag& Tag : QueryTags)
		{
			if (!TagHandleMap.Contains(Tag))
			{
				UpdateTargetTags(Tag, ASC->GetTagCount(Tag));
				TagHandleMap.Add(Tag, ASC->RegisterGameplayTagEvent(Tag).AddUObject(this, &UAbilityTask_WaitGameplayTagQuery::UpdateTargetTags));
			}
		}

		EvaluateTagQuery();

		bRegisteredCallbacks = true;
	}
}

void UAbilityTask_WaitGameplayTagQuery::UpdateTargetTags(const FGameplayTag Tag, int32 NewCount)
{
	if (NewCount <= 0)
	{
		TargetTags.RemoveTag(Tag);
	}
	else
	{
		TargetTags.AddTag(Tag);
	}

	if (bRegisteredCallbacks)
	{
		EvaluateTagQuery();
	}
}

void UAbilityTask_WaitGameplayTagQuery::EvaluateTagQuery()
{
	if (TagQuery.IsEmpty())
	{
		return;
	}

	const bool bMatchesQuery = TagQuery.Matches(TargetTags);
	const bool bStateChanged = bMatchesQuery != bQueryState;
	bQueryState = bMatchesQuery;

	bool bTriggerDelegate = false;
	if (bStateChanged)
	{
		if (TriggerCondition == EWaitGameplayTagQueryTriggerCondition::WhenTrue && bQueryState)
		{
			bTriggerDelegate = true;
		}
		else if (TriggerCondition == EWaitGameplayTagQueryTriggerCondition::WhenFalse && !bQueryState)
		{
			bTriggerDelegate = true;
		}
	}

	if (bTriggerDelegate)
	{
		Triggered.Broadcast();
		if (bOnlyTriggerOnce)
		{
			EndTask();
		}
	}
}

void UAbilityTask_WaitGameplayTagQuery::OnDestroy(bool AbilityIsEnding)
{
	UAbilitySystemComponent* ASC = bRegisteredCallbacks ? GetTargetASC() : nullptr;
	if (ASC != nullptr)
	{
		for (TPair<FGameplayTag, FDelegateHandle> Pair : TagHandleMap)
		{
			if (Pair.Value.IsValid())
			{
				ASC->UnregisterGameplayTagEvent(Pair.Value, Pair.Key);
			}
		}
	}

	TagHandleMap.Empty();
	TargetTags.Reset();

	Super::OnDestroy(AbilityIsEnding);
}

UAbilitySystemComponent* UAbilityTask_WaitGameplayTagQuery::GetTargetASC()
{	
	return bUseExternalTarget ? ToRawPtr(OptionalExternalTarget) : AbilitySystemComponent.Get();
}

void UAbilityTask_WaitGameplayTagQuery::SetExternalTarget(const AActor* Actor)
{
	if (Actor != nullptr)
	{
		bUseExternalTarget = true;
		OptionalExternalTarget = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Actor);
	}
}

UAbilityTask_WaitGameplayTagQuery* UAbilityTask_WaitGameplayTagQuery::WaitGameplayTagQuery(UGameplayAbility* OwningAbility, 
																						   const FGameplayTagQuery TagQuery, 
																						   const AActor* InOptionalExternalTarget /*= nullptr*/, 
																						   const EWaitGameplayTagQueryTriggerCondition TriggerCondition /*= EWaitGameplayTagQueryTriggerCondition::WhenTrue*/, 
																						   const bool bOnlyTriggerOnce /*= false*/)
{
	UAbilityTask_WaitGameplayTagQuery* MyObj = NewAbilityTask<UAbilityTask_WaitGameplayTagQuery>(OwningAbility);
	MyObj->TagQuery = TagQuery;
	MyObj->SetExternalTarget(InOptionalExternalTarget);
	MyObj->TriggerCondition = TriggerCondition;
	MyObj->bOnlyTriggerOnce = bOnlyTriggerOnce;

	return MyObj;
}

