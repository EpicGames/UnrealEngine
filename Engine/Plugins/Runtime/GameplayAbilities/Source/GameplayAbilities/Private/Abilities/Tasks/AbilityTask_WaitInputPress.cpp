// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitInputPress.h"
#include "AbilitySystemComponent.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitInputPress)

UAbilityTask_WaitInputPress::UAbilityTask_WaitInputPress(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	StartTime = 0.f;
	bTestInitialState = false;
}

void UAbilityTask_WaitInputPress::OnPressCallback()
{
	float ElapsedTime = GetWorld()->GetTimeSeconds() - StartTime;

	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (!Ability || !ASC)
	{
		return;
	}

	ASC->AbilityReplicatedEventDelegate(EAbilityGenericReplicatedEvent::InputPressed, GetAbilitySpecHandle(), GetActivationPredictionKey()).Remove(DelegateHandle);

	FScopedPredictionWindow ScopedPrediction(ASC, IsPredictingClient());

	if (IsPredictingClient())
	{
		// Tell the server about this
		ASC->ServerSetReplicatedEvent(EAbilityGenericReplicatedEvent::InputPressed, GetAbilitySpecHandle(), GetActivationPredictionKey(), ASC->ScopedPredictionKey);
	}
	else
	{
		ASC->ConsumeGenericReplicatedEvent(EAbilityGenericReplicatedEvent::InputPressed, GetAbilitySpecHandle(), GetActivationPredictionKey());
	}

	// We are done. Kill us so we don't keep getting broadcast messages
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnPress.Broadcast(ElapsedTime);
	}
	EndTask();
}

UAbilityTask_WaitInputPress* UAbilityTask_WaitInputPress::WaitInputPress(class UGameplayAbility* OwningAbility, bool bTestAlreadyPressed)
{
	UAbilityTask_WaitInputPress* Task = NewAbilityTask<UAbilityTask_WaitInputPress>(OwningAbility);
	Task->bTestInitialState = bTestAlreadyPressed;
	return Task;
}

void UAbilityTask_WaitInputPress::Activate()
{
	StartTime = GetWorld()->GetTimeSeconds();

	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (ASC && Ability)
	{
		if (bTestInitialState && IsLocallyControlled())
		{
			FGameplayAbilitySpec *Spec = Ability->GetCurrentAbilitySpec();
			if (Spec && Spec->InputPressed)
			{
				OnPressCallback();
				return;
			}
		}

		DelegateHandle = ASC->AbilityReplicatedEventDelegate(EAbilityGenericReplicatedEvent::InputPressed, GetAbilitySpecHandle(), GetActivationPredictionKey()).AddUObject(this, &UAbilityTask_WaitInputPress::OnPressCallback);
		if (IsForRemoteClient())
		{
			if (!ASC->CallReplicatedEventDelegateIfSet(EAbilityGenericReplicatedEvent::InputPressed, GetAbilitySpecHandle(), GetActivationPredictionKey()))
			{
				SetWaitingOnRemotePlayerData();
			}
		}
	}
}
