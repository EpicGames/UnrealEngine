// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitInputRelease.h"
#include "AbilitySystemComponent.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitInputRelease)

UAbilityTask_WaitInputRelease::UAbilityTask_WaitInputRelease(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	StartTime = 0.f;
	bTestInitialState = false;
}

void UAbilityTask_WaitInputRelease::OnReleaseCallback()
{
	float ElapsedTime = GetWorld()->GetTimeSeconds() - StartTime;

	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (!Ability || !ASC)
	{
		return;
	}

	ASC->AbilityReplicatedEventDelegate(EAbilityGenericReplicatedEvent::InputReleased, GetAbilitySpecHandle(), GetActivationPredictionKey()).Remove(DelegateHandle);

	FScopedPredictionWindow ScopedPrediction(ASC, IsPredictingClient());

	if (IsPredictingClient())
	{
		// Tell the server about this
		ASC->ServerSetReplicatedEvent(EAbilityGenericReplicatedEvent::InputReleased, GetAbilitySpecHandle(), GetActivationPredictionKey(), ASC->ScopedPredictionKey);
	}
	else
	{
		ASC->ConsumeGenericReplicatedEvent(EAbilityGenericReplicatedEvent::InputReleased, GetAbilitySpecHandle(), GetActivationPredictionKey());
	}

	// We are done. Kill us so we don't keep getting broadcast messages
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		OnRelease.Broadcast(ElapsedTime);
	}
	EndTask();
}

UAbilityTask_WaitInputRelease* UAbilityTask_WaitInputRelease::WaitInputRelease(class UGameplayAbility* OwningAbility, bool bTestAlreadyReleased)
{
	UAbilityTask_WaitInputRelease* Task = NewAbilityTask<UAbilityTask_WaitInputRelease>(OwningAbility);
	Task->bTestInitialState = bTestAlreadyReleased;
	return Task;
}

void UAbilityTask_WaitInputRelease::Activate()
{
	StartTime = GetWorld()->GetTimeSeconds();

	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (ASC && Ability)
	{
		if (bTestInitialState && IsLocallyControlled())
		{
			FGameplayAbilitySpec *Spec = Ability->GetCurrentAbilitySpec();
			if (Spec && !Spec->InputPressed)
			{
				OnReleaseCallback();
				return;
			}
		}

		DelegateHandle = ASC->AbilityReplicatedEventDelegate(EAbilityGenericReplicatedEvent::InputReleased, GetAbilitySpecHandle(), GetActivationPredictionKey()).AddUObject(this, &UAbilityTask_WaitInputRelease::OnReleaseCallback);
		if (IsForRemoteClient())
		{
			if (!ASC->CallReplicatedEventDelegateIfSet(EAbilityGenericReplicatedEvent::InputReleased, GetAbilitySpecHandle(), GetActivationPredictionKey()))
			{
				SetWaitingOnRemotePlayerData();
			}
		}
	}
}
