// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitMovementModeChange.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitMovementModeChange)

UAbilityTask_WaitMovementModeChange::UAbilityTask_WaitMovementModeChange(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RequiredMode = MOVE_None;
}

UAbilityTask_WaitMovementModeChange* UAbilityTask_WaitMovementModeChange::CreateWaitMovementModeChange(class UGameplayAbility* OwningAbility, EMovementMode NewMode)
{
	UAbilityTask_WaitMovementModeChange* MyObj = NewAbilityTask<UAbilityTask_WaitMovementModeChange>(OwningAbility);
	MyObj->RequiredMode = NewMode;
	return MyObj;
}

void UAbilityTask_WaitMovementModeChange::Activate()
{
	ACharacter* Character = Cast<ACharacter>(GetAvatarActor());
	if (Character)
	{
		Character->MovementModeChangedDelegate.AddDynamic(this, &UAbilityTask_WaitMovementModeChange::OnMovementModeChange);
		MyCharacter = Character;
	}

	SetWaitingOnAvatar();
}

void UAbilityTask_WaitMovementModeChange::OnMovementModeChange(ACharacter * Character, EMovementMode PrevMovementMode, uint8 PreviousCustomMode)
{
	if (Character)
	{
		if (UCharacterMovementComponent *MoveComp = Cast<UCharacterMovementComponent>(Character->GetMovementComponent()))
		{
			if (RequiredMode == MOVE_None || MoveComp->MovementMode == RequiredMode)
			{
				if (ShouldBroadcastAbilityTaskDelegates())
				{
					OnChange.Broadcast(MoveComp->MovementMode);
				}
				EndTask();
				return;
			}
		}
	}
}

void UAbilityTask_WaitMovementModeChange::OnDestroy(bool AbilityEnded)
{
	if (MyCharacter.IsValid())
	{
		MyCharacter->MovementModeChangedDelegate.RemoveDynamic(this, &UAbilityTask_WaitMovementModeChange::OnMovementModeChange);
	}

	Super::OnDestroy(AbilityEnded);
}

