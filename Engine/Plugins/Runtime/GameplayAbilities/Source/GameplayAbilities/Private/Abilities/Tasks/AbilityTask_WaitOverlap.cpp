// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitOverlap.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitOverlap)

UAbilityTask_WaitOverlap::UAbilityTask_WaitOverlap(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

void UAbilityTask_WaitOverlap::OnHitCallback(UPrimitiveComponent* HitComp, AActor* OtherActor, UPrimitiveComponent* OtherComp, FVector NormalImpulse, const FHitResult& Hit)
{
	if(OtherActor)
	{
		// Construct TargetData
		FGameplayAbilityTargetData_SingleTargetHit * TargetData = new FGameplayAbilityTargetData_SingleTargetHit(Hit);

		// Give it a handle and return
		FGameplayAbilityTargetDataHandle	Handle;
		Handle.Data.Add(TSharedPtr<FGameplayAbilityTargetData>(TargetData));
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			OnOverlap.Broadcast(Handle);
		}

		// We are done. Kill us so we don't keep getting broadcast messages
		EndTask();
	}
}

/**
*	Need:
*	-Easy way to specify which primitive components should be used for this overlap test
*	-Easy way to specify which types of actors/collision overlaps that we care about/want to block on
*/

UAbilityTask_WaitOverlap* UAbilityTask_WaitOverlap::WaitForOverlap(UGameplayAbility* OwningAbility)
{
	UAbilityTask_WaitOverlap* MyObj = NewAbilityTask<UAbilityTask_WaitOverlap>(OwningAbility);
	return MyObj;
}

void UAbilityTask_WaitOverlap::Activate()
{
	SetWaitingOnAvatar();

	UPrimitiveComponent* PrimComponent = GetComponent();
	if (PrimComponent)
	{
		PrimComponent->OnComponentHit.AddDynamic(this, &UAbilityTask_WaitOverlap::OnHitCallback);
	}
}

void UAbilityTask_WaitOverlap::OnDestroy(bool AbilityEnded)
{
	UPrimitiveComponent* PrimComponent = GetComponent();
	if (PrimComponent)
	{
		PrimComponent->OnComponentHit.RemoveDynamic(this, &UAbilityTask_WaitOverlap::OnHitCallback);
	}

	Super::OnDestroy(AbilityEnded);
}

UPrimitiveComponent* UAbilityTask_WaitOverlap::GetComponent()
{
	// TEMP - we are just using root component's collision. A real system will need more data to specify which component to use
	UPrimitiveComponent * PrimComponent = nullptr;
	AActor* ActorOwner = GetAvatarActor();
	if (ActorOwner)
	{
		PrimComponent = Cast<UPrimitiveComponent>(ActorOwner->GetRootComponent());
		if (!PrimComponent)
		{
			PrimComponent = ActorOwner->FindComponentByClass<UPrimitiveComponent>();
		}
	}

	return PrimComponent;
}

