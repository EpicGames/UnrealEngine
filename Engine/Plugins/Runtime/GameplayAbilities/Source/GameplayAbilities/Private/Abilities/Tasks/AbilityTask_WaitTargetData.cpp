// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitTargetData.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitTargetData)

UAbilityTask_WaitTargetData::UAbilityTask_WaitTargetData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	
}

UAbilityTask_WaitTargetData* UAbilityTask_WaitTargetData::WaitTargetData(UGameplayAbility* OwningAbility, FName TaskInstanceName, TEnumAsByte<EGameplayTargetingConfirmation::Type> ConfirmationType, TSubclassOf<AGameplayAbilityTargetActor> InTargetClass)
{
	UAbilityTask_WaitTargetData* MyObj = NewAbilityTask<UAbilityTask_WaitTargetData>(OwningAbility, TaskInstanceName);		//Register for task list here, providing a given FName as a key
	MyObj->TargetClass = InTargetClass;
	MyObj->TargetActor = nullptr;
	MyObj->ConfirmationType = ConfirmationType;
	return MyObj;
}

UAbilityTask_WaitTargetData* UAbilityTask_WaitTargetData::WaitTargetDataUsingActor(UGameplayAbility* OwningAbility, FName TaskInstanceName, TEnumAsByte<EGameplayTargetingConfirmation::Type> ConfirmationType, AGameplayAbilityTargetActor* InTargetActor)
{
	UAbilityTask_WaitTargetData* MyObj = NewAbilityTask<UAbilityTask_WaitTargetData>(OwningAbility, TaskInstanceName);		//Register for task list here, providing a given FName as a key
	MyObj->TargetClass = nullptr;
	MyObj->TargetActor = InTargetActor;
	MyObj->ConfirmationType = ConfirmationType;
	return MyObj;
}

void UAbilityTask_WaitTargetData::Activate()
{
	// Need to handle case where target actor was passed into task
	if (Ability && (TargetClass == nullptr))
	{
		if (TargetActor)
		{
			AGameplayAbilityTargetActor* SpawnedActor = TargetActor;
			TargetClass = SpawnedActor->GetClass();

			RegisterTargetDataCallbacks();


			if (!IsValidChecked(this))
			{
				return;
			}

			if (ShouldSpawnTargetActor())
			{
				InitializeTargetActor(SpawnedActor);
				FinalizeTargetActor(SpawnedActor);

				// Note that the call to FinalizeTargetActor, this task could finish and our owning ability may be ended.
			}
			else
			{
				TargetActor = nullptr;

				// We may need a better solution here.  We don't know the target actor isn't needed till after it's already been spawned.
				SpawnedActor->Destroy();
				SpawnedActor = nullptr;
			}
		}
		else
		{
			EndTask();
		}
	}
}

bool UAbilityTask_WaitTargetData::BeginSpawningActor(UGameplayAbility* OwningAbility, TSubclassOf<AGameplayAbilityTargetActor> InTargetClass, AGameplayAbilityTargetActor*& SpawnedActor)
{
	SpawnedActor = nullptr;

	if (Ability)
	{
		if (ShouldSpawnTargetActor())
		{
			UClass* Class = *InTargetClass;
			if (Class != nullptr)
			{
				if (UWorld* World = GEngine->GetWorldFromContextObject(OwningAbility, EGetWorldErrorMode::LogAndReturnNull))
				{
					SpawnedActor = World->SpawnActorDeferred<AGameplayAbilityTargetActor>(Class, FTransform::Identity, nullptr, nullptr, ESpawnActorCollisionHandlingMethod::AlwaysSpawn);
				}
			}

			if (SpawnedActor)
			{
				TargetActor = SpawnedActor;
				InitializeTargetActor(SpawnedActor);
			}
		}

		RegisterTargetDataCallbacks();
	}

	return (SpawnedActor != nullptr);
}

void UAbilityTask_WaitTargetData::FinishSpawningActor(UGameplayAbility* OwningAbility, AGameplayAbilityTargetActor* SpawnedActor)
{
	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (ASC && IsValid(SpawnedActor))
	{
		check(TargetActor == SpawnedActor);

		const FTransform SpawnTransform = ASC->GetOwner()->GetTransform();

		SpawnedActor->FinishSpawning(SpawnTransform);

		FinalizeTargetActor(SpawnedActor);
	}
}

bool UAbilityTask_WaitTargetData::ShouldSpawnTargetActor() const
{
	check(TargetClass);
	check(Ability);

	// Spawn the actor if this is a locally controlled ability (always) or if this is a replicating targeting mode.
	// (E.g., server will spawn this target actor to replicate to all non owning clients)

	const AGameplayAbilityTargetActor* CDO = CastChecked<AGameplayAbilityTargetActor>(TargetClass->GetDefaultObject());

	const bool bReplicates = CDO->GetIsReplicated();
	const bool bIsLocallyControlled = Ability->GetCurrentActorInfo()->IsLocallyControlled();
	const bool bShouldProduceTargetDataOnServer = CDO->ShouldProduceTargetDataOnServer;

	return (bReplicates || bIsLocallyControlled || bShouldProduceTargetDataOnServer);
}

void UAbilityTask_WaitTargetData::InitializeTargetActor(AGameplayAbilityTargetActor* SpawnedActor) const
{
	check(SpawnedActor);
	check(Ability);

	SpawnedActor->PrimaryPC = Ability->GetCurrentActorInfo()->PlayerController.Get();

	// If we spawned the target actor, always register the callbacks for when the data is ready.
	SpawnedActor->TargetDataReadyDelegate.AddUObject(const_cast<UAbilityTask_WaitTargetData*>(this), &UAbilityTask_WaitTargetData::OnTargetDataReadyCallback);
	SpawnedActor->CanceledDelegate.AddUObject(const_cast<UAbilityTask_WaitTargetData*>(this), &UAbilityTask_WaitTargetData::OnTargetDataCancelledCallback);
}

void UAbilityTask_WaitTargetData::FinalizeTargetActor(AGameplayAbilityTargetActor* SpawnedActor) const
{
	check(SpawnedActor);
	check(Ability);

	if (UAbilitySystemComponent* ASC = AbilitySystemComponent.Get())
	{
		// User ability activation is inhibited while this is active
		ASC->SpawnedTargetActors.Push(SpawnedActor);
	}

	SpawnedActor->StartTargeting(Ability);

	if (SpawnedActor->ShouldProduceTargetData())
	{
		// If instant confirm, then stop targeting immediately.
		// Note this is kind of bad: we should be able to just call a static func on the CDO to do this. 
		// But then we wouldn't get to set ExposeOnSpawnParameters.
		if (ConfirmationType == EGameplayTargetingConfirmation::Instant)
		{
			SpawnedActor->ConfirmTargeting();
		}
		else if (ConfirmationType == EGameplayTargetingConfirmation::UserConfirmed)
		{
			// Bind to the Cancel/Confirm Delegates (called from local confirm or from repped confirm)
			SpawnedActor->BindToConfirmCancelInputs();
		}
	}
}

void UAbilityTask_WaitTargetData::RegisterTargetDataCallbacks()
{
	if (!ensure(IsValidChecked(this)))
	{
		return;
	}

	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (!ASC)
	{
		return;
	}

	check(TargetClass);
	check(Ability);

	const AGameplayAbilityTargetActor* CDO = CastChecked<AGameplayAbilityTargetActor>(TargetClass->GetDefaultObject());

	const bool bIsLocallyControlled = Ability->GetCurrentActorInfo()->IsLocallyControlled();
	const bool bShouldProduceTargetDataOnServer = CDO->ShouldProduceTargetDataOnServer;

	// If not locally controlled (server for remote client), see if TargetData was already sent
	// else register callback for when it does get here.
	if (!bIsLocallyControlled)
	{
		// Register with the TargetData callbacks if we are expecting client to send them
		if (!bShouldProduceTargetDataOnServer)
		{
			FGameplayAbilitySpecHandle	SpecHandle = GetAbilitySpecHandle();
			FPredictionKey ActivationPredictionKey = GetActivationPredictionKey();

			//Since multifire is supported, we still need to hook up the callbacks
			ASC->AbilityTargetDataSetDelegate(SpecHandle, ActivationPredictionKey ).AddUObject(this, &UAbilityTask_WaitTargetData::OnTargetDataReplicatedCallback);
			ASC->AbilityTargetDataCancelledDelegate(SpecHandle, ActivationPredictionKey ).AddUObject(this, &UAbilityTask_WaitTargetData::OnTargetDataReplicatedCancelledCallback);

			ASC->CallReplicatedTargetDataDelegatesIfSet(SpecHandle, ActivationPredictionKey );

			SetWaitingOnRemotePlayerData();
		}
	}
}

/** Valid TargetData was replicated to use (we are server, was sent from client) */
void UAbilityTask_WaitTargetData::OnTargetDataReplicatedCallback(const FGameplayAbilityTargetDataHandle& Data, FGameplayTag ActivationTag)
{
	FGameplayAbilityTargetDataHandle MutableData = Data;

	if (UAbilitySystemComponent* ASC = AbilitySystemComponent.Get())
	{
		ASC->ConsumeClientReplicatedTargetData(GetAbilitySpecHandle(), GetActivationPredictionKey());
	}

	/** 
	 *  Call into the TargetActor to sanitize/verify the data. If this returns false, we are rejecting
	 *	the replicated target data and will treat this as a cancel.
	 *	
	 *	This can also be used for bandwidth optimizations. OnReplicatedTargetDataReceived could do an actual
	 *	trace/check/whatever server side and use that data. So rather than having the client send that data
	 *	explicitly, the client is basically just sending a 'confirm' and the server is now going to do the work
	 *	in OnReplicatedTargetDataReceived.
	 */
	if (TargetActor && !TargetActor->OnReplicatedTargetDataReceived(MutableData))
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			Cancelled.Broadcast(MutableData);
		}
	}
	else
	{
		if (ShouldBroadcastAbilityTaskDelegates())
		{
			ValidData.Broadcast(MutableData);
		}
	}

	if (ConfirmationType != EGameplayTargetingConfirmation::CustomMulti)
	{
		EndTask();
	}
}

/** Client canceled this Targeting Task (we are the server) */
void UAbilityTask_WaitTargetData::OnTargetDataReplicatedCancelledCallback()
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		Cancelled.Broadcast(FGameplayAbilityTargetDataHandle());
	}
	EndTask();
}

/** The TargetActor we spawned locally has called back with valid target data */
void UAbilityTask_WaitTargetData::OnTargetDataReadyCallback(const FGameplayAbilityTargetDataHandle& Data)
{
	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (!Ability || !ASC)
	{
		return;
	}

	FScopedPredictionWindow	ScopedPrediction(ASC, ShouldReplicateDataToServer());
	
	const FGameplayAbilityActorInfo* Info = Ability->GetCurrentActorInfo();
	if (IsPredictingClient())
	{
		if (!TargetActor->ShouldProduceTargetDataOnServer)
		{
			FGameplayTag ApplicationTag; // Fixme: where would this be useful?
			ASC->CallServerSetReplicatedTargetData(GetAbilitySpecHandle(), GetActivationPredictionKey(), Data, ApplicationTag, ASC->ScopedPredictionKey);
		}
		else if (ConfirmationType == EGameplayTargetingConfirmation::UserConfirmed)
		{
			// We aren't going to send the target data, but we will send a generic confirmed message.
			ASC->ServerSetReplicatedEvent(EAbilityGenericReplicatedEvent::GenericConfirm, GetAbilitySpecHandle(), GetActivationPredictionKey(), ASC->ScopedPredictionKey);
		}
	}

	if (ShouldBroadcastAbilityTaskDelegates())
	{
		ValidData.Broadcast(Data);
	}

	if (ConfirmationType != EGameplayTargetingConfirmation::CustomMulti)
	{
		EndTask();
	}
}

/** The TargetActor we spawned locally has called back with a cancel event (they still include the 'last/best' targetdata but the consumer of this may want to discard it) */
void UAbilityTask_WaitTargetData::OnTargetDataCancelledCallback(const FGameplayAbilityTargetDataHandle& Data)
{
	UAbilitySystemComponent* ASC = AbilitySystemComponent.Get();
	if (!ASC)
	{
		return;
	}

	FScopedPredictionWindow ScopedPrediction(ASC, IsPredictingClient());

	if (IsPredictingClient())
	{
		if (!TargetActor->ShouldProduceTargetDataOnServer)
		{
			ASC->ServerSetReplicatedTargetDataCancelled(GetAbilitySpecHandle(), GetActivationPredictionKey(), ASC->ScopedPredictionKey );
		}
		else
		{
			// We aren't going to send the target data, but we will send a generic confirmed message.
			ASC->ServerSetReplicatedEvent(EAbilityGenericReplicatedEvent::GenericCancel, GetAbilitySpecHandle(), GetActivationPredictionKey(), ASC->ScopedPredictionKey);
		}
	}
	Cancelled.Broadcast(Data);
	EndTask();
}

/** Called when the ability is asked to confirm from an outside node. What this means depends on the individual task. By default, this does nothing other than ending if bEndTask is true. */
void UAbilityTask_WaitTargetData::ExternalConfirm(bool bEndTask)
{
	if (TargetActor)
	{
		if (TargetActor->ShouldProduceTargetData())
		{
			TargetActor->ConfirmTargetingAndContinue();
		}
	}
	Super::ExternalConfirm(bEndTask);
}

/** Called when the ability is asked to confirm from an outside node. What this means depends on the individual task. By default, this does nothing other than ending if bEndTask is true. */
void UAbilityTask_WaitTargetData::ExternalCancel()
{
	if (ShouldBroadcastAbilityTaskDelegates())
	{
		Cancelled.Broadcast(FGameplayAbilityTargetDataHandle());
	}
	Super::ExternalCancel();
}

void UAbilityTask_WaitTargetData::OnDestroy(bool AbilityEnded)
{
	if (TargetActor)
	{
		TargetActor->Destroy();
	}

	Super::OnDestroy(AbilityEnded);
}

bool UAbilityTask_WaitTargetData::ShouldReplicateDataToServer() const
{
	if (!Ability || !TargetActor)
	{
		return false;
	}

	// Send TargetData to the server IFF we are the client and this isn't a GameplayTargetActor that can produce data on the server	
	const FGameplayAbilityActorInfo* Info = Ability->GetCurrentActorInfo();
	if (!Info->IsNetAuthority() && !TargetActor->ShouldProduceTargetDataOnServer)
	{
		return true;
	}

	return false;
}


// --------------------------------------------------------------------------------------

