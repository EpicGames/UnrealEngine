// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/Tasks/AbilityTask_WaitVelocityChange.h"
#include "AbilitySystemLog.h"
#include "GameFramework/MovementComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_WaitVelocityChange)

UAbilityTask_WaitVelocityChange::UAbilityTask_WaitVelocityChange(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bTickingTask = true;
}

void UAbilityTask_WaitVelocityChange::TickTask(float DeltaTime)
{
	if (UMovementComponent* LocalMoveComp = CachedMovementComponent.Get())
	{
		float dot = FVector::DotProduct(Direction, LocalMoveComp->Velocity);

		if (dot > MinimumMagnitude)
		{
			if (ShouldBroadcastAbilityTaskDelegates())
			{
				OnVelocityChage.Broadcast();
			}
			EndTask();
		}
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilityTask_WaitVelocityChange ticked without a valid movement component. ending."));
		EndTask();
	}
}

UAbilityTask_WaitVelocityChange* UAbilityTask_WaitVelocityChange::CreateWaitVelocityChange(UGameplayAbility* OwningAbility, FVector InDirection, float InMinimumMagnitude)
{
	UAbilityTask_WaitVelocityChange* MyObj = NewAbilityTask<UAbilityTask_WaitVelocityChange>(OwningAbility);

	MyObj->MinimumMagnitude = InMinimumMagnitude;
	MyObj->Direction = InDirection.GetSafeNormal();
	

	return MyObj;
}

void UAbilityTask_WaitVelocityChange::Activate()
{
	const FGameplayAbilityActorInfo* ActorInfo = Ability->GetCurrentActorInfo();
	CachedMovementComponent = ActorInfo->MovementComponent;
	SetWaitingOnAvatar();
}

