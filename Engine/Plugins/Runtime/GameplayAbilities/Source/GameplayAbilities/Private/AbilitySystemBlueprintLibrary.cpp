// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbilitySystemBlueprintLibrary.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"
#include "AbilitySystemPrivate.h"
#include "Engine/World.h"
#include "GameplayEffect.h"
#include "GameplayEffectAggregator.h"
#include "GameplayEffectComponents/AdditionalEffectsGameplayEffectComponent.h"
#include "GameplayEffectUIData.h"
#include "GameplayAbilitySpec.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilitySystemBlueprintLibrary)

FGameplayTagChangedEventWrapperSpecHandle::FGameplayTagChangedEventWrapperSpecHandle()
	: Data(nullptr)
{
}

FGameplayTagChangedEventWrapperSpecHandle::FGameplayTagChangedEventWrapperSpecHandle(FGameplayTagChangedEventWrapperSpec* DataPtr)
	: Data(DataPtr)
{
}

bool FGameplayTagChangedEventWrapperSpecHandle::operator==(FGameplayTagChangedEventWrapperSpecHandle const& Other) const
{
	const bool bBothValid = Data.IsValid() && Other.Data.IsValid();
	const bool bBothInvalid = !Data.IsValid() && !Other.Data.IsValid();
	return (bBothInvalid || (bBothValid && (Data.Get() == Other.Data.Get())));
}

bool FGameplayTagChangedEventWrapperSpecHandle::operator!=(FGameplayTagChangedEventWrapperSpecHandle const& Other) const
{
	return !(FGameplayTagChangedEventWrapperSpecHandle::operator==(Other));
}

FGameplayTagChangedEventWrapperSpec::FGameplayTagChangedEventWrapperSpec(
	UAbilitySystemComponent* AbilitySystemComponent, 
	FOnGameplayTagChangedEventWrapperSignature InGameplayTagChangedEventWrapperDelegate, 
	EGameplayTagEventType::Type InTagListeningPolicy)
	: AbilitySystemComponentWk(AbilitySystemComponent)
	, GameplayTagChangedEventWrapperDelegate(InGameplayTagChangedEventWrapperDelegate)
	, TagListeningPolicy(InTagListeningPolicy)
	, DelegateBindings{}
{
}

FGameplayTagChangedEventWrapperSpec::~FGameplayTagChangedEventWrapperSpec()
{
	const int32 RemainingDelegateBindingsCount = DelegateBindings.Num();
	if (RemainingDelegateBindingsCount > 0)
	{
		// We still have delegates bound to the ASC - we need to warn the user!
		// We expect the user to unbind delegates they bound.

		// The exception is if the ASC itself is not valid which indicates things are tearing down - in that case, we'll give them a pass since it's a moot point that
		//  we are still bound if the ASC isn't around anymore.
		UAbilitySystemComponent* AbilitySystemComponent = AbilitySystemComponentWk.Get();
		if (IsValid(AbilitySystemComponent))
		{
			ABILITY_LOG(
				Error, 
				TEXT("~FGameplayTagChangedEventWrapperSpec: our bound spec is being destroyed but we still have %d delegate bindings bound to the ASC on '%s'! Please cache off the Bound delegate handle and unbind it when finished."),
				RemainingDelegateBindingsCount,
				*GetNameSafe(AbilitySystemComponent->GetOwner()));
		}
	}
}

UAbilitySystemBlueprintLibrary::UAbilitySystemBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

UAbilitySystemComponent* UAbilitySystemBlueprintLibrary::GetAbilitySystemComponent(AActor *Actor)
{
	return UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Actor);
}

void UAbilitySystemBlueprintLibrary::SendGameplayEventToActor(AActor* Actor, FGameplayTag EventTag, FGameplayEventData Payload)
{
	if (::IsValid(Actor))
	{
		UAbilitySystemComponent* AbilitySystemComponent = GetAbilitySystemComponent(Actor);
		if (AbilitySystemComponent != nullptr && IsValidChecked(AbilitySystemComponent))
		{
			using namespace UE::AbilitySystem::Private;
			if (EnumHasAnyFlags(static_cast<EAllowPredictiveGEFlags>(CVarAllowPredictiveGEFlagsValue), EAllowPredictiveGEFlags::AllowGameplayEventToApplyGE))
			{
				FScopedPredictionWindow NewScopedWindow(AbilitySystemComponent, true);
				AbilitySystemComponent->HandleGameplayEvent(EventTag, &Payload);
			}
			else
			{
				AbilitySystemComponent->HandleGameplayEvent(EventTag, &Payload);
			}
		}
		else
		{
			ABILITY_LOG(Error, TEXT("UAbilitySystemBlueprintLibrary::SendGameplayEventToActor: Invalid ability system component retrieved from Actor %s. EventTag was %s"), *Actor->GetName(), *EventTag.ToString());
		}
	}
}

FGameplayTagChangedEventWrapperSpecHandle UAbilitySystemBlueprintLibrary::BindEventWrapperToGameplayTagChanged(
	UAbilitySystemComponent* AbilitySystemComponent,
	FGameplayTag Tag, 
	FOnGameplayTagChangedEventWrapperSignature GameplayTagChangedEventWrapperDelegate, 
	bool bExecuteImmediatelyIfTagApplied /*= true*/, 
	EGameplayTagEventType::Type TagListeningPolicy /*= EGameplayTagEventType::NewOrRemoved*/)
{
	if (!::IsValid(AbilitySystemComponent))
	{
		return FGameplayTagChangedEventWrapperSpecHandle();
	}

	FGameplayTagChangedEventWrapperSpec* TagBindingSpec = new FGameplayTagChangedEventWrapperSpec(AbilitySystemComponent, GameplayTagChangedEventWrapperDelegate, TagListeningPolicy);
	FGameplayTagChangedEventWrapperSpecHandle TagBindingHandle(TagBindingSpec);

	// Bind to the ASC's tag change listening delegate (which is not a 'dynamic' delegate and thereby can't be used in BP).
	const FDelegateHandle TagChangedDelegateHandle = AbilitySystemComponent->RegisterGameplayTagEvent(Tag, TagListeningPolicy).AddLambda(
		[GameplayTagChangedEventWrapperDelegate]
		(const FGameplayTag GameplayTag, int32 GameplayTagCount)
	{
		UAbilitySystemBlueprintLibrary::ProcessGameplayTagChangedEventWrapper(GameplayTag, GameplayTagCount, GameplayTagChangedEventWrapperDelegate);
	});

	TagBindingSpec->DelegateBindings.Add(Tag, TagChangedDelegateHandle);

	if (bExecuteImmediatelyIfTagApplied)
	{
		const int32 GameplayTagCount = AbilitySystemComponent->GetGameplayTagCount(Tag);
		if (GameplayTagCount > 0)
		{
			GameplayTagChangedEventWrapperDelegate.ExecuteIfBound(Tag, GameplayTagCount);
		}
	}
	
	return TagBindingHandle;
}

FGameplayTagChangedEventWrapperSpecHandle UAbilitySystemBlueprintLibrary::BindEventWrapperToAnyOfGameplayTagsChanged(
	UAbilitySystemComponent* AbilitySystemComponent,
	const TArray<FGameplayTag>& Tags, 
	FOnGameplayTagChangedEventWrapperSignature GameplayTagChangedEventWrapperDelegate, 
	bool bExecuteImmediatelyIfTagApplied/* = true*/,
	EGameplayTagEventType::Type TagListeningPolicy/* = EGameplayTagEventType::NewOrRemoved*/)
{
	if (!::IsValid(AbilitySystemComponent))
	{
		return FGameplayTagChangedEventWrapperSpecHandle();
	}

	FGameplayTagChangedEventWrapperSpec* TagBindingSpec = new FGameplayTagChangedEventWrapperSpec(AbilitySystemComponent, GameplayTagChangedEventWrapperDelegate, TagListeningPolicy);
	FGameplayTagChangedEventWrapperSpecHandle TagBindingHandle(TagBindingSpec);

	TagBindingSpec->DelegateBindings.Reserve(Tags.Num());

	// Bind each tag and add to the DelegateBindings container.
	for (const FGameplayTag& Tag : Tags)
	{
		// Bind to the ASC's tag change listening delegate (which is not a 'dynamic' delegate and thereby can't be used in BP).
		const FDelegateHandle TagChangedDelegateHandle = AbilitySystemComponent->RegisterGameplayTagEvent(Tag, TagListeningPolicy).AddLambda(
			[GameplayTagChangedEventWrapperDelegate]
			(const FGameplayTag GameplayTag, int32 GameplayTagCount)
		{
			UAbilitySystemBlueprintLibrary::ProcessGameplayTagChangedEventWrapper(GameplayTag, GameplayTagCount, GameplayTagChangedEventWrapperDelegate);
		});

		TagBindingSpec->DelegateBindings.Add(Tag, TagChangedDelegateHandle);
	}

	if (bExecuteImmediatelyIfTagApplied)
	{
		for (const FGameplayTag& Tag : Tags)
		{
			const int32 GameplayTagCount = AbilitySystemComponent->GetGameplayTagCount(Tag);
			if (GameplayTagCount > 0)
			{
				GameplayTagChangedEventWrapperDelegate.ExecuteIfBound(Tag, GameplayTagCount);
			}
		}
	}

	return TagBindingHandle;
}

FGameplayTagChangedEventWrapperSpecHandle UAbilitySystemBlueprintLibrary::BindEventWrapperToAnyOfGameplayTagContainerChanged(
	UAbilitySystemComponent* AbilitySystemComponent,
	const FGameplayTagContainer TagContainer, 
	FOnGameplayTagChangedEventWrapperSignature GameplayTagChangedEventWrapperDelegate, 
	bool bExecuteImmediatelyIfTagApplied/* = true*/,
	EGameplayTagEventType::Type TagListeningPolicy/* = EGameplayTagEventType::NewOrRemoved*/)
{
	TArray<FGameplayTag> Tags;
	TagContainer.GetGameplayTagArray(Tags);
	
	return BindEventWrapperToAnyOfGameplayTagsChanged(AbilitySystemComponent, Tags, GameplayTagChangedEventWrapperDelegate, bExecuteImmediatelyIfTagApplied, TagListeningPolicy);
}

void UAbilitySystemBlueprintLibrary::UnbindAllGameplayTagChangedEventWrappersForHandle(FGameplayTagChangedEventWrapperSpecHandle Handle)
{
	FGameplayTagChangedEventWrapperSpec* GameplayTagChangedEventDataPtr = Handle.Data.Get();
	if (GameplayTagChangedEventDataPtr == nullptr)
	{
		return;
	}

	UAbilitySystemComponent* AbilitySystemComponent = GameplayTagChangedEventDataPtr->AbilitySystemComponentWk.Get();
	if (AbilitySystemComponent == nullptr)
	{
		return;
	}

	for (auto Iterator = GameplayTagChangedEventDataPtr->DelegateBindings.CreateConstIterator(); Iterator; ++Iterator)
	{
		AbilitySystemComponent->UnregisterGameplayTagEvent(Iterator->Value, Iterator->Key, GameplayTagChangedEventDataPtr->TagListeningPolicy);
	}

	GameplayTagChangedEventDataPtr->DelegateBindings.Reset();
}

void UAbilitySystemBlueprintLibrary::UnbindGameplayTagChangedEventWrapperForHandle(FGameplayTag Tag, FGameplayTagChangedEventWrapperSpecHandle Handle)
{
	FGameplayTagChangedEventWrapperSpec* GameplayTagChangedEventDataPtr = Handle.Data.Get();
	if (GameplayTagChangedEventDataPtr == nullptr)
	{
		return;
	}

	UAbilitySystemComponent* AbilitySystemComponent = GameplayTagChangedEventDataPtr->AbilitySystemComponentWk.Get();
	if (AbilitySystemComponent == nullptr)
	{
		return;
	}

	for (auto DelegateBindingIterator = GameplayTagChangedEventDataPtr->DelegateBindings.CreateIterator(); DelegateBindingIterator; ++DelegateBindingIterator)
	{
		const FGameplayTag& BoundTag = DelegateBindingIterator->Key;
		if (!BoundTag.MatchesTagExact(Tag))
		{
			continue;
		}

		AbilitySystemComponent->UnregisterGameplayTagEvent(DelegateBindingIterator->Value, BoundTag, GameplayTagChangedEventDataPtr->TagListeningPolicy);
		DelegateBindingIterator.RemoveCurrent();
	}
}

void UAbilitySystemBlueprintLibrary::ProcessGameplayTagChangedEventWrapper(
	const FGameplayTag GameplayTag, 
	int32 GameplayTagCount,
	FOnGameplayTagChangedEventWrapperSignature GameplayTagChangedEventWrapperDelegate)
{
	GameplayTagChangedEventWrapperDelegate.ExecuteIfBound(GameplayTag, GameplayTagCount);
}

bool UAbilitySystemBlueprintLibrary::IsValid(FGameplayAttribute Attribute)
{
	return Attribute.IsValid();
}

float UAbilitySystemBlueprintLibrary::GetFloatAttribute(const  AActor* Actor, FGameplayAttribute Attribute, bool& bSuccessfullyFoundAttribute)
{
	const UAbilitySystemComponent* const AbilitySystem = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Actor);

	return GetFloatAttributeFromAbilitySystemComponent(AbilitySystem, Attribute, bSuccessfullyFoundAttribute);
}

float UAbilitySystemBlueprintLibrary::GetFloatAttributeFromAbilitySystemComponent(const  UAbilitySystemComponent* AbilitySystem, FGameplayAttribute Attribute, bool& bSuccessfullyFoundAttribute)
{
	bSuccessfullyFoundAttribute = true;

	if (!AbilitySystem || !AbilitySystem->HasAttributeSetForAttribute(Attribute))
	{
		bSuccessfullyFoundAttribute = false;
		return 0.f;
	}

	const float Result = AbilitySystem->GetNumericAttribute(Attribute);
	return Result;
}

float UAbilitySystemBlueprintLibrary::GetFloatAttributeBase(const AActor* Actor, FGameplayAttribute Attribute, bool& bSuccessfullyFoundAttribute)
{
	const UAbilitySystemComponent* const AbilitySystem = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Actor);
	return GetFloatAttributeBaseFromAbilitySystemComponent(AbilitySystem, Attribute, bSuccessfullyFoundAttribute);
}

float UAbilitySystemBlueprintLibrary::GetFloatAttributeBaseFromAbilitySystemComponent(const UAbilitySystemComponent* AbilitySystemComponent, FGameplayAttribute Attribute, bool& bSuccessfullyFoundAttribute)
{
	float Result = 0.f;
	bSuccessfullyFoundAttribute = false;

	if (AbilitySystemComponent && AbilitySystemComponent->HasAttributeSetForAttribute(Attribute))
	{
		bSuccessfullyFoundAttribute = true;
		Result = AbilitySystemComponent->GetNumericAttributeBase(Attribute);
	}

	return Result;
}

float UAbilitySystemBlueprintLibrary::EvaluateAttributeValueWithTags(UAbilitySystemComponent* AbilitySystem, FGameplayAttribute Attribute, const FGameplayTagContainer& SourceTags, const FGameplayTagContainer& TargetTags, bool& bSuccess)
{
	float RetVal = 0.f;
	if (!AbilitySystem || !AbilitySystem->HasAttributeSetForAttribute(Attribute))
	{
		bSuccess = false;
		return RetVal;
	}

	FGameplayEffectAttributeCaptureDefinition Capture(Attribute, EGameplayEffectAttributeCaptureSource::Source, true);

	FGameplayEffectAttributeCaptureSpec CaptureSpec(Capture);
	AbilitySystem->CaptureAttributeForGameplayEffect(CaptureSpec);

	FAggregatorEvaluateParameters EvalParams;

	EvalParams.SourceTags = &SourceTags;
	EvalParams.TargetTags = &TargetTags;

	bSuccess = CaptureSpec.AttemptCalculateAttributeMagnitude(EvalParams, RetVal);

	return RetVal;
}

float UAbilitySystemBlueprintLibrary::EvaluateAttributeValueWithTagsAndBase(UAbilitySystemComponent* AbilitySystem, FGameplayAttribute Attribute, const FGameplayTagContainer& SourceTags, const FGameplayTagContainer& TargetTags, float BaseValue, bool& bSuccess)
{
	float RetVal = 0.f;
	if (!AbilitySystem || !AbilitySystem->HasAttributeSetForAttribute(Attribute))
	{
		bSuccess = false;
		return RetVal;
	}

	FGameplayEffectAttributeCaptureDefinition Capture(Attribute, EGameplayEffectAttributeCaptureSource::Source, true);

	FGameplayEffectAttributeCaptureSpec CaptureSpec(Capture);
	AbilitySystem->CaptureAttributeForGameplayEffect(CaptureSpec);

	FAggregatorEvaluateParameters EvalParams;

	EvalParams.SourceTags = &SourceTags;
	EvalParams.TargetTags = &TargetTags;

	bSuccess = CaptureSpec.AttemptCalculateAttributeMagnitudeWithBase(EvalParams, BaseValue, RetVal);

	return RetVal;
}

bool UAbilitySystemBlueprintLibrary::EqualEqual_GameplayAttributeGameplayAttribute(FGameplayAttribute AttributeA, FGameplayAttribute AttributeB)
{
	return (AttributeA == AttributeB);
}

bool UAbilitySystemBlueprintLibrary::NotEqual_GameplayAttributeGameplayAttribute(FGameplayAttribute AttributeA, FGameplayAttribute AttributeB)
{
	return (AttributeA != AttributeB);
}

FString UAbilitySystemBlueprintLibrary::GetDebugStringFromGameplayAttribute(const FGameplayAttribute& Attribute)
{
	if (const UClass* AttributeSetClass = Attribute.GetAttributeSetClass())
	{
		return FString::Format(TEXT("{0}.{1}"), { AttributeSetClass->GetName(), Attribute.GetName() });
	}

	return Attribute.GetName();
}

FGameplayAbilityTargetDataHandle UAbilitySystemBlueprintLibrary::AppendTargetDataHandle(FGameplayAbilityTargetDataHandle TargetHandle, const FGameplayAbilityTargetDataHandle& HandleToAdd)
{
	TargetHandle.Append(HandleToAdd);
	return TargetHandle;
}

FGameplayAbilityTargetDataHandle UAbilitySystemBlueprintLibrary::AbilityTargetDataFromLocations(const FGameplayAbilityTargetingLocationInfo& SourceLocation, const FGameplayAbilityTargetingLocationInfo& TargetLocation)
{
	// Construct TargetData
	FGameplayAbilityTargetData_LocationInfo*	NewData = new FGameplayAbilityTargetData_LocationInfo();
	NewData->SourceLocation = SourceLocation;
	NewData->TargetLocation = TargetLocation;

	// Give it a handle and return
	FGameplayAbilityTargetDataHandle	Handle;
	Handle.Data.Add(TSharedPtr<FGameplayAbilityTargetData_LocationInfo>(NewData));
	return Handle;
}

FGameplayAbilityTargetDataHandle UAbilitySystemBlueprintLibrary::AbilityTargetDataFromActor(AActor* Actor)
{
	// Construct TargetData
	FGameplayAbilityTargetData_ActorArray*	NewData = new FGameplayAbilityTargetData_ActorArray();
	NewData->TargetActorArray.Add(Actor);
	FGameplayAbilityTargetDataHandle		Handle(NewData);
	return Handle;
}
FGameplayAbilityTargetDataHandle UAbilitySystemBlueprintLibrary::AbilityTargetDataFromActorArray(const TArray<AActor*>& ActorArray, bool OneTargetPerHandle)
{
	// Construct TargetData
	if (OneTargetPerHandle)
	{
		FGameplayAbilityTargetDataHandle Handle;
		for (int32 i = 0; i < ActorArray.Num(); ++i)
		{
			if (::IsValid(ActorArray[i]))
			{
				FGameplayAbilityTargetDataHandle TempHandle = AbilityTargetDataFromActor(ActorArray[i]);
				Handle.Append(TempHandle);
			}
		}
		return Handle;
	}
	else
	{
		FGameplayAbilityTargetData_ActorArray*	NewData = new FGameplayAbilityTargetData_ActorArray();
		NewData->TargetActorArray.Reset();
		for (auto Actor : ActorArray)
		{
			NewData->TargetActorArray.Add(Actor);
		}
		FGameplayAbilityTargetDataHandle		Handle(NewData);
		return Handle;
	}
}

FGameplayAbilityTargetDataHandle UAbilitySystemBlueprintLibrary::FilterTargetData(const FGameplayAbilityTargetDataHandle& TargetDataHandle, FGameplayTargetDataFilterHandle FilterHandle)
{
	FGameplayAbilityTargetDataHandle ReturnDataHandle;
	
	for (int32 i = 0; TargetDataHandle.IsValid(i); ++i)
	{
		const FGameplayAbilityTargetData* UnfilteredData = TargetDataHandle.Get(i);
		check(UnfilteredData);
		const TArray<TWeakObjectPtr<AActor>> UnfilteredActors = UnfilteredData->GetActors();
		if (UnfilteredActors.Num() > 0)
		{
			TArray<TWeakObjectPtr<AActor>> FilteredActors = UnfilteredActors.FilterByPredicate(FilterHandle);
			if (FilteredActors.Num() > 0)
			{
				//Copy the data first, since we don't understand the internals of it
				const UScriptStruct* ScriptStruct = UnfilteredData->GetScriptStruct();
				FGameplayAbilityTargetData* NewData = (FGameplayAbilityTargetData*)FMemory::Malloc(ScriptStruct->GetCppStructOps()->GetSize());
				ScriptStruct->InitializeStruct(NewData);
				ScriptStruct->CopyScriptStruct(NewData, UnfilteredData);
				ReturnDataHandle.Data.Add(TSharedPtr<FGameplayAbilityTargetData>(NewData, FGameplayAbilityTargetDataDeleter()));
				if (FilteredActors.Num() < UnfilteredActors.Num())
				{
					//We have lost some, but not all, of our actors, so replace the array. This should only be possible with targeting types that permit actor-array setting.
					if (!NewData->SetActors(FilteredActors))
					{
						//This is an error, though we could ignore it. We somehow filtered out part of a list, but the class doesn't support changing the list, so now it's all or nothing.
						check(false);
					}
				}
			}
		}
	}

	return ReturnDataHandle;
}

FGameplayTargetDataFilterHandle UAbilitySystemBlueprintLibrary::MakeFilterHandle(FGameplayTargetDataFilter Filter, AActor* FilterActor)
{
	FGameplayTargetDataFilterHandle FilterHandle;
	FGameplayTargetDataFilter* NewFilter = new FGameplayTargetDataFilter(Filter);
	NewFilter->InitializeFilterContext(FilterActor);
	FilterHandle.Filter = TSharedPtr<FGameplayTargetDataFilter>(NewFilter);
	return FilterHandle;
}

FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::MakeSpecHandle(UGameplayEffect* InGameplayEffect, AActor* InInstigator, AActor* InEffectCauser, float InLevel)
{
	if (InGameplayEffect)
	{
		FGameplayEffectContext* EffectContext = UAbilitySystemGlobals::Get().AllocGameplayEffectContext();
		EffectContext->AddInstigator(InInstigator, InEffectCauser);
		return FGameplayEffectSpecHandle(new FGameplayEffectSpec(InGameplayEffect, FGameplayEffectContextHandle(EffectContext), InLevel));
	}
	
	const FString InstigatorName = InInstigator ? InInstigator->GetActorNameOrLabel() : TEXT("None");
	const FString CauserName = InEffectCauser ? InEffectCauser->GetActorNameOrLabel() : TEXT("None");

	ABILITY_LOG(Warning, TEXT("[%hs] called with null GameplayEffect. Instigator: %s, Causer: %s"), __func__, *InstigatorName, *CauserName);
	
	return FGameplayEffectSpecHandle();
}

FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::MakeSpecHandleByClass(TSubclassOf<UGameplayEffect> GameplayEffect, AActor* Instigator, AActor* EffectCauser, float Level)
{
	if (const UGameplayEffect* GameplayEffectCDO = GameplayEffect.GetDefaultObject())
	{
		FGameplayEffectContext* EffectContext = UAbilitySystemGlobals::Get().AllocGameplayEffectContext();
		EffectContext->AddInstigator(Instigator, EffectCauser);

		return FGameplayEffectSpecHandle(new FGameplayEffectSpec(GameplayEffectCDO, FGameplayEffectContextHandle(EffectContext), Level));
	}

	ABILITY_LOG(Warning, TEXT("%hs was called with invalid GameplayEffect"), __func__);
	return FGameplayEffectSpecHandle();
}

FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::CloneSpecHandle(AActor* InNewInstigator, AActor* InEffectCauser, FGameplayEffectSpecHandle GameplayEffectSpecHandle_Clone)
{
	FGameplayEffectContext* EffectContext = UAbilitySystemGlobals::Get().AllocGameplayEffectContext();
	EffectContext->AddInstigator(InNewInstigator, InEffectCauser);

	return FGameplayEffectSpecHandle(new FGameplayEffectSpec(*GameplayEffectSpecHandle_Clone.Data.Get(), FGameplayEffectContextHandle(EffectContext)));
}


FGameplayAbilityTargetDataHandle UAbilitySystemBlueprintLibrary::AbilityTargetDataFromHitResult(const FHitResult& HitResult)
{
	// Construct TargetData
	FGameplayAbilityTargetData_SingleTargetHit* TargetData = new FGameplayAbilityTargetData_SingleTargetHit(HitResult);

	// Give it a handle and return
	FGameplayAbilityTargetDataHandle	Handle;
	Handle.Data.Add(TSharedPtr<FGameplayAbilityTargetData>(TargetData));

	return Handle;
}

int32 UAbilitySystemBlueprintLibrary::GetDataCountFromTargetData(const FGameplayAbilityTargetDataHandle& TargetData)
{
	return TargetData.Data.Num();
}

TArray<AActor*> UAbilitySystemBlueprintLibrary::GetActorsFromTargetData(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index)
{
	if (TargetData.Data.IsValidIndex(Index))
	{
		const FGameplayAbilityTargetData* Data = TargetData.Data[Index].Get();
		TArray<AActor*>	ResolvedArray;
		if (Data)
		{
			TArray<TWeakObjectPtr<AActor>> WeakArray = Data->GetActors();
			for (TWeakObjectPtr<AActor>& WeakPtr : WeakArray)
			{
				ResolvedArray.Add(WeakPtr.Get());
			}
		}
		return ResolvedArray;
	}
	return TArray<AActor*>();
}

TArray<AActor*> UAbilitySystemBlueprintLibrary::GetAllActorsFromTargetData(const FGameplayAbilityTargetDataHandle& TargetData)
{
	TArray<AActor*>	Result;
	for (int32 TargetDataIndex = 0; TargetDataIndex < TargetData.Data.Num(); ++TargetDataIndex)
	{
		if (TargetData.Data.IsValidIndex(TargetDataIndex))
		{
			const FGameplayAbilityTargetData* DataAtIndex = TargetData.Data[TargetDataIndex].Get();
			if (DataAtIndex)
			{
				TArray<TWeakObjectPtr<AActor>> WeakArray = DataAtIndex->GetActors();
				for (TWeakObjectPtr<AActor>& WeakPtr : WeakArray)
				{
					Result.Add(WeakPtr.Get());
				}
			}
		}
	}
	return Result;
}

bool UAbilitySystemBlueprintLibrary::DoesTargetDataContainActor(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index, AActor* Actor)
{
	if (TargetData.Data.IsValidIndex(Index))
	{
		FGameplayAbilityTargetData* Data = TargetData.Data[Index].Get();
		if (Data)
		{
			TArray<TWeakObjectPtr<AActor>> WeakArray = Data->GetActors();
			for (TWeakObjectPtr<AActor>& WeakPtr : WeakArray)
			{
				if (WeakPtr == Actor)
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool UAbilitySystemBlueprintLibrary::TargetDataHasActor(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index)
{
	if (TargetData.Data.IsValidIndex(Index))
	{
		FGameplayAbilityTargetData* Data = TargetData.Data[Index].Get();
		if (Data)
		{
			return (Data->GetActors().Num() > 0);
		}
	}
	return false;
}

bool UAbilitySystemBlueprintLibrary::TargetDataHasHitResult(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index)
{
	if (TargetData.Data.IsValidIndex(Index))
	{
		FGameplayAbilityTargetData* Data = TargetData.Data[Index].Get();
		if (Data)
		{
			return Data->HasHitResult();
		}
	}
	return false;
}

FHitResult UAbilitySystemBlueprintLibrary::GetHitResultFromTargetData(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index)
{
	if (TargetData.Data.IsValidIndex(Index))
	{
		FGameplayAbilityTargetData* Data = TargetData.Data[Index].Get();
		if (Data)
		{
			const FHitResult* HitResultPtr = Data->GetHitResult();
			if (HitResultPtr)
			{
				return *HitResultPtr;
			}
		}
	}

	return FHitResult();
}

bool UAbilitySystemBlueprintLibrary::TargetDataHasOrigin(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index)
{
	if (TargetData.Data.IsValidIndex(Index) == false)
	{
		return false;
	}

	FGameplayAbilityTargetData* Data = TargetData.Data[Index].Get();
	if (Data)
	{
		return (Data->HasHitResult() || Data->HasOrigin());
	}
	return false;
}

FTransform UAbilitySystemBlueprintLibrary::GetTargetDataOrigin(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index)
{
	if (TargetData.Data.IsValidIndex(Index) == false)
	{
		return FTransform::Identity;
	}

	FGameplayAbilityTargetData* Data = TargetData.Data[Index].Get();
	if (Data)
	{
		if (Data->HasOrigin())
		{
			return Data->GetOrigin();
		}
		if (Data->HasHitResult())
		{
			const FHitResult* HitResultPtr = Data->GetHitResult();
			FTransform ReturnTransform;
			ReturnTransform.SetLocation(HitResultPtr->TraceStart);
			ReturnTransform.SetRotation((HitResultPtr->Location - HitResultPtr->TraceStart).GetSafeNormal().Rotation().Quaternion());
			return ReturnTransform;
		}
	}

	return FTransform::Identity;
}

bool UAbilitySystemBlueprintLibrary::TargetDataHasEndPoint(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index)
{
	if (TargetData.Data.IsValidIndex(Index))
	{
		FGameplayAbilityTargetData* Data = TargetData.Data[Index].Get();
		if (Data)
		{
			return (Data->HasHitResult() || Data->HasEndPoint());
		}
	}
	return false;
}

FVector UAbilitySystemBlueprintLibrary::GetTargetDataEndPoint(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index)
{
	if (TargetData.Data.IsValidIndex(Index))
	{
		FGameplayAbilityTargetData* Data = TargetData.Data[Index].Get();
		if (Data)
		{
			const FHitResult* HitResultPtr = Data->GetHitResult();
			if (HitResultPtr)
			{
				return HitResultPtr->Location;
			}
			else if (Data->HasEndPoint())
			{
				return Data->GetEndPoint();
			}
		}
	}

	return FVector::ZeroVector;
}

FTransform UAbilitySystemBlueprintLibrary::GetTargetDataEndPointTransform(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index)
{
	if (TargetData.Data.IsValidIndex(Index))
	{
		FGameplayAbilityTargetData* Data = TargetData.Data[Index].Get();
		if (Data)
		{
			return Data->GetEndPointTransform();
		}
	}

	return FTransform::Identity;
}


// -------------------------------------------------------------------------------------

bool UAbilitySystemBlueprintLibrary::EffectContextIsValid(FGameplayEffectContextHandle EffectContext)
{
	return EffectContext.IsValid();
}

bool UAbilitySystemBlueprintLibrary::EffectContextIsInstigatorLocallyControlled(FGameplayEffectContextHandle EffectContext)
{
	return EffectContext.IsLocallyControlled();
}

FHitResult UAbilitySystemBlueprintLibrary::EffectContextGetHitResult(FGameplayEffectContextHandle EffectContext)
{
	if (EffectContext.GetHitResult())
	{
		return *EffectContext.GetHitResult();
	}

	return FHitResult();
}

bool UAbilitySystemBlueprintLibrary::EffectContextHasHitResult(FGameplayEffectContextHandle EffectContext)
{
	return EffectContext.GetHitResult() != NULL;
}

void UAbilitySystemBlueprintLibrary::EffectContextAddHitResult(FGameplayEffectContextHandle EffectContext, FHitResult HitResult, bool bReset)
{
	EffectContext.AddHitResult(HitResult, bReset);
}

AActor*	UAbilitySystemBlueprintLibrary::EffectContextGetInstigatorActor(FGameplayEffectContextHandle EffectContext)
{
	return EffectContext.GetInstigator();
}

AActor*	UAbilitySystemBlueprintLibrary::EffectContextGetOriginalInstigatorActor(FGameplayEffectContextHandle EffectContext)
{
	return EffectContext.GetOriginalInstigator();
}

AActor*	UAbilitySystemBlueprintLibrary::EffectContextGetEffectCauser(FGameplayEffectContextHandle EffectContext)
{
	return EffectContext.GetEffectCauser();
}

UObject* UAbilitySystemBlueprintLibrary::EffectContextGetSourceObject(FGameplayEffectContextHandle EffectContext)
{
	return const_cast<UObject*>( EffectContext.GetSourceObject() );
}

FVector UAbilitySystemBlueprintLibrary::EffectContextGetOrigin(FGameplayEffectContextHandle EffectContext)
{
	if (EffectContext.HasOrigin())
	{
		return EffectContext.GetOrigin();
	}

	return FVector::ZeroVector;
}

void UAbilitySystemBlueprintLibrary::EffectContextSetOrigin(FGameplayEffectContextHandle EffectContext, FVector Origin)
{
	EffectContext.AddOrigin(Origin);
}

bool UAbilitySystemBlueprintLibrary::IsInstigatorLocallyControlled(FGameplayCueParameters Parameters)
{
	return Parameters.IsInstigatorLocallyControlled();
}

bool UAbilitySystemBlueprintLibrary::IsInstigatorLocallyControlledPlayer(FGameplayCueParameters Parameters)
{
	return Parameters.IsInstigatorLocallyControlledPlayer();
}

int32 UAbilitySystemBlueprintLibrary::GetActorCount(FGameplayCueParameters Parameters)
{
	return Parameters.EffectContext.GetActors().Num();
}

AActor* UAbilitySystemBlueprintLibrary::GetActorByIndex(FGameplayCueParameters Parameters, int32 Index)
{
	const TArray<TWeakObjectPtr<AActor>> WeakActors = Parameters.EffectContext.GetActors();
	if (WeakActors.IsValidIndex(Index))
	{
		return WeakActors[Index].Get();
	}
	return NULL;
}

FHitResult UAbilitySystemBlueprintLibrary::GetHitResult(FGameplayCueParameters Parameters)
{
	if (Parameters.EffectContext.GetHitResult())
	{
		return *Parameters.EffectContext.GetHitResult();
	}
	
	return FHitResult();
}

bool UAbilitySystemBlueprintLibrary::HasHitResult(FGameplayCueParameters Parameters)
{
	return Parameters.EffectContext.GetHitResult() != NULL;
}

void UAbilitySystemBlueprintLibrary::ForwardGameplayCueToTarget(TScriptInterface<IGameplayCueInterface> TargetCueInterface, EGameplayCueEvent::Type EventType, FGameplayCueParameters Parameters)
{
	UObject* TargetObject = TargetCueInterface.GetObject();
	if (TargetCueInterface && TargetObject)
	{
		TargetCueInterface->HandleGameplayCue(TargetObject, Parameters.OriginalTag, EventType, Parameters);
	}
}

AActor*	UAbilitySystemBlueprintLibrary::GetInstigatorActor(FGameplayCueParameters Parameters)
{
	return Parameters.GetInstigator();
}

FTransform UAbilitySystemBlueprintLibrary::GetInstigatorTransform(FGameplayCueParameters Parameters)
{
	AActor* InstigatorActor = GetInstigatorActor(Parameters);
	if (InstigatorActor)
	{
		return InstigatorActor->GetTransform();
	}

	ABILITY_LOG(Warning, TEXT("UAbilitySystemBlueprintLibrary::GetInstigatorTransform called on GameplayCue with no valid instigator"));
	return FTransform::Identity;
}

FVector UAbilitySystemBlueprintLibrary::GetOrigin(FGameplayCueParameters Parameters)
{
	if (Parameters.EffectContext.HasOrigin())
	{
		return Parameters.EffectContext.GetOrigin();
	}

	return Parameters.Location;
}

bool UAbilitySystemBlueprintLibrary::GetGameplayCueEndLocationAndNormal(AActor* TargetActor, FGameplayCueParameters Parameters, FVector& Location, FVector& Normal)
{
	FGameplayEffectContext* Data = Parameters.EffectContext.Get();
	if (Parameters.Location.IsNearlyZero() == false)
	{
		Location = Parameters.Location;
		Normal = Parameters.Normal;
	}
	else if (Data && Data->GetHitResult())
	{
		Location = Data->GetHitResult()->Location;
		Normal = Data->GetHitResult()->Normal;
		return true;
	}
	else if(TargetActor)
	{
		Location = TargetActor->GetActorLocation();
		Normal = TargetActor->GetActorForwardVector();
		return true;
	}

	return false;
}

bool UAbilitySystemBlueprintLibrary::GetGameplayCueDirection(AActor* TargetActor, FGameplayCueParameters Parameters, FVector& Direction)
{
	if (Parameters.Normal.IsNearlyZero() == false)
	{
		Direction = -Parameters.Normal;
		return true;
	}

	if (FGameplayEffectContext* Ctx = Parameters.EffectContext.Get())
	{
		if (Ctx->GetHitResult())
		{
			// Most projectiles and melee attacks will use this
			Direction = (-1.f * Ctx->GetHitResult()->Normal);
			return true;
		}
		else if (TargetActor && Ctx->HasOrigin())
		{
			// Fallback to trying to use the target location and the origin of the effect
			FVector NewVec = (TargetActor->GetActorLocation() - Ctx->GetOrigin());
			NewVec.Normalize();
			Direction = NewVec;
			return true;
		}
		else if (TargetActor && Ctx->GetEffectCauser())
		{
			// Finally, try to use the direction between the causer of the effect and the target of the effect
			FVector NewVec = (TargetActor->GetActorLocation() - Ctx->GetEffectCauser()->GetActorLocation());
			NewVec.Normalize();
			Direction = NewVec;
			return true;
		}
	}

	Direction = FVector::ZeroVector;
	return false;
}

bool UAbilitySystemBlueprintLibrary::DoesGameplayCueMeetTagRequirements(FGameplayCueParameters Parameters, const FGameplayTagRequirements& SourceTagReqs, const FGameplayTagRequirements& TargetTagReqs)
{
	return SourceTagReqs.RequirementsMet(Parameters.AggregatedSourceTags)
		&& TargetTagReqs.RequirementsMet(Parameters.AggregatedSourceTags);
}

FGameplayCueParameters UAbilitySystemBlueprintLibrary::MakeGameplayCueParameters(float NormalizedMagnitude, float RawMagnitude, FGameplayEffectContextHandle EffectContext, FGameplayTag MatchedTagName, FGameplayTag OriginalTag, FGameplayTagContainer AggregatedSourceTags, FGameplayTagContainer AggregatedTargetTags, FVector Location, FVector Normal, AActor* Instigator, AActor* EffectCauser, UObject* SourceObject, UPhysicalMaterial* PhysicalMaterial, int32 GameplayEffectLevel, int32 AbilityLevel, USceneComponent* TargetAttachComponent, bool bReplicateLocationWhenUsingMinimalRepProxy)
{
	FGameplayCueParameters Parameters;
	Parameters.NormalizedMagnitude = NormalizedMagnitude;
	Parameters.RawMagnitude = RawMagnitude;
	Parameters.EffectContext = EffectContext;
	Parameters.MatchedTagName = MatchedTagName;
	Parameters.OriginalTag = OriginalTag;
	Parameters.AggregatedSourceTags = AggregatedSourceTags;
	Parameters.AggregatedTargetTags = AggregatedTargetTags;
	Parameters.Location = Location;
	Parameters.Normal = Normal;
	Parameters.Instigator = Instigator;
	Parameters.EffectCauser = EffectCauser;
	Parameters.SourceObject = SourceObject;
	Parameters.PhysicalMaterial = PhysicalMaterial;
	Parameters.GameplayEffectLevel = GameplayEffectLevel;
	Parameters.AbilityLevel = AbilityLevel;
	Parameters.TargetAttachComponent = TargetAttachComponent;
	Parameters.bReplicateLocationWhenUsingMinimalRepProxy = bReplicateLocationWhenUsingMinimalRepProxy;
	return Parameters;
}

void UAbilitySystemBlueprintLibrary::BreakGameplayCueParameters(const struct FGameplayCueParameters& Parameters, float& NormalizedMagnitude, float& RawMagnitude, FGameplayEffectContextHandle& EffectContext, FGameplayTag& MatchedTagName, FGameplayTag& OriginalTag, FGameplayTagContainer& AggregatedSourceTags, FGameplayTagContainer& AggregatedTargetTags, FVector& Location, FVector& Normal, AActor*& Instigator, AActor*& EffectCauser, UObject*& SourceObject, UPhysicalMaterial*& PhysicalMaterial, int32& GameplayEffectLevel, int32& AbilityLevel, USceneComponent*& TargetAttachComponent, bool& bReplicateLocationWhenUsingMinimalRepProxy)
{
	NormalizedMagnitude = Parameters.NormalizedMagnitude;
	RawMagnitude = Parameters.RawMagnitude;
	EffectContext = Parameters.EffectContext;
	MatchedTagName = Parameters.MatchedTagName;
	OriginalTag = Parameters.OriginalTag;
	AggregatedSourceTags = Parameters.AggregatedSourceTags;
	AggregatedTargetTags = Parameters.AggregatedTargetTags;
	Location = Parameters.Location;
	Normal = Parameters.Normal;
	Instigator = Parameters.Instigator.Get();
	EffectCauser = Parameters.EffectCauser.Get();
	SourceObject = const_cast<UObject*>(Parameters.SourceObject.Get());
	PhysicalMaterial = const_cast<UPhysicalMaterial*>(Parameters.PhysicalMaterial.Get());
	GameplayEffectLevel = Parameters.GameplayEffectLevel;
	AbilityLevel = Parameters.AbilityLevel;
	TargetAttachComponent = Parameters.TargetAttachComponent.Get();
	bReplicateLocationWhenUsingMinimalRepProxy = Parameters.bReplicateLocationWhenUsingMinimalRepProxy;
}

// ---------------------------------------------------------------------------------------

FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::AssignSetByCallerMagnitude(FGameplayEffectSpecHandle SpecHandle, FName DataName, float Magnitude)
{
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		
		Spec->SetSetByCallerMagnitude(DataName, Magnitude);
		
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilitySystemBlueprintLibrary::AssignSetByCallerMagnitude called with invalid SpecHandle"));
	}

	return SpecHandle;
}

FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::AssignTagSetByCallerMagnitude(FGameplayEffectSpecHandle SpecHandle, FGameplayTag DataTag, float Magnitude)
{
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec)
	{
		Spec->SetSetByCallerMagnitude(DataTag, Magnitude);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilitySystemBlueprintLibrary::AssignSetByCallerTagMagnitude called with invalid SpecHandle"));
	}

	return SpecHandle;
}

FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::SetDuration(FGameplayEffectSpecHandle SpecHandle, float Duration)
{
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec)
	{
		Spec->SetDuration(Duration, true);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilitySystemBlueprintLibrary::SetDuration called with invalid SpecHandle"));
	}

	return SpecHandle;
}

FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::AddGrantedTag(FGameplayEffectSpecHandle SpecHandle, FGameplayTag NewGameplayTag)
{
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec)
	{
		Spec->DynamicGrantedTags.AddTag(NewGameplayTag);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilitySystemBlueprintLibrary::AddGrantedTag called with invalid SpecHandle"));
	}

	return SpecHandle;
}

FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::AddGrantedTags(FGameplayEffectSpecHandle SpecHandle, FGameplayTagContainer NewGameplayTags)
{
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec)
	{
		Spec->DynamicGrantedTags.AppendTags(NewGameplayTags);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilitySystemBlueprintLibrary::AddGrantedTags called with invalid SpecHandle"));
	}

	return SpecHandle;
}

FGameplayTagContainer UAbilitySystemBlueprintLibrary::GetGrantedTags(FGameplayEffectSpecHandle SpecHandle)
{
	FGameplayTagContainer AllGrantedTags;
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec)
	{
		Spec->GetAllGrantedTags(AllGrantedTags);
	}
	return AllGrantedTags;
}

FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::AddAssetTag(FGameplayEffectSpecHandle SpecHandle, FGameplayTag NewGameplayTag)
{
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec)
	{
		Spec->AddDynamicAssetTag(NewGameplayTag);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilitySystemBlueprintLibrary::AddEffectTag called with invalid SpecHandle"));
	}

	return SpecHandle;
}

FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::AddAssetTags(FGameplayEffectSpecHandle SpecHandle, FGameplayTagContainer NewGameplayTags)
{
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec)
	{
		Spec->AppendDynamicAssetTags(NewGameplayTags);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilitySystemBlueprintLibrary::AddEffectTags called with invalid SpecHandle"));
	}

	return SpecHandle;
}

FGameplayTagContainer UAbilitySystemBlueprintLibrary::GetAssetTags(FGameplayEffectSpecHandle SpecHandle)
{
	FGameplayTagContainer AllAssetTags;
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec)
	{
		Spec->GetAllAssetTags(AllAssetTags);
	}
	return AllAssetTags;
}
	
FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::AddLinkedGameplayEffectSpec(FGameplayEffectSpecHandle SpecHandle, FGameplayEffectSpecHandle LinkedGameplayEffectSpec)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec)
	{
		Spec->TargetEffectSpecs.Add(LinkedGameplayEffectSpec);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilitySystemBlueprintLibrary::AddLinkedGameplayEffectSpec called with invalid SpecHandle"));
	}

	return SpecHandle;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::AddLinkedGameplayEffect(FGameplayEffectSpecHandle SpecHandle, TSubclassOf<UGameplayEffect> LinkedGameplayEffect)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGameplayEffectSpecHandle LinkedSpecHandle;
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec)
	{
		FGameplayEffectSpec* LinkedSpec = new FGameplayEffectSpec();
		LinkedSpec->InitializeFromLinkedSpec(LinkedGameplayEffect->GetDefaultObject<UGameplayEffect>(), *Spec);

		LinkedSpecHandle = FGameplayEffectSpecHandle(LinkedSpec);
		Spec->TargetEffectSpecs.Add(LinkedSpecHandle);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("UAbilitySystemBlueprintLibrary::AddLinkedGameplayEffectSpec called with invalid SpecHandle"));
	}

	return LinkedSpecHandle;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::SetStackCount(FGameplayEffectSpecHandle SpecHandle, int32 StackCount)
{
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec)
	{
		Spec->SetStackCount(StackCount);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("%s called with invalid SpecHandle"), ANSI_TO_TCHAR(__func__));
	}
	return SpecHandle;
}
	
FGameplayEffectSpecHandle UAbilitySystemBlueprintLibrary::SetStackCountToMax(FGameplayEffectSpecHandle SpecHandle)
{
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	if (Spec && Spec->Def)
	{
		Spec->SetStackCount(Spec->Def->StackLimitCount);
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("%s called with invalid SpecHandle"), ANSI_TO_TCHAR(__func__));
	}
	return SpecHandle;
}

FGameplayEffectContextHandle UAbilitySystemBlueprintLibrary::GetEffectContext(FGameplayEffectSpecHandle SpecHandle)
{
	if (FGameplayEffectSpec* Spec = SpecHandle.Data.Get())
	{
		return Spec->GetEffectContext();
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("%s called with invalid SpecHandle"), ANSI_TO_TCHAR(__func__));
	}

	return FGameplayEffectContextHandle();
}

TArray<FGameplayEffectSpecHandle> UAbilitySystemBlueprintLibrary::GetAllLinkedGameplayEffectSpecHandles(FGameplayEffectSpecHandle SpecHandle)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if (FGameplayEffectSpec* Spec = SpecHandle.Data.Get())
	{
		return Spec->TargetEffectSpecs;
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("%s called with invalid SpecHandle"), ANSI_TO_TCHAR(__func__));
	}

	TArray<FGameplayEffectSpecHandle> Handles;
	return Handles;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAbilitySystemBlueprintLibrary::IsActiveGameplayEffectHandleValid(FActiveGameplayEffectHandle Handle)
{
	return Handle.IsValid();
}

bool UAbilitySystemBlueprintLibrary::IsActiveGameplayEffectHandleActive(FActiveGameplayEffectHandle Handle)
{
	// GetOwningAbilitySystemComponent will only return a valid pointer if the effect is still active
	return Handle.IsValid() && Handle.GetOwningAbilitySystemComponent();
}

UAbilitySystemComponent* UAbilitySystemBlueprintLibrary::GetAbilitySystemComponentFromActiveGameplayEffectHandle(FActiveGameplayEffectHandle Handle)
{
	return Handle.GetOwningAbilitySystemComponent();
}

int32 UAbilitySystemBlueprintLibrary::GetActiveGameplayEffectStackCount(FActiveGameplayEffectHandle ActiveHandle)
{
	UAbilitySystemComponent* ASC = ActiveHandle.GetOwningAbilitySystemComponent();
	if (ASC)
	{
		return ASC->GetCurrentStackCount(ActiveHandle);
	}
	return 0;
}

int32 UAbilitySystemBlueprintLibrary::GetActiveGameplayEffectStackLimitCount(FActiveGameplayEffectHandle ActiveHandle)
{
	UAbilitySystemComponent* ASC = ActiveHandle.GetOwningAbilitySystemComponent();
	if (ASC)
	{
		const UGameplayEffect* ActiveGE = ASC->GetGameplayEffectDefForHandle(ActiveHandle);
		if (ActiveGE)
		{
			return ActiveGE->GetStackLimitCount();
		}
	}
	return 0;
}

float UAbilitySystemBlueprintLibrary::GetActiveGameplayEffectStartTime(FActiveGameplayEffectHandle ActiveHandle)
{
	UAbilitySystemComponent* ASC = ActiveHandle.GetOwningAbilitySystemComponent();
	if (ASC)
	{
		if (const FActiveGameplayEffect* ActiveGE = ASC->GetActiveGameplayEffect(ActiveHandle))
		{
			return ActiveGE->StartWorldTime;
		}
	}
	return 0;
}
	
float UAbilitySystemBlueprintLibrary::GetActiveGameplayEffectExpectedEndTime(FActiveGameplayEffectHandle ActiveHandle)
{
	UAbilitySystemComponent* ASC = ActiveHandle.GetOwningAbilitySystemComponent();
	if (ASC)
	{
		if (const FActiveGameplayEffect* ActiveGE = ASC->GetActiveGameplayEffect(ActiveHandle))
		{
			return ActiveGE->GetEndTime();
		}
	}
	return 0;
}

float UAbilitySystemBlueprintLibrary::GetActiveGameplayEffectTotalDuration(FActiveGameplayEffectHandle ActiveHandle)
{
	UAbilitySystemComponent* ASC = ActiveHandle.GetOwningAbilitySystemComponent();
	if (ASC)
	{
		if (const FActiveGameplayEffect* ActiveGE = ASC->GetActiveGameplayEffect(ActiveHandle))
		{
			return ActiveGE->GetDuration();
		}
	}
	return 0;
}
float UAbilitySystemBlueprintLibrary::GetActiveGameplayEffectRemainingDuration(UObject* WorldContextObject, FActiveGameplayEffectHandle ActiveHandle)
{
	UAbilitySystemComponent* ASC = ActiveHandle.GetOwningAbilitySystemComponent();
	if (ASC)
	{
		if (const FActiveGameplayEffect* ActiveGE = ASC->GetActiveGameplayEffect(ActiveHandle))
		{
			if (WorldContextObject)
			{
				if (UWorld* World = WorldContextObject->GetWorld())
				{
					return ActiveGE->GetTimeRemaining(World->GetTimeSeconds());
				}
			}
		}
	}
	return 0;
}

float UAbilitySystemBlueprintLibrary::GetModifiedAttributeMagnitude(const FGameplayEffectSpec& Spec, FGameplayAttribute Attribute)
{
	float Delta = 0.f;
	for (const FGameplayEffectModifiedAttribute &Mod : Spec.ModifiedAttributes)
	{
		if (Mod.Attribute == Attribute)
		{
			Delta += Mod.TotalMagnitude;
		}
	}
	return Delta;
}

float UAbilitySystemBlueprintLibrary::GetModifiedAttributeMagnitude(FGameplayEffectSpecHandle SpecHandle, FGameplayAttribute Attribute)
{
	FGameplayEffectSpec* Spec = SpecHandle.Data.Get();
	float Delta = 0.f;
	if (Spec)
	{
		return GetModifiedAttributeMagnitude(*Spec, Attribute);
	}
	return 0;
}

FString UAbilitySystemBlueprintLibrary::GetActiveGameplayEffectDebugString(FActiveGameplayEffectHandle ActiveHandle)
{
	FString Str;
	UAbilitySystemComponent* ASC = ActiveHandle.GetOwningAbilitySystemComponent();
	if (ASC)
	{
		Str = ASC->GetActiveGEDebugString(ActiveHandle);
	}
	return Str;
}

bool UAbilitySystemBlueprintLibrary::AddLooseGameplayTags(AActor* Actor, const FGameplayTagContainer& GameplayTags, bool bShouldReplicate)
{
	if (UAbilitySystemComponent* AbilitySysComp = GetAbilitySystemComponent(Actor))
	{
		AbilitySysComp->AddLooseGameplayTags(GameplayTags, 1, bShouldReplicate ? EGameplayTagReplicationState::CountToOwner : EGameplayTagReplicationState::None);

		return true;
	}

	return false;
}

bool UAbilitySystemBlueprintLibrary::RemoveLooseGameplayTags(AActor* Actor, const FGameplayTagContainer& GameplayTags, bool bShouldReplicate)
{
	if (UAbilitySystemComponent* AbilitySysComp = GetAbilitySystemComponent(Actor))
	{
		AbilitySysComp->RemoveLooseGameplayTags(GameplayTags, 1, bShouldReplicate ? EGameplayTagReplicationState::CountToOwner : EGameplayTagReplicationState::None);

		return true;
	}

	return false;
}

bool UAbilitySystemBlueprintLibrary::AddGameplayTags(AActor* Actor, const FGameplayTagContainer& GameplayTags, EGameplayTagReplicationState ReplicationRule)
{
	if (UAbilitySystemComponent* AbilitySysComp = GetAbilitySystemComponent(Actor))
	{
		AbilitySysComp->AddLooseGameplayTags(GameplayTags, 1, ReplicationRule);

		return true;
	}

	return false;
}

bool UAbilitySystemBlueprintLibrary::RemoveGameplayTags(AActor* Actor, const FGameplayTagContainer& GameplayTags, EGameplayTagReplicationState ReplicationRule)
{
	if (UAbilitySystemComponent* AbilitySysComp = GetAbilitySystemComponent(Actor))
	{
		AbilitySysComp->RemoveLooseGameplayTags(GameplayTags, 1, ReplicationRule);

		return true;
	}

	return false;
}

const UGameplayEffect* UAbilitySystemBlueprintLibrary::GetGameplayEffectFromSpecHandle(FGameplayEffectSpecHandle Handle)
{
	return Handle.Data ? Handle.Data->Def.Get() : nullptr;
}

bool UAbilitySystemBlueprintLibrary::IsDurationGameplayEffectSpecHandle(FGameplayEffectSpecHandle Handle)
{
	if (Handle.Data && Handle.Data->Def)
	{
		return Handle.Data->Def->DurationPolicy != EGameplayEffectDurationType::Instant;
	}

	ABILITY_LOG(Warning, TEXT("IsDurationGameplayEffectSpecHandle called on invalid handle"));
	return false;
}

EGameplayEffectDurationType UAbilitySystemBlueprintLibrary::GetDurationPolicyFromGameplayEffectSpecHandle(FGameplayEffectSpecHandle Handle)
{
	if (Handle.Data && Handle.Data->Def)
	{
		return Handle.Data->Def->DurationPolicy;
	}

	ABILITY_LOG(Warning, TEXT("GetDurationPolicyFromGameplayEffectSpecHandle called on invalid handle"));
	return EGameplayEffectDurationType::Instant;
}

const UGameplayEffectUIData* UAbilitySystemBlueprintLibrary::GetGameplayEffectUIData(TSubclassOf<UGameplayEffect> EffectClass, TSubclassOf<UGameplayEffectUIData> DataType)
{
	if (const UGameplayEffect* Effect = EffectClass.GetDefaultObject())
	{
		const UGameplayEffectUIData* UIData = Effect->FindComponent<UGameplayEffectUIData>();
		if (!UIData)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			UIData = Effect->UIData;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		if ((UIData != nullptr) && (DataType != nullptr) && UIData->IsA(DataType))
		{
			return UIData;
		}
	}

	return nullptr;
}

bool UAbilitySystemBlueprintLibrary::EqualEqual_ActiveGameplayEffectHandle(const FActiveGameplayEffectHandle& A, const FActiveGameplayEffectHandle& B)
{
	return A == B;
}

bool UAbilitySystemBlueprintLibrary::NotEqual_ActiveGameplayEffectHandle(const FActiveGameplayEffectHandle& A, const FActiveGameplayEffectHandle& B)
{
	return A != B;
}

const UGameplayEffect* UAbilitySystemBlueprintLibrary::GetGameplayEffectFromActiveEffectHandle(const FActiveGameplayEffectHandle& ActiveHandle)
{
	const UAbilitySystemComponent* ASC = ActiveHandle.GetOwningAbilitySystemComponent();

	if (ASC)
	{
		return ASC->GetGameplayEffectCDO(ActiveHandle);
	}

	ABILITY_LOG(Error, TEXT("GetGameplayAbilityFromSpecHandle() called with an invalid Active Gameplay Effect Handle"));

	return nullptr;
}

const FGameplayTagContainer& UAbilitySystemBlueprintLibrary::GetGameplayEffectAssetTags(TSubclassOf<UGameplayEffect> EffectClass)
{
	if (const UGameplayEffect* DefaultGE = EffectClass.GetDefaultObject())
	{
		return DefaultGE->GetAssetTags();
	}

	static const FGameplayTagContainer Empty{};
	return Empty;
}

const FGameplayTagContainer& UAbilitySystemBlueprintLibrary::GetGameplayEffectGrantedTags(TSubclassOf<UGameplayEffect> EffectClass)
{
	if (const UGameplayEffect* DefaultGE = EffectClass.GetDefaultObject())
	{
		return DefaultGE->GetGrantedTags();
	}

	static const FGameplayTagContainer Empty{};
	return Empty;
}

const UGameplayAbility* UAbilitySystemBlueprintLibrary::GetGameplayAbilityFromSpecHandle(UAbilitySystemComponent* AbilitySystem, const FGameplayAbilitySpecHandle& AbilitySpecHandle, bool& bIsInstance)
{
	// validate the ASC
	if (!AbilitySystem)
	{
		ABILITY_LOG(Error, TEXT("GetGameplayAbilityFromSpecHandle() called with an invalid Ability System Component"));

		bIsInstance = false;
		return nullptr;
	}

	// get and validate the ability spec
	FGameplayAbilitySpec* AbilitySpec = AbilitySystem->FindAbilitySpecFromHandle(AbilitySpecHandle);
	if (!AbilitySpec)
	{
		ABILITY_LOG(Error, TEXT("GetGameplayAbilityFromSpecHandle() Ability Spec not found on passed Ability System Component"));

		bIsInstance = false;
		return nullptr;
	}

	// try to get the ability instance
	UGameplayAbility* AbilityInstance = AbilitySpec->GetPrimaryInstance();
	bIsInstance = true;

	// default to the CDO if we can't
	if (!AbilityInstance)
	{
		AbilityInstance = AbilitySpec->Ability;
		bIsInstance = false;
	}

	return AbilityInstance;
}

bool UAbilitySystemBlueprintLibrary::IsGameplayAbilityActive(const UGameplayAbility* GameplayAbility)
{
	if (!GameplayAbility)
	{
		UE_LOGF(LogAbilitySystem, Error, "%s passed in invalid (null) GameplayAbility", __func__);
		return false;
	}
	else if (!GameplayAbility->IsInstantiated())
	{
		UE_LOGF(LogAbilitySystem, Error, "%s passed a non-instantiated instance: %ls", __func__, *GetNameSafe(GameplayAbility));
		return false;
	}

	return GameplayAbility->IsActive();
}

bool UAbilitySystemBlueprintLibrary::HasAnyAbilitiesByPredicate(
	const UAbilitySystemComponent* AbilitySystemComponent, 
	const TFunctionRef<bool(const UGameplayAbility& AbilityRef)> Predicate, 
	bool bOnlyRunPredicateOnAbilityCDOs /*= false*/)
{
	if (AbilitySystemComponent == nullptr)
	{
		return false;
	}

	// Copy off the ActivatableAbilities for safety to avoid situations where the Predicate is affecting the list while iterating.
	TArray<FGameplayAbilitySpec> ActivatableAbilitiesCopy = AbilitySystemComponent->GetActivatableAbilities();
	for (const FGameplayAbilitySpec& Spec : ActivatableAbilitiesCopy)
	{
		if (Spec.Ability == nullptr)
		{
			continue;
		}

		TArray<UGameplayAbility*> AbilitiesToCheck;

		if (!bOnlyRunPredicateOnAbilityCDOs && Spec.Ability->GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced)
		{
			AbilitiesToCheck.Append(Spec.GetAbilityInstances());
		}
		else
		{
			AbilitiesToCheck.Add(Spec.Ability);
		}

		for (UGameplayAbility* AbilityToCheck : AbilitiesToCheck)
		{
			if (AbilityToCheck == nullptr)
			{
				continue;
			}

			const bool bPassesCondition = Predicate(*AbilityToCheck);
			if (bPassesCondition)
			{
				return true;
			}
		}
	}

	return false;
}

bool UAbilitySystemBlueprintLibrary::HasAnyAbilitiesWithAssetTag(const UAbilitySystemComponent* AbilitySystemComponent, FGameplayTag AssetTag)
{
	if (AbilitySystemComponent == nullptr)
	{
		return false;
	}

	constexpr bool bOnlyCheckAbilityCDO = true;
	return HasAnyAbilitiesByPredicate(AbilitySystemComponent, [AssetTag](const UGameplayAbility& AbilityRef) -> bool
	{
		return AbilityRef.GetAssetTags().HasTag(AssetTag);
	}, bOnlyCheckAbilityCDO);
}

bool UAbilitySystemBlueprintLibrary::EqualEqual_GameplayAbilitySpecHandle(const FGameplayAbilitySpecHandle& A, const FGameplayAbilitySpecHandle& B)
{
	return A == B;
}

bool UAbilitySystemBlueprintLibrary::NotEqual_GameplayAbilitySpecHandle(const FGameplayAbilitySpecHandle& A, const FGameplayAbilitySpecHandle& B)
{
	return A != B;
}

float UAbilitySystemBlueprintLibrary::Conv_ScalableFloatToFloat(const FScalableFloat& Input, float Level)
{
	return Input.GetValueAtLevel(Level);
}

double UAbilitySystemBlueprintLibrary::Conv_ScalableFloatToDouble(const FScalableFloat& Input, float Level)
{
	return static_cast<double>(Input.GetValueAtLevel(Level));
}
