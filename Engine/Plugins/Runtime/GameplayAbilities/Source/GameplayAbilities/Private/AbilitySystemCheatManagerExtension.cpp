// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbilitySystemCheatManagerExtension.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"

#include "Engine/Blueprint.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilitySystemCheatManagerExtension)

namespace  UE::AbilitySystem::Private
{
	/**
	 * Common logic used to fuzzy-find a requested class (or alternatively, a passed-in asset path).
	 */
	template<typename ClassToFind>
	TSubclassOf<ClassToFind> FuzzyFindClass(FString SearchString)
	{
		TSubclassOf<ClassToFind> FoundClass;

		// See if we passed in a class name of a Class that already exists in memory.
		// If we passed-in Default__, just remove that part since we're looking for Classes, not CDO names.
		SearchString.RemoveFromStart(TEXT("Default__"));
		const int SearchStringLen = SearchString.Len();
		int BestClassMatchLen = INT_MAX;
		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			const bool bClassMatches = ClassIt->IsChildOf(ClassToFind::StaticClass());
			if (!bClassMatches)
			{
				continue;
			}

			// Class name search
			const FString ClassName = ClassIt->GetName();
			const int ClassNameLen = ClassName.Len();
			if (ClassNameLen < BestClassMatchLen && ClassNameLen >= SearchStringLen)
			{
				bool bContains = ClassName.Contains(SearchString);
				if (bContains)
				{
					FoundClass = *ClassIt;
					BestClassMatchLen = ClassNameLen;
				}
			}
		}

		// If it wasn't a class name, then perhaps it was the path to a specific asset
		if (!FoundClass)
		{
			FSoftObjectPath SoftObjectPath{ SearchString };
			if (UObject* ReferencedObject = SoftObjectPath.ResolveObject())
			{
				if (UPackage* ReferencedPackage = Cast<UPackage>(ReferencedObject))
				{
					ReferencedObject = ReferencedPackage->FindAssetInPackage();
				}

				if (UBlueprint* ReferencedBlueprint = Cast<UBlueprint>(ReferencedObject))
				{
					FoundClass = ReferencedBlueprint->GeneratedClass;
				}
				else if (ClassToFind* ReferencedGA = Cast<ClassToFind>(ReferencedObject))
				{
					FoundClass = ReferencedGA->GetClass();
				}
			}
		}

		return FoundClass;
	}
}

struct FScopedCanActivateAbilityLogGatherer : public FOutputDevice
{
#if NO_LOGGING
	FScopedCanActivateAbilityLogGatherer(const FNoLoggingCategory& InLogCategoryToCapture) {}
#else
	FScopedCanActivateAbilityLogGatherer(const FLogCategoryBase& InLogCategoryToCapture)
		: LogCategoryToCapture{ InLogCategoryToCapture.GetCategoryName() }
	{
		GLog->AddOutputDevice(this);
	}

	~FScopedCanActivateAbilityLogGatherer()
	{
		GLog->RemoveOutputDevice(this);
	}
#endif

	/** Structure for the Log Entries */
	struct FLogEntry
	{
		const FName Category;
		const ELogVerbosity::Type Verbosity;
		const FString Text;
	};

	TArray<FLogEntry>&& GetCapturedLogs()
	{
		FScopeLock _(&CriticalSection);
		return MoveTemp(Logs);
	}

protected:
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& InCategory) override
	{
		if (InCategory != LogCategoryToCapture)
		{
			return;
		}

		FScopeLock _(&CriticalSection);
		Logs.Emplace(FLogEntry{ InCategory, Verbosity, V });
	}

	FCriticalSection	CriticalSection;
	FName				LogCategoryToCapture;
	TArray<FLogEntry>	Logs;
};


UAbilitySystemCheatManagerExtension::UAbilitySystemCheatManagerExtension()
{
#if WITH_SERVER_CODE && UE_WITH_CHEAT_MANAGER
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		UCheatManager::RegisterForOnCheatManagerCreated(FOnCheatManagerCreated::FDelegate::CreateLambda(
			[](UCheatManager* CheatManager)
			{
				CheatManager->AddCheatManagerExtension(NewObject<ThisClass>(CheatManager));
			}));

	}
#endif
}


void UAbilitySystemCheatManagerExtension::AbilityGrant(const FString& AssetSearchString) const
{
	UAbilitySystemGlobals& AbilitySystemGlobals = UAbilitySystemGlobals::Get();

	APlayerController* PC = GetPlayerController();
	UAbilitySystemComponent* ASC = PC ? AbilitySystemGlobals.GetAbilitySystemComponentFromActor(PC->GetPawn()) : nullptr;
	if (!ASC)
	{
		UE_LOGF(LogConsoleResponse, Log, "%ls did not have AbilitySystemComponent", *GetNameSafe(PC));
		return;
	}

	if (AssetSearchString.IsEmpty())
	{
		AbilityListGranted();
		return;
	}

	// We couldn't find anything the user was searching for, so early out
	TSubclassOf<UGameplayAbility> ActivateAbilityClass = UE::AbilitySystem::Private::FuzzyFindClass<UGameplayAbility>(AssetSearchString);
	if (!ActivateAbilityClass)
	{
		UE_LOGF(LogConsoleResponse, Log, "Could not find a valid Gameplay Ability based on Search String '%ls'", *AssetSearchString);
		if (PC->GetWorld()->IsPlayInEditor())
		{
			UE_LOGF(LogConsoleResponse, Log, "Reminder: If it's a non-native class, make sure it's loaded in the Editor.");
		}

		return;
	}

	// Check if it's already granted
	if (const FGameplayAbilitySpec* ExistingSpec = ASC->FindAbilitySpecFromClass(ActivateAbilityClass))
	{
		UE_LOGF(LogConsoleResponse, Log, "Existing Ability Spec '%ls' on Player '%ls' (It is already granted).", *GetNameSafe(*ActivateAbilityClass), *GetNameSafe(PC));
		return;
	}

	// If we're not the authority, we need to send the command to the server because we can't grant locally.
	if (!ASC->IsOwnerActorAuthoritative())
	{
		const FString ServerCommand = FString::Printf(TEXT("%hs %s"), __func__, *AssetSearchString);
		PC->ServerExec(ServerCommand);

		UE_LOGF(LogConsoleResponse, Log, "Sent Command 'ServerExec %ls' from Player '%ls' to Server because only the authority can grant abilities.", *ServerCommand, *GetNameSafe(PC));
		return;
	}

	// It wasn't granted, let's grant it now.
	FGameplayAbilitySpec AbilitySpec{ ActivateAbilityClass };
	FGameplayAbilitySpecHandle SpecHandle = ASC->GiveAbility(AbilitySpec);
	if (SpecHandle.IsValid())
	{
		UE_LOGF(LogConsoleResponse, Log, "Successfully Granted '%ls' on Player '%ls'.", *GetNameSafe(ActivateAbilityClass.Get()), *GetNameSafe(PC));
	}
	else
	{
		UE_LOGF(LogConsoleResponse, Log, "Failed to Grant '%ls' on Player '%ls'.", *GetNameSafe(ActivateAbilityClass.Get()), *GetNameSafe(PC));
	}
}

void UAbilitySystemCheatManagerExtension::AbilityListGranted() const
{
	APlayerController* PC = GetPlayerController();
	UAbilitySystemComponent* ASC = PC ? UAbilitySystemGlobals::Get().GetAbilitySystemComponentFromActor(PC->GetPawn()) : nullptr;
	if (!ASC)
	{
		UE_LOGF(LogConsoleResponse, Log, "%ls did not have AbilitySystemComponent", *GetNameSafe(PC));
		return;
	}

	const UEnum* ExecutionEnumPtr = FindObject<UEnum>(nullptr, TEXT("/Script/GameplayAbilities.EGameplayAbilityNetExecutionPolicy"), EFindObjectFlags::ExactClass);
	check(ExecutionEnumPtr && TEXT("Couldn't locate EGameplayAbilityNetExecutionPolicy enum!"));

	const UEnum* SecurityEnumPtr = FindObject<UEnum>(nullptr, TEXT("/Script/GameplayAbilities.EGameplayAbilityNetSecurityPolicy"), EFindObjectFlags::ExactClass);
	check(SecurityEnumPtr && TEXT("Couldn't locate EGameplayAbilityNetSecurityPolicy enum!"));

	UE_LOGF(LogConsoleResponse, Log, "Granted abilities to %ls (ASC: '%ls'):", *PC->GetName(), *ASC->GetFullName());

	for (FGameplayAbilitySpec& Activatable : ASC->GetActivatableAbilities())
	{
		const TCHAR* ActiveText = Activatable.IsActive() ? TEXT("**ACTIVE**") : TEXT("");
		UE_LOGF(LogConsoleResponse, Log, "   %ls (%ls - %ls) %ls", *Activatable.Ability->GetName(), *ExecutionEnumPtr->GetDisplayNameTextByIndex(Activatable.Ability->GetNetExecutionPolicy()).ToString(), *SecurityEnumPtr->GetDisplayNameTextByIndex(Activatable.Ability->GetNetSecurityPolicy()).ToString(), ActiveText);
	}
}

void UAbilitySystemCheatManagerExtension::AbilityActivate(const FString& PartialName) const
{
	UAbilitySystemGlobals& AbilitySystemGlobals = UAbilitySystemGlobals::Get();

	APlayerController* PC = GetPlayerController();
	UAbilitySystemComponent* ASC = PC ? AbilitySystemGlobals.GetAbilitySystemComponentFromActor(PC->GetPawn()) : nullptr;
	if (!ASC)
	{
		UE_LOGF(LogConsoleResponse, Log, "%ls did not have AbilitySystemComponent", *GetNameSafe(PC));
		return;
	}

	if (PartialName.IsEmpty())
	{
		AbilityListGranted();
		return;
	}

	// Start by figuring out if we're trying to execute by GameplayTag.
	const FName SearchName{ TCHAR_TO_ANSI(*PartialName), FNAME_Find };
	if (SearchName != NAME_None)
	{
		const FGameplayTag GameplayTag = FGameplayTag::RequestGameplayTag(SearchName, false);
		if (GameplayTag.IsValid())
		{
			FGameplayTagContainer GameplayTagContainer{ GameplayTag };
			TArray<FGameplayAbilitySpec*> MatchingSpecs;
			constexpr bool bTagRequirementsMustMatch = false;
			ASC->GetActivatableGameplayAbilitySpecsByAllMatchingTags(GameplayTagContainer, MatchingSpecs, bTagRequirementsMustMatch);
			if (MatchingSpecs.Num() > 0)
			{
				FString MatchingAbilitiesString = FString::JoinBy(MatchingSpecs, TEXT(", "), [](FGameplayAbilitySpec* Item) { return Item->GetDebugString(); });

				bool bSuccess = ASC->TryActivateAbilitiesByTag(GameplayTagContainer);
				if (bSuccess)
				{
					UE_LOGF(LogConsoleResponse, Log, "Requested Tag '%ls' successfully executed one of: %ls.", *SearchName.ToString(), *MatchingAbilitiesString);
				}
				else
				{
					UE_LOGF(LogConsoleResponse, Log, "Requested Tag '%ls' was expected to execute one of: %ls. But it failed to do so due to tag requirements.", *SearchName.ToString(), *MatchingAbilitiesString);
				}
			}
			else
			{
				UE_LOGF(LogConsoleResponse, Log, "Requested Tag '%ls' matched no given Gameplay Abilities to %ls.", *SearchName.ToString(), *GetNameSafe(PC));
			}

			return;
		}
	}

	// We couldn't find anything the user was searching for, so early out
	TSubclassOf<UGameplayAbility> ActivateAbilityClass = UE::AbilitySystem::Private::FuzzyFindClass<UGameplayAbility>(PartialName);
	if (!ActivateAbilityClass)
	{
		UE_LOGF(LogConsoleResponse, Log, "Could not find a valid Gameplay Ability based on Search String '%ls'", *PartialName);
		return;
	}

	// Check if we already have this ability granted, because it affects how we can execute it.
	const FGameplayAbilitySpec* ExistingSpec = ASC->FindAbilitySpecFromClass(ActivateAbilityClass);

	// If we're not the authority, we should check if we need to send the command to the server
	if (!ASC->IsOwnerActorAuthoritative())
	{
		const UGameplayAbility* Ability = ActivateAbilityClass.GetDefaultObject();
		EGameplayAbilityNetExecutionPolicy::Type NetExecutionPolicy = Ability->GetNetExecutionPolicy();

		const bool bMustActivateLocally = (NetExecutionPolicy == EGameplayAbilityNetExecutionPolicy::LocalOnly || NetExecutionPolicy == EGameplayAbilityNetExecutionPolicy::LocalPredicted);
		if (!ExistingSpec && bMustActivateLocally)
		{
			UE_LOGF(LogConsoleResponse, Log, "Cannot Activate '%ls' on '%ls' as the ability is not granted.  Since its NetExecutionPolicy must activate locally, you must first grant it (use the AbilityGrant command).", *GetNameSafe(ActivateAbilityClass.Get()), *GetNameSafe(PC));
			return;
		}

		const bool bSendToServer = (NetExecutionPolicy >= EGameplayAbilityNetExecutionPolicy::ServerInitiated);
		if (bSendToServer)
		{
			const FString ServerCommand = FString::Printf(TEXT("%hs %s"), __func__, *PartialName);
			PC->ServerExec(ServerCommand);

			UE_LOGF(LogConsoleResponse, Log, "Sent Command '%ls' from Player '%ls' to Server (Reason: Net Execution Policy).", *ServerCommand, *GetNameSafe(PC));
			return;
		}
	}


	// We found what the user was searching for, so let's try to activate it.  First, let's assume the Ability was already granted.
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogAbilitySystem, ELogVerbosity::VeryVerbose);
	FScopedCanActivateAbilityLogGatherer LogGatherer{ LogAbilitySystem };
	FScopedCanActivateAbilityLogEnabler LogEnabler;
	if (FGameplayAbilitySpec* GrantedAbilitySpec = ASC->FindAbilitySpecFromClass(ActivateAbilityClass))
	{
		constexpr bool bAllowRemoteActivation = false;
		bool bSuccess = ASC->TryActivateAbility(GrantedAbilitySpec->Handle, bAllowRemoteActivation);
#if !NO_LOGGING
		if (bSuccess)
		{
			UE_LOGF(LogConsoleResponse, Log, "Successfully Activated previously Granted Ability '%ls' on Player '%ls'.", *GetNameSafe(ActivateAbilityClass.Get()), *GetNameSafe(PC));
		}
		else
		{
			UE_LOGF(LogConsoleResponse, Log, "Failed to Activate previously Granted Ability '%ls' on Player '%ls'. Logs:", *GetNameSafe(ActivateAbilityClass.Get()), *GetNameSafe(PC));

			TArray<FScopedCanActivateAbilityLogGatherer::FLogEntry> CapturedLogs = LogGatherer.GetCapturedLogs();
			for (const FScopedCanActivateAbilityLogGatherer::FLogEntry& LogEntry : CapturedLogs)
			{
				FMsg::LogV(__FILE__, __LINE__, LogConsoleResponse.GetCategoryName(), LogEntry.Verbosity, *LogEntry.Text, {});
			}
		}
#endif
		return;
	}

	// It wasn't granted, let's grant it, then activate it.
	FGameplayAbilitySpec AbilitySpec{ ActivateAbilityClass };
	FGameplayAbilitySpecHandle SpecHandle = ASC->GiveAbilityAndActivateOnce(AbilitySpec);
#if !NO_LOGGING
	if (SpecHandle.IsValid())
	{
		UE_LOGF(LogConsoleResponse, Log, "Successfully Granted, then Activated '%ls' on Player '%ls'.", *GetNameSafe(ActivateAbilityClass.Get()), *GetNameSafe(PC));
	}
	else
	{
		UE_LOGF(LogConsoleResponse, Log, "Failed to Grant & Activate '%ls' on Player '%ls'. Logs:", *GetNameSafe(ActivateAbilityClass.Get()), *GetNameSafe(PC));

		TArray<FScopedCanActivateAbilityLogGatherer::FLogEntry> CapturedLogs = LogGatherer.GetCapturedLogs();
		for (const FScopedCanActivateAbilityLogGatherer::FLogEntry& LogEntry : CapturedLogs)
		{
			FMsg::LogV(__FILE__, __LINE__, LogConsoleResponse.GetCategoryName(), LogEntry.Verbosity, *LogEntry.Text, {});
		}
	}
#endif
}

void UAbilitySystemCheatManagerExtension::AbilityCancel(const FString& PartialName) const
{
	APlayerController* PC = GetPlayerController();
	UAbilitySystemComponent* ASC = PC ? UAbilitySystemGlobals::Get().GetAbilitySystemComponentFromActor(PC->GetPawn()) : nullptr;
	if (!ASC)
	{
		UE_LOGF(LogConsoleResponse, Log, "PlayerController %ls did not have an AbilitySystemComponent", *GetNameSafe(PC));
		return;
	}

	FString SearchString = PartialName;
	SearchString.RemoveFromStart(TEXT("Default__"));

	bool bFound = false;
	for (FGameplayAbilitySpec& GASpec : ASC->GetActivatableAbilities())
	{
		if (GASpec.IsActive())
		{
			if (!GASpec.Ability || !GASpec.Ability->GetName().Contains(SearchString))
			{
				continue;
			}

			TArray<UGameplayAbility*> ActiveAbilities = GASpec.GetAbilityInstances();
			if (ActiveAbilities.Num() > 0)
			{
				for (UGameplayAbility* Instance : ActiveAbilities)
				{
					if (Instance)
					{
						UE_LOGF(LogConsoleResponse, Log, "%ls (%ls): Cancelling (instanced) %ls", *PC->GetName(), *ASC->GetName(), *Instance->GetName());
						Instance->CancelAbility(GASpec.Handle, ASC->AbilityActorInfo.Get(), Instance->GetCurrentActivationInfoRef(), true);
						bFound = true;
					}
				}
			}
			else
			{
				UE_LOGF(LogConsoleResponse, Log, "%ls (%ls): Cancelling (non-instanced) %ls", *PC->GetName(), *ASC->GetName(), *GASpec.Ability->GetName());
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				GASpec.Ability->CancelAbility(GASpec.Handle, ASC->AbilityActorInfo.Get(), GASpec.ActivationInfo, true);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				bFound = true;
			}
		}
	}

	if (!bFound)
	{
		UE_LOGF(LogConsoleResponse, Log, "Found no Active Gameplay Abilities on %ls (%ls) that matched '%ls'", *PC->GetName(), *ASC->GetName(), *SearchString);
	}
}

void UAbilitySystemCheatManagerExtension::EffectListActive() const
{
	APlayerController* PC = GetPlayerController();
	UAbilitySystemComponent* ASC = PC ? UAbilitySystemGlobals::Get().GetAbilitySystemComponentFromActor(PC->GetPawn()) : nullptr;
	if (!ASC)
	{
		UE_LOGF(LogConsoleResponse, Log, "PlayerController %ls did not have an AbilitySystemComponent", *GetNameSafe(PC));
		return;
	}

	UE_LOGF(LogConsoleResponse, Log, "Active Gameplay Effects on %ls (%ls):", *PC->GetName(), *ASC->GetName());

	const FActiveGameplayEffectsContainer& ActiveGEContainer = ASC->GetActiveGameplayEffects();
	for (const FActiveGameplayEffectHandle& ActiveGEHandle : ActiveGEContainer.GetAllActiveEffectHandles())
	{
		if (const FActiveGameplayEffect* ActiveGE = ASC->GetActiveGameplayEffect(ActiveGEHandle))
		{
			const UGameplayEffect* GESpecDef = ActiveGE->Spec.Def;
			const FString ActiveState = ActiveGE->bIsInhibited ? FString(TEXT("Inhibited")) : FString::Printf(TEXT("Active (Stack: %d)"), ActiveGE->Spec.GetStackCount());

			UE_LOGF(LogConsoleResponse, Log, " [%u] %ls: %ls", GetTypeHash(ActiveGEHandle), *GetNameSafe(GESpecDef), *ActiveState);
		}
	}
}

void UAbilitySystemCheatManagerExtension::EffectApply(const FString& PartialName, float EffectLevel /*= FGameplayEffectConstants::INVALID_LEVEL*/) const
{
	APlayerController* PC = GetPlayerController();
	UAbilitySystemComponent* ASC = PC ? UAbilitySystemGlobals::Get().GetAbilitySystemComponentFromActor(PC->GetPawn()) : nullptr;
	if (!ASC)
	{
		UE_LOGF(LogConsoleResponse, Log, "PlayerController %ls did not have an AbilitySystemComponent", *GetNameSafe(PC));
		return;
	}

	// We couldn't find anything the user was searching for, so early out
	TSubclassOf<UGameplayEffect> GameplayEffectClass = UE::AbilitySystem::Private::FuzzyFindClass<UGameplayEffect>(PartialName);
	if (!GameplayEffectClass)
	{
		UE_LOGF(LogConsoleResponse, Log, "Could not find a valid Gameplay Effect based on Search String '%ls'", *PartialName);
		return;
	}

	// Create a GameplayEffectSpec that executes the passed-in parameters
	FGameplayEffectContextHandle GEContextHandle = ASC->MakeEffectContext();
	APawn* Pawn = PC->GetPawn();
	GEContextHandle.AddInstigator(Pawn, Pawn);
	GEContextHandle.AddOrigin(Pawn->GetActorLocation());
	FGameplayEffectSpec GESpec{ GameplayEffectClass.GetDefaultObject(), GEContextHandle, EffectLevel };

	// We need to create a new valid prediction key (as if we just started activating an ability) so the GE will fire even if we are not the authority.
	FPredictionKey FakePredictionKey = (PC->GetLocalRole() == ENetRole::ROLE_Authority) ? FPredictionKey() : FPredictionKey::CreateNewPredictionKey(ASC);
	FActiveGameplayEffectHandle ActiveGEHandle = ASC->ApplyGameplayEffectSpecToSelf(GESpec, FakePredictionKey);
	if (ActiveGEHandle.IsValid())
	{
		UE_LOGF(LogConsoleResponse, Log, "Successfully Applied (Non-Instant) Gameplay Effect '%ls' on Player '%ls'.", *GameplayEffectClass->GetName(), *PC->GetName());
	}
	else if (ActiveGEHandle.WasSuccessfullyApplied())
	{
		UE_LOGF(LogConsoleResponse, Log, "Executed (Instant) Gameplay Effect '%ls' on Player '%ls'.", *GameplayEffectClass->GetName(), *PC->GetName());
	}
	else
	{
		UE_LOGF(LogConsoleResponse, Log, "Failed to Apply Gameplay Effect '%ls' on Player '%ls'.", *GameplayEffectClass->GetName(), *PC->GetName());
	}
}

void UAbilitySystemCheatManagerExtension::EffectRemove(const FString& NameOrHandle) const
{
	UAbilitySystemGlobals& AbilitySystemGlobals = UAbilitySystemGlobals::Get();

	APlayerController* PC = GetPlayerController();
	UAbilitySystemComponent* ASC = PC ? AbilitySystemGlobals.GetAbilitySystemComponentFromActor(PC->GetPawn()) : nullptr;
	if (!ASC)
	{
		UE_LOGF(LogConsoleResponse, Log, "PlayerController %ls did not have an AbilitySystemComponent", *GetNameSafe(PC));
		return;
	}

	uint32 SearchHandleValue = 0;
	if (NameOrHandle.IsNumeric())
	{
		int64 SearchHash64 = FCString::Atoi64(*NameOrHandle);
		if (SearchHash64 > 0)
		{
			SearchHandleValue = static_cast<uint32>(SearchHash64 & UINT_MAX);
		}
	}

	bool bAnyMatch = false;
	const FActiveGameplayEffectsContainer& ActiveGEContainer = ASC->GetActiveGameplayEffects();
	for (const FActiveGameplayEffectHandle& ActiveGEHandle : ActiveGEContainer.GetAllActiveEffectHandles())
	{
		if (const FActiveGameplayEffect* ActiveGE = ASC->GetActiveGameplayEffect(ActiveGEHandle))
		{
			const UGameplayEffect* GESpecDef = ActiveGE->Spec.Def;
			const uint32 ActiveGEHandleValue = GetTypeHash(ActiveGEHandle);

			const bool bMatch = (SearchHandleValue > 0) ? (ActiveGEHandleValue == SearchHandleValue) : GESpecDef->GetName().Contains(NameOrHandle);
			if (bMatch)
			{
				UE_LOGF(LogConsoleResponse, Log, "Removing Active GE Handle %u (%ls)", ActiveGEHandleValue, *GESpecDef->GetName());
				ASC->RemoveActiveGameplayEffect(ActiveGEHandle);
				bAnyMatch = true;
			}
		}
	}

	if (!bAnyMatch)
	{
		UE_LOGF(LogConsoleResponse, Log, "Found no Active Gameplay Effects on %ls (%ls) that matched '%ls'", *PC->GetName(), *ASC->GetName(), *NameOrHandle);
	}
}


/**
 * Handy wrapper function that executes a named exec function with the given Arguments, but through the CheatManager.  This allows the function to execute with context
 * if RPC'd to the server, thus routing to the proper PlayerController.  All of the functions support -server as an optional parameter, routing the function directly to the server.
 */
void ForwardToAbilitySystemCheatManagerExtension(const FName& ExecFunctionName, const TArray<FString>& Args, UWorld* World, FOutputDevice& OutputDevice)
{
	if (APlayerController* PlayerController = World->GetFirstPlayerController())
	{
		APawn* PCPawn = PlayerController->GetPawnOrSpectator();
		if (UCheatManager* CheatManager = PlayerController->CheatManager)
		{
			// All of these commands allow -server to force to be sent to the server
			TArray<FString> MutableArgs = Args;
			const bool bExecuteOnServer = MutableArgs.RemoveSingle(TEXT("-server")) > 0;

			const FString Cmd = FString::Printf(TEXT("%s %s"), *ExecFunctionName.ToString(), *FString::Join(MutableArgs, TEXT(" ")));
			if (bExecuteOnServer)
			{
				OutputDevice.Logf(TEXT("Executing 'ServerExec %s':"), *Cmd);
				PlayerController->ServerExec(Cmd);
			}
			else
			{
				const bool bSuccess = CheatManager->ProcessConsoleExec(*Cmd, OutputDevice, PCPawn);
				if (!bSuccess)
				{
					OutputDevice.Logf(TEXT("CheatManager %s on PlayerController %s did not process command '%s'"), *GetNameSafe(CheatManager), *GetNameSafe(PlayerController), *Cmd);
				}
			}
		}
		else
		{
			OutputDevice.Logf(TEXT("PlayerController %s did not have a CheatManager"), *GetNameSafe(PlayerController));
		}
	}
	else
	{
		OutputDevice.Logf(TEXT("Could not find Player Controller in %s"), *GetNameSafe(World));
	}
}

//
// AbilitySystem.Ability Debug Commands forward to the CheatManager
//

FAutoConsoleCommand DebugAbilitySystemAbilityGrantCommand(TEXT("AbilitySystem.Ability.Grant"), TEXT("<ClassName/AssetName>. Grants an Ability to the Player. Granting only happens on the Authority, so this command will be sent to the Server."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World, FOutputDevice& OutputDevice)
		{
			if (Args.Num() < 1)
			{
				OutputDevice.Logf(TEXT("Missing Arguments: <ClassName/AssetName>"));
				OutputDevice.Logf(TEXT(" ClassName: If supplied, the ClassName of the loaded GameplayAbility to Grant."));
				OutputDevice.Logf(TEXT(" AssetName: If supplied, the AssetName of the loaded GameplayAbility to Grant."));

				ForwardToAbilitySystemCheatManagerExtension(GET_FUNCTION_NAME_CHECKED(UAbilitySystemCheatManagerExtension, AbilityListGranted), Args, World, OutputDevice);
				return;
			}

			ForwardToAbilitySystemCheatManagerExtension(GET_FUNCTION_NAME_CHECKED(UAbilitySystemCheatManagerExtension, AbilityGrant), Args, World, OutputDevice);
		}), ECVF_Cheat);

FAutoConsoleCommand DebugAbilitySystemAbilityCancelCommand(TEXT("AbilitySystem.Ability.Cancel"), TEXT("[-Server] <PartialName>. Cancels (prematurely Ends) a currently executing Gameplay Ability"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World, FOutputDevice& OutputDevice)
		{
			if (Args.Num() < 1)
			{
				OutputDevice.Logf(TEXT("Missing Arguments: <PartialName>"));
				OutputDevice.Logf(TEXT(" -Server: Indicates this command should be sent to the server, rather than executed locally."));
				OutputDevice.Logf(TEXT(" PartialName: A substring of the currently active abilities to cancel. All matching abilities will be cancelled."));

				ForwardToAbilitySystemCheatManagerExtension(GET_FUNCTION_NAME_CHECKED(UAbilitySystemCheatManagerExtension, AbilityListGranted), Args, World, OutputDevice);
				return;
			}

			ForwardToAbilitySystemCheatManagerExtension(GET_FUNCTION_NAME_CHECKED(UAbilitySystemCheatManagerExtension, AbilityCancel), Args, World, OutputDevice);
		}), ECVF_Cheat);

FAutoConsoleCommand DebugAbilitySystemAbilityActivateCommand(TEXT("AbilitySystem.Ability.Activate"), TEXT("[-Server] <TagName/ClassName/AssetName>. Activate a Gameplay Ability.\nSubstring name matching works for Activation Tags (on already granted abilities), Asset Paths (on non-granted abilities), or Class Names on both."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World, FOutputDevice& OutputDevice)
		{
			if (Args.Num() < 1)
			{
				OutputDevice.Logf(TEXT("This command Activates an Ability.  If the Ability is not already granted, then it is activated on the server using GiveAbilityAndActivateOnce, which will remove it after completion."));
				OutputDevice.Logf(TEXT("Missing Arguments: <TagName or ClassName or AssetName>"));
				OutputDevice.Logf(TEXT(" -Server: Indicates this command should be sent to the server, rather than executed locally."));
				OutputDevice.Logf(TEXT(" TagName: If the argument is a GameplayTag, it will be activated using TryActivateAbilitiesByTag which requires the Gameplay Ability be previously granted."));
				OutputDevice.Logf(TEXT(" ClassName or AssetName: A substring of class/asset.  Only the best match that is currently loaded will be considered."));

				ForwardToAbilitySystemCheatManagerExtension(GET_FUNCTION_NAME_CHECKED(UAbilitySystemCheatManagerExtension, AbilityListGranted), Args, World, OutputDevice);
				return;
			}

			ForwardToAbilitySystemCheatManagerExtension(GET_FUNCTION_NAME_CHECKED(UAbilitySystemCheatManagerExtension, AbilityActivate), Args, World, OutputDevice);
		}), ECVF_Cheat);


FAutoConsoleCommand DebugAbilitySystemAbilityListGrantedCommand(TEXT("AbilitySystem.Ability.ListGranted"), TEXT("List the Gameplay Abilities that are granted to the local player"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World, FOutputDevice& OutputDevice)
		{
			ForwardToAbilitySystemCheatManagerExtension(GET_FUNCTION_NAME_CHECKED(UAbilitySystemCheatManagerExtension, AbilityListGranted), Args, World, OutputDevice);
		}), ECVF_Cheat);

//
// AbilitySystem.Effect Debug Commands forward to the CheatManager
//

FAutoConsoleCommand DebugAbilitySystemEffectListActiveCommand(TEXT("AbilitySystem.Effect.ListActive"), TEXT("[-Server] Lists all of the Gameplay Effects currently active on the Player"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World, FOutputDevice& OutputDevice)
		{
			ForwardToAbilitySystemCheatManagerExtension(GET_FUNCTION_NAME_CHECKED(UAbilitySystemCheatManagerExtension, EffectListActive), Args, World, OutputDevice);
		}), ECVF_Cheat);

FAutoConsoleCommand DebugAbilitySystemEffectRemoveCommand(TEXT("AbilitySystem.Effect.Remove"), TEXT("[-Server] <Handle/Name>. Remove a Gameplay Effect that is currently active on the Player"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World, FOutputDevice& OutputDevice)
		{
			if (Args.Num() < 1)
			{
				OutputDevice.Logf(TEXT("Missing Arguments: [-Server] <Handle or Name>"));
				OutputDevice.Logf(TEXT(" -Server: Indicates this command should be sent to the server, rather than executed locally."));
				OutputDevice.Logf(TEXT(" Handle: The numeric handle of the Gameplay Effect to remove (found through EffectListActive). This is not compatible with -server."));
				OutputDevice.Logf(TEXT(" Name: Partial name of the Gameplay Effect to remove."));
				return;
			}

			// Let's just make sure the user isn't about to do something that's going to be profoundly confusing
			const bool bExecuteOnServer = Args.Contains("-server");
			if (bExecuteOnServer)
			{
				const bool bArgIsHandle = Args.ContainsByPredicate([](const FString& Value) { return FCString::Atoi64(*Value) > 0; });
				if (bArgIsHandle)
				{
					UE_LOGF(LogConsoleResponse, Log, "Error: Search by Handle Value is not permitted with -server (because ActiveGE Handles are not replicated to the same handle value)");
					return;
				}
			}

			ForwardToAbilitySystemCheatManagerExtension(GET_FUNCTION_NAME_CHECKED(UAbilitySystemCheatManagerExtension, EffectRemove), Args, World, OutputDevice);
		}), ECVF_Cheat);

FAutoConsoleCommand DebugAbilitySystemEffectApply(TEXT("AbilitySystem.Effect.Apply"), TEXT("[-Server] <Class/AssetName> [Level]. Apply a Gameplay Effect on the Player.  Substring name matching works for Asset Tags, Asset Paths, or Class Names.  Use -Server to send to the server (default is apply locally)."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World, FOutputDevice& OutputDevice)
		{
			if (Args.Num() < 1)
			{
				OutputDevice.Logf(TEXT("Missing Arguments: [-Server] <MatchString> [Level]"));
				OutputDevice.Logf(TEXT(" -Server: Indicates this command should be sent to the server, rather than predicted locally."));
				OutputDevice.Logf(TEXT(" MatchString: Supply a substring of a class or asset name to match. Only loaded GameplayEffects are searched."));
				OutputDevice.Logf(TEXT(" Level: Optionally supply a Level such as 4.  Can be omitted (in which case the GE is level-less)."));
				return;
			}

			ForwardToAbilitySystemCheatManagerExtension(GET_FUNCTION_NAME_CHECKED(UAbilitySystemCheatManagerExtension, EffectApply), Args, World, OutputDevice);
		}), ECVF_Cheat);
