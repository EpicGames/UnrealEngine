// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/CheatManager.h"

#include "AbilitySystemCheatManagerExtension.generated.h"

/** Cheats related to GAS */
UCLASS(NotBlueprintable)
class UAbilitySystemCheatManagerExtension final : public UCheatManagerExtension
{
	GENERATED_BODY()

public:
	UAbilitySystemCheatManagerExtension();

	// Gameplay Abilities

	/** List all of the Abilities Granted to the owning Player */
	UFUNCTION(Exec)
	void AbilityListGranted() const;

	/** Grant a specified GameplayAbility to the owning Player */
	UFUNCTION(Exec)
	void AbilityGrant(const FString& AssetSearchString) const;

	/** Activate a previously granted GameplayAbility on the owning Player */
	UFUNCTION(Exec)
	void AbilityActivate(const FString& PartialName) const;

	/** Cancel a previously activated GameplayAbility on the owning Player */
	UFUNCTION(Exec)
	void AbilityCancel(const FString& PartialName) const;

	// Gameplay Effects

	/** List the Active Gameplay Effects on the owning Player */
	UFUNCTION(Exec)
	void EffectListActive() const;

	/** Apply a Gameplay Effect on the owning Player */
	UFUNCTION(Exec)
	void EffectApply(const FString& PartialName, float EffectLevel = -1.0f) const;

	/** Remove an Active Gameplay Effect on the owning Player */
	UFUNCTION(Exec)
	void EffectRemove(const FString& NameOrHandle) const;
};
