// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectMigrationContext.h"
#include "Engine/Canvas.h"
#include "DisplayDebugHelpers.h"
#include "Engine/Engine.h"
#include "GameFramework/HUD.h"
#include "AbilitySystemStats.h"
#include "AbilitySystemGlobals.h"
#include "GameplayCueManager.h"
#include "AbilitySystemPrivate.h"

#include "Net/UnrealNetwork.h"
#include "Engine/ActorChannel.h"
#include "GameplayEffectCustomApplicationRequirement.h"
#include "TimerManager.h"
#include "Net/Core/PushModel/PushModel.h"
#include "HAL/IConsoleManager.h"
#include "Iris/IrisConfig.h"
#include "Serialization/Archive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilitySystemComponent)

DEFINE_LOG_CATEGORY(LogAbilitySystemComponent);

DECLARE_CYCLE_STAT(TEXT("AbilitySystemComp ApplyGameplayEffectSpecToTarget"), STAT_AbilitySystemComp_ApplyGameplayEffectSpecToTarget, STATGROUP_AbilitySystem);
DECLARE_CYCLE_STAT(TEXT("AbilitySystemComp ApplyGameplayEffectSpecToSelf"), STAT_AbilitySystemComp_ApplyGameplayEffectSpecToSelf, STATGROUP_AbilitySystem);
DECLARE_CYCLE_STAT(TEXT("AbilitySystemComp InvokeGameplayCueEvent"), STAT_AbilitySystemComp_InvokeGameplayCueEvent, STATGROUP_AbilitySystem);
DECLARE_CYCLE_STAT(TEXT("AbilitySystemComp OnGameplayEffectAppliedToSelf"), STAT_AbilitySystemComp_OnGameplayEffectAppliedToSelf, STATGROUP_AbilitySystem);
DECLARE_CYCLE_STAT(TEXT("AbilitySystemComp OnGameplayEffectAppliedToTarget"), STAT_AbilitySystemComp_OnGameplayEffectAppliedToTarget, STATGROUP_AbilitySystem);
DECLARE_CYCLE_STAT(TEXT("AbilitySystemComp ExecuteGameplayEffect"), STAT_AbilitySystemComp_ExecuteGameplayEffect, STATGROUP_AbilitySystem);

#define LOCTEXT_NAMESPACE "AbilitySystemComponent"

/** Enable to log out all render state create, destroy and updatetransform events */
#define LOG_RENDER_STATE 0

static bool bUseReplicationConditionForActiveGameplayEffects = true;
static FAutoConsoleVariableRef CVarUseReplicationConditionForActiveGameplayEffects(TEXT("AbilitySystem.UseReplicationConditionForActiveGameplayEffects"), bUseReplicationConditionForActiveGameplayEffects, TEXT("Whether to be able to determine the replication condition for AbilitySystemComponent::ActiveGameplayEffects at runtime. Removes the need for executing custom logic in FActiveGameplayEffects::NetDeltaSerialize. Default is true."));

static bool bReplicateAbilitiesToSimulatedProxies = false;
static FAutoConsoleVariableRef CVarReplicateGameplayAbilitiesToOwnerOnly(TEXT("AbilitySystem.Fix.ReplicateAbilitiesToSimulatedProxies"), bReplicateAbilitiesToSimulatedProxies, TEXT("Default: False.  When false, Gameplay Abilities replicate to AutonomousProxies only, not SimulatedProxies (surmised to be a bug)"));

static bool bForceReplicationAlsoUpdatesReplicatedProxyInterface = true;
static FAutoConsoleVariableRef CVarForceReplicationAlsoUpdatesReplicatedProxyInterface(TEXT("AbilitySystem.Fix.ForceReplicationAlsoUpdatesReplicatedProxyInterface"), bForceReplicationAlsoUpdatesReplicatedProxyInterface, TEXT("Default: True.  When true, Calling ForceReplication() on the AbilitySystemComponent will also call ForceReplication() on the ReplicationProxy to ensure prompt replication of Cues and Tags"));

extern TAutoConsoleVariable<bool> CVarReplicateTagCountContainerWithIris;

UAbilitySystemComponent::UAbilitySystemComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, GameplayTagCountContainer()
{
	bWantsInitializeComponent = true;

	PrimaryComponentTick.bStartWithTickEnabled = true; // FIXME! Just temp until timer manager figured out
	bAutoActivate = true;	// Forcing AutoActivate since above we manually force tick enabled.
							// if we don't have this, UpdateShouldTick() fails to have any effect
							// because we'll be receiving ticks but bIsActive starts as false
	
	bCachedIsNetSimulated = false;
	UserAbilityActivationInhibited = false;

	GenericConfirmInputID = INDEX_NONE;
	GenericCancelInputID = INDEX_NONE;

	bSuppressGrantAbility = false;
	bSuppressGameplayCues = false;
	bPendingMontageRep = false;
	AffectedAnimInstanceTag = NAME_None; 

	AbilityScopeLockCount = 0;
	bAbilityPendingClearAll = false;
	AbilityLastActivatedTime = 0.f;

	ReplicationMode = EGameplayEffectReplicationMode::Full;

	ClientActivateAbilityFailedStartTime = 0.f;
	ClientActivateAbilityFailedCountRecent = 0;

	bDestroyActiveStateInitiated = false;
}

const UAttributeSet* UAbilitySystemComponent::InitStats(TSubclassOf<class UAttributeSet> Attributes, const UDataTable* DataTable)
{
	const UAttributeSet* AttributeObj = nullptr;
	if (Attributes)
	{
		AttributeObj = GetOrCreateAttributeSubobject(Attributes);
		if (AttributeObj && DataTable)
		{
			// This const_cast is OK - this is one of the few places we want to directly modify our AttributeSet properties rather
			// than go through a gameplay effect
			const_cast<UAttributeSet*>(AttributeObj)->InitFromMetaDataTable(DataTable);
		}
	}
	return AttributeObj;
}

void UAbilitySystemComponent::K2_InitStats(TSubclassOf<class UAttributeSet> Attributes, const UDataTable* DataTable)
{
	InitStats(Attributes, DataTable);
}

const UAttributeSet* UAbilitySystemComponent::GetOrCreateAttributeSubobject(TSubclassOf<UAttributeSet> AttributeClass)
{
	AActor* OwningActor = GetOwner();
	const UAttributeSet* MyAttributes = nullptr;
	if (OwningActor && AttributeClass)
	{
		MyAttributes = GetAttributeSubobject(AttributeClass);
		if (!MyAttributes)
		{
			UAttributeSet* Attributes = NewObject<UAttributeSet>(OwningActor, AttributeClass);
			AddSpawnedAttribute(Attributes);
			MyAttributes = Attributes;
		}
	}

	return MyAttributes;
}

const UAttributeSet* UAbilitySystemComponent::GetAttributeSubobjectChecked(const TSubclassOf<UAttributeSet> AttributeClass) const
{
	const UAttributeSet* Set = GetAttributeSubobject(AttributeClass);
	check(Set);
	return Set;
}

const UAttributeSet* UAbilitySystemComponent::GetAttributeSubobject(const TSubclassOf<UAttributeSet> AttributeClass) const
{
	for (const UAttributeSet* Set : GetSpawnedAttributes())
	{
		if (Set && Set->IsA(AttributeClass))
		{
			return Set;
		}
	}
	return nullptr;
}

bool UAbilitySystemComponent::HasAttributeSetForAttribute(FGameplayAttribute Attribute) const
{
	return (Attribute.IsValid() && (Attribute.IsSystemAttribute() || GetAttributeSubobject(Attribute.GetAttributeSetClass()) != nullptr));
}

void UAbilitySystemComponent::GetAllAttributes(TArray<FGameplayAttribute>& OutAttributes)
{
	for (const UAttributeSet* Set : GetSpawnedAttributes())
	{
		if (!Set)
		{
			continue;
		}

		UAttributeSet::GetAttributesFromSetClass(Set->GetClass(), OutAttributes);
	}
}

const UAttributeSet* UAbilitySystemComponent::GetAttributeSet(TSubclassOf<UAttributeSet> AttributeSetClass) const
{
	// get the attribute set
	const UAttributeSet* AttributeSet = GetAttributeSubobject(AttributeSetClass);

	// return the pointer
	return AttributeSet;
}

float UAbilitySystemComponent::GetGameplayAttributeValue(FGameplayAttribute Attribute, OUT bool& bFound) const
{
	// validate the attribute
	if (Attribute.IsValid())
	{
		// get the associated AttributeSet
		const UAttributeSet* InternalAttributeSet = GetAttributeSubobject(Attribute.GetAttributeSetClass());

		if (InternalAttributeSet)
		{
			// NOTE: this is currently not taking predicted gameplay effect modifiers into consideration, so the value may not be accurate on client
			bFound = true;
			return Attribute.GetNumericValue(InternalAttributeSet);
		}
	}

	// the attribute was not found
	bFound = false;
	return 0.0f;
}

void UAbilitySystemComponent::OnRegister()
{
	Super::OnRegister();

	/** Allocate an AbilityActorInfo. Note: this goes through a global function and is a SharedPtr so projects can make their own AbilityActorInfo */
	if (!AbilityActorInfo.IsValid())
	{
		AbilityActorInfo = TSharedPtr<FGameplayAbilityActorInfo>(UAbilitySystemGlobals::Get().AllocAbilityActorInfo());
	}

	// Cached off netrole to avoid constant checking on owning actor
	CacheIsNetSimulated();

	// Init starting data
	for (int32 i=0; i < DefaultStartingData.Num(); ++i)
	{
		if (DefaultStartingData[i].Attributes && DefaultStartingData[i].DefaultStartingTable)
		{
			UAttributeSet* Attributes = const_cast<UAttributeSet*>(GetOrCreateAttributeSubobject(DefaultStartingData[i].Attributes));
			Attributes->InitFromMetaDataTable(DefaultStartingData[i].DefaultStartingTable);
		}
	}

	ActiveGameplayEffects.RegisterWithOwner(this);
	ActiveGameplayEffects.SetIsUsingReplicationCondition(bUseReplicationConditionForActiveGameplayEffects);
	ActivatableAbilities.RegisterWithOwner(this);
	ActiveGameplayCues.bMinimalReplication = false;
	ActiveGameplayCues.SetOwner(this);	
	MinimalReplicationGameplayCues.bMinimalReplication = true;
	MinimalReplicationGameplayCues.SetOwner(this);

	UpdateActiveGameplayEffectsReplicationCondition();
	UpdateMinimalReplicationGameplayCuesCondition();

	GameplayTagCountContainer.SetOwner(this);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MinimalReplicationTags.Owner = this;
	ReplicatedLooseTags.Owner = this;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Ensure bDestroyActiveStateInitiated is clear in case component is re-entering play 
	bDestroyActiveStateInitiated = false;
}

void UAbilitySystemComponent::OnUnregister()
{
	Super::OnUnregister();

#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (const FUObjectMigrationContext* MigrationContext = FUObjectMigrationContext::GetCurrentMigrationContext())
	{
		// Do not DestroyActiveState when migrating-out.  There's too many side effects which we should instead handle in PostMigrate.
		if (MigrationContext->MigrationSide == EObjectMigrationSide::Send)
		{
			return;
		}
	}
#endif

	DestroyActiveState();
}

void UAbilitySystemComponent::BeginPlay()
{
	Super::BeginPlay();

	// Cache net role here as well since for map-placed actors on clients, the Role may not be set correctly yet in OnRegister.
	CacheIsNetSimulated();
}

void UAbilitySystemComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsMigratingRemoteObjects())
	{
		bool bHasAbilityActorInfo = AbilityActorInfo.IsValid();
		Ar << bHasAbilityActorInfo;
		if (bHasAbilityActorInfo)
		{
			// Can happen when loading (the sending side had an ability info we didn't yet allocate)
			if (!AbilityActorInfo.IsValid())
			{
				AbilityActorInfo = TSharedPtr<FGameplayAbilityActorInfo>( UAbilitySystemGlobals::Get().AllocAbilityActorInfo() );
			}

			// TODO: This just uses the default serializer but does not account for polymorphic structs
			ensureMsgf(sizeof(*AbilityActorInfo.Get()) == sizeof(FGameplayAbilityActorInfo), TEXT("We do not handle polymorphic AbilityActorInfo properly"));
			FGameplayAbilityActorInfo DefaultStruct;
			FGameplayAbilityActorInfo::StaticStruct()->SerializeItem(Ar, AbilityActorInfo.Get(), &DefaultStruct);
		}

		if (Ar.IsSaving())
		{
			auto SaveTagToGASpecMap = [&Ar](TMap<FGameplayTag, TArray<FGameplayAbilitySpecHandle>>& Map)
			{
				int32 NumEntries = Map.Num();
				Ar << NumEntries;
				for (TPair<FGameplayTag, TArray<FGameplayAbilitySpecHandle>>& Pair : Map)
				{
					FGameplayTag& GameplayTag = Pair.Key;
					TArray<FGameplayAbilitySpecHandle>& GASpecs = Pair.Value;

					Ar << GameplayTag;
					Ar << GASpecs;
				}
			};

			SaveTagToGASpecMap(GameplayEventTriggeredAbilities);
			SaveTagToGASpecMap(OwnedTagTriggeredAbilities);
		}
		else if (Ar.IsLoading())
		{
			auto LoadTagToGASpecMap = [&Ar](TMap<FGameplayTag, TArray<FGameplayAbilitySpecHandle>>& Map)
			{
				int32 NumEntries = Map.Num();
				Ar << NumEntries;

				Map.Reset();

				FGameplayTag GameplayTag;
				TArray<FGameplayAbilitySpecHandle> GASpecs;

				for (int Idx = 0; Idx < NumEntries; ++Idx)
				{
					Ar << GameplayTag;
					Ar << GASpecs;
					Map.Emplace(GameplayTag, GASpecs);
				}
			};

			LoadTagToGASpecMap(GameplayEventTriggeredAbilities);
			LoadTagToGASpecMap(OwnedTagTriggeredAbilities);
		}
	}
}

void UAbilitySystemComponent::ReadyForReplication()
{
	Super::ReadyForReplication();

	// Register the spawned attributes to the replicated sub object list.
	if (IsUsingRegisteredSubObjectList())
	{
		for (UGameplayAbility* ReplicatedAbility : AllReplicatedInstancedAbilities)
		{
			if (ReplicatedAbility)
			{
				const ELifetimeCondition LifetimeCondition = bReplicateAbilitiesToSimulatedProxies ? COND_None : COND_ReplayOrOwner;
				AddReplicatedSubObject(ReplicatedAbility, LifetimeCondition);
			}
		}

		for (UAttributeSet* ReplicatedAttribute : SpawnedAttributes)
		{
			if (ReplicatedAttribute)
			{
				AddReplicatedSubObject(ReplicatedAttribute);
			}
		}
	}
}

void UAbilitySystemComponent::CacheIsNetSimulated()
{
	bCachedIsNetSimulated = IsNetSimulating();
	ActiveGameplayEffects.OwnerIsNetAuthority = IsOwnerActorAuthoritative();
	UpdateActiveGameplayEffectsReplicationCondition();
}

FActiveGameplayEffectHandle UAbilitySystemComponent::SetActiveGameplayEffectInhibit(FActiveGameplayEffectHandle&& ActiveGEHandle, bool bInhibit, bool bInvokePredictedEffects)
{
	FActiveGameplayEffect* ActiveGE = ActiveGameplayEffects.GetActiveGameplayEffect(ActiveGEHandle);
	if (!ActiveGE)
	{
		ABILITY_LOG(Error, TEXT("%s received bad Active GameplayEffect Handle: %s"), ANSI_TO_TCHAR(__func__), *ActiveGEHandle.ToString());
		return FActiveGameplayEffectHandle();
	}

	if (ActiveGE->bIsInhibited != bInhibit)
	{
		ActiveGE->bIsInhibited = bInhibit;

		// It's possible the adding or removing of the tags can invalidate the ActiveGE.  As such,
		// let's make sure we hold on to that memory until this function is done.
		FScopedActiveGameplayEffectLock ScopeLockActiveGameplayEffects(ActiveGameplayEffects);

		// All OnDirty callbacks must be inhibited until we update this entire GameplayEffect.
		FScopedAggregatorOnDirtyBatch	AggregatorOnDirtyBatcher;
		if (bInhibit)
		{
			// Remove our ActiveGameplayEffects modifiers with our Attribute Aggregators
			ActiveGameplayEffects.RemoveActiveGameplayEffectGrantedTagsAndModifiers(*ActiveGE, bInvokePredictedEffects);
		}
		else
		{
			ActiveGameplayEffects.AddActiveGameplayEffectGrantedTagsAndModifiers(*ActiveGE, bInvokePredictedEffects);
		}

		// The act of executing anything on the ActiveGE can invalidate it.  So we need to recheck if we can continue to execute the callbacks.
		if (!ActiveGE->IsPendingRemove)
		{
			ActiveGE->EventSet.OnInhibitionChanged.Broadcast(ActiveGEHandle, ActiveGE->bIsInhibited);
		}

		// We lost that it was active somewhere along the way, let the caller know
		if (ActiveGE->IsPendingRemove)
		{
			return FActiveGameplayEffectHandle();
		}
	}

	// Normal case is the passed-in ActiveGEHandle is still active and thus can continue execution
	return MoveTemp(ActiveGEHandle);
}



const FActiveGameplayEffect* UAbilitySystemComponent::GetActiveGameplayEffect(const FActiveGameplayEffectHandle Handle) const
{
	return ActiveGameplayEffects.GetActiveGameplayEffect(Handle);
}

const FActiveGameplayEffectsContainer& UAbilitySystemComponent::GetActiveGameplayEffects() const
{
	return ActiveGameplayEffects;
}

const UGameplayEffect* UAbilitySystemComponent::GetGameplayEffectCDO(const FActiveGameplayEffectHandle Handle) const
{
	// get the active gameplay effect struct
	const FActiveGameplayEffect* ActiveGE = GetActiveGameplayEffect(Handle);

	if (ActiveGE)
	{
		// return the spec's CDO
		return ActiveGE->Spec.Def;
	}

	// active effect not found
	return nullptr;
}

const FGameplayTagContainer* UAbilitySystemComponent::GetGameplayEffectSourceTagsFromHandle(FActiveGameplayEffectHandle Handle) const
{
	return ActiveGameplayEffects.GetGameplayEffectSourceTagsFromHandle(Handle);
}

const FGameplayTagContainer* UAbilitySystemComponent::GetGameplayEffectTargetTagsFromHandle(FActiveGameplayEffectHandle Handle) const
{
	return ActiveGameplayEffects.GetGameplayEffectTargetTagsFromHandle(Handle);
}

void UAbilitySystemComponent::CaptureAttributeForGameplayEffect(OUT FGameplayEffectAttributeCaptureSpec& OutCaptureSpec)
{
	// Verify the capture is happening on an attribute the component actually has a set for; if not, can't capture the value
	const FGameplayAttribute& AttributeToCapture = OutCaptureSpec.BackingDefinition.AttributeToCapture;
	if (AttributeToCapture.IsValid() && (AttributeToCapture.IsSystemAttribute() || GetAttributeSubobject(AttributeToCapture.GetAttributeSetClass())))
	{
		ActiveGameplayEffects.CaptureAttributeForGameplayEffect(OutCaptureSpec);
	}
}

bool UAbilitySystemComponent::HasNetworkAuthorityToApplyGameplayEffect(FPredictionKey PredictionKey) const
{
	return (IsOwnerActorAuthoritative() || PredictionKey.IsValidForMorePrediction());
}

void UAbilitySystemComponent::SetNumericAttributeBase(const FGameplayAttribute &Attribute, float NewFloatValue)
{
	// Go through our active gameplay effects container so that aggregation/mods are handled properly.
	ActiveGameplayEffects.SetAttributeBaseValue(Attribute, NewFloatValue);
}

float UAbilitySystemComponent::GetNumericAttributeBase(const FGameplayAttribute &Attribute) const
{
	if (Attribute.IsSystemAttribute())
	{
		return 0.f;
	}

	return ActiveGameplayEffects.GetAttributeBaseValue(Attribute);
}

void UAbilitySystemComponent::SetNumericAttribute_Internal(const FGameplayAttribute &Attribute, float& NewFloatValue)
{
	// Set the attribute directly: update the FProperty on the attribute set.
	const UAttributeSet* AttributeSet = GetAttributeSubobjectChecked(Attribute.GetAttributeSetClass());
	Attribute.SetNumericValueChecked(NewFloatValue, const_cast<UAttributeSet*>(AttributeSet));
}

float UAbilitySystemComponent::GetNumericAttribute(const FGameplayAttribute &Attribute) const
{
	if (Attribute.IsSystemAttribute())
	{
		return 0.f;
	}

	const UAttributeSet* const AttributeSetOrNull = GetAttributeSubobject(Attribute.GetAttributeSetClass());
	if (AttributeSetOrNull == nullptr)
	{
		return 0.f;
	}

	return Attribute.GetNumericValue(AttributeSetOrNull);
}

float UAbilitySystemComponent::GetNumericAttributeChecked(const FGameplayAttribute &Attribute) const
{
	if(Attribute.IsSystemAttribute())
	{
		return 0.f;
	}

	const UAttributeSet* AttributeSet = GetAttributeSubobjectChecked(Attribute.GetAttributeSetClass());
	return Attribute.GetNumericValueChecked(AttributeSet);
}

void UAbilitySystemComponent::ApplyModToAttribute(const FGameplayAttribute &Attribute, TEnumAsByte<EGameplayModOp::Type> ModifierOp, float ModifierMagnitude)
{
	// We can only apply loose mods on the authority. If we ever need to predict these, they would need to be turned into GEs and be given a prediction key so that
	// they can be rolled back.
	if (IsOwnerActorAuthoritative())
	{
		ActiveGameplayEffects.ApplyModToAttribute(Attribute, ModifierOp, ModifierMagnitude);
	}
}

void UAbilitySystemComponent::ApplyModToAttributeUnsafe(const FGameplayAttribute &Attribute, TEnumAsByte<EGameplayModOp::Type> ModifierOp, float ModifierMagnitude)
{
	ActiveGameplayEffects.ApplyModToAttribute(Attribute, ModifierOp, ModifierMagnitude);
}

FGameplayEffectSpecHandle UAbilitySystemComponent::MakeOutgoingSpec(TSubclassOf<UGameplayEffect> GameplayEffectClass, float Level, FGameplayEffectContextHandle Context) const
{
	SCOPE_CYCLE_COUNTER(STAT_GetOutgoingSpec);
	if (Context.IsValid() == false)
	{
		Context = MakeEffectContext();
	}

	if (GameplayEffectClass)
	{
		UGameplayEffect* GameplayEffect = GameplayEffectClass->GetDefaultObject<UGameplayEffect>();

		FGameplayEffectSpec* NewSpec = new FGameplayEffectSpec(GameplayEffect, Context, Level);
		return FGameplayEffectSpecHandle(NewSpec);
	}

	return FGameplayEffectSpecHandle(nullptr);
}

FGameplayEffectContextHandle UAbilitySystemComponent::MakeEffectContext() const
{
	FGameplayEffectContextHandle Context = FGameplayEffectContextHandle(UAbilitySystemGlobals::Get().AllocGameplayEffectContext());
	
	// By default use the owner and avatar as the instigator and causer
	if (ensureMsgf(AbilityActorInfo.IsValid(), TEXT("Unable to make effect context because AbilityActorInfo is not valid.")))
	{
		Context.AddInstigator(AbilityActorInfo->OwnerActor.Get(), AbilityActorInfo->AvatarActor.Get());
	}
	
	return Context;
}

int32 UAbilitySystemComponent::GetGameplayEffectCount(TSubclassOf<UGameplayEffect> SourceGameplayEffect, UAbilitySystemComponent* OptionalInstigatorFilterComponent, bool bEnforceOnGoingCheck) const
{
	int32 Count = 0;

	if (SourceGameplayEffect)
	{
		FGameplayEffectQuery Query;
		Query.CustomMatchDelegate.BindLambda([&](const FActiveGameplayEffect& CurEffect)
		{
			bool bMatches = false;

			// First check at matching: backing GE class must be the exact same
			if (CurEffect.Spec.Def && SourceGameplayEffect == CurEffect.Spec.Def->GetClass())
			{
				// If an instigator is specified, matching is dependent upon it
				if (OptionalInstigatorFilterComponent)
				{
					bMatches = (OptionalInstigatorFilterComponent == CurEffect.Spec.GetEffectContext().GetInstigatorAbilitySystemComponent());
				}
				else
				{
					bMatches = true;
				}
			}

			return bMatches;
		});

		Count = ActiveGameplayEffects.GetActiveEffectCount(Query, bEnforceOnGoingCheck);
	}

	return Count;
}

int32 UAbilitySystemComponent::GetGameplayEffectCount_IfLoaded(TSoftClassPtr<UGameplayEffect> SoftSourceGameplayEffect, UAbilitySystemComponent* OptionalInstigatorFilterComponent, bool bEnforceOnGoingCheck) const
{
	TSubclassOf<UGameplayEffect> SourceGameplayEffect = SoftSourceGameplayEffect.Get();

	//if the gameplay effect is not loaded, then there must be none active
	if (!SourceGameplayEffect)
	{
		return 0;
	}

	return GetGameplayEffectCount(SourceGameplayEffect, OptionalInstigatorFilterComponent, bEnforceOnGoingCheck);
}

int32 UAbilitySystemComponent::GetAggregatedStackCount(const FGameplayEffectQuery& Query) const
{
	return ActiveGameplayEffects.GetActiveEffectCount(Query);
}

FActiveGameplayEffectHandle UAbilitySystemComponent::BP_ApplyGameplayEffectToTarget(TSubclassOf<UGameplayEffect> GameplayEffectClass, UAbilitySystemComponent* Target, float Level, FGameplayEffectContextHandle Context)
{
	if (Target == nullptr)
	{
		ABILITY_LOG(Log, TEXT("UAbilitySystemComponent::BP_ApplyGameplayEffectToTarget called with null Target. %s. Context: %s"), *GetFullName(), *Context.ToString());
		return FActiveGameplayEffectHandle();
	}

	if (GameplayEffectClass == nullptr)
	{
		ABILITY_LOG(Error, TEXT("UAbilitySystemComponent::BP_ApplyGameplayEffectToTarget called with null GameplayEffectClass. %s. Context: %s"), *GetFullName(), *Context.ToString());
		return FActiveGameplayEffectHandle();
	}

	UGameplayEffect* GameplayEffect = GameplayEffectClass->GetDefaultObject<UGameplayEffect>();
	return ApplyGameplayEffectToTarget(GameplayEffect, Target, Level, Context);	
}

/** This is a helper function used in automated testing, I'm not sure how useful it will be to gamecode or blueprints */
FActiveGameplayEffectHandle UAbilitySystemComponent::ApplyGameplayEffectToTarget(UGameplayEffect *GameplayEffect, UAbilitySystemComponent *Target, float Level, FGameplayEffectContextHandle Context, FPredictionKey PredictionKey)
{
	check(GameplayEffect);
	if (HasNetworkAuthorityToApplyGameplayEffect(PredictionKey))
	{
		if (!Context.IsValid())
		{
			Context = MakeEffectContext();
		}

		FGameplayEffectSpec	Spec(GameplayEffect, Context, Level);
		return ApplyGameplayEffectSpecToTarget(Spec, Target, PredictionKey);
	}

	return FActiveGameplayEffectHandle();
}

/** Helper function since we can't have default/optional values for FModifierQualifier in K2 function */
FActiveGameplayEffectHandle UAbilitySystemComponent::BP_ApplyGameplayEffectToSelf(TSubclassOf<UGameplayEffect> GameplayEffectClass, float Level, FGameplayEffectContextHandle EffectContext)
{
	if ( GameplayEffectClass )
	{
		if (!EffectContext.IsValid())
		{
			EffectContext = MakeEffectContext();
		}

		UGameplayEffect* GameplayEffect = GameplayEffectClass->GetDefaultObject<UGameplayEffect>();
		return ApplyGameplayEffectToSelf(GameplayEffect, Level, EffectContext);
	}

	return FActiveGameplayEffectHandle();
}

/** This is a helper function - it seems like this will be useful as a blueprint interface at the least, but Level parameter may need to be expanded */
FActiveGameplayEffectHandle UAbilitySystemComponent::ApplyGameplayEffectToSelf(const UGameplayEffect *GameplayEffect, float Level, const FGameplayEffectContextHandle& EffectContext, FPredictionKey PredictionKey)
{
	if (GameplayEffect == nullptr)
	{
		ABILITY_LOG(Error, TEXT("UAbilitySystemComponent::ApplyGameplayEffectToSelf called by Instigator %s with a null GameplayEffect."), *EffectContext.ToString());
		return FActiveGameplayEffectHandle();
	}

	if (HasNetworkAuthorityToApplyGameplayEffect(PredictionKey))
	{
		FGameplayEffectSpec	Spec(GameplayEffect, EffectContext, Level);
		return ApplyGameplayEffectSpecToSelf(Spec, PredictionKey);
	}

	return FActiveGameplayEffectHandle();
}

FActiveGameplayEffectEvents* UAbilitySystemComponent::GetActiveEffectEventSet(FActiveGameplayEffectHandle Handle)
{
	FActiveGameplayEffect* ActiveEffect = ActiveGameplayEffects.GetActiveGameplayEffect(Handle);
	return ActiveEffect ? &ActiveEffect->EventSet : nullptr;
}

FOnActiveGameplayEffectRemoved_Info* UAbilitySystemComponent::OnGameplayEffectRemoved_InfoDelegate(FActiveGameplayEffectHandle Handle)
{
	FActiveGameplayEffect* ActiveEffect = ActiveGameplayEffects.GetActiveGameplayEffect(Handle);
	return ActiveEffect ? &ActiveEffect->EventSet.OnEffectRemoved : nullptr;
}

FOnActiveGameplayEffectStackChange* UAbilitySystemComponent::OnGameplayEffectStackChangeDelegate(FActiveGameplayEffectHandle Handle)
{
	FActiveGameplayEffect* ActiveEffect = ActiveGameplayEffects.GetActiveGameplayEffect(Handle);
	return ActiveEffect ? &ActiveEffect->EventSet.OnStackChanged : nullptr;
}

FOnActiveGameplayEffectTimeChange* UAbilitySystemComponent::OnGameplayEffectTimeChangeDelegate(FActiveGameplayEffectHandle Handle)
{
	FActiveGameplayEffect* ActiveEffect = ActiveGameplayEffects.GetActiveGameplayEffect(Handle);
	return ActiveEffect ? &ActiveEffect->EventSet.OnTimeChanged : nullptr;
}

FOnActiveGameplayEffectInhibitionChanged* UAbilitySystemComponent::OnGameplayEffectInhibitionChangedDelegate(FActiveGameplayEffectHandle Handle)
{
	FActiveGameplayEffect* ActiveEffect = ActiveGameplayEffects.GetActiveGameplayEffect(Handle);
	return ActiveEffect ? &ActiveEffect->EventSet.OnInhibitionChanged : nullptr;
}

FOnGivenActiveGameplayEffectRemoved& UAbilitySystemComponent::OnAnyGameplayEffectRemovedDelegate()
{
	return ActiveGameplayEffects.OnActiveGameplayEffectRemovedDelegate;
}

void UAbilitySystemComponent::UpdateTagMapSingle_Internal(const FGameplayTag& Tag, const int32 CountDelta, EGameplayTagReplicationState TagRepState)
{
	if (UE::Net::ShouldUseIrisReplication() && !CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread())
	{
		// Iris replication doesn't work the same so we're falling back to using the Minimal and replicated loose tags until it's fixed.
		const UWorld* World = GetWorld();
		ENetMode NetMode = World ? World->GetNetMode() : NM_Standalone;
		if (NetMode == NM_DedicatedServer || NetMode == NM_ListenServer)
		{
			if (TagRepState > EGameplayTagReplicationState::TagOnly)
			{
				int32& TagCount = GetReplicatedLooseTags_Mutable().TagMap.FindOrAdd(Tag);
				GetReplicatedLooseTags_Mutable().SetTagCount(Tag, FMath::Max(0, TagCount + CountDelta));
			}
			else if (TagRepState > EGameplayTagReplicationState::None)
			{
				int32& TagCount = GetMinimalReplicationTags_Mutable().TagMap.FindOrAdd(Tag);
				GetMinimalReplicationTags_Mutable().SetTagCount(Tag, FMath::Max(0, TagCount + CountDelta));
			}
		}
	}
	
	// For removal, reorder calls so that FillParentTags is only called once
	if (CountDelta > 0)
	{
		if (GameplayTagCountContainer.UpdateTagCount(Tag, CountDelta, TagRepState))
		{
			OnTagUpdated(Tag, true);
		}
	}
	else if (CountDelta < 0)
	{
		TArray<FDeferredTagChangeDelegate> DeferredTagChangeDelegates;
		if (GameplayTagCountContainer.UpdateTagCount_DeferredParentRemoval(Tag, CountDelta, DeferredTagChangeDelegates))
		{
			GameplayTagCountContainer.FillParentTags();
			OnTagUpdated(Tag, false);
			if (UE::Net::ShouldUseIrisReplication() && !CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread())
			{
				// Iris replication doesn't work the same so we're falling back to using the Minimal and replicated loose tags until it's fixed.
				const UWorld* World = GetWorld();
				ENetMode NetMode = World ? World->GetNetMode() : NM_Standalone;
				if (NetMode == NM_DedicatedServer || NetMode == NM_ListenServer)
				{
					FMinimalReplicationTagCountMap& MTagMap = GetMinimalReplicationTags_Mutable();
					if (MTagMap.TagMap.Contains(Tag))
					{
						MTagMap.RemoveTag(Tag);
					}

					FMinimalReplicationTagCountMap& RLTagMap = GetReplicatedLooseTags_Mutable();
					if (RLTagMap.TagMap.Contains(Tag))
					{
						RLTagMap.RemoveTag(Tag);
					}
				}
			}

			for (FDeferredTagChangeDelegate& Delegate : DeferredTagChangeDelegates)
			{
				Delegate.Execute();
			}
		}
	}
	
	if (UE::Net::ShouldUseIrisReplication() && CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread())
	{
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, GameplayTagCountContainer, this);
	}
}

void UAbilitySystemComponent::UpdateTagMap_Internal(const FGameplayTagContainer& Container, const int32 CountDelta, EGameplayTagReplicationState TagRepState)
{
	if (UE::Net::ShouldUseIrisReplication() && !CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread())
	{
		// Iris replication doesn't work the same so we're falling back to using the Minimal and replicated loose tags until it's fixed.
		const UWorld* World = GetWorld();
		ENetMode NetMode = World ? World->GetNetMode() : NM_Standalone;
		if (NetMode == NM_DedicatedServer || NetMode == NM_ListenServer)
		{
			if (TagRepState > EGameplayTagReplicationState::TagOnly)
			{
				for (TArray<FGameplayTag>::TConstIterator TagIt = Container.CreateConstIterator(); TagIt; ++TagIt)
				{
					const FGameplayTag& Tag = *TagIt;
					int32& TagCount = GetReplicatedLooseTags_Mutable().TagMap.FindOrAdd(Tag);
					GetReplicatedLooseTags_Mutable().SetTagCount(Tag, FMath::Max(0, TagCount + CountDelta));
				}
			}
			else if (TagRepState > EGameplayTagReplicationState::None)
			{
				for (TArray<FGameplayTag>::TConstIterator TagIt = Container.CreateConstIterator(); TagIt; ++TagIt)
				{
					const FGameplayTag& Tag = *TagIt;
					int32& TagCount = GetMinimalReplicationTags_Mutable().TagMap.FindOrAdd(Tag);
					GetMinimalReplicationTags_Mutable().SetTagCount(Tag, FMath::Max(0, TagCount + CountDelta));
				}
			}
		}
	}
	
	// For removal, reorder calls so that FillParentTags is only called once
	if (CountDelta > 0)
	{
		for (TArray<FGameplayTag>::TConstIterator TagIt = Container.CreateConstIterator(); TagIt; ++TagIt)
		{
			const FGameplayTag& Tag = *TagIt;
			if (GameplayTagCountContainer.UpdateTagCount(Tag, CountDelta, TagRepState))
			{
				OnTagUpdated(Tag, true);
			}
		}
	}
	else if (CountDelta < 0)
	{
		// Defer FillParentTags and calling delegates until all Tags have been removed
		TArray<FGameplayTag> RemovedTags;
		RemovedTags.Reserve(Container.Num()); // pre-allocate max number (if all are removed)
		TArray<FDeferredTagChangeDelegate> DeferredTagChangeDelegates;

		for (TArray<FGameplayTag>::TConstIterator TagIt = Container.CreateConstIterator(); TagIt; ++TagIt)
		{
			const FGameplayTag& Tag = *TagIt;
			if (GameplayTagCountContainer.UpdateTagCount_DeferredParentRemoval(Tag, CountDelta, DeferredTagChangeDelegates))
			{
				RemovedTags.Add(Tag);
				if (UE::Net::ShouldUseIrisReplication() && !CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread())
				{
					// Iris replication doesn't work the same so we're falling back to using the Minimal and replicated loose tags until it's fixed.
					const UWorld* World = GetWorld();
					ENetMode NetMode = World ? World->GetNetMode() : NM_Standalone;
					if (NetMode == NM_DedicatedServer || NetMode == NM_ListenServer)
					{
						FMinimalReplicationTagCountMap& MTagMap = GetMinimalReplicationTags_Mutable();
						if (MTagMap.TagMap.Contains(Tag))
						{
							MTagMap.SetTagCount(Tag, 0);
						}

						FMinimalReplicationTagCountMap& RLTagMap = GetReplicatedLooseTags_Mutable();
						if (RLTagMap.TagMap.Contains(Tag))
						{
							RLTagMap.SetTagCount(Tag, 0);
						}
					}
				}
			}
		}

		// now do the work that was deferred
		if (RemovedTags.Num() > 0)
		{
			GameplayTagCountContainer.FillParentTags();
		}

		for (FDeferredTagChangeDelegate& Delegate : DeferredTagChangeDelegates)
		{
			Delegate.Execute();
		}

		// Notify last in case OnTagUpdated queries this container
		for (FGameplayTag& Tag : RemovedTags)
		{
			OnTagUpdated(Tag, false);
		}
	}
	
	if (UE::Net::ShouldUseIrisReplication() && CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread())
	{
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, GameplayTagCountContainer, this);
	}
}

int32 UAbilitySystemComponent::GetGameplayTagCount(FGameplayTag GameplayTag) const
{
	// NOTE -- no authority check to allow read-only access on all clients.
	// This may lead to incorrect values due to network lag. 
	return GetTagCount(GameplayTag);
}

FOnGameplayEffectTagCountChanged& UAbilitySystemComponent::RegisterGameplayTagEvent(FGameplayTag Tag, EGameplayTagEventType::Type EventType)
{
	return GameplayTagCountContainer.RegisterGameplayTagEvent(Tag, EventType);
}

bool UAbilitySystemComponent::UnregisterGameplayTagEvent(FDelegateHandle DelegateHandle, FGameplayTag Tag, EGameplayTagEventType::Type EventType)
{
	return GameplayTagCountContainer.RegisterGameplayTagEvent(Tag, EventType).Remove(DelegateHandle);
}

FDelegateHandle UAbilitySystemComponent::RegisterAndCallGameplayTagEvent(FGameplayTag Tag, FOnGameplayEffectTagCountChanged::FDelegate Delegate, EGameplayTagEventType::Type EventType)
{
	FDelegateHandle DelegateHandle = GameplayTagCountContainer.RegisterGameplayTagEvent(Tag, EventType).Add(Delegate);

	const int32 TagCount = GetTagCount(Tag);
	if (TagCount > 0)
	{
		Delegate.Execute(Tag, TagCount);
	}

	return DelegateHandle;
}

FOnGameplayEffectTagCountChanged& UAbilitySystemComponent::RegisterGenericGameplayTagEvent()
{
	return GameplayTagCountContainer.RegisterGenericGameplayEvent();
}

FOnGameplayAttributeChange& UAbilitySystemComponent::RegisterGameplayAttributeEvent(FGameplayAttribute Attribute)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return ActiveGameplayEffects.RegisterGameplayAttributeEvent(Attribute);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FOnGameplayAttributeValueChange& UAbilitySystemComponent::GetGameplayAttributeValueChangeDelegate(FGameplayAttribute Attribute)
{
	return ActiveGameplayEffects.GetGameplayAttributeValueChangeDelegate(Attribute);
}

FProperty* UAbilitySystemComponent::GetOutgoingDurationProperty()
{
	static FProperty* DurationProperty = FindFieldChecked<FProperty>(UAbilitySystemComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UAbilitySystemComponent, OutgoingDuration));
	return DurationProperty;
}

FProperty* UAbilitySystemComponent::GetIncomingDurationProperty()
{
	static FProperty* DurationProperty = FindFieldChecked<FProperty>(UAbilitySystemComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UAbilitySystemComponent, IncomingDuration));
	return DurationProperty;
}

const FGameplayEffectAttributeCaptureDefinition& UAbilitySystemComponent::GetOutgoingDurationCapture()
{
	// We will just always take snapshots of the source's duration mods
	static FGameplayEffectAttributeCaptureDefinition OutgoingDurationCapture(GetOutgoingDurationProperty(), EGameplayEffectAttributeCaptureSource::Source, true);
	return OutgoingDurationCapture;

}
const FGameplayEffectAttributeCaptureDefinition& UAbilitySystemComponent::GetIncomingDurationCapture()
{
	// Never take snapshots of the target's duration mods: we are going to evaluate this on apply only.
	static FGameplayEffectAttributeCaptureDefinition IncomingDurationCapture(GetIncomingDurationProperty(), EGameplayEffectAttributeCaptureSource::Target, false);
	return IncomingDurationCapture;
}

void UAbilitySystemComponent::ResetTagMap()
{
	GameplayTagCountContainer.Reset();
}

void UAbilitySystemComponent::NotifyTagMap_StackCountChange(const FGameplayTagContainer& Container)
{
	for (TArray<FGameplayTag>::TConstIterator TagIt = Container.CreateConstIterator(); TagIt; ++TagIt)
	{
		const FGameplayTag& Tag = *TagIt;
		GameplayTagCountContainer.Notify_StackCountChange(Tag);
	}
}

FActiveGameplayEffectHandle UAbilitySystemComponent::ApplyGameplayEffectSpecToTarget(const FGameplayEffectSpec &Spec, UAbilitySystemComponent *Target, FPredictionKey PredictionKey)
{
	SCOPE_CYCLE_COUNTER(STAT_AbilitySystemComp_ApplyGameplayEffectSpecToTarget);
	UAbilitySystemGlobals& AbilitySystemGlobals = UAbilitySystemGlobals::Get();

	if (!AbilitySystemGlobals.ShouldPredictTargetGameplayEffects())
	{
		// If we don't want to predict target effects, clear prediction key
		PredictionKey = FPredictionKey();
	}

	FActiveGameplayEffectHandle ReturnHandle;

	if (Target)
	{
		ReturnHandle = Target->ApplyGameplayEffectSpecToSelf(Spec, PredictionKey);
	}

	return ReturnHandle;
}

FActiveGameplayEffectHandle UAbilitySystemComponent::ApplyGameplayEffectSpecToSelf(const FGameplayEffectSpec &Spec, FPredictionKey PredictionKey)
{
#if WITH_SERVER_CODE
	SCOPE_CYCLE_COUNTER(STAT_AbilitySystemComp_ApplyGameplayEffectSpecToSelf);
#endif

	// Scope lock the container after the addition has taken place to prevent the new effect from potentially getting mangled during the remainder
	// of the add operation
	FScopedActiveGameplayEffectLock ScopeLock(ActiveGameplayEffects);

	FScopeCurrentGameplayEffectBeingApplied ScopedGEApplication(&Spec, this);

	// Check if the Spec definition is valid
	if (Spec.Def == nullptr)
	{
		ABILITY_LOG(Verbose, TEXT("UAbilitySystemComponent::ApplyGameplayEffectSpecToSelf called with a Null effect definition"));
		return FActiveGameplayEffectHandle();
	}

	// Check Network Authority
	if (!HasNetworkAuthorityToApplyGameplayEffect(PredictionKey))
	{
		return FActiveGameplayEffectHandle();
	}

	// Don't allow prediction of periodic effects
	if (PredictionKey.IsValidKey() && Spec.GetPeriod() > 0.f)
	{
		if (IsOwnerActorAuthoritative())
		{
			// Server continue with invalid prediction key
			PredictionKey = FPredictionKey();
		}
		else
		{
			// Client just return now
			return FActiveGameplayEffectHandle();
		}
	}

	// Check if there is a registered "application" query that can block the application
	for (const FGameplayEffectApplicationQuery& ApplicationQuery : GameplayEffectApplicationQueries)
	{
		const bool bAllowed = ApplicationQuery.Execute(ActiveGameplayEffects, Spec);
		if (!bAllowed)
		{
			return FActiveGameplayEffectHandle();
		}
	}

	// check if the effect being applied actually succeeds
	if (!Spec.Def->CanApply(ActiveGameplayEffects, Spec))
	{
		return FActiveGameplayEffectHandle();
	}

	// Check AttributeSet requirements: make sure all attributes are valid
	// We may want to cache this off in some way to make the runtime check quicker.
	// We also need to handle things in the execution list
	for (const FGameplayModifierInfo& Mod : Spec.Def->Modifiers)
	{
		if (!Mod.Attribute.IsValid())
		{
			ABILITY_LOG(Warning, TEXT("%s has a null modifier attribute."), *Spec.Def->GetPathName());
			return FActiveGameplayEffectHandle();
		}
	}


	// Clients should treat predicted instant effects as if they have infinite duration. The effects will be cleaned up later.
	bool bTreatAsInfiniteDuration = GetOwnerRole() != ROLE_Authority && PredictionKey.IsLocalClientKey() && Spec.Def->DurationPolicy == EGameplayEffectDurationType::Instant;

	// Make sure we create our copy of the spec in the right place
	// We default to returning an FActiveGameplayEffectHandle that denotes the GE was successfully executed instantly
	FActiveGameplayEffectHandle MyHandle = FActiveGameplayEffectHandle::GetInstantExecutedHandle();
	bool bInvokeGameplayCueApplied = Spec.Def->DurationPolicy != EGameplayEffectDurationType::Instant; // Cache this now before possibly modifying predictive instant effect to infinite duration effect.
	bool bFoundExistingStackableGE = false;

	FActiveGameplayEffect* AppliedEffect = nullptr;
	FGameplayEffectSpec* OurCopyOfSpec = nullptr;
	TUniquePtr<FGameplayEffectSpec> StackSpec;
	{
		if (Spec.Def->DurationPolicy != EGameplayEffectDurationType::Instant || bTreatAsInfiniteDuration)
		{
			AppliedEffect = ActiveGameplayEffects.ApplyGameplayEffectSpec(Spec, PredictionKey, bFoundExistingStackableGE);
			if (!AppliedEffect)
			{
				return FActiveGameplayEffectHandle();
			}

			MyHandle = AppliedEffect->Handle;
			OurCopyOfSpec = &(AppliedEffect->Spec);

			// Log results of applied GE spec
			if (UE_LOG_ACTIVE(VLogAbilitySystem, Log))
			{
				UE_VLOG(GetOwnerActor(), VLogAbilitySystem, Log, TEXT("Applied %s"), *OurCopyOfSpec->Def->GetFName().ToString());

				for (const FGameplayModifierInfo& Modifier : Spec.Def->Modifiers)
				{
					float Magnitude = 0.f;
					Modifier.ModifierMagnitude.AttemptCalculateMagnitude(Spec, Magnitude);
					UE_VLOG(GetOwnerActor(), VLogAbilitySystem, Log, TEXT("         %s: %s %f"), *Modifier.Attribute.GetName(), *EGameplayModOpToString(Modifier.ModifierOp), Magnitude);
				}
			}
		}

		if (!OurCopyOfSpec)
		{
			StackSpec = MakeUnique<FGameplayEffectSpec>(Spec);
			OurCopyOfSpec = StackSpec.Get();

			UAbilitySystemGlobals::Get().GlobalPreGameplayEffectSpecApply(*OurCopyOfSpec, this);
			OurCopyOfSpec->CaptureAttributeDataFromTarget(this);
		}

		// if necessary add a modifier to OurCopyOfSpec to force it to have an infinite duration
		if (bTreatAsInfiniteDuration)
		{
			// This should just be a straight set of the duration float now
			OurCopyOfSpec->SetDuration(UGameplayEffect::INFINITE_DURATION, true);
		}
	}

	// Update (not push) the global spec being applied [we want to switch it to our copy, from the const input copy)
	UAbilitySystemGlobals::Get().SetCurrentAppliedGE(OurCopyOfSpec);

	// UE5.4: We are following the same previous implementation that there is a special case for Gameplay Cues here (caveat: may not be true):
	// We are Stacking an existing Gameplay Effect.  That means the GameplayCues should already be Added/WhileActive and we do not have a proper
	// way to replicate the fact that it's been retriggered, hence the RPC here.  I say this may not be true because any number of things could have
	// removed the GameplayCue by the time we getting a Stacking GE (e.g. RemoveGameplayCue).
	if (!bSuppressGameplayCues && !Spec.Def->bSuppressStackingCues && bFoundExistingStackableGE && AppliedEffect && !AppliedEffect->bIsInhibited)
	{
		ensureMsgf(OurCopyOfSpec, TEXT("OurCopyOfSpec will always be valid if bFoundExistingStackableGE"));
		if (OurCopyOfSpec && OurCopyOfSpec->GetStackCount() >= Spec.GetStackCount())
		{
			// If we're predicting we want to activate on ourselves
			if (!IsOwnerActorAuthoritative() && PredictionKey.IsLocalClientKey())
			{
				InvokeGameplayCueEvent(*OurCopyOfSpec, EGameplayCueEvent::OnActive);
				InvokeGameplayCueEvent(*OurCopyOfSpec, EGameplayCueEvent::WhileActive);
			}
			else
			{
				UAbilitySystemGlobals::Get().GetGameplayCueManager()->InvokeGameplayCueAddedAndWhileActive_FromSpec(this, *OurCopyOfSpec, PredictionKey);
			}
		}
	}
	
	// Execute the GE at least once (if instant, this will execute once and be done. If persistent, it was added to ActiveGameplayEffects in ApplyGameplayEffectSpec)
	
	// Execute if this is an instant application effect
	if (bTreatAsInfiniteDuration)
	{
		// This is an instant application but we are treating it as an infinite duration for prediction. We should still predict the execute GameplayCUE.
		// (in non predictive case, this will happen inside ::ExecuteGameplayEffect)

		if (!bSuppressGameplayCues)
		{
			UAbilitySystemGlobals::Get().GetGameplayCueManager()->InvokeGameplayCueExecuted_FromSpec(this, *OurCopyOfSpec, PredictionKey);
		}
	}
	else if (Spec.Def->DurationPolicy == EGameplayEffectDurationType::Instant)
	{
		// This is a non-predicted instant effect (it never gets added to ActiveGameplayEffects)
		ExecuteGameplayEffect(*OurCopyOfSpec, PredictionKey);
	}

	// Notify the Gameplay Effect (and its Components) that it has been successfully applied
	Spec.Def->OnApplied(ActiveGameplayEffects, *OurCopyOfSpec, PredictionKey);

	UAbilitySystemComponent* InstigatorASC = Spec.GetContext().GetInstigatorAbilitySystemComponent();

	// Send ourselves a callback	
	OnGameplayEffectAppliedToSelf(InstigatorASC, *OurCopyOfSpec, MyHandle);

	// Send the instigator a callback
	if (InstigatorASC)
	{
		InstigatorASC->OnGameplayEffectAppliedToTarget(this, *OurCopyOfSpec, MyHandle);
	}

	return MyHandle;
}

FActiveGameplayEffectHandle UAbilitySystemComponent::BP_ApplyGameplayEffectSpecToTarget(const FGameplayEffectSpecHandle& SpecHandle, UAbilitySystemComponent* Target)
{
	FActiveGameplayEffectHandle ReturnHandle;
	if (SpecHandle.IsValid() && Target)
	{
		ReturnHandle = ApplyGameplayEffectSpecToTarget(*SpecHandle.Data.Get(), Target);
	}

	return ReturnHandle;
}

FActiveGameplayEffectHandle UAbilitySystemComponent::BP_ApplyGameplayEffectSpecToSelf(const FGameplayEffectSpecHandle& SpecHandle)
{
	FActiveGameplayEffectHandle ReturnHandle;
	if (SpecHandle.IsValid())
	{
		ReturnHandle = ApplyGameplayEffectSpecToSelf(*SpecHandle.Data.Get());
	}

	return ReturnHandle;
}

void UAbilitySystemComponent::ExecutePeriodicEffect(FActiveGameplayEffectHandle	Handle)
{
	ActiveGameplayEffects.ExecutePeriodicGameplayEffect(Handle);
}

void UAbilitySystemComponent::ExecuteGameplayEffect(FGameplayEffectSpec &Spec, FPredictionKey PredictionKey)
{
#if WITH_SERVER_CODE
	SCOPE_CYCLE_COUNTER(STAT_AbilitySystemComp_ExecuteGameplayEffect);
#endif

	// Should only ever execute effects that are instant application or periodic application
	// Effects with no period and that aren't instant application should never be executed
	check( (Spec.GetDuration() == UGameplayEffect::INSTANT_APPLICATION || Spec.GetPeriod() != UGameplayEffect::NO_PERIOD) );

	if (UE_LOG_ACTIVE(VLogAbilitySystem, Log))
	{
		UE_VLOG(GetOwnerActor(), VLogAbilitySystem, Log, TEXT("Executed %s"), *Spec.Def->GetFName().ToString());
		
		for (const FGameplayModifierInfo& Modifier : Spec.Def->Modifiers)
		{
			float Magnitude = 0.f;
			Modifier.ModifierMagnitude.AttemptCalculateMagnitude(Spec, Magnitude);
			UE_VLOG(GetOwnerActor(), VLogAbilitySystem, Log, TEXT("         %s: %s %f"), *Modifier.Attribute.GetName(), *EGameplayModOpToString(Modifier.ModifierOp), Magnitude);
		}
	}

	ActiveGameplayEffects.ExecuteActiveEffectsFrom(Spec, PredictionKey);
}

void UAbilitySystemComponent::CheckDurationExpired(FActiveGameplayEffectHandle Handle)
{
	ActiveGameplayEffects.CheckDuration(Handle);
}

const UGameplayEffect* UAbilitySystemComponent::GetGameplayEffectDefForHandle(FActiveGameplayEffectHandle Handle)
{
	FActiveGameplayEffect* ActiveGE = ActiveGameplayEffects.GetActiveGameplayEffect(Handle);
	if (ActiveGE)
	{
		return ActiveGE->Spec.Def;
	}

	return nullptr;
}

bool UAbilitySystemComponent::RemoveActiveGameplayEffect(FActiveGameplayEffectHandle Handle, int32 StacksToRemove)
{
	using namespace UE::AbilitySystem::Private;
	if (!EnumHasAnyFlags(static_cast<EAllowPredictiveGEFlags>(CVarAllowPredictiveGEFlagsValue), EAllowPredictiveGEFlags::AllowNativeRemoveByHandle))
	{
		if (!IsOwnerActorAuthoritative())
		{
			const UGameplayEffect* GEDef = GetGameplayEffectDefForHandle(Handle);
			UE_LOGF(LogAbilitySystem, Warning, "%s called without Authority when attempting to remove %ls. Fix-up code, or temporarily patch using AbilitySystem.Fix.AllowPredictiveGEFlags", __func__, *GetNameSafe(GEDef));
			return false;
		}
	}

	return ActiveGameplayEffects.RemoveActiveGameplayEffect(Handle, StacksToRemove);
}

void UAbilitySystemComponent::RemoveActiveGameplayEffect_NoReturn(FActiveGameplayEffectHandle Handle, int32 StacksToRemove)
{
	// Legacy version allowed client removal, so let's keep that for now.
	RemoveActiveGameplayEffect_AllowClientRemoval(Handle, StacksToRemove);
}

void UAbilitySystemComponent::RemoveActiveGameplayEffect_AllowClientRemoval(FActiveGameplayEffectHandle Handle, int32 StacksToRemove)
{
	ActiveGameplayEffects.RemoveActiveGameplayEffect(Handle, StacksToRemove);
}

void UAbilitySystemComponent::OnRejectedActiveGameplayEffect(FActiveGameplayEffectHandle Handle, int32 StacksToRemove)
{
	constexpr bool bPredictionRejected = true;
	ActiveGameplayEffects.RemoveActiveGameplayEffect(Handle, StacksToRemove, bPredictionRejected);
}

void UAbilitySystemComponent::OnCaughtUpActiveGameplayEffect(FActiveGameplayEffectHandle Handle, int32 StacksToRemove)
{
	ActiveGameplayEffects.RemoveActiveGameplayEffect(Handle, StacksToRemove);
}

void UAbilitySystemComponent::OnPredictiveGameplayEffectStackCaughtUp(FActiveGameplayEffectHandle Handle)
{
	ActiveGameplayEffects.OnPredictiveGameplayEffectStackCaughtUp(Handle);
}

void UAbilitySystemComponent::RemoveActiveGameplayEffectBySourceEffect(TSubclassOf<UGameplayEffect> GameplayEffect, UAbilitySystemComponent* InstigatorAbilitySystemComponent, int32 StacksToRemove /*= -1*/)
{
	if (GameplayEffect)
	{
		FGameplayEffectQuery Query;
		Query.CustomMatchDelegate.BindLambda([&](const FActiveGameplayEffect& CurEffect)
		{
			bool bMatches = false;

			// First check at matching: backing GE class must be the exact same
			if (CurEffect.Spec.Def && GameplayEffect == CurEffect.Spec.Def->GetClass())
			{
				// If an instigator is specified, matching is dependent upon it
				if (InstigatorAbilitySystemComponent)
				{
					bMatches = (InstigatorAbilitySystemComponent == CurEffect.Spec.GetEffectContext().GetInstigatorAbilitySystemComponent());
				}
				else
				{
					bMatches = true;
				}
			}

			return bMatches;
		});
		ActiveGameplayEffects.RemoveActiveEffects(Query, StacksToRemove);
	}
}

float UAbilitySystemComponent::GetGameplayEffectDuration(FActiveGameplayEffectHandle Handle) const
{
	float StartEffectTime = 0.0f;
	float Duration = 0.0f;
	ActiveGameplayEffects.GetGameplayEffectStartTimeAndDuration(Handle, StartEffectTime, Duration);

	return Duration;
}

void UAbilitySystemComponent::RecomputeGameplayEffectStartTimes(const float WorldTime, const float ServerWorldTime)
{
	ActiveGameplayEffects.RecomputeStartWorldTimes(WorldTime, ServerWorldTime);
}

void UAbilitySystemComponent::GetGameplayEffectStartTimeAndDuration(FActiveGameplayEffectHandle Handle, float& StartEffectTime, float& Duration) const
{
	return ActiveGameplayEffects.GetGameplayEffectStartTimeAndDuration(Handle, StartEffectTime, Duration);
}

void UAbilitySystemComponent::UpdateActiveGameplayEffectSetByCallerMagnitude(FActiveGameplayEffectHandle ActiveHandle, FGameplayTag SetByCallerTag, float NewValue)
{
	ActiveGameplayEffects.UpdateActiveGameplayEffectSetByCallerMagnitude(ActiveHandle, SetByCallerTag, NewValue);
}

void UAbilitySystemComponent::UpdateActiveGameplayEffectSetByCallerMagnitudes(FActiveGameplayEffectHandle ActiveHandle, const TMap<FGameplayTag, float>& NewSetByCallerValues)
{
	ActiveGameplayEffects.UpdateActiveGameplayEffectSetByCallerMagnitudes(ActiveHandle, NewSetByCallerValues);
}

float UAbilitySystemComponent::GetGameplayEffectMagnitude(FActiveGameplayEffectHandle Handle, FGameplayAttribute Attribute) const
{
	return ActiveGameplayEffects.GetGameplayEffectMagnitude(Handle, Attribute);
}

void UAbilitySystemComponent::SetActiveGameplayEffectLevel(FActiveGameplayEffectHandle ActiveHandle, int32 NewLevel)
{
	ActiveGameplayEffects.SetActiveGameplayEffectLevel(ActiveHandle, NewLevel);
}

void UAbilitySystemComponent::SetActiveGameplayEffectLevelUsingQuery(FGameplayEffectQuery Query, int32 NewLevel)
{
	TArray<FActiveGameplayEffectHandle> ActiveGameplayEffectHandles = ActiveGameplayEffects.GetActiveEffects(Query);
	for (FActiveGameplayEffectHandle ActiveHandle : ActiveGameplayEffectHandles)
	{
		SetActiveGameplayEffectLevel(ActiveHandle, NewLevel);
	}
}

int32 UAbilitySystemComponent::GetCurrentStackCount(FActiveGameplayEffectHandle Handle) const
{
	if (const FActiveGameplayEffect* ActiveGE = ActiveGameplayEffects.GetActiveGameplayEffect(Handle))
	{
		return ActiveGE->Spec.GetStackCount();
	}
	return 0;
}

int32 UAbilitySystemComponent::GetCurrentStackCount(FGameplayAbilitySpecHandle Handle) const
{
	FActiveGameplayEffectHandle GEHandle = FindActiveGameplayEffectHandle(Handle);
	if (GEHandle.IsValid())
	{
		return GetCurrentStackCount(GEHandle);
	}
	return 0;
}

FString UAbilitySystemComponent::GetActiveGEDebugString(FActiveGameplayEffectHandle Handle) const
{
	FString Str;

	if (const FActiveGameplayEffect* ActiveGE = ActiveGameplayEffects.GetActiveGameplayEffect(Handle))
	{
		Str = FString::Printf(TEXT("%s - (Level: %.2f. Stacks: %d)"), *ActiveGE->Spec.Def->GetName(), ActiveGE->Spec.GetLevel(), ActiveGE->Spec.GetStackCount());
	}

	return Str;
}

/** Gets the GE Handle of the GE that granted the passed in Ability */
FActiveGameplayEffectHandle UAbilitySystemComponent::FindActiveGameplayEffectHandle(FGameplayAbilitySpecHandle Handle) const
{
	for (const FActiveGameplayEffect& ActiveGE : &ActiveGameplayEffects)
	{
		// Old, deprecated way of handling these (before AbilitiesGameplayEffectComponent):
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (const FGameplayAbilitySpecDef& AbilitySpecDef : ActiveGE.Spec.GrantedAbilitySpecs)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			if (AbilitySpecDef.AssignedHandle == Handle)
			{
				return ActiveGE.Handle;
			}
		}

		// Where AbilitiesGameplayEffectComponent stores its data
		if (ActiveGE.GrantedAbilityHandles.Contains(Handle))
		{
			return ActiveGE.Handle;
		}
	}
	return FActiveGameplayEffectHandle();
}

void UAbilitySystemComponent::InitDefaultGameplayCueParameters(FGameplayCueParameters& Parameters)
{
	Parameters.Instigator = GetOwnerActor();
	Parameters.EffectCauser = GetAvatarActor_Direct();
}

bool UAbilitySystemComponent::IsReadyForGameplayCues()
{
	// check if the avatar actor is valid and ready to take gameplaycues
	AActor* ActorAvatar = nullptr;
	if (AbilityActorInfo.IsValid())
	{
		ActorAvatar = AbilityActorInfo->AvatarActor.Get();
	}
	return ActorAvatar != nullptr;
}

void UAbilitySystemComponent::InvokeGameplayCueEvent(const FGameplayEffectSpecForRPC &Spec, EGameplayCueEvent::Type EventType)
{
#if WITH_SERVER_CODE
	SCOPE_CYCLE_COUNTER(STAT_AbilitySystemComp_InvokeGameplayCueEvent);
#endif

	AActor* ActorAvatar = AbilityActorInfo->AvatarActor.Get();
	if (ActorAvatar == nullptr || bSuppressGameplayCues)
	{
		// No avatar actor to call this gameplaycue on.
		return;
	}

	if (!Spec.Def)
	{
		ABILITY_LOG(Warning, TEXT("InvokeGameplayCueEvent Actor %s that has no gameplay effect!"), ActorAvatar ? *ActorAvatar->GetName() : TEXT("nullptr"));
		return;
	}
	
	float ExecuteLevel = Spec.GetLevel();

	FGameplayCueParameters CueParameters(Spec);

	FGameplayEffectQuery EffectQuery;
	EffectQuery.EffectDefinition = Spec.Def->GetClass();
	CueParameters.bGameplayEffectActive = Spec.Def->DurationPolicy == EGameplayEffectDurationType::Instant || ActiveGameplayEffects.GetActiveEffectCount(EffectQuery) > 0;

	for (FGameplayEffectCue CueInfo : Spec.Def->GameplayCues)
	{
		if (CueInfo.MagnitudeAttribute.IsValid())
		{
			if (const FGameplayEffectModifiedAttribute* ModifiedAttribute = Spec.GetModifiedAttribute(CueInfo.MagnitudeAttribute))
			{
				CueParameters.RawMagnitude = ModifiedAttribute->TotalMagnitude;
			}
			else
			{
				CueParameters.RawMagnitude = 0.0f;
			}
		}
		else
		{
			CueParameters.RawMagnitude = 0.0f;
		}

		CueParameters.NormalizedMagnitude = CueInfo.NormalizeLevel(ExecuteLevel);

		UAbilitySystemGlobals::Get().GetGameplayCueManager()->HandleGameplayCues(ActorAvatar, CueInfo.GameplayCueTags, EventType, CueParameters);
	}
}

void UAbilitySystemComponent::InvokeGameplayCueEvent(const FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, FGameplayEffectContextHandle EffectContext)
{
	FGameplayCueParameters CueParameters(EffectContext);

	CueParameters.NormalizedMagnitude = 1.f;
	CueParameters.RawMagnitude = 0.f;

	InvokeGameplayCueEvent(GameplayCueTag, EventType, CueParameters);
}

void UAbilitySystemComponent::InvokeGameplayCueEvent(const FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& GameplayCueParameters)
{
	if(ensureMsgf(AbilityActorInfo != nullptr, TEXT("AbilityActorInfo is null for %s. Probably OnRegister of the AbilitySystemComponent was not called yet."), *this->GetOwner()->GetName()))
	{
		AActor* ActorAvatar = AbilityActorInfo->AvatarActor.Get();
		
		if (ActorAvatar != nullptr && !bSuppressGameplayCues)
		{
			UAbilitySystemGlobals::Get().GetGameplayCueManager()->HandleGameplayCue(ActorAvatar, GameplayCueTag, EventType, GameplayCueParameters);
		}
	}
}

void UAbilitySystemComponent::ExecuteGameplayCue(const FGameplayTag GameplayCueTag, FGameplayEffectContextHandle EffectContext)
{
	// Send to the wrapper on the cue manager
	UAbilitySystemGlobals::Get().GetGameplayCueManager()->InvokeGameplayCueExecuted(this, GameplayCueTag, ScopedPredictionKey, EffectContext);
}

void UAbilitySystemComponent::ExecuteGameplayCue(const FGameplayTag GameplayCueTag, const FGameplayCueParameters& GameplayCueParameters)
{
	// Send to the wrapper on the cue manager
	UAbilitySystemGlobals::Get().GetGameplayCueManager()->InvokeGameplayCueExecuted_WithParams(this, GameplayCueTag, ScopedPredictionKey, GameplayCueParameters);
}

void UAbilitySystemComponent::AddGameplayCue(const FGameplayTag GameplayCueTag, FGameplayEffectContextHandle EffectContext)
{
	AddGameplayCue_Internal(GameplayCueTag, EffectContext, ActiveGameplayCues);
}

void UAbilitySystemComponent::AddGameplayCue(const FGameplayTag GameplayCueTag, const FGameplayCueParameters& GameplayCueParameters)
{
	AddGameplayCue_Internal(GameplayCueTag, GameplayCueParameters, ActiveGameplayCues);
}

void UAbilitySystemComponent::AddGameplayCue_MinimalReplication(const FGameplayTag GameplayCueTag, FGameplayEffectContextHandle EffectContext)
{
	AddGameplayCue_Internal(GameplayCueTag, EffectContext, MinimalReplicationGameplayCues);
}

void UAbilitySystemComponent::AddGameplayCue_Internal(const FGameplayTag GameplayCueTag, FGameplayEffectContextHandle& EffectContext, FActiveGameplayCueContainer& GameplayCueContainer)
{
	if (EffectContext.IsValid() == false)
	{
		EffectContext = MakeEffectContext();
	}

	FGameplayCueParameters Parameters(EffectContext);

	AddGameplayCue_Internal(GameplayCueTag, Parameters, GameplayCueContainer);
}

void UAbilitySystemComponent::AddGameplayCue_Internal(const FGameplayTag GameplayCueTag, const FGameplayCueParameters& GameplayCueParameters, FActiveGameplayCueContainer& GameplayCueContainer)
{
	if (IsOwnerActorAuthoritative())
	{
		if (!GameplayCueTag.IsValid())
		{
			return;
		}

		const bool bWasInList = GameplayCueContainer.HasCue(GameplayCueTag);

		ForceReplication();
		GameplayCueContainer.AddCue(GameplayCueTag, ScopedPredictionKey, GameplayCueParameters);
		
		// For mixed minimal replication mode, we do NOT want the owning client to play the OnActive event through this RPC, since it will get the full replicated 
		// GE in its AGE array. Generate a server-side prediction key for it, which it will look for on the _Implementation function and ignore. (<--- Original Hack)
		{
			FPredictionKey PredictionKeyForRPC = ScopedPredictionKey; // Key we send for RPC. Start with the regular old ScopedPredictionKey

			// Special stuff for mixed replication mode
			if (ReplicationMode == EGameplayEffectReplicationMode::Mixed)
			{
				if (GameplayCueContainer.bMinimalReplication)
				{
					// For *replicated to sim proxies only* container, Create a Server Initiated PK to avoid double playing on the auto proxy in mixed replication mode (Original Hack)
					PredictionKeyForRPC = FPredictionKey::CreateNewServerInitiatedKey(this);
				}
				else
				{
					// For "replicated to everyone" cue container, we need to clear server replicated prediction keys, or else they will trip the same absorption code that we added for the first hack above.
					// Its ok to just throw out a server replicated prediction key because (outside of mixed replication mode) it will not affect what the client does in NetMulticast_InvokeGameplayCueAdded_WithParams_Implementation
					// (E.g, the client only skips the InvokeCall if the key is locally generated, not for server generated ones anways)
					if (ScopedPredictionKey.IsServerInitiatedKey())
					{
						PredictionKeyForRPC = FPredictionKey();
					}
				}
			}
			
			// Finally, call the RPC to play the OnActive event
			if (IAbilitySystemReplicationProxyInterface* ReplicationInterface = GetReplicationInterface())
			{
				ReplicationInterface->Call_InvokeGameplayCueAdded_WithParams(GameplayCueTag, PredictionKeyForRPC, GameplayCueParameters);
			}
		}

		if (!bWasInList)
		{
			// Call on server here, clients get it from repnotify
			InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::WhileActive, GameplayCueParameters);
		}
	}
	else if (ScopedPredictionKey.IsLocalClientKey())
	{
		GameplayCueContainer.PredictiveAdd(GameplayCueTag, ScopedPredictionKey);

		// Allow for predictive gameplaycue events? Needs more thought
		InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::OnActive, GameplayCueParameters);
		InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::WhileActive, GameplayCueParameters);
	}
}

void UAbilitySystemComponent::RemoveGameplayCue(const FGameplayTag GameplayCueTag)
{
	RemoveGameplayCue_Internal(GameplayCueTag, ActiveGameplayCues);
}

void UAbilitySystemComponent::RemoveGameplayCue_MinimalReplication(const FGameplayTag GameplayCueTag)
{
	RemoveGameplayCue_Internal(GameplayCueTag, MinimalReplicationGameplayCues);
}

void UAbilitySystemComponent::RemoveGameplayCue_Internal(const FGameplayTag GameplayCueTag, FActiveGameplayCueContainer& GameplayCueContainer)
{
	if (IsOwnerActorAuthoritative())
	{
		GameplayCueContainer.RemoveCue(GameplayCueTag);
	}
	else if (ScopedPredictionKey.IsLocalClientKey())
	{
		GameplayCueContainer.PredictiveRemove(GameplayCueTag);
	}
}

void UAbilitySystemComponent::RemoveAllGameplayCues()
{
	TArray<FGameplayTag, TInlineAllocator<16>> GameplayCueTagsToRemove;
	for (const FActiveGameplayCue& Cue : ActiveGameplayCues.GameplayCues)
	{
		GameplayCueTagsToRemove.Emplace(Cue.GameplayCueTag);
	}
	for (const FGameplayTag& CueTagToRemove : GameplayCueTagsToRemove)
	{
		RemoveGameplayCue(CueTagToRemove);
	}
}

void UAbilitySystemComponent::NetMulticast_InvokeGameplayCueExecuted_FromSpec_Implementation(const FGameplayEffectSpecForRPC Spec, FPredictionKey PredictionKey)
{
	if (IsOwnerActorAuthoritative() || PredictionKey.IsLocalClientKey() == false)
	{
		InvokeGameplayCueEvent(Spec, EGameplayCueEvent::Executed);
	}
}

void UAbilitySystemComponent::NetMulticast_InvokeGameplayCueExecuted_Implementation(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext)
{
	if (IsOwnerActorAuthoritative() || PredictionKey.IsLocalClientKey() == false)
	{
		InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::Executed, EffectContext);
	}
}

void UAbilitySystemComponent::NetMulticast_InvokeGameplayCuesExecuted_Implementation(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext)
{
	if (IsOwnerActorAuthoritative() || PredictionKey.IsLocalClientKey() == false)
	{
		for (const FGameplayTag& GameplayCueTag : GameplayCueTags)
		{
			InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::Executed, EffectContext);
		}
	}
}

void UAbilitySystemComponent::NetMulticast_InvokeGameplayCueExecuted_WithParams_Implementation(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters)
{
	if (IsOwnerActorAuthoritative() || PredictionKey.IsLocalClientKey() == false)
	{
		InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::Executed, GameplayCueParameters);
	}
}

void UAbilitySystemComponent::NetMulticast_InvokeGameplayCuesExecuted_WithParams_Implementation(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters)
{
	if (IsOwnerActorAuthoritative() || PredictionKey.IsLocalClientKey() == false)
	{
		for (const FGameplayTag& GameplayCueTag : GameplayCueTags)
		{
			InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::Executed, GameplayCueParameters);
		}
	}
}

void UAbilitySystemComponent::NetMulticast_InvokeGameplayCueAdded_Implementation(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext)
{
	if (IsOwnerActorAuthoritative() || PredictionKey.IsLocalClientKey() == false)
	{
		InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::OnActive, EffectContext);
	}
}

void UAbilitySystemComponent::NetMulticast_InvokeGameplayCueAdded_WithParams_Implementation(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters Parameters)
{
	// If server generated prediction key and auto proxy, skip this message. 
	// This is an RPC from mixed replication mode code, we will get the "real" message from our OnRep on the autonomous proxy
	// See UAbilitySystemComponent::AddGameplayCue_Internal for more info.
	bool bIsMixedReplicationFromServer = (ReplicationMode == EGameplayEffectReplicationMode::Mixed && PredictionKey.IsServerInitiatedKey() && AbilityActorInfo->IsLocallyControlledPlayer());

	if (IsOwnerActorAuthoritative() || (PredictionKey.IsLocalClientKey() == false && !bIsMixedReplicationFromServer))
	{
		InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::OnActive, Parameters);
	}
}

void UAbilitySystemComponent::NetMulticast_InvokeGameplayCueAddedAndWhileActive_FromSpec_Implementation(const FGameplayEffectSpecForRPC& Spec, FPredictionKey PredictionKey)
{
	if (IsOwnerActorAuthoritative() || PredictionKey.IsLocalClientKey() == false)
	{
		InvokeGameplayCueEvent(Spec, EGameplayCueEvent::OnActive);
		InvokeGameplayCueEvent(Spec, EGameplayCueEvent::WhileActive);
	}
}

void UAbilitySystemComponent::NetMulticast_InvokeGameplayCueAddedAndWhileActive_WithParams_Implementation(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters)
{
	if (IsOwnerActorAuthoritative() || PredictionKey.IsLocalClientKey() == false)
	{
		InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::OnActive, GameplayCueParameters);
		InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::WhileActive, GameplayCueParameters);
	}
}

void UAbilitySystemComponent::NetMulticast_InvokeGameplayCuesAddedAndWhileActive_WithParams_Implementation(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters)
{
	if (IsOwnerActorAuthoritative() || PredictionKey.IsLocalClientKey() == false)
	{
		for (const FGameplayTag& GameplayCueTag : GameplayCueTags)
		{
			InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::OnActive, GameplayCueParameters);
			InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::WhileActive, GameplayCueParameters);
		}
	}
}

TArray<float> UAbilitySystemComponent::GetActiveEffectsTimeRemaining(const FGameplayEffectQuery& Query) const
{
	return ActiveGameplayEffects.GetActiveEffectsTimeRemaining(Query);
}

TArray<TPair<float,float>> UAbilitySystemComponent::GetActiveEffectsTimeRemainingAndDuration(const FGameplayEffectQuery& Query) const
{
	return ActiveGameplayEffects.GetActiveEffectsTimeRemainingAndDuration(Query);
}

TArray<float> UAbilitySystemComponent::GetActiveEffectsDuration(const FGameplayEffectQuery& Query) const
{
	return ActiveGameplayEffects.GetActiveEffectsDuration(Query);
}

TArray<FActiveGameplayEffectHandle> UAbilitySystemComponent::GetActiveEffects(const FGameplayEffectQuery& Query) const
{
	return ActiveGameplayEffects.GetActiveEffects(Query);
}

TArray<FActiveGameplayEffectHandle> UAbilitySystemComponent::GetActiveEffectsWithAllTags(FGameplayTagContainer Tags) const
{
	return GetActiveEffects(FGameplayEffectQuery::MakeQuery_MatchAllEffectTags(Tags));
}

float UAbilitySystemComponent::GetActiveEffectsEndTime(const FGameplayEffectQuery& Query) const
{
	TArray<AActor*> DummyInstigators;
	return ActiveGameplayEffects.GetActiveEffectsEndTime(Query, DummyInstigators);
}

float UAbilitySystemComponent::GetActiveEffectsEndTimeWithInstigators(const FGameplayEffectQuery& Query, TArray<AActor*>& Instigators) const
{
	return ActiveGameplayEffects.GetActiveEffectsEndTime(Query, Instigators);
}

bool UAbilitySystemComponent::GetActiveEffectsEndTimeAndDuration(const FGameplayEffectQuery& Query, float& EndTime, float& Duration) const
{
	TArray<AActor*> DummyInstigators;
	return ActiveGameplayEffects.GetActiveEffectsEndTimeAndDuration(Query, EndTime, Duration, DummyInstigators);
}

void UAbilitySystemComponent::ModifyActiveEffectStartTime(FActiveGameplayEffectHandle Handle, float StartTimeDiff)
{
	ActiveGameplayEffects.ModifyActiveEffectStartTime(Handle, StartTimeDiff);
}

int32 UAbilitySystemComponent::RemoveActiveEffectsWithTags(const FGameplayTagContainer Tags)
{
	if (IsOwnerActorAuthoritative())
	{
		return RemoveActiveEffects(FGameplayEffectQuery::MakeQuery_MatchAnyEffectTags(Tags));
	}
	return 0;
}

int32 UAbilitySystemComponent::RemoveActiveEffectsWithSourceTags(FGameplayTagContainer Tags)
{
	if (IsOwnerActorAuthoritative())
	{
		return RemoveActiveEffects(FGameplayEffectQuery::MakeQuery_MatchAnySourceSpecTags(Tags));
	}
	return 0;
}

int32 UAbilitySystemComponent::RemoveActiveEffectsWithAppliedTags(FGameplayTagContainer Tags)
{
	if (IsOwnerActorAuthoritative())
	{
		return RemoveActiveEffects(FGameplayEffectQuery::MakeQuery_MatchAnyOwningTags(Tags));
	}
	return 0;
}

int32 UAbilitySystemComponent::RemoveActiveEffectsWithGrantedTags(const FGameplayTagContainer Tags)
{
	if (IsOwnerActorAuthoritative())
	{
		return RemoveActiveEffects(FGameplayEffectQuery::MakeQuery_MatchAnyOwningTags(Tags));
	}
	return 0;
}

int32 UAbilitySystemComponent::RemoveActiveEffects(const FGameplayEffectQuery& Query, int32 StacksToRemove)
{
	if (IsOwnerActorAuthoritative())
	{
		return ActiveGameplayEffects.RemoveActiveEffects(Query, StacksToRemove);
	}

	return 0;
}

void UAbilitySystemComponent::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	// Fast Arrays don't use push model, but there's no harm in marking them with it.
	// The flag will just be ignored.
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;

	Params.Condition = (bUseReplicationConditionForActiveGameplayEffects ? COND_Dynamic : COND_None);
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, ActiveGameplayEffects, Params);

	Params.Condition = COND_None;
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, SpawnedAttributes, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, ActiveGameplayCues, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, RepAnimMontageInfo, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, OwnerActor, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, AvatarActor, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, GameplayTagCountContainer, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, ReplicatedLooseTags, Params);

	Params.Condition = COND_ReplayOrOwner;
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, ActivatableAbilities, Params);

	Params.Condition = COND_ReplayOnly;
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, ClientDebugStrings, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, ServerDebugStrings, Params);
	
	Params.Condition = COND_OwnerOnly;
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, BlockedAbilityBindings, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, ReplicatedPredictionKeyMap, Params);
	
	Params.Condition = COND_SkipOwner;
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, MinimalReplicationGameplayCues, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UAbilitySystemComponent, MinimalReplicationTags, Params);

	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
}

void UAbilitySystemComponent::GetReplicatedCustomConditionState(FCustomPropertyConditionState& OutActiveState) const
{
	if (ActiveGameplayEffects.IsUsingReplicationCondition())
	{
		DOREPDYNAMICCONDITION_INITCONDITION_FAST(ThisClass, ActiveGameplayEffects, ActiveGameplayEffects.GetReplicationCondition());
	}
	DOREPCUSTOMCONDITION_ACTIVE_FAST(ThisClass, MinimalReplicationGameplayCues, MinimalReplicationGameplayCues.ShouldReplicate());
}

void UAbilitySystemComponent::UpdateActiveGameplayEffectsReplicationCondition()
{
	if (ActiveGameplayEffects.IsUsingReplicationCondition())
	{
		DOREPDYNAMICCONDITION_SETCONDITION_FAST(ThisClass, ActiveGameplayEffects, ActiveGameplayEffects.GetReplicationCondition());
	}
}

void UAbilitySystemComponent::UpdateMinimalReplicationGameplayCuesCondition()
{
	DOREPCUSTOMCONDITION_SETACTIVE_FAST(ThisClass, MinimalReplicationGameplayCues, MinimalReplicationGameplayCues.ShouldReplicate());
}

void UAbilitySystemComponent::ForceReplication()
{
	AActor *OwningActor = GetOwner();
	if (OwningActor)
	{
		OwningActor->ForceNetUpdate();

		if (ReplicationProxyEnabled && bForceReplicationAlsoUpdatesReplicatedProxyInterface)
		{
			IAbilitySystemReplicationProxyInterface* ReplicationProxy = GetReplicationInterface();
			if (ReplicationProxy && ReplicationProxy != this)
			{
				ReplicationProxy->ForceReplication();
			}
		}
	}
}

void UAbilitySystemComponent::ForceAvatarReplication()
{
	if (AActor* LocalAvatarActor = GetAvatarActor_Direct())
	{
		LocalAvatarActor->ForceNetUpdate();
	}
}

bool UAbilitySystemComponent::ReplicateSubobjects(UActorChannel* Channel, FOutBunch* Bunch, FReplicationFlags *RepFlags)
{
#if SUBOBJECT_TRANSITION_VALIDATION
	// When true it means we are calling this function to find any leftover replicated subobjects in classes that transitioned to the new registry list.
	// This shared class needs to keep supporting the old ways until we fully deprecate the API, so by only returning false we prevent the ensures to trigger
	if (UActorChannel::CanIgnoreDeprecatedReplicateSubObjects())
	{
		return false;
	}
#endif

	bool WroteSomething = Super::ReplicateSubobjects(Channel, Bunch, RepFlags);

	for (const UAttributeSet* Set : GetSpawnedAttributes())
	{
		if (IsValid(Set))
		{
			WroteSomething |= Channel->ReplicateSubobject(const_cast<UAttributeSet*>(Set), *Bunch, *RepFlags);
		}
	}

	for (UGameplayAbility* Ability : GetReplicatedInstancedAbilities())
	{
		if (IsValid(Ability))
		{
			WroteSomething |= Channel->ReplicateSubobject(Ability, *Bunch, *RepFlags);
		}
	}

	return WroteSomething;
}

void UAbilitySystemComponent::PreNetReceive()
{
	// Update the cached IsNetSimulated value here if this component is still considered authority.
	// Even though the value is also cached in OnRegister and BeginPlay, clients may
	// receive properties before OnBeginPlay, so this ensures the role is correct
	// for that case.
	if (!bCachedIsNetSimulated)
	{
		CacheIsNetSimulated();
	}
	ActiveGameplayEffects.IncrementLock();
}
	
void UAbilitySystemComponent::PostNetReceive()
{
	ActiveGameplayEffects.DecrementLock();
}

bool UAbilitySystemComponent::HasAuthorityOrPredictionKey(const FGameplayAbilityActivationInfo* ActivationInfo) const
{
	return ((ActivationInfo->ActivationMode == EGameplayAbilityActivationMode::Authority) || CanPredict());
}

void UAbilitySystemComponent::SetReplicationMode(EGameplayEffectReplicationMode NewReplicationMode)
{
	ReplicationMode = NewReplicationMode;

	// The changing of replication mode can affect replication conditions for ActiveGameplayEffects and MinimalReplicationGameplayCues.
	// It's ok to call these before the component is replicated, GetReplicatedCustomConditionState will make sure the conditions are up-to-date.
	UpdateActiveGameplayEffectsReplicationCondition();
	UpdateMinimalReplicationGameplayCuesCondition();
}

IAbilitySystemReplicationProxyInterface* UAbilitySystemComponent::GetReplicationInterface()
{
	if (ReplicationProxyEnabled)
	{
		// Note the expectation is that when the avatar actor is null (e.g during a respawn) that we do return null and calling code handles this (by probably not replicating whatever it was going to)
		return Cast<IAbilitySystemReplicationProxyInterface>(GetAvatarActor_Direct());
	}

	return Cast<IAbilitySystemReplicationProxyInterface>(this);
}

void UAbilitySystemComponent::OnPredictiveGameplayCueCatchup(FGameplayTag Tag)
{
	// Remove it
	RemoveLooseGameplayTag(Tag);

	if (HasMatchingGameplayTag(Tag) == 0)
	{
		// Invoke Removed event if we no longer have this tag (probably a mispredict)
		InvokeGameplayCueEvent(Tag, EGameplayCueEvent::Removed);
	}
}

void UAbilitySystemComponent::PrintAllGameplayEffects() const
{
	ABILITY_LOG(Log, TEXT("Owner: %s. Avatar: %s"), *GetOwner()->GetName(), *AbilityActorInfo->AvatarActor->GetName());
	ActiveGameplayEffects.PrintAllGameplayEffects();
}

#if ENABLE_VISUAL_LOG
static FVisualLogStatusCategory GrabDebugSnapshot_GameplayAbilities(const UAbilitySystemComponent* ASC)
{
	FVisualLogStatusCategory AllAbilitiesStatus;
	AllAbilitiesStatus.Category = TEXT("Gameplay Abilities");

	for (const FGameplayAbilitySpec& Spec : ASC->GetActivatableAbilities())
	{
		FVisualLogStatusCategory AbilityStatus;
		const UGameplayAbility* AbilitySource = Spec.GetPrimaryInstance() ? Spec.GetPrimaryInstance() : Spec.Ability.Get();
		AbilityStatus.Category = FString::Printf(TEXT("%s[%s] %s"), Spec.IsActive() ? TEXT("**") : TEXT(""), *Spec.Handle.ToString(), *GetNameSafe(AbilitySource));

		AbilityStatus.Add(TEXT("ActiveCount"), FString::Printf(TEXT("%d"), Spec.ActiveCount));
		AbilityStatus.Add(TEXT("Level"), FString::Printf(TEXT("%d"), Spec.Level));

		if (UObject* SourceObject = Spec.SourceObject.Get())
		{
			AbilityStatus.Add(TEXT("SourceObject"), *GetNameSafe(SourceObject));
		}

		FActiveGameplayEffectHandle AGEHandle = ASC->FindActiveGameplayEffectHandle(Spec.Handle);
		if (AGEHandle.IsValid())
		{
			if (const FActiveGameplayEffect* ActiveGE = ASC->GetActiveGameplayEffect(AGEHandle))
			{
				AbilityStatus.Add(TEXT("Granting Effect"), FString::Printf(TEXT("[%s] %s"), *AGEHandle.ToString(), *ActiveGE->Spec.ToSimpleString()));
			}
			else
			{
				AbilityStatus.Add(TEXT("Granting Effect"), FString::Printf(TEXT("[%s] NOT FOUND"), *AGEHandle.ToString()));
			}
		}

		if (FGameplayEventData* GameplayEventData = Spec.GameplayEventData.Get())
		{
			FVisualLogStatusCategory EventDataStatus;
			EventDataStatus.Category = TEXT("GameplayEventData");

			EventDataStatus.Add(TEXT("EventTag"), GameplayEventData->EventTag.ToString());

#define EventDataStatus_AddOptional(x) if (GameplayEventData-> x) { EventDataStatus.Add(TEXT("##x"), *GetNameSafe(GameplayEventData-> x)); }
			EventDataStatus_AddOptional(Instigator);
			EventDataStatus_AddOptional(Target);
			EventDataStatus_AddOptional(OptionalObject);
			EventDataStatus_AddOptional(OptionalObject2);
#undef EventDataStatus_AddOptional

			EventDataStatus.Add(TEXT("Context"), GameplayEventData->ContextHandle.ToString());

			if (GameplayEventData->InstigatorTags.Num())
			{
				EventDataStatus.Add(TEXT("InstigatorTags"), GameplayEventData->InstigatorTags.ToStringSimple());
			}

			if (GameplayEventData->TargetTags.Num())
			{
				EventDataStatus.Add(TEXT("TargetTags"), GameplayEventData->TargetTags.ToStringSimple());
			}

			if (GameplayEventData->EventMagnitude != 0.0f)
			{
				EventDataStatus.Add(TEXT("EventMagnitude"), FString::Printf(TEXT("%.3f"), GameplayEventData->EventMagnitude));
			}

			for (int32 Index = 0; Index < GameplayEventData->TargetData.Num(); ++Index)
			{
				FGameplayAbilityTargetData* TargetData = GameplayEventData->TargetData.Get(Index);
				EventDataStatus.Add(FString::Printf(TEXT("TargetData[%d]"), Index), TargetData ? *TargetData->ToString() : TEXT("null"));
			}

			AbilityStatus.AddChild(EventDataStatus);
		}

		AllAbilitiesStatus.AddChild(AbilityStatus);
	}

	return AllAbilitiesStatus;
}

void UAbilitySystemComponent::GrabDebugSnapshot(FVisualLogEntry* Snapshot) const
{
	Super::GrabDebugSnapshot(Snapshot);

	if (ActivatableAbilities.Items.Num() > 0)
	{
		FVisualLogStatusCategory AbilitiesStatus = GrabDebugSnapshot_GameplayAbilities(this);
		Snapshot->Status.Add(AbilitiesStatus);
	}

	ActiveGameplayEffects.DescribeSelfToVisLog(Snapshot);
}
#endif


bool UAbilitySystemComponent::IsOwnerActorAuthoritative() const
{
	return !bCachedIsNetSimulated;
}

bool UAbilitySystemComponent::ShouldRecordMontageReplication() const
{
	// Returns true IF the owner is authoritative OR the world is recording a replay.
	if (IsOwnerActorAuthoritative())
	{
		return true;
	}

	const UWorld* World = GetWorld();
	return World && World->IsRecordingReplay();
}

void UAbilitySystemComponent::OnAttributeAggregatorDirty(FAggregator* Aggregator, FGameplayAttribute Attribute, bool bFromRecursiveCall)
{
	ActiveGameplayEffects.OnAttributeAggregatorDirty(Aggregator, Attribute, bFromRecursiveCall);
}

void UAbilitySystemComponent::OnMagnitudeDependencyChange(FActiveGameplayEffectHandle Handle, const FAggregator* ChangedAggregator)
{
	ActiveGameplayEffects.OnMagnitudeDependencyChange(Handle, ChangedAggregator);
}

void UAbilitySystemComponent::OnGameplayEffectDurationChange(struct FActiveGameplayEffect& ActiveEffect)
{

}

void UAbilitySystemComponent::OnGameplayEffectAppliedToTarget(UAbilitySystemComponent* Target, const FGameplayEffectSpec& SpecApplied, FActiveGameplayEffectHandle ActiveHandle)
{
#if WITH_SERVER_CODE
	SCOPE_CYCLE_COUNTER(STAT_AbilitySystemComp_OnGameplayEffectAppliedToTarget);
#endif
	OnGameplayEffectAppliedDelegateToTarget.Broadcast(Target, SpecApplied, ActiveHandle);
}

void UAbilitySystemComponent::OnGameplayEffectAppliedToSelf(UAbilitySystemComponent* Source, const FGameplayEffectSpec& SpecApplied, FActiveGameplayEffectHandle ActiveHandle)
{
#if WITH_SERVER_CODE
	SCOPE_CYCLE_COUNTER(STAT_AbilitySystemComp_OnGameplayEffectAppliedToSelf);
#endif
	OnGameplayEffectAppliedDelegateToSelf.Broadcast(Source, SpecApplied, ActiveHandle);
}

void UAbilitySystemComponent::OnPeriodicGameplayEffectExecuteOnTarget(UAbilitySystemComponent* Target, const FGameplayEffectSpec& SpecExecuted, FActiveGameplayEffectHandle ActiveHandle)
{
	OnPeriodicGameplayEffectExecuteDelegateOnTarget.Broadcast(Target, SpecExecuted, ActiveHandle);
}

void UAbilitySystemComponent::OnPeriodicGameplayEffectExecuteOnSelf(UAbilitySystemComponent* Source, const FGameplayEffectSpec& SpecExecuted, FActiveGameplayEffectHandle ActiveHandle)
{
	OnPeriodicGameplayEffectExecuteDelegateOnSelf.Broadcast(Source, SpecExecuted, ActiveHandle);
}

TArray<TObjectPtr<UGameplayTask>>&	UAbilitySystemComponent::GetAbilityActiveTasks(UGameplayAbility* Ability)
{
	return Ability->ActiveTasks;
}

AActor* UAbilitySystemComponent::GetGameplayTaskAvatar(const UGameplayTask* Task) const
{
	check(AbilityActorInfo.IsValid());
	return AbilityActorInfo->AvatarActor.Get();
}

AActor* UAbilitySystemComponent::GetAvatarActor() const
{
	check(AbilityActorInfo.IsValid());
	return AbilityActorInfo->AvatarActor.Get();
}

void UAbilitySystemComponent::HandleDeferredGameplayCues(const FActiveGameplayEffectsContainer* GameplayEffectsContainer)
{
	for (const FActiveGameplayEffect& Effect : GameplayEffectsContainer)
	{
		if (Effect.bIsInhibited == false)
		{
			if (Effect.bPendingRepOnActiveGC)
			{
				InvokeGameplayCueEvent(Effect.Spec, EGameplayCueEvent::OnActive);
			}
			if (Effect.bPendingRepWhileActiveGC)
			{
				InvokeGameplayCueEvent(Effect.Spec, EGameplayCueEvent::WhileActive);
			}
		}

		Effect.bPendingRepOnActiveGC = false;
		Effect.bPendingRepWhileActiveGC = false;
	}
}

void UAbilitySystemComponent::DebugCyclicAggregatorBroadcasts(FAggregator* Aggregator)
{
	ActiveGameplayEffects.DebugCyclicAggregatorBroadcasts(Aggregator);
}

void UAbilitySystemComponent::OnRep_ClientDebugString()
{
	const TArray<FString>& LocalClientStrings = GetClientDebugStrings();

	ABILITY_LOG(Display, TEXT(" "));
	ABILITY_LOG(Display, TEXT("Received Client AbilitySystem Debug information: (%d lines)"), LocalClientStrings.Num());
	for (const FString& Str : LocalClientStrings)
	{
		ABILITY_LOG(Display, TEXT("%s"), *Str);
	}
}
void UAbilitySystemComponent::OnRep_ServerDebugString()
{
	const TArray<FString>& LocalServerStrings = GetClientDebugStrings();

	ABILITY_LOG(Display, TEXT(" "));
	ABILITY_LOG(Display, TEXT("Server AbilitySystem Debug information: (%d lines)"), LocalServerStrings.Num());
	for (const FString& Str : LocalServerStrings)
	{
		ABILITY_LOG(Display, TEXT("%s"), *Str);
	}
}

float UAbilitySystemComponent::GetFilteredAttributeValue(const FGameplayAttribute& Attribute, const FGameplayTagRequirements& SourceTags, const FGameplayTagContainer& TargetTags, const TArray<FActiveGameplayEffectHandle>& HandlesToIgnore)
{
	float AttributeValue = 0.f;

	UE_CLOGF(!SourceTags.TagQuery.IsEmpty(), LogAbilitySystem, Error, "GetFilteredAttributeValue: SourceTags cannot contain a TagQuery (it is ignored)");
	UE_CLOGF(!SourceTags.IgnoreTags.IsEmpty(), LogAbilitySystem, Error, "GetFilteredAttributeValue: SourceTags cannot contain IgnoreTags (they are not used)");

	if (SourceTags.RequireTags.Num() == 0 && SourceTags.IgnoreTags.Num() == 0 && HandlesToIgnore.Num() == 0)
	{
		// No qualifiers so we can just read this attribute normally
		AttributeValue = GetNumericAttribute(Attribute);
	}
	else
	{
		// Need to capture qualified attributes
		FGameplayEffectAttributeCaptureDefinition CaptureDef(Attribute.GetUProperty(), EGameplayEffectAttributeCaptureSource::Source, false);
		FGameplayEffectAttributeCaptureSpec CaptureSpec(CaptureDef);

		CaptureAttributeForGameplayEffect(CaptureSpec);

		// Source Tags
		static FGameplayTagContainer QuerySourceTags;
		QuerySourceTags.Reset();

		GetOwnedGameplayTags(QuerySourceTags);
		QuerySourceTags.AppendTags(SourceTags.RequireTags);

		// Target Tags
		static FGameplayTagContainer QueryTargetTags;
		QueryTargetTags.Reset();

		QueryTargetTags.AppendTags(TargetTags);

		FAggregatorEvaluateParameters Params;
		Params.SourceTags = &QuerySourceTags;
		Params.TargetTags = &QueryTargetTags;
		Params.IncludePredictiveMods = true;
		Params.IgnoreHandles = HandlesToIgnore;

		if (CaptureSpec.AttemptCalculateAttributeMagnitude(Params, AttributeValue) == false)
		{
			UE_LOGF(LogAbilitySystemComponent, Warning, "Failed to calculate Attribute %ls. On: %ls", *Attribute.GetName(), *GetFullName());
		}
	}

	return AttributeValue;
}

bool UAbilitySystemComponent::ServerPrintDebug_RequestWithStrings_Validate(const TArray<FString>& Strings)
{
	return true;
}

void UAbilitySystemComponent::ServerPrintDebug_RequestWithStrings_Implementation(const TArray<FString>& Strings)
{
	ABILITY_LOG(Display, TEXT(" "));
	ABILITY_LOG(Display, TEXT("Received Client AbilitySystem Debug information: "));
	for (const FString& Str : Strings)
	{
		ABILITY_LOG(Display, TEXT("%s"), *Str);
	}

	SetClientDebugStrings(TArray<FString>(Strings));
	ServerPrintDebug_Request_Implementation();
}

bool UAbilitySystemComponent::ServerPrintDebug_Request_Validate()
{
	return true;
}

void UAbilitySystemComponent::ServerPrintDebug_Request_Implementation()
{
	OnServerPrintDebug_Request();

	FAbilitySystemComponentDebugInfo DebugInfo;
	DebugInfo.bShowAbilities = true;
	DebugInfo.bShowAttributes = true;
	DebugInfo.bShowGameplayEffects = true;
	DebugInfo.Accumulate = true;
	DebugInfo.bPrintToLog = true;

	Debug_Internal(DebugInfo);

	SetServerDebugStrings(MoveTemp(DebugInfo.Strings));

	ClientPrintDebug_Response(DebugInfo.Strings, DebugInfo.GameFlags);
}

void UAbilitySystemComponent::OnServerPrintDebug_Request()
{

}

void UAbilitySystemComponent::ClientPrintDebug_Response_Implementation(const TArray<FString>& Strings, int32 GameFlags)
{
	OnClientPrintDebug_Response(Strings, GameFlags);
}
void UAbilitySystemComponent::OnClientPrintDebug_Response(const TArray<FString>& Strings, int32 GameFlags)
{
	ABILITY_LOG(Display, TEXT(" "));
	ABILITY_LOG(Display, TEXT("Server State: "));
	for (const FString& Str : Strings)
	{
		ABILITY_LOG(Display, TEXT("%s"), *Str);
	}


	// Now that we've heard back from server, append its strings and broadcast the delegate
	UAbilitySystemGlobals::Get().AbilitySystemDebugStrings.Append(Strings);
	UAbilitySystemGlobals::Get().OnClientServerDebugAvailable.Broadcast();
	UAbilitySystemGlobals::Get().AbilitySystemDebugStrings.Reset(); // we are done with this now. Clear it to signal that this can be ran again
}

FString UAbilitySystemComponent::CleanupName(FString Str)
{
	Str.RemoveFromStart(TEXT("Default__"));
	Str.RemoveFromEnd(TEXT("_c"));
	return Str;
}

void UAbilitySystemComponent::AccumulateScreenPos(FAbilitySystemComponentDebugInfo& Info)
{
	const float ColumnWidth = Info.Canvas ? Info.Canvas->ClipX * 0.4f : 0.f;

	float NewY = Info.YPos + Info.YL;
	if (NewY > Info.MaxY)
	{
		// Need new column, reset Y to original height
		NewY = Info.NewColumnYPadding;
		Info.XPos += ColumnWidth;
	}
	Info.YPos = NewY;
}

void UAbilitySystemComponent::DebugLine(FAbilitySystemComponentDebugInfo& Info, FString Str, float XOffset, float YOffset, int32 MinTextRowsToAdvance)
{
	if (Info.Canvas)
	{
		FFontRenderInfo RenderInfo = FFontRenderInfo();
		RenderInfo.bEnableShadow = true;
		if (const UFont* Font = GEngine->GetTinyFont())
		{
			float ScaleY = 1.f;
			Info.YL = Info.Canvas->DrawText(Font, Str, Info.XPos + XOffset, Info.YPos, 1.f, ScaleY, RenderInfo);
			if (Info.YL < MinTextRowsToAdvance * (Font->GetMaxCharHeight() * ScaleY))
			{
				Info.YL = MinTextRowsToAdvance * (Font->GetMaxCharHeight() * ScaleY);
			}
			AccumulateScreenPos(Info);
		}
	}

	if (Info.bPrintToLog)
	{
		FString LogStr;
		for (int32 i=0; i < (int32)XOffset; ++i)
		{
			LogStr += TEXT(" ");
		}
		LogStr += Str;
		ABILITY_LOG(Warning, TEXT("%s"), *LogStr);
	}

	if (Info.Accumulate)
	{
		FString LogStr;
		for (int32 i=0; i < (int32)XOffset; ++i)
		{
			LogStr += TEXT(" ");
		}
		LogStr += Str;
		Info.Strings.Add(Str);
	}
}

struct FASCDebugTargetInfo
{
	FASCDebugTargetInfo()
	{
		DebugCategoryIndex = 0;
		DebugCategories.Add(TEXT("Attributes"));
		DebugCategories.Add(TEXT("GameplayEffects"));
		DebugCategories.Add(TEXT("Ability"));
	}

	TArray<FName> DebugCategories;
	int32 DebugCategoryIndex;

	TWeakObjectPtr<UWorld>	TargetWorld;
	TWeakObjectPtr<UAbilitySystemComponent>	LastDebugTarget;
};

TArray<FASCDebugTargetInfo>	AbilitySystemDebugInfoList;

FASCDebugTargetInfo* GetDebugTargetInfo(UWorld* World)
{
	FASCDebugTargetInfo* TargetInfo = nullptr;
	for (FASCDebugTargetInfo& Info : AbilitySystemDebugInfoList )
	{
		if (Info.TargetWorld.Get() == World)
		{
			TargetInfo = &Info;
			break;
		}
	}
	if (TargetInfo == nullptr)
	{
		TargetInfo = &AbilitySystemDebugInfoList[AbilitySystemDebugInfoList.AddDefaulted()];
		TargetInfo->TargetWorld = World;
	}
	return TargetInfo;
}

static void CycleDebugCategory(UWorld* InWorld)
{
	FASCDebugTargetInfo* TargetInfo = GetDebugTargetInfo(InWorld);
	TargetInfo->DebugCategoryIndex = (TargetInfo->DebugCategoryIndex+1) % TargetInfo->DebugCategories.Num();
}

static void SetDebugCategory(const TArray<FString>& Args, UWorld* InWorld)
{
	if (Args.Num() < 1)
	{
		ABILITY_LOG(Error, TEXT("Missing category name parameter. Usage: AbilitySystem.Debug.SetCategory <CategoryName>"))
		return;
	}

	FASCDebugTargetInfo* TargetInfo = GetDebugTargetInfo(InWorld);
	check(TargetInfo);

	for (int32 CategoryIndex = 0; CategoryIndex < TargetInfo->DebugCategories.Num(); ++CategoryIndex)
	{
		const FString CategoryString = TargetInfo->DebugCategories[CategoryIndex].ToString();
		if (CategoryString.Equals(Args[0], ESearchCase::IgnoreCase))
		{
			TargetInfo->DebugCategoryIndex = CategoryIndex;
			return;
		}
	}

	ABILITY_LOG(Error, TEXT("Unable to match category name parameter [%s]. Usage: AbilitySystem.Debug.SetCategory <CategoryName>"), *Args[0]);
}

UAbilitySystemComponent* GetDebugTarget(FASCDebugTargetInfo* Info)
{
	// Return target if we already have one
	if (UAbilitySystemComponent* ASC = Info->LastDebugTarget.Get())
	{
		return ASC;
	}

	// Find one
	for (TObjectIterator<UAbilitySystemComponent> It; It; ++It)
	{
		if (UAbilitySystemComponent* ASC = *It)
		{
			// Make use it belongs to our world and will be valid in a TWeakObjPtr (e.g.  not pending kill)
			if (ASC->GetWorld() == Info->TargetWorld.Get() && MakeWeakObjectPtr(ASC).Get())
			{
				Info->LastDebugTarget = ASC;
				if (ASC->AbilityActorInfo != nullptr && ASC->AbilityActorInfo->IsLocallyControlledPlayer())
				{
					// Default to local player first
					break;
				}
			}
		}
	}

	return Info->LastDebugTarget.Get();
}

FAutoConsoleCommandWithWorld AbilitySystemDebugNextCategoryCmd(
	TEXT("AbilitySystem.Debug.NextCategory"),
	TEXT("Switches to the next ShowDebug AbilitySystem category"),
	FConsoleCommandWithWorldDelegate::CreateStatic(CycleDebugCategory)
	);

FAutoConsoleCommandWithWorldAndArgs AbilitySystemDebugSetCategoryCmd(
	TEXT("AbilitySystem.Debug.SetCategory"),
	TEXT("Sets the ShowDebug AbilitySystem category. Usage: AbilitySystem.Debug.SetCategory <CategoryName>"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(SetDebugCategory)
);

void UAbilitySystemComponent::OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	if (DisplayInfo.IsDisplayOn(TEXT("AbilitySystem")))
	{
		UWorld* World = HUD->GetWorld();
		FASCDebugTargetInfo* TargetInfo = GetDebugTargetInfo(World);
	
		UAbilitySystemComponent* ASC = nullptr;

		if (UAbilitySystemGlobals::Get().ShouldUseDebugTargetFromHud())
		{
			ASC = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(HUD->GetCurrentDebugTargetActor());
		}
		else
		{
			ASC = GetDebugTarget(TargetInfo);
		}

		if (ASC)
		{
			TArray<FName> LocalDisplayNames;
			LocalDisplayNames.Add( TargetInfo->DebugCategories[ TargetInfo->DebugCategoryIndex ] );

			FDebugDisplayInfo LocalDisplayInfo( LocalDisplayNames, TArray<FName>() );

			ASC->DisplayDebug(Canvas, LocalDisplayInfo, YL, YPos);
		}
	}
}

void UAbilitySystemComponent::DisplayDebug(class UCanvas* Canvas, const class FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	FAbilitySystemComponentDebugInfo DebugInfo;

	if (DebugDisplay.IsDisplayOn(FName(TEXT("Attributes"))))
	{
		DebugInfo.bShowAbilities = false;
		DebugInfo.bShowAttributes = true;
		DebugInfo.bShowGameplayEffects = false;
	}
	if (DebugDisplay.IsDisplayOn(FName(TEXT("Ability"))))
	{
		DebugInfo.bShowAbilities = true;
		DebugInfo.bShowAttributes = false;
		DebugInfo.bShowGameplayEffects = false;
	}
	else if (DebugDisplay.IsDisplayOn(FName(TEXT("GameplayEffects"))))
	{
		DebugInfo.bShowAbilities = false;
		DebugInfo.bShowAttributes = false;
		DebugInfo.bShowGameplayEffects = true;
	}

	DebugInfo.bPrintToLog = false;
	DebugInfo.Canvas = Canvas;
	DebugInfo.XPos = 0.f;
	DebugInfo.YPos = YPos;
	DebugInfo.OriginalX = 0.f;
	DebugInfo.OriginalY = YPos;
	DebugInfo.MaxY = Canvas->ClipY - 150.f; // Give some padding for any non-columnizing debug output following this output
	DebugInfo.NewColumnYPadding = 30.f;

	Debug_Internal(DebugInfo);

	YPos = DebugInfo.YPos;
	YL = DebugInfo.YL;
}

bool UAbilitySystemComponent::ShouldSendClientDebugStringsToServer() const
{
	// This implements basic throttling so that debug strings can't be sent more than once a second to the server
	const double MinTimeBetweenClientDebugSends = 1.f;
	static double LastSendTime = 0.f;

	double CurrentTime = FPlatformTime::Seconds();
	bool ShouldSend = (CurrentTime - LastSendTime) > MinTimeBetweenClientDebugSends;
	if (ShouldSend)
	{
		LastSendTime = CurrentTime;
	}
	return ShouldSend;
}

void UAbilitySystemComponent::PrintDebug()
{
	FAbilitySystemComponentDebugInfo DebugInfo;
	DebugInfo.bShowAbilities = true;
	DebugInfo.bShowAttributes = true;
	DebugInfo.bShowGameplayEffects = true;
	DebugInfo.bPrintToLog = true;
	DebugInfo.Accumulate = true;

	Debug_Internal(DebugInfo);

	// Store our local strings in the global debug array. Wait for server to respond with its strings.
	if (UAbilitySystemGlobals::Get().AbilitySystemDebugStrings.Num() > 0)
	{
		ABILITY_LOG(Warning, TEXT("UAbilitySystemComponent::PrintDebug called while AbilitySystemDebugStrings was not empty. Still waiting for server response from a previous call?"));
	}
		
	UAbilitySystemGlobals::Get().AbilitySystemDebugStrings = DebugInfo.Strings;

	if (IsOwnerActorAuthoritative() == false)
	{
		// See what the server thinks
		if (ShouldSendClientDebugStringsToServer())
		{
			ServerPrintDebug_RequestWithStrings(DebugInfo.Strings);
		}
		else
		{
			ServerPrintDebug_Request();
		}
	}
	else
	{
		UAbilitySystemGlobals::Get().OnClientServerDebugAvailable.Broadcast();
		UAbilitySystemGlobals::Get().AbilitySystemDebugStrings.Reset();
	}
}

void UAbilitySystemComponent::Debug_Internal(FAbilitySystemComponentDebugInfo& Info)
{
	// Draw title at top of screen (default HUD debug text starts at 50 ypos, we can position this on top)*
	//   *until someone changes it unknowingly
	{
		FString DebugTitle("");
		// Category
		if (Info.bShowAbilities)
		{
			DebugTitle += TEXT("ABILITIES ");
		}
		if (Info.bShowAttributes)
		{
			DebugTitle += TEXT("ATTRIBUTES ");
		}
		if (Info.bShowGameplayEffects)
		{
			DebugTitle += TEXT("GAMEPLAYEFFECTS ");
		}
		AActor* LocalAvatarActor = GetAvatarActor_Direct();
		AActor* LocalOwnerActor = GetOwnerActor();

		// Avatar info
		if (LocalAvatarActor)
		{
			const ENetRole AvatarRole = LocalAvatarActor->GetLocalRole();
			DebugTitle += FString::Printf(TEXT("for avatar %s "), *LocalAvatarActor->GetName());
			if (AvatarRole == ROLE_AutonomousProxy)
			{
				DebugTitle += TEXT("(local player) ");
			}
			else if (AvatarRole == ROLE_SimulatedProxy)
			{
				DebugTitle += TEXT("(simulated) ");
			}
			else if (AvatarRole == ROLE_Authority)
			{
				DebugTitle += TEXT("(authority) ");
			}
		}
		// Owner info
		if (LocalOwnerActor && LocalOwnerActor != LocalAvatarActor)
		{
			const ENetRole OwnerRole = LocalOwnerActor->GetLocalRole();
			DebugTitle += FString::Printf(TEXT("for owner %s "), *LocalOwnerActor->GetName());
			if (OwnerRole == ROLE_AutonomousProxy)
			{
				DebugTitle += TEXT("(autonomous) ");
			}
			else if (OwnerRole == ROLE_SimulatedProxy)
			{
				DebugTitle += TEXT("(simulated) ");
			}
			else if (OwnerRole == ROLE_Authority)
			{
				DebugTitle += TEXT("(authority) ");
			}
		}

		if (Info.Canvas)
		{
			Info.Canvas->SetDrawColor(FColor::White);
			FFontRenderInfo RenderInfo = FFontRenderInfo();
			RenderInfo.bEnableShadow = true;
			Info.Canvas->DrawText(GEngine->GetLargeFont(), DebugTitle, Info.XPos + 4.f, 10.f, 1.5f, 1.5f, RenderInfo);
		}
		else
		{
			DebugLine(Info, DebugTitle, 0.f, 0.f);
		}
	}

	DebugLine(Info, TEXT("Tip: Use the GameplayDebugger for enhanced functionality"), 4.0f, 0.0f);

	FGameplayTagContainer OwnerTags;
	GetOwnedGameplayTags(OwnerTags);

	if (Info.Canvas)
	{
		Info.Canvas->SetDrawColor(FColor::White);
	}

	FString TagsStrings;
	int32 TagCount = 1;
	const int32 NumTags = OwnerTags.Num();
	for (FGameplayTag Tag : OwnerTags)
	{
		TagsStrings.Append(FString::Printf(TEXT("%s (%d)"), *Tag.ToString(), GetTagCount(Tag)));

		if (TagCount++ < NumTags)
		{
			TagsStrings += TEXT(", ");
		}
	}
	DebugLine(Info, FString::Printf(TEXT("Owned Tags: %s"), *TagsStrings), 4.f, 0.f, 4);

	if (BlockedAbilityTags.GetExplicitGameplayTags().Num() > 0)
	{
		FString BlockedTagsStrings;
		int32 BlockedTagCount = 1;
		for (FGameplayTag Tag : BlockedAbilityTags.GetExplicitGameplayTags())
		{
			BlockedTagsStrings.Append(FString::Printf(TEXT("%s (%d)"), *Tag.ToString(), BlockedAbilityTags.GetTagCount(Tag)));

			if (BlockedTagCount++ < NumTags)
			{
				BlockedTagsStrings += TEXT(", ");
			}
		}
		DebugLine(Info, FString::Printf(TEXT("BlockedAbilityTags: %s"), *BlockedTagsStrings), 4.f, 0.f);
	}
	else
	{
		DebugLine(Info, FString::Printf(TEXT("BlockedAbilityTags: ")), 4.f, 0.f);
	}

	TSet<FGameplayAttribute> DrawAttributes;

	float MaxCharHeight = 10;
	if (GetOwner()->GetNetMode() != NM_DedicatedServer)
	{
		MaxCharHeight = GEngine->GetTinyFont()->GetMaxCharHeight();
	}

	// -------------------------------------------------------------

	if (Info.bShowAttributes)
	{
		// Draw the attribute aggregator map.
		for (auto It = ActiveGameplayEffects.AttributeAggregatorMap.CreateConstIterator(); It; ++It)
		{
			FGameplayAttribute Attribute = It.Key();
			const FAggregatorRef& AggregatorRef = It.Value();
			if (AggregatorRef.Get())
			{
				FAggregator& Aggregator = *AggregatorRef.Get();

				FAggregatorEvaluateParameters EmptyParams;

				TMap<EGameplayModEvaluationChannel, const TArray<FAggregatorMod>*> ModMap;
				Aggregator.EvaluateQualificationForAllMods(EmptyParams);
				Aggregator.GetAllAggregatorMods(ModMap);

				if (ModMap.Num() == 0)
				{
					continue;
				}

				float FinalValue = GetNumericAttribute(Attribute);
				float BaseValue = Aggregator.GetBaseValue();

				FString AttributeString = FString::Printf(TEXT("%s %.2f "), *Attribute.GetName(), GetNumericAttribute(Attribute));
				if (FMath::Abs<float>(BaseValue - FinalValue) > SMALL_NUMBER)
				{
					AttributeString += FString::Printf(TEXT(" (Base: %.2f)"), BaseValue);
				}

				if (Info.Canvas)
				{
					Info.Canvas->SetDrawColor(FColor::White);
				}

				DebugLine(Info, AttributeString, 4.f, 0.f);

				DrawAttributes.Add(Attribute);

 				for (const auto& CurMapElement : ModMap)
 				{
					const EGameplayModEvaluationChannel Channel = CurMapElement.Key;
					const TArray<FAggregatorMod>* ModArrays = CurMapElement.Value;

					const FString ChannelNameString = UAbilitySystemGlobals::Get().GetGameplayModEvaluationChannelAlias(Channel).ToString();
					for (int32 ModOpIdx = 0; ModOpIdx < EGameplayModOp::Max; ++ModOpIdx)
					{
						const TArray<FAggregatorMod>& CurModArray = ModArrays[ModOpIdx];
						for (const FAggregatorMod& Mod : CurModArray)
						{
							bool IsActivelyModifyingAttribute = Mod.Qualifies();
							if (Info.Canvas)
							{
								Info.Canvas->SetDrawColor(IsActivelyModifyingAttribute ? FColor::Yellow : FColor(128, 128, 128));
							}

							FActiveGameplayEffect* ActiveGE = ActiveGameplayEffects.GetActiveGameplayEffect(Mod.ActiveHandle);
							FString SrcName = ActiveGE ? ActiveGE->Spec.Def->GetName() : FString(TEXT(""));

							if (IsActivelyModifyingAttribute == false)
							{
								if (Mod.SourceTagReqs)
								{
									SrcName += FString::Printf(TEXT(" SourceTags: [%s] "), *Mod.SourceTagReqs->ToString());
								}
								if (Mod.TargetTagReqs)
								{
									SrcName += FString::Printf(TEXT("TargetTags: [%s]"), *Mod.TargetTagReqs->ToString());
								}
							}

							DebugLine(Info, FString::Printf(TEXT("   %s %s\t %.2f - %s"), *ChannelNameString, *EGameplayModOpToString(ModOpIdx), Mod.EvaluatedMagnitude, *SrcName), 7.f, 0.f);
							Info.NewColumnYPadding = FMath::Max<float>(Info.NewColumnYPadding, Info.YPos + Info.YL);
						}
					}
 				}

				AccumulateScreenPos(Info);
			}
		}
	}

	// -------------------------------------------------------------

	if (Info.bShowGameplayEffects)
	{
		for (FActiveGameplayEffect& ActiveGE : &ActiveGameplayEffects)
		{
			if (Info.Canvas)
			{
				Info.Canvas->SetDrawColor(FColor::White);
			}

			FString DurationStr = TEXT("Infinite Duration ");
			if (ActiveGE.GetDuration() > 0.f)
			{
				DurationStr = FString::Printf(TEXT("Duration: %.2f. Remaining: %.2f (Start: %.2f / %.2f / %.2f) %s "), ActiveGE.GetDuration(), ActiveGE.GetTimeRemaining(GetWorld()->GetTimeSeconds()), ActiveGE.StartServerWorldTime, ActiveGE.CachedStartServerWorldTime, ActiveGE.StartWorldTime, ActiveGE.DurationHandle.IsValid() ? TEXT("Valid Handle") : TEXT("INVALID Handle") );
				if (ActiveGE.DurationHandle.IsValid())
				{
					DurationStr += FString::Printf(TEXT("(Local Duration: %.2f)"), GetWorld()->GetTimerManager().GetTimerRemaining(ActiveGE.DurationHandle));
				}
			}
			if (ActiveGE.GetPeriod() > 0.f)
			{
				DurationStr += FString::Printf(TEXT("Period: %.2f"), ActiveGE.GetPeriod());
			}

			FString StackString;
			if (ActiveGE.Spec.GetStackCount() > 1)
			{

				if (ActiveGE.Spec.Def->GetStackingType() == EGameplayEffectStackingType::AggregateBySource)
				{
					StackString = FString::Printf(TEXT("(Stacks: %d. From: %s) "), ActiveGE.Spec.GetStackCount(), ActiveGE.Spec.GetContext().GetInstigatorAbilitySystemComponent() ? *GetNameSafe(ActiveGE.Spec.GetContext().GetInstigatorAbilitySystemComponent()->GetAvatarActor_Direct()) : TEXT("Invalid"));
				}
				else
				{
					StackString = FString::Printf(TEXT("(Stacks: %d) "), ActiveGE.Spec.GetStackCount());
				}
			}

			FString LevelString;
			if (ActiveGE.Spec.GetLevel() > 1.f)
			{
				LevelString = FString::Printf(TEXT("Level: %.2f"), ActiveGE.Spec.GetLevel());
			}

			FString PredictionString;
			if (ActiveGE.PredictionKey.IsValidKey())
			{
				if (ActiveGE.PredictionKey.WasLocallyGenerated() )
				{
					PredictionString = FString::Printf(TEXT("(Predicted and Waiting)"));
				}
				else
				{
					PredictionString = FString::Printf(TEXT("(Predicted and Caught Up)"));
				}
			}

			if (Info.Canvas)
			{
				Info.Canvas->SetDrawColor(ActiveGE.bIsInhibited ? FColor(128, 128, 128) : FColor::White);
			}

			DebugLine(Info, FString::Printf(TEXT("%s %s %s %s %s"), *CleanupName(GetNameSafe(ActiveGE.Spec.Def)), *DurationStr, *StackString, *LevelString, *PredictionString), 4.f, 0.f);

			FGameplayTagContainer GrantedTags;
			ActiveGE.Spec.GetAllGrantedTags(GrantedTags);
			if (GrantedTags.Num() > 0)
			{
				DebugLine(Info, FString::Printf(TEXT("Granted Tags: %s"), *GrantedTags.ToStringSimple()), 7.f, 0.f);
			}

			for (int32 ModIdx = 0; ModIdx < ActiveGE.Spec.Modifiers.Num(); ++ModIdx)
			{
				if (ActiveGE.Spec.Def == nullptr)
				{
					DebugLine(Info, FString::Printf(TEXT("null def! (Backwards compat?)")), 7.f, 0.f);
					continue;
				}

				const FModifierSpec& ModSpec = ActiveGE.Spec.Modifiers[ModIdx];
				const FGameplayModifierInfo& ModInfo = ActiveGE.Spec.Def->Modifiers[ModIdx];

				DebugLine(Info, FString::Printf(TEXT("Mod: %s. %s. %.2f"), *ModInfo.Attribute.GetName(), *EGameplayModOpToString(ModInfo.ModifierOp), ModSpec.GetEvaluatedMagnitude()), 7.f, 0.f);

				if (Info.Canvas)
				{
					Info.Canvas->SetDrawColor(ActiveGE.bIsInhibited ? FColor(128, 128, 128) : FColor::White);
				}
			}

			AccumulateScreenPos(Info);
		}
	}

	// -------------------------------------------------------------

	if (Info.bShowAttributes)
	{
		if (Info.Canvas)
		{
			Info.Canvas->SetDrawColor(FColor::White);
		}
		for (UAttributeSet* Set : GetSpawnedAttributes())
		{
			if (!Set)
			{
				continue;
			}

			TArray<FGameplayAttribute> Attributes;
			UAttributeSet::GetAttributesFromSetClass(Set->GetClass(), Attributes);
			for (const FGameplayAttribute& Attribute : Attributes)
			{
				if (DrawAttributes.Contains(Attribute))
					continue;

				if (Attribute.IsValid())
				{
					const float Value = GetNumericAttribute(Attribute);
					DebugLine(Info, FString::Printf(TEXT("%s %.2f"), *Attribute.GetName(), Value), 4.f, 0.f);
				}
			}
		}
		AccumulateScreenPos(Info);
	}

	// -------------------------------------------------------------

	bool bShowAbilityTaskDebugMessages = true;

	if (Info.bShowAbilities)
	{
		for (const FGameplayAbilitySpec& AbilitySpec : GetActivatableAbilities())
		{
			if (AbilitySpec.Ability == nullptr)
				continue;

			FString StatusText;
			FColor AbilityTextColor = FColor(128, 128, 128);
			FGameplayTagContainer FailureTags;
			const TArray<uint8>& LocalBlockedAbilityBindings = GetBlockedAbilityBindings();

			// We prefer executing on the instanced ability if we're instancing
			UGameplayAbility* AbilitySource = AbilitySpec.GetPrimaryInstance();
			if (!AbilitySource)
			{
				AbilitySource = AbilitySpec.Ability.Get();
			}

			if (AbilitySpec.IsActive())
			{
				StatusText = FString::Printf(TEXT(" (Active %d)"), AbilitySpec.ActiveCount);
				AbilityTextColor = FColor::Yellow;
			}
			else if (LocalBlockedAbilityBindings.IsValidIndex(AbilitySpec.InputID) && LocalBlockedAbilityBindings[AbilitySpec.InputID])
			{
				StatusText = TEXT(" (InputBlocked)");
				AbilityTextColor = FColor::Red;
			}
			else if (AbilitySource->GetAssetTags().HasAny(BlockedAbilityTags.GetExplicitGameplayTags()))
			{
				StatusText = TEXT(" (TagBlocked)");
				AbilityTextColor = FColor::Red;
			}
			else if (AbilitySource->CanActivateAbility(AbilitySpec.Handle, AbilityActorInfo.Get(), nullptr, nullptr, &FailureTags) == false)
			{
				StatusText = FString::Printf(TEXT(" (CantActivate %s)"), *FailureTags.ToString());
				AbilityTextColor = FColor::Red;

				float Cooldown =  AbilitySpec.Ability->GetCooldownTimeRemaining(AbilityActorInfo.Get());
				if (Cooldown > 0.f)
				{
					StatusText += FString::Printf(TEXT("   Cooldown: %.2f\n"), Cooldown);
				}
			}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FString InputPressedStr = AbilitySpec.InputPressed ? TEXT("(InputPressed)") : TEXT("");
			FString ActivationModeStr = AbilitySpec.IsActive() ? UEnum::GetValueAsString(TEXT("GameplayAbilities.EGameplayAbilityActivationMode"), AbilitySpec.ActivationInfo.ActivationMode) : TEXT("");
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			if (Info.Canvas)
			{
				Info.Canvas->SetDrawColor(AbilityTextColor);
			}

			DebugLine(Info, FString::Printf(TEXT("%s %s %s %s"), *CleanupName(GetNameSafe(AbilitySpec.Ability)), *StatusText, *InputPressedStr, *ActivationModeStr), 4.f, 0.f);

			if (AbilitySpec.IsActive())
			{
				TArray<UGameplayAbility*> Instances = AbilitySpec.GetAbilityInstances();
				for (int32 InstanceIdx = 0; InstanceIdx < Instances.Num(); ++InstanceIdx)
				{
					UGameplayAbility* Instance = Instances[InstanceIdx];
					if (!Instance)
						continue;

					if (Info.Canvas)
					{
						Info.Canvas->SetDrawColor(FColor::White);
					}
					for (UGameplayTask* Task : Instance->ActiveTasks)
					{
						if (Task)
						{
							DebugLine(Info, FString::Printf(TEXT("%s"), *Task->GetDebugString()), 7.f, 0.f);

							if (bShowAbilityTaskDebugMessages)
							{
								for (FAbilityTaskDebugMessage& Msg : Instance->TaskDebugMessages)
								{
									if (Msg.FromTask == Task)
									{
										DebugLine(Info, FString::Printf(TEXT("%s"), *Msg.Message), 9.f, 0.f);
									}
								}
							}
						}
					}

					bool FirstTaskMsg=true;
					int32 MsgCount = 0;
					for (FAbilityTaskDebugMessage& Msg : Instance->TaskDebugMessages)
					{
						// Cap finished task msgs to 5 per ability if we are printing to screen (else things will scroll off)
						if ( Info.Canvas && ++MsgCount > 5 )
						{
							break;
						}

						if (Instance->ActiveTasks.Contains(Msg.FromTask) == false)
						{
							if (FirstTaskMsg)
							{
								DebugLine(Info, TEXT("[FinishedTasks]"), 7.f, 0.f);
								FirstTaskMsg = false;
							}

							DebugLine(Info, FString::Printf(TEXT("%s"), *Msg.Message), 9.f, 0.f);
						}
					}

					if (InstanceIdx < Instances.Num() - 2)
					{
						if (Info.Canvas)
						{
							Info.Canvas->SetDrawColor(FColor(128, 128, 128));
						}
						DebugLine(Info, FString::Printf(TEXT("--------")), 7.f, 0.f);
					}
				}
			}
		}
		AccumulateScreenPos(Info);
	}

	if (Info.XPos > Info.OriginalX)
	{
		// We flooded to new columns, returned YPos should be max Y (and some padding)
		Info.YPos = Info.MaxY + MaxCharHeight*2.f;
	}
	Info.YL = MaxCharHeight;
}

void UAbilitySystemComponent::SetOwnerActor(AActor* NewOwnerActor)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(UAbilitySystemComponent, OwnerActor, this);
	if (OwnerActor)
	{
		OwnerActor->OnDestroyed.RemoveDynamic(this, &UAbilitySystemComponent::OnOwnerActorDestroyed);
	}
	OwnerActor = NewOwnerActor;
	if (OwnerActor)
	{
		OwnerActor->OnDestroyed.AddUniqueDynamic(this, &UAbilitySystemComponent::OnOwnerActorDestroyed);
	}
}

void UAbilitySystemComponent::SetAvatarActor_Direct(AActor* NewAvatarActor)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(UAbilitySystemComponent, AvatarActor, this);
	if (AvatarActor)
	{
		AvatarActor->OnDestroyed.RemoveDynamic(this, &UAbilitySystemComponent::OnAvatarActorDestroyed);
	}
	AvatarActor = NewAvatarActor;
	if (AvatarActor)
	{
		AvatarActor->OnDestroyed.AddUniqueDynamic(this, &UAbilitySystemComponent::OnAvatarActorDestroyed);
	}
}

void UAbilitySystemComponent::OnAvatarActorDestroyed(AActor* InActor)
{
	if (InActor == AvatarActor)
	{
		AvatarActor = nullptr;
		MARK_PROPERTY_DIRTY_FROM_NAME(UAbilitySystemComponent, AvatarActor, this);
	}
}

void UAbilitySystemComponent::OnOwnerActorDestroyed(AActor* InActor)
{
	if (InActor == OwnerActor)
	{
		OwnerActor = nullptr;
		MARK_PROPERTY_DIRTY_FROM_NAME(UAbilitySystemComponent, OwnerActor, this);
	}
}

void UAbilitySystemComponent::SetSpawnedAttributes(const TArray<UAttributeSet*>& NewSpawnedAttributes)
{
	for (UAttributeSet* AttributeSet : SpawnedAttributes)
	{
		if (AttributeSet)
		{
			AActor* ActorOwner = AttributeSet->GetTypedOuter<AActor>();
			if (ActorOwner)
			{
				ActorOwner->OnEndPlay.RemoveDynamic(this, &UAbilitySystemComponent::OnSpawnedAttributesEndPlayed);
			}
		}
	}

	// Clean the previous list
	RemoveAllSpawnedAttributes();

	// Add the elements from the new list
	for (UAttributeSet* NewAttribute : NewSpawnedAttributes)
	{
		if (NewAttribute)
		{
			AddSpawnedAttribute(NewAttribute);

			AActor* ActorOwner = NewAttribute->GetTypedOuter<AActor>();
			if (ActorOwner)
			{
				ActorOwner->OnEndPlay.AddUniqueDynamic(this, &UAbilitySystemComponent::OnSpawnedAttributesEndPlayed);
			}
		}
	}

	SetSpawnedAttributesListDirty();
}

void UAbilitySystemComponent::AddSpawnedAttribute(UAttributeSet* Attribute)
{
	if (!IsValid(Attribute))
	{
		return;
	}

	if (SpawnedAttributes.Find(Attribute) == INDEX_NONE)
	{
		if (IsUsingRegisteredSubObjectList() && IsReadyForReplication())
		{
			AddReplicatedSubObject(Attribute);
		}

		SpawnedAttributes.Add(Attribute);
		SetSpawnedAttributesListDirty();
	}
}

void UAbilitySystemComponent::RemoveSpawnedAttribute(UAttributeSet* AttributeSet)
{
	if (SpawnedAttributes.RemoveSingle(AttributeSet) > 0)
	{
		if (IsUsingRegisteredSubObjectList())
		{
			RemoveReplicatedSubObject(AttributeSet);
		}

		TArray<FGameplayAttribute> Attributes;
		UAttributeSet::GetAttributesFromSetClass(AttributeSet->GetClass(), Attributes);
		for (const FGameplayAttribute& Attribute : Attributes)
		{
			ABILITY_LOG(Log, TEXT("Cleaning up aggregator for attribute '%s' due to RemoveSpawnedAttribute removing attribute set '%s'"), *Attribute.GetName(), *AttributeSet->GetName());
			ActiveGameplayEffects.CleanupAttributeAggregator(Attribute);
		}

		SetSpawnedAttributesListDirty();
	}
}

void UAbilitySystemComponent::RemoveAllSpawnedAttributes()
{
	if (IsUsingRegisteredSubObjectList())
	{
		for (UAttributeSet* OldAttribute : SpawnedAttributes)
		{
			RemoveReplicatedSubObject(OldAttribute);
		}
	}

	SpawnedAttributes.Empty();
	SetSpawnedAttributesListDirty();
}

void UAbilitySystemComponent::SetSpawnedAttributesListDirty()
{
	MARK_PROPERTY_DIRTY_FROM_NAME(UAbilitySystemComponent, SpawnedAttributes, this);
}

TArray<TObjectPtr<UAttributeSet>>& UAbilitySystemComponent::GetSpawnedAttributes_Mutable()
{
	SetSpawnedAttributesListDirty();
	return SpawnedAttributes;
}

const TArray<UAttributeSet*>& UAbilitySystemComponent::GetSpawnedAttributes() const
{
	return SpawnedAttributes;
}

void UAbilitySystemComponent::OnRep_SpawnedAttributes(const TArray<UAttributeSet*>& PreviousSpawnedAttributes)
{
	if (IsUsingRegisteredSubObjectList())
	{
		// Find the attributes that got removed
		for (UAttributeSet* PreviousAttributeSet : PreviousSpawnedAttributes)
		{
			if (PreviousAttributeSet)
			{
				const bool bWasRemoved = SpawnedAttributes.Find(PreviousAttributeSet) == INDEX_NONE;
				if (bWasRemoved)
				{
					RemoveReplicatedSubObject(PreviousAttributeSet);
				}
			}
		}

		// Find the attributes that got added
		for (UAttributeSet* NewAttributeSet : SpawnedAttributes)
		{
			if (IsValid(NewAttributeSet))
			{
				const bool bIsAdded = PreviousSpawnedAttributes.Find(NewAttributeSet) == INDEX_NONE;
				if (bIsAdded)
				{
					AddReplicatedSubObject(NewAttributeSet);
				}
			}
		}
	}

	// Create a map of attribute set class to latest object to detect removed and replaced attribute sets.
	TMap<TSubclassOf<UAttributeSet>, UAttributeSet*> NewAttributeSetClasses;
	for (UAttributeSet* NewAttributeSet : SpawnedAttributes)
	{
		if (IsValid(NewAttributeSet))
		{
			NewAttributeSetClasses.Add(NewAttributeSet->GetClass(), NewAttributeSet);
		}
	}

	// Find the attribute sets that got removed or replaced by one of the same class. Replacement here means that the an attribute set instance
	// is no longer in SpawnedAttributes, but a new one exists of the same class. This happens when the client and server both created an attribute set
	// and the server's instance is replicated over, for example when using DefaultStartingData. For removed attribute sets, their aggregators should be 
	// cleaned up. For replaced attribute sets, the aggregators should remain valid.
	for (UAttributeSet* PreviousAttributeSet : PreviousSpawnedAttributes)
	{
		if (PreviousAttributeSet)
		{
			UClass* AttributeSetClass = PreviousAttributeSet->GetClass();
			TArray<FGameplayAttribute> Attributes;
			UAttributeSet::GetAttributesFromSetClass(PreviousAttributeSet->GetClass(), Attributes);

			// Case: Attribute set removed and not replaced
			if (!NewAttributeSetClasses.Contains(PreviousAttributeSet->GetClass()))
			{
				// The attribute set no longer exists after removal on the server.
				// Remove any aggregators for all the attributes in it.
				for (const FGameplayAttribute& Attribute : Attributes)
				{
					ABILITY_LOG(Log, TEXT("Cleaning up aggregator for attribute '%s' due to OnRep_SpawnedAttributes detecting removal of '%s' without replacement"), *Attribute.GetName(), *PreviousAttributeSet->GetName());
					ActiveGameplayEffects.CleanupAttributeAggregator(Attribute);
				}
			}
			// Case: Attribute set replaced by server-replicated object
			else if (PreviousAttributeSet != NewAttributeSetClasses[AttributeSetClass])
			{
				UAttributeSet* NewAttributeSet = NewAttributeSetClasses[AttributeSetClass];
				for (const FGameplayAttribute& Attribute : Attributes)
				{
					ABILITY_LOG(Log, TEXT("Keeping aggregator for attribute '%s' due to OnRep_SpawnedAttributes detecting replacement '%s' for '%s'"), *Attribute.GetName(), *NewAttributeSet->GetName(), *PreviousAttributeSet->GetName());

					// Since we're replacing the attribute set with the server replicated one, give it a chance to do setup on the 
					// existing aggregator like metadata and binding to its delegates. However, only do this for non-system attributes, 
					// mirroring behavior in FindOrCreateAttributeAggregator(). Also don't actually created the aggregator if it doesn't 
					// exist. Aggregators are created on-demand when they are first modified.
					if (Attribute.IsSystemAttribute() == false && ActiveGameplayEffects.AttributeAggregatorMap.Contains(Attribute))
					{
						FAggregatorRef& Aggregator = ActiveGameplayEffects.FindOrCreateAttributeAggregator(Attribute);
						NewAttributeSet->OnAttributeAggregatorCreated(Attribute, Aggregator.Get());
					}
				}
			}
		}
	}
}

void UAbilitySystemComponent::AddReplicatedInstancedAbility(UGameplayAbility* GameplayAbility)
{
	if (AllReplicatedInstancedAbilities.Find(GameplayAbility) == INDEX_NONE)
	{
		AllReplicatedInstancedAbilities.Add(GameplayAbility);
		
		if (IsUsingRegisteredSubObjectList() && IsReadyForReplication())
		{
			const ELifetimeCondition LifetimeCondition = bReplicateAbilitiesToSimulatedProxies ? COND_None : COND_ReplayOrOwner;
			AddReplicatedSubObject(GameplayAbility, LifetimeCondition);
		}
	}
}

void UAbilitySystemComponent::RemoveReplicatedInstancedAbility(UGameplayAbility* GameplayAbility)
{
	const bool bWasRemoved = AllReplicatedInstancedAbilities.RemoveSingle(GameplayAbility) > 0;
	if (bWasRemoved && IsUsingRegisteredSubObjectList() && IsReadyForReplication())
	{
		RemoveReplicatedSubObject(GameplayAbility);
	}
}

void UAbilitySystemComponent::RemoveAllReplicatedInstancedAbilities()
{
	if (IsUsingRegisteredSubObjectList() && IsReadyForReplication())
	{
		for (UGameplayAbility* ReplicatedAbility : AllReplicatedInstancedAbilities)
		{
			RemoveReplicatedSubObject(ReplicatedAbility);
		}
	}

	AllReplicatedInstancedAbilities.Empty();
}

void UAbilitySystemComponent::OnSpawnedAttributesEndPlayed(AActor* InActor, EEndPlayReason::Type EndPlayReason)
{
	for (int32 Index = SpawnedAttributes.Num() - 1; Index >= 0; --Index)
	{
		UAttributeSet* AttributeSet = SpawnedAttributes[Index];
		if (AttributeSet && AttributeSet->GetTypedOuter<AActor>() == InActor)
		{
			if (IsUsingRegisteredSubObjectList())
			{
				RemoveReplicatedSubObject(AttributeSet);
			}
			
			SpawnedAttributes[Index] = nullptr;
		}
	}

	SetSpawnedAttributesListDirty();
}

void UAbilitySystemComponent::SetClientDebugStrings(TArray<FString>&& NewClientDebugStrings)
{
	GetClientDebugStrings_Mutable() = NewClientDebugStrings;
}


TArray<FString>& UAbilitySystemComponent::GetClientDebugStrings_Mutable()
{
	MARK_PROPERTY_DIRTY_FROM_NAME(UAbilitySystemComponent, ClientDebugStrings, this);
	return ClientDebugStrings;
}

const TArray<FString>& UAbilitySystemComponent::GetClientDebugStrings() const
{
	MARK_PROPERTY_DIRTY_FROM_NAME(UAbilitySystemComponent, ClientDebugStrings, this);
	return ClientDebugStrings;
}

void UAbilitySystemComponent::SetServerDebugStrings(TArray<FString>&& NewServerDebugStrings)
{
	GetServerDebugStrings_Mutable() = NewServerDebugStrings;
}

TArray<FString>& UAbilitySystemComponent::GetServerDebugStrings_Mutable()
{
	MARK_PROPERTY_DIRTY_FROM_NAME(UAbilitySystemComponent, ServerDebugStrings, this);
	return ServerDebugStrings;
}

const TArray<FString>& UAbilitySystemComponent::GetServerDebugStrings() const
{
	return ServerDebugStrings;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UAbilitySystemComponent::SetRepAnimMontageInfo(const FGameplayAbilityRepAnimMontage& NewRepAnimMontageInfo)
{
	GetRepAnimMontageInfo_Mutable() = NewRepAnimMontageInfo;
}

FGameplayAbilityRepAnimMontage& UAbilitySystemComponent::GetRepAnimMontageInfo_Mutable()
{
	MARK_PROPERTY_DIRTY_FROM_NAME(UAbilitySystemComponent, RepAnimMontageInfo, this);
	return RepAnimMontageInfo;
}

const FGameplayAbilityRepAnimMontage& UAbilitySystemComponent::GetRepAnimMontageInfo() const
{
	return RepAnimMontageInfo;
}

void UAbilitySystemComponent::SetBlockedAbilityBindings(const TArray<uint8>& NewBlockedAbilityBindings)
{
	GetBlockedAbilityBindings_Mutable() = NewBlockedAbilityBindings;
}

TArray<uint8>& UAbilitySystemComponent::GetBlockedAbilityBindings_Mutable()
{
	MARK_PROPERTY_DIRTY_FROM_NAME(UAbilitySystemComponent, BlockedAbilityBindings, this);
	return BlockedAbilityBindings;
}

const TArray<uint8>& UAbilitySystemComponent::GetBlockedAbilityBindings() const
{
	return BlockedAbilityBindings;
}

void UAbilitySystemComponent::SetMinimalReplicationTags(const FMinimalReplicationTagCountMap& NewMinimalReplicationTags)
{
	GetMinimalReplicationTags_Mutable() = NewMinimalReplicationTags;
}

FMinimalReplicationTagCountMap& UAbilitySystemComponent::GetMinimalReplicationTags_Mutable()
{
	MARK_PROPERTY_DIRTY_FROM_NAME(UAbilitySystemComponent, MinimalReplicationTags, this);
	return MinimalReplicationTags;
}

const FMinimalReplicationTagCountMap& UAbilitySystemComponent::GetMinimalReplicationTags() const
{
	return MinimalReplicationTags;
}

FMinimalReplicationTagCountMap& UAbilitySystemComponent::GetReplicatedLooseTags_Mutable()
{
	MARK_PROPERTY_DIRTY_FROM_NAME(UAbilitySystemComponent, ReplicatedLooseTags, this);
	return ReplicatedLooseTags;
}

const FMinimalReplicationTagCountMap& UAbilitySystemComponent::GetReplicatedLooseTags() const
{
	return ReplicatedLooseTags;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
