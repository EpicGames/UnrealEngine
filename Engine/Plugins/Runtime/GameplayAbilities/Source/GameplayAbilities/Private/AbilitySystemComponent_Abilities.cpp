// Copyright Epic Games, Inc. All Rights Reserved.
// ActorComponent.cpp: Actor component implementation.

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "EngineDefines.h"
#include "Engine/NetSerialization.h"
#include "Engine/World.h"
#include "Templates/SubclassOf.h"
#include "Components/InputComponent.h"
#include "GameplayTagContainer.h"
#include "TimerManager.h"
#include "AbilitySystemLog.h"
#include "AttributeSet.h"
#include "GameplayAbilitiesDeveloperSettings.h"
#include "GameplayPrediction.h"
#include "GameplayEffectTypes.h"
#include "GameplayAbilitySpec.h"
#include "UObject/UObjectHash.h"
#include "GameFramework/PlayerController.h"
#include "Abilities/GameplayAbilityTypes.h"
#include "AbilitySystemStats.h"
#include "AbilitySystemGlobals.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimInstance.h"
#include "Abilities/GameplayAbilityTargetTypes.h"
#include "Abilities/GameplayAbility.h"
#include "AbilitySystemComponent.h"
#include "Abilities/GameplayAbilityTargetActor.h"
#include "TickableAttributeSetInterface.h"
#include "GameplayTagResponseTable.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "VisualLogger/VisualLogger.h"

#define LOCTEXT_NAMESPACE "AbilitySystemComponent"

/** Enable to log out all render state create, destroy and updatetransform events */
#define LOG_RENDER_STATE 0
DECLARE_CYCLE_STAT(TEXT("AbilitySystemComp ServerTryActivate"), STAT_AbilitySystemComp_ServerTryActivate, STATGROUP_AbilitySystem);
DECLARE_CYCLE_STAT(TEXT("AbilitySystemComp ServerEndAbility"), STAT_AbilitySystemComp_ServerEndAbility, STATGROUP_AbilitySystem);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);

static TAutoConsoleVariable<float> CVarReplayMontageErrorThreshold(TEXT("replay.MontageErrorThreshold"), 0.5f, TEXT("Tolerance level for when montage playback position correction occurs in replays"));
static TAutoConsoleVariable<bool> CVarAbilitySystemSetActivationInfoMultipleTimes(TEXT("AbilitySystem.SetActivationInfoMultipleTimes"), false, TEXT("Set this to true if some replicated Gameplay Abilities aren't setting their owning actors correctly"));
static TAutoConsoleVariable<bool> CVarGasFixClientSideMontageBlendOutTime(TEXT("AbilitySystem.Fix.ClientSideMontageBlendOutTime"), true, TEXT("Enable a fix to replicate the Montage BlendOutTime for (recently) stopped Montages"));
static TAutoConsoleVariable<bool> CVarUpdateMontageSectionIdToPlay(TEXT("AbilitySystem.UpdateMontageSectionIdToPlay"), true, TEXT("During tick, update the section ID that replicated montages should use"));
static TAutoConsoleVariable<bool> CVarReplicateMontageNextSectionId(TEXT("AbilitySystem.ReplicateMontageNextSectionId"), true, TEXT("Apply the replicated next section Id to montages when skipping position replication"));
static TAutoConsoleVariable<bool> CVarEnsureAbilitiesEndGracefully(TEXT("AbilitySystem.EnsureAbilitiesEndGracefully"), true, TEXT("When shutting down (during ClearAllAbilities) we should check if all GameplayAbilities gracefully ended. This should be disabled if you have NonInstanced abilities that are designed for multiple concurrent executions."));
static TAutoConsoleVariable<bool> CVarAllowSimulatedMontageRestart(TEXT("AbilitySystem.AllowSimulatedMontageRestart"), true, TEXT("Due to bad lag times, montages may time out on Simulated clients. This variable allows simulated montages to be restarted in such scenarios."));

/**
 * Mark the passed-in GA (and any SubObjects) as Garbage
 */
void MarkAbilityAsGarbage(UGameplayAbility* GameplayAbility)
{
	if (GameplayAbility)
	{
		GameplayAbility->MarkAsGarbage();

		// We must also mark all SubObjects as garbage, gather any ones not already marked.
		TArray<UObject*> GASubObjects;
		constexpr EObjectFlags ExclusionFlags = EObjectFlags::RF_MirroredGarbage;
		GetObjectsWithOuter(GameplayAbility, GASubObjects, EGetObjectsFlags::IncludeNestedObjects, ExclusionFlags);

		for (UObject* GASubObject : GASubObjects)
		{
			GASubObject->MarkAsGarbage();
		}
	}
}

void UAbilitySystemComponent::InitializeComponent()
{
	Super::InitializeComponent();

	// Look for DSO AttributeSets (note we are currently requiring all attribute sets to be subobjects of the same owner. This doesn't *have* to be the case forever.
	AActor *Owner = GetOwner();
	InitAbilityActorInfo(Owner, Owner);	// Default init to our outer owner

	// cleanup any bad data that may have gotten into SpawnedAttributes
	for (int32 Idx = SpawnedAttributes.Num()-1; Idx >= 0; --Idx)
	{
		if (SpawnedAttributes[Idx] == nullptr)
		{
			SpawnedAttributes.RemoveAt(Idx);
		}
	}

	TArray<UObject*> ChildObjects;
	GetObjectsWithOuter(Owner, ChildObjects, EGetObjectsFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage);

	for (UObject* Obj : ChildObjects)
	{
		UAttributeSet* Set = Cast<UAttributeSet>(Obj);
		if (Set)  
		{
			SpawnedAttributes.AddUnique(Set);
		}
	}

	SetSpawnedAttributesListDirty();
}

void UAbilitySystemComponent::UninitializeComponent()
{
	Super::UninitializeComponent();
	
	ActiveGameplayEffects.Uninitialize();
}

void UAbilitySystemComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	DestroyActiveState();

	// The MarkPendingKill on these attribute sets used to be done in UninitializeComponent,
	// but it was moved here instead since it's possible for the component to be uninitialized,
	// and later re-initialized, without being destroyed - and the attribute sets need to be preserved
	// in this case. This can happen when the owning actor's level is removed and later re-added
	// to the world, since EndPlay (and therefore UninitializeComponents) will be called on
	// the owning actor when its level is removed.
	for (UAttributeSet* Set : GetSpawnedAttributes())
	{
		if (Set)
		{
			Set->MarkAsGarbage();
		}
	}

	// Call the super at the end, after we've done what we needed to do
	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

void UAbilitySystemComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{	
	SCOPE_CYCLE_COUNTER(STAT_TickAbilityTasks);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(AbilityTasks);

	if (IsOwnerActorAuthoritative())
	{
		AnimMontage_UpdateReplicatedData();
	}

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	for (UAttributeSet* AttributeSet : GetSpawnedAttributes())
	{
		ITickableAttributeSetInterface* TickableSet = Cast<ITickableAttributeSetInterface>(AttributeSet);
		if (TickableSet && TickableSet->ShouldTick())
		{
			TickableSet->Tick(DeltaTime);
		}
	}
}

void UAbilitySystemComponent::InitAbilityActorInfo(AActor* InOwnerActor, AActor* InAvatarActor)
{
	check(AbilityActorInfo.IsValid());
	bool WasAbilityActorNull = (AbilityActorInfo->AvatarActor == nullptr);
	bool AvatarChanged = (InAvatarActor != AbilityActorInfo->AvatarActor);

	AbilityActorInfo->InitFromActor(InOwnerActor, InAvatarActor, this);

	SetOwnerActor(InOwnerActor);

	// caching the previous value of the actor so we can check against it but then setting the value to the new because it may get used
	const AActor* PrevAvatarActor = GetAvatarActor_Direct();
	SetAvatarActor_Direct(InAvatarActor);

	// if the avatar actor was null but won't be after this, we want to run the deferred gameplaycues that may not have run in NetDeltaSerialize
	// Conversely, if the ability actor was previously null, then the effects would not run in the NetDeltaSerialize. As such we want to run them now.
	if ((WasAbilityActorNull || PrevAvatarActor == nullptr) && InAvatarActor != nullptr)
	{
		HandleDeferredGameplayCues(&ActiveGameplayEffects);
	}

	if (AvatarChanged)
	{
		ABILITYLIST_SCOPE_LOCK();
		for (FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
		{
			if (Spec.Ability)
			{
				if (Spec.Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerActor)
				{
					UGameplayAbility* AbilityInstance = Spec.GetPrimaryInstance();
					// If we don't have the ability instance, it was either already destroyed or will get called on creation
					if (AbilityInstance)
					{
						AbilityInstance->OnAvatarSet(AbilityActorInfo.Get(), Spec);
					}
				}
				else
				{
					Spec.Ability->OnAvatarSet(AbilityActorInfo.Get(), Spec);
				}
			}
		}
	}

	if (UGameplayTagReponseTable* TagTable = UAbilitySystemGlobals::Get().GetGameplayTagResponseTable())
	{
		TagTable->RegisterResponseForEvents(this);
	}
	
	LocalAnimMontageInfo = FGameplayAbilityLocalAnimMontage();
	if (IsOwnerActorAuthoritative())
	{
		SetRepAnimMontageInfo(FGameplayAbilityRepAnimMontage());
	}

	if (bPendingMontageRep)
	{
		OnRep_ReplicatedAnimMontage();
	}
}

bool UAbilitySystemComponent::GetShouldTick() const 
{
	const bool bHasReplicatedMontageInfoToUpdate = (IsOwnerActorAuthoritative() && GetRepAnimMontageInfo().IsStopped == false);
	
	if (bHasReplicatedMontageInfoToUpdate)
	{
		return true;
	}

	bool bResult = Super::GetShouldTick();	
	if (bResult == false)
	{
		for (const UAttributeSet* AttributeSet : GetSpawnedAttributes())
		{
			const ITickableAttributeSetInterface* TickableAttributeSet = Cast<const ITickableAttributeSetInterface>(AttributeSet);
			if (TickableAttributeSet && TickableAttributeSet->ShouldTick())
			{
				bResult = true;
				break;
			}
		}
	}
	
	return bResult;
}

void UAbilitySystemComponent::SetAvatarActor(AActor* InAvatarActor)
{
	check(AbilityActorInfo.IsValid());
	InitAbilityActorInfo(GetOwnerActor(), InAvatarActor);
}

void UAbilitySystemComponent::ClearActorInfo()
{
	check(AbilityActorInfo.IsValid());
	AbilityActorInfo->ClearActorInfo();
	SetOwnerActor(nullptr);
	SetAvatarActor_Direct(nullptr);
}

void UAbilitySystemComponent::OnRep_OwningActor()
{
	if (!ensure(AbilityActorInfo.IsValid()))
	{
		return;
	}

	AActor* LocalOwnerActor = GetOwnerActor();
	AActor* LocalAvatarActor = GetAvatarActor_Direct();

	if (LocalOwnerActor != AbilityActorInfo->OwnerActor || LocalAvatarActor != AbilityActorInfo->AvatarActor)
	{
		if (LocalOwnerActor != nullptr)
		{
			InitAbilityActorInfo(LocalOwnerActor, LocalAvatarActor);
		}
		else
		{
			ClearActorInfo();
		}
	}
}

void UAbilitySystemComponent::RefreshAbilityActorInfo()
{
	check(AbilityActorInfo.IsValid());
	AbilityActorInfo->InitFromActor(AbilityActorInfo->OwnerActor.Get(), AbilityActorInfo->AvatarActor.Get(), this);
}

FGameplayAbilitySpecHandle UAbilitySystemComponent::GiveAbility(const FGameplayAbilitySpec& Spec)
{
	if (!IsValid(Spec.Ability))
	{
		ABILITY_LOG(Error, TEXT("GiveAbility called with an invalid Ability Class."));

		return FGameplayAbilitySpecHandle();
	}

	if (!IsOwnerActorAuthoritative())
	{
		ABILITY_LOG(Error, TEXT("GiveAbility called on ability %s on the client, not allowed!"), *Spec.Ability->GetName());

		return FGameplayAbilitySpecHandle();
	}

	// If locked, add to pending list. The Spec.Handle is not regenerated when we receive, so returning this is ok.
	if (AbilityScopeLockCount > 0)
	{
		UE_LOGF(LogAbilitySystem, Verbose, "%ls: GiveAbility %ls delayed (ScopeLocked)", *GetNameSafe(GetOwner()), *GetNameSafe(Spec.Ability));
		AbilityPendingAdds.Add(Spec);
		return Spec.Handle;
	}
	
	ABILITYLIST_SCOPE_LOCK();
	FGameplayAbilitySpec& OwnedSpec = ActivatableAbilities.Items[ActivatableAbilities.Items.Add(Spec)];
	
	if (OwnedSpec.Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerActor)
	{
		// Create the instance at creation time
		CreateNewInstanceOfAbility(OwnedSpec, Spec.Ability);
	}
	
	OnGiveAbility(OwnedSpec);
	MarkAbilitySpecDirty(OwnedSpec, true);

	UE_LOGF(LogAbilitySystem, Log, "%ls: GiveAbility %ls [%ls] Level: %d Source: %ls", *GetNameSafe(GetOwner()), *GetNameSafe(Spec.Ability), *Spec.Handle.ToString(), Spec.Level, *GetNameSafe(Spec.SourceObject.Get()));
	UE_VLOG(GetOwner(), VLogAbilitySystem, Log, TEXT("GiveAbility %s [%s] Level: %d Source: %s"), *GetNameSafe(Spec.Ability), *Spec.Handle.ToString(), Spec.Level, *GetNameSafe(Spec.SourceObject.Get()));
	return OwnedSpec.Handle;
}

FGameplayAbilitySpecHandle UAbilitySystemComponent::GiveAbilityAndActivateOnce(FGameplayAbilitySpec& Spec, const FGameplayEventData* GameplayEventData)
{
	if (!IsValid(Spec.Ability))
	{
		ABILITY_LOG(Error, TEXT("GiveAbilityAndActivateOnce called with an invalid Ability Class."));

		return FGameplayAbilitySpecHandle();
	}

	if (Spec.Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::NonInstanced || Spec.Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalOnly)
	{
		ABILITY_LOG(Error, TEXT("GiveAbilityAndActivateOnce called on ability %s that is non instanced or won't execute on server, not allowed!"), *Spec.Ability->GetName());

		return FGameplayAbilitySpecHandle();
	}

	if (!IsOwnerActorAuthoritative())
	{
		ABILITY_LOG(Error, TEXT("GiveAbilityAndActivateOnce called on ability %s on the client, not allowed!"), *Spec.Ability->GetName());

		return FGameplayAbilitySpecHandle();
	}

	Spec.bActivateOnce = true;

	FGameplayAbilitySpecHandle AddedAbilityHandle = GiveAbility(Spec);

	FGameplayAbilitySpec* FoundSpec = FindAbilitySpecFromHandle(AddedAbilityHandle);

	if (FoundSpec)
	{
		FoundSpec->RemoveAfterActivation = true;

		if (!InternalTryActivateAbility(AddedAbilityHandle, FPredictionKey(), nullptr, nullptr, GameplayEventData))
		{
			// We failed to activate it, so remove it now
			ClearAbility(AddedAbilityHandle);

			return FGameplayAbilitySpecHandle();
		}
	}
	else if (GameplayEventData)
	{
		// Cache the GameplayEventData in the pending spec (if it was correctly queued)
		FGameplayAbilitySpec& PendingSpec = AbilityPendingAdds.Last();
		if (PendingSpec.Handle == AddedAbilityHandle)
		{
			PendingSpec.GameplayEventData = MakeShared<FGameplayEventData>(*GameplayEventData);
		}
	}

	return AddedAbilityHandle;
}

FGameplayAbilitySpecHandle UAbilitySystemComponent::K2_GiveAbility(TSubclassOf<UGameplayAbility> AbilityClass, int32 Level /*= 0*/, int32 InputID /*= -1*/)
{
	// build and validate the ability spec
	FGameplayAbilitySpec AbilitySpec = BuildAbilitySpecFromClass(AbilityClass, Level, InputID);

	// validate the class
	if (!IsValid(AbilitySpec.Ability))
	{
		ABILITY_LOG(Error, TEXT("K2_GiveAbility() called with an invalid Ability Class."));

		return FGameplayAbilitySpecHandle();
	}

	// grant the ability and return the handle. This will run validation and authority checks
	return GiveAbility(AbilitySpec);
}

FGameplayAbilitySpecHandle UAbilitySystemComponent::K2_GiveAbilityAndActivateOnce(TSubclassOf<UGameplayAbility> AbilityClass, int32 Level /* = 0*/, int32 InputID /* = -1*/)
{
	// build and validate the ability spec
	FGameplayAbilitySpec AbilitySpec = BuildAbilitySpecFromClass(AbilityClass, Level, InputID);

	// validate the class
	if (!IsValid(AbilitySpec.Ability))
	{
		ABILITY_LOG(Error, TEXT("K2_GiveAbilityAndActivateOnce() called with an invalid Ability Class."));

		return FGameplayAbilitySpecHandle();
	}

	return GiveAbilityAndActivateOnce(AbilitySpec);
}

void UAbilitySystemComponent::SetRemoveAbilityOnEnd(FGameplayAbilitySpecHandle AbilitySpecHandle)
{
	FGameplayAbilitySpec* FoundSpec = FindAbilitySpecFromHandle(AbilitySpecHandle);
	if (FoundSpec)
	{
		if (FoundSpec->IsActive())
		{
			FoundSpec->RemoveAfterActivation = true;
		}
		else
		{
			ClearAbility(AbilitySpecHandle);
		}
	}
}

void UAbilitySystemComponent::ClearAllAbilities()
{
	// If this is called inside an ability scope lock, postpone the workload until end of scope.
	// This was introduced for abilities that trigger their owning actor's destruction on ability
	// activation.
	if (AbilityScopeLockCount > 0)
	{
		bAbilityPendingClearAll = true;
		return;
	}

	if (!IsOwnerActorAuthoritative())
	{
		ABILITY_LOG(Error, TEXT("Attempted to call ClearAllAbilities() without authority."));

		return;
	}

	// Note we aren't marking any old abilities pending kill. This shouldn't matter since they will be garbage collected.
	ABILITYLIST_SCOPE_LOCK();
	for (FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		OnRemoveAbility(Spec);
	}

	// Let's add some enhanced checking if requested
	if (CVarEnsureAbilitiesEndGracefully.GetValueOnGameThread())
	{
		for (FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
		{
			if (Spec.IsActive())
			{
				ensureAlwaysMsgf(Spec.Ability->GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced, TEXT("%hs: %s was still active (ActiveCount = %d). Since it's not instanced, it's likely that TryActivateAbility and EndAbility are not matched."), __func__, *GetNameSafe(Spec.Ability), Spec.ActiveCount);
				ensureAlwaysMsgf(Spec.Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::NonInstanced, TEXT("%hs: %s was still active. Since it's an instanced ability, it's likely that there's an issue with the flow of EndAbility or RemoveAbility (such as not calling the Super function)."), __func__, *GetNameSafe(Spec.Ability));
			}
		}
	}

	ActivatableAbilities.Items.Empty(ActivatableAbilities.Items.Num());
	ActivatableAbilities.MarkArrayDirty();

	CheckForClearedAbilities();
	bAbilityPendingClearAll = false;
}

void UAbilitySystemComponent::ClearAllAbilitiesWithInputID(int32 InputID /*= 0*/)
{
	// find all matching abilities
	TArray<const FGameplayAbilitySpec*> OutSpecs;
	FindAllAbilitySpecsFromInputID(InputID, OutSpecs);

	// iterate through the bound abilities
	for (const FGameplayAbilitySpec* CurrentSpec : OutSpecs)
	{
		// clear the ability
		ClearAbility(CurrentSpec->Handle);
	}
}

void UAbilitySystemComponent::ClearAbility(const FGameplayAbilitySpecHandle& Handle)
{
	if (!IsOwnerActorAuthoritative())
	{
		ABILITY_LOG(Error, TEXT("Attempted to call ClearAbility() on the client. This is not allowed!"));

		return;
	}

	for (int Idx = 0; Idx < AbilityPendingAdds.Num(); ++Idx)
	{
		if (AbilityPendingAdds[Idx].Handle == Handle)
		{
			AbilityPendingAdds.RemoveAtSwap(Idx, EAllowShrinking::No);
			return;
		}
	}

	for (int Idx = 0; Idx < ActivatableAbilities.Items.Num(); ++Idx)
	{
		check(ActivatableAbilities.Items[Idx].Handle.IsValid());
		if (ActivatableAbilities.Items[Idx].Handle == Handle)
		{
			if (AbilityScopeLockCount > 0)
			{
				if (ActivatableAbilities.Items[Idx].PendingRemove == false)
				{
					ActivatableAbilities.Items[Idx].PendingRemove = true;
					AbilityPendingRemoves.Add(Handle);
				}
			}
			else
			{
				{
					// OnRemoveAbility will possibly call EndAbility. EndAbility can "do anything" including remove this abilityspec again. So a scoped list lock is necessary here.
					ABILITYLIST_SCOPE_LOCK();
					OnRemoveAbility(ActivatableAbilities.Items[Idx]);
					ActivatableAbilities.Items.RemoveAtSwap(Idx);
					ActivatableAbilities.MarkArrayDirty();
				}
				CheckForClearedAbilities();
			}
			return;
		}
	}
}

void UAbilitySystemComponent::OnGiveAbility(FGameplayAbilitySpec& Spec)
{
	if (!Spec.Ability)
	{
		return;
	}

	const UGameplayAbility* SpecAbility = Spec.Ability;
	const bool bInstancedPerActor = SpecAbility->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerActor;
	if (bInstancedPerActor && SpecAbility->GetReplicationPolicy() == EGameplayAbilityReplicationPolicy::ReplicateNo)
	{
		// If we don't replicate and are missing an instance, add one
		if (Spec.NonReplicatedInstances.Num() == 0)
		{
			CreateNewInstanceOfAbility(Spec, SpecAbility);
		}
	}

	// If this Ability Spec specified that it was created from an Active Gameplay Effect, then link the handle to the Active Gameplay Effect.
	if (Spec.GameplayEffectHandle.IsValid())
	{
		UAbilitySystemComponent* SourceASC = Spec.GameplayEffectHandle.GetOwningAbilitySystemComponent();
		UE_CLOGF(!SourceASC, LogAbilitySystem, Error, "OnGiveAbility Spec '%ls' GameplayEffectHandle had invalid Owning Ability System Component", *Spec.GetDebugString());
		if (SourceASC)
		{
			FActiveGameplayEffect* SourceActiveGE = SourceASC->ActiveGameplayEffects.GetActiveGameplayEffect(Spec.GameplayEffectHandle);
			UE_CLOGF(!SourceActiveGE, LogAbilitySystem, Error, "OnGiveAbility Spec '%ls' GameplayEffectHandle was not active on Owning Ability System Component '%ls'", *Spec.GetDebugString(), *SourceASC->GetName());
			if (SourceActiveGE)
			{
				SourceActiveGE->GrantedAbilityHandles.AddUnique(Spec.Handle);
				SourceASC->ActiveGameplayEffects.MarkItemDirty(*SourceActiveGE);
			}
		}
	}

	RegisterAbilityTriggers(Spec, Spec.Ability->AbilityTriggers);
	RegisterAbilityTriggers(Spec, Spec.DynamicAbilityTriggers);

	// If there's already a primary instance, it should be the one to receive the OnGiveAbility call
	UGameplayAbility* PrimaryInstance = bInstancedPerActor ? Spec.GetPrimaryInstance() : nullptr;
	if (PrimaryInstance)
	{
		PrimaryInstance->OnGiveAbility(AbilityActorInfo.Get(), Spec);
	}
	else
	{
		Spec.Ability->OnGiveAbility(AbilityActorInfo.Get(), Spec);
	}
}

void UAbilitySystemComponent::RegisterAbilityTriggers(const FGameplayAbilitySpec& Spec, TConstArrayView<FAbilityTriggerData> AbilityTriggers)
{
	for (const FAbilityTriggerData& TriggerData : AbilityTriggers)
	{
		FGameplayTag EventTag = TriggerData.TriggerTag;

		TMap<FGameplayTag, TArray<FGameplayAbilitySpecHandle>>& TriggeredAbilityMap = (TriggerData.TriggerSource == EGameplayAbilityTriggerSource::GameplayEvent) ? GameplayEventTriggeredAbilities : OwnedTagTriggeredAbilities;

		if (TriggeredAbilityMap.Contains(EventTag))
		{
			TriggeredAbilityMap[EventTag].AddUnique(Spec.Handle);	// Fixme: is this right? Do we want to trigger the ability directly of the spec?
		}
		else
		{
			TArray<FGameplayAbilitySpecHandle> Triggers;
			Triggers.Add(Spec.Handle);
			TriggeredAbilityMap.Add(EventTag, Triggers);
		}

		if (TriggerData.TriggerSource != EGameplayAbilityTriggerSource::GameplayEvent)
		{
			FOnGameplayEffectTagCountChanged& CountChangedEvent = RegisterGameplayTagEvent(EventTag);
			// Add a change callback if it isn't on it already

			if (!CountChangedEvent.IsBoundToObject(this))
			{
				MonitoredTagChangedDelegateHandle = CountChangedEvent.AddUObject(this, &UAbilitySystemComponent::MonitoredTagChanged);
			}
		}
	}
}

void UAbilitySystemComponent::OnRemoveAbility(FGameplayAbilitySpec& Spec)
{
	ensureMsgf(AbilityScopeLockCount > 0, TEXT("%hs called without an Ability List Lock.  It can produce side effects and should be locked to pin the Spec argument."), __func__);

	if (!Spec.Ability)
	{
		return;
	}

	UE_LOGF(LogAbilitySystem, Log, "%ls: Removing Ability [%ls] %ls Level: %d", *GetNameSafe(GetOwner()), *Spec.Handle.ToString(), *GetNameSafe(Spec.Ability), Spec.Level);
	UE_VLOG(GetOwner(), VLogAbilitySystem, Log, TEXT("Removing Ability [%s] %s Level: %d"), *Spec.Handle.ToString(), *GetNameSafe(Spec.Ability), Spec.Level);

	UnregisterAbilityTriggers(Spec, Spec.Ability->AbilityTriggers);
	UnregisterAbilityTriggers(Spec, Spec.DynamicAbilityTriggers);

	TArray<UGameplayAbility*> Instances = Spec.GetAbilityInstances();

	for (auto Instance : Instances)
	{
		if (Instance)
		{
			if (Instance->IsActive())
			{
				// ReplicateEndAbility if the GA itself isn't replicated.
				bool bReplicateEndAbility = Instance->GetReplicationPolicy() == EGameplayAbilityReplicationPolicy::ReplicateNo;
				bool bWasCancelled = false;
				Instance->EndAbility(Instance->CurrentSpecHandle, Instance->CurrentActorInfo, Instance->CurrentActivationInfo, bReplicateEndAbility, bWasCancelled);
			}
			else
			{
				// Ability isn't active, but still needs to be destroyed
				if (GetOwnerRole() == ROLE_Authority)
				{
					// Only destroy if we're the server or this isn't replicated. Can't destroy on the client or replication will fail when it replicates the end state
					RemoveReplicatedInstancedAbility(Instance);
				}

				if (Instance->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerExecution)
				{
					ABILITY_LOG(Error, TEXT("%s was InActive, yet still instanced during OnRemove"), *Instance->GetName());
					MarkAbilityAsGarbage(Instance);
				}
			}
		}
	}

	// Notify the ability that it has been removed.  It follows the same pattern as OnGiveAbility() and is only called on the primary instance of the ability or the CDO.
	UGameplayAbility* PrimaryInstance = Spec.GetPrimaryInstance();
	if (PrimaryInstance)
	{
		PrimaryInstance->OnRemoveAbility(AbilityActorInfo.Get(), Spec);
		
		// Make sure we remove this before marking it as garbage.
		if (GetOwnerRole() == ROLE_Authority)
		{
			RemoveReplicatedInstancedAbility(PrimaryInstance);
		}
		MarkAbilityAsGarbage(PrimaryInstance);
	}
	else
	{
		// This is error handling some edge cases
		if (Spec.IsActive())
		{
			if (Spec.Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerExecution)
			{
#if DO_ENSURE
				// Let's make sure we aren't recursively executing this ability.
				Instances = Spec.GetAbilityInstances();
				for (UGameplayAbility* Instance : Instances)
				{
					ensureMsgf(Instance->bIsAbilityEnding, TEXT("All instances of %s on %s should have been ended by now. Maybe it was retriggered from OnEndAbility (bad)?"), *GetNameSafe(Spec.Ability), *GetName());
				}
#endif

				// If we're a client, fall-through anyway as this is our last chance to execute OnRemoveAbility.
				if (IsOwnerActorAuthoritative())
				{
					// We assume we are inside an EndAbility callstack (bIsActive && bIsAbilityEnding), so remove us when we get out of it (NotifyAbilityEnded).
					Spec.RemoveAfterActivation = true;
					return;
				}
			}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			else if (ensureMsgf(Spec.Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::NonInstanced, TEXT("We should never have an instanced Gameplay Ability that is still active by this point. All instances should have EndAbility called just before here.")))
			{
				// Seems like it should be cancelled, but we're just following the existing pattern (could be due to functionality from OnRep)
				constexpr bool bReplicateEndAbility = false;
				constexpr bool bWasCancelled = false;
				Spec.Ability->EndAbility(Spec.Handle, AbilityActorInfo.Get(), Spec.ActivationInfo, bReplicateEndAbility, bWasCancelled);
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		Spec.Ability->OnRemoveAbility(AbilityActorInfo.Get(), Spec);
	}

	// If this Ability Spec specified that it was created from an Active Gameplay Effect, then unlink the handle to the Active Gameplay Effect.
	// Note: It's possible (maybe even likely) that the ActiveGE is no longer considered active by this point.
	// That means we can't use FindActiveGameplayEffectHandle (which fails if ActiveGE is PendingRemove), but also many of these checks will fail
	// if the ActiveGE has completed its removal.
	if (Spec.GameplayEffectHandle.IsValid()) // This can only be true on the network authority
	{
		if (UAbilitySystemComponent* SourceASC = Spec.GameplayEffectHandle.GetOwningAbilitySystemComponent())
		{
			FActiveGameplayEffect* SourceActiveGE = SourceASC->ActiveGameplayEffects.GetActiveGameplayEffect(Spec.GameplayEffectHandle);
			if (ensureMsgf(SourceActiveGE, TEXT("FActiveGameplayEffectHandle::GetOwningAbilitySystemComponent returned an ASC which did not contain the handle")))
			{
				SourceActiveGE->GrantedAbilityHandles.Remove(Spec.Handle);
			}
		}
	}

	Spec.ReplicatedInstances.Empty();
	Spec.NonReplicatedInstances.Empty();
}

void UAbilitySystemComponent::UnregisterAbilityTriggers(const FGameplayAbilitySpec& Spec, TConstArrayView<FAbilityTriggerData> AbilityTriggers)
{
	for (const FAbilityTriggerData& TriggerData : AbilityTriggers)
	{
		FGameplayTag EventTag = TriggerData.TriggerTag;

		TMap<FGameplayTag, TArray<FGameplayAbilitySpecHandle>>& TriggeredAbilityMap = (TriggerData.TriggerSource == EGameplayAbilityTriggerSource::GameplayEvent) ? GameplayEventTriggeredAbilities : OwnedTagTriggeredAbilities;

		if (ensureMsgf(TriggeredAbilityMap.Contains(EventTag),
			TEXT("%s::%s not found in TriggeredAbilityMap while removing, TriggerSource: %d"), *Spec.Ability->GetName(), *EventTag.ToString(), (int32)TriggerData.TriggerSource))
		{
			TriggeredAbilityMap[EventTag].Remove(Spec.Handle);
			if (TriggeredAbilityMap[EventTag].Num() == 0)
			{
				TriggeredAbilityMap.Remove(EventTag);
			}
		}
	}
}

void UAbilitySystemComponent::CheckForClearedAbilities()
{
	for (auto& Triggered : GameplayEventTriggeredAbilities)
	{
		// Make sure all triggered abilities still exist, if not remove
		for (int32 i = 0; i < Triggered.Value.Num(); i++)
		{
			FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(Triggered.Value[i]);

			if (!Spec)
			{
				Triggered.Value.RemoveAt(i);
				i--;
			}
		}
		
		// We leave around the empty trigger stub, it's likely to be added again
	}

	for (auto& Triggered : OwnedTagTriggeredAbilities)
	{
		bool bRemovedTrigger = false;
		// Make sure all triggered abilities still exist, if not remove
		for (int32 i = 0; i < Triggered.Value.Num(); i++)
		{
			FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(Triggered.Value[i]);

			if (!Spec)
			{
				Triggered.Value.RemoveAt(i);
				i--;
				bRemovedTrigger = true;
			}
		}
		
		if (bRemovedTrigger && Triggered.Value.Num() == 0)
		{
			// If we removed all triggers, remove the callback
			FOnGameplayEffectTagCountChanged& CountChangedEvent = RegisterGameplayTagEvent(Triggered.Key);
		
			if (CountChangedEvent.IsBoundToObject(this))
			{
				CountChangedEvent.Remove(MonitoredTagChangedDelegateHandle);
			}
		}

		// We leave around the empty trigger stub, it's likely to be added again
	}

	for (int32 i = 0; i < AllReplicatedInstancedAbilities.Num(); i++)
	{
		UGameplayAbility* Ability = AllReplicatedInstancedAbilities[i];
		if (!IsValid(Ability))
		{
			if (Ability && IsUsingRegisteredSubObjectList())
			{
				RemoveReplicatedSubObject(Ability);
			}

			AllReplicatedInstancedAbilities.RemoveAt(i);
			i--;
		}
	}

	// Clear any out of date ability spec handles on active gameplay effects
	for (FActiveGameplayEffect& ActiveGE : &ActiveGameplayEffects)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (FGameplayAbilitySpecDef& AbilitySpec : ActiveGE.Spec.GrantedAbilitySpecs)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			if (AbilitySpec.AssignedHandle.IsValid() && FindAbilitySpecFromHandle(AbilitySpec.AssignedHandle) == nullptr)
			{
				bool bIsPendingAdd = false;
				for (const FAbilityListLockActiveChange* ActiveChange : AbilityListLockActiveChanges)
				{
					for (const FGameplayAbilitySpec& PendingSpec : ActiveChange->Adds)
					{
						if (PendingSpec.Handle == AbilitySpec.AssignedHandle)
						{
							bIsPendingAdd = true;
							break;
						}
					}

					if (bIsPendingAdd)
					{
						break;
					}
				}

				for (const FGameplayAbilitySpec& PendingSpec : AbilityPendingAdds)
				{
					if (PendingSpec.Handle == AbilitySpec.AssignedHandle)
					{
						bIsPendingAdd = true;
						break;
					}
				}

				if (bIsPendingAdd)
				{
					ABILITY_LOG(Verbose, TEXT("Skipped clearing AssignedHandle %s from GE %s / %s, as it is pending being added."), *AbilitySpec.AssignedHandle.ToString(), *ActiveGE.GetDebugString(), *ActiveGE.Handle.ToString());
					continue;
				}

				ABILITY_LOG(Verbose, TEXT("::CheckForClearedAbilities is clearing AssignedHandle %s from GE %s / %s"), *AbilitySpec.AssignedHandle.ToString(), *ActiveGE.GetDebugString(), *ActiveGE.Handle.ToString() );
				AbilitySpec.AssignedHandle = FGameplayAbilitySpecHandle();
			}
		}
	}
}

void UAbilitySystemComponent::IncrementAbilityListLock()
{
	AbilityScopeLockCount++;
}
void UAbilitySystemComponent::DecrementAbilityListLock()
{
	if (--AbilityScopeLockCount == 0)
	{
		if (bAbilityPendingClearAll)
		{
			ClearAllAbilities();

			// When there are pending adds but also a pending clear-all, prioritize clear-all since ClearAllAbilities() based on an assumption 
			// that the clear-all is likely end-of-life cleanup. There may be cases where someone intentionally calls ClearAllAbilities() and 
			// then GiveAbility() within one ability scope lock like an ability that removes all abilities and grants an ability. In the future 
			// we could support this by keeping a chronological list of pending add/remove/clear-all actions and executing them in order.
			if (AbilityPendingAdds.Num() > 0)
			{
				ABILITY_LOG(Warning, TEXT("GiveAbility and ClearAllAbilities were both called within an ability scope lock. Prioritizing clear all abilities by ignoring pending adds."));
				AbilityPendingAdds.Reset();
			}

			// Pending removes are no longer relevant since all abilities have been removed
			AbilityPendingRemoves.Reset();
		}
		else if (AbilityPendingAdds.Num() > 0 || AbilityPendingRemoves.Num() > 0)
		{
			FAbilityListLockActiveChange ActiveChange(*this, AbilityPendingAdds, AbilityPendingRemoves);

			for (FGameplayAbilitySpec& Spec : ActiveChange.Adds)
			{
				if (Spec.bActivateOnce)
				{
					GiveAbilityAndActivateOnce(Spec, Spec.GameplayEventData.Get());
				}
				else
				{
					GiveAbility(Spec);
				}
			}

			for (FGameplayAbilitySpecHandle& Handle : ActiveChange.Removes)
			{
				ClearAbility(Handle);
			}
		}
	}
}

FGameplayAbilitySpec* UAbilitySystemComponent::FindAbilitySpecFromHandle(FGameplayAbilitySpecHandle Handle, EConsiderPending ConsiderPending) const
{
	SCOPE_CYCLE_COUNTER(STAT_FindAbilitySpecFromHandle);

	for (const FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		if (Spec.Handle == Handle)
		{
			if (!Spec.PendingRemove || EnumHasAnyFlags(ConsiderPending, EConsiderPending::PendingRemove))
			{
				return const_cast<FGameplayAbilitySpec*>(&Spec);
			}
		}
	}

	if (EnumHasAnyFlags(ConsiderPending, EConsiderPending::PendingAdd))
	{
		for (const FGameplayAbilitySpec& Spec : AbilityPendingAdds)
		{
			if (!Spec.PendingRemove || EnumHasAnyFlags(ConsiderPending, EConsiderPending::PendingRemove))
			{
				return const_cast<FGameplayAbilitySpec*>(&Spec);
			}
		}
	}

	return nullptr;
}

TArray<const FGameplayAbilitySpec*> UAbilitySystemComponent::FindAbilitySpecsFromGEHandle(const FScopedAbilityListLock& /*Used as a Contract*/, FActiveGameplayEffectHandle ActiveGEHandle, EConsiderPending ConsiderPending) const
{
	TArray<const FGameplayAbilitySpec*> ReturnValue;

	if (!ensureMsgf(IsOwnerActorAuthoritative(), TEXT("%hs is only valid on authority as FGameplayAbilitySpec::GameplayEffectHandle is not replicated and ability granting only happens on the server"), __func__))
	{
		return ReturnValue;
	}

	auto GatherGAsByGEHandle = [ActiveGEHandle, ConsiderPending, &ReturnValue](const TArrayView<const FGameplayAbilitySpec> AbilitiesToConsider)
		{
			for (const FGameplayAbilitySpec& GASpec : AbilitiesToConsider)
			{
				if (GASpec.GameplayEffectHandle == ActiveGEHandle)
				{
					if (!GASpec.PendingRemove || EnumHasAnyFlags(ConsiderPending, EConsiderPending::PendingRemove))
					{
						ReturnValue.Emplace(&GASpec);
					}
				}
			}
		};

	// All activatable abilities (which will include abilities that are in AbilityPendingRemoves
	GatherGAsByGEHandle(GetActivatableAbilities());

	// If requested, specifically look for abilities that are pending add
	if (EnumHasAnyFlags(ConsiderPending,EConsiderPending::PendingAdd))
	{
		GatherGAsByGEHandle(AbilityPendingAdds);
	}

	return ReturnValue;
}


FGameplayAbilitySpec* UAbilitySystemComponent::FindAbilitySpecFromClass(TSubclassOf<UGameplayAbility> InAbilityClass) const
{
	SCOPE_CYCLE_COUNTER(STAT_FindAbilitySpecFromHandle);

	for (const FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		if (Spec.Ability == nullptr)
		{
			continue;
		}

		if (Spec.Ability->GetClass() == InAbilityClass)
		{
			return const_cast<FGameplayAbilitySpec*>(&Spec);
		}
	}

	return nullptr;
}

void UAbilitySystemComponent::MarkAbilitySpecDirty(FGameplayAbilitySpec& Spec, bool WasAddOrRemove)
{
	if (IsOwnerActorAuthoritative())
	{
		// Don't mark dirty for specs that are server only unless it was an add/remove
		if (!(Spec.Ability && Spec.Ability->NetExecutionPolicy == EGameplayAbilityNetExecutionPolicy::ServerOnly && !WasAddOrRemove))
		{
			ActivatableAbilities.MarkItemDirty(Spec);
		}
		AbilitySpecDirtiedCallbacks.Broadcast(Spec);
	}
	else
	{
		// Clients predicting should call MarkArrayDirty to force the internal replication map to be rebuilt.
		ActivatableAbilities.MarkArrayDirty();
	}
}

FGameplayAbilitySpec* UAbilitySystemComponent::FindAbilitySpecFromInputID(int32 InputID) const
{
	if (InputID != INDEX_NONE)
	{
		for (const FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
		{
			if (Spec.InputID == InputID)
			{
				return const_cast<FGameplayAbilitySpec*>(&Spec);
			}
		}
	}
	return nullptr;
}

void UAbilitySystemComponent::FindAllAbilitySpecsFromInputID(int32 InputID, TArray<const FGameplayAbilitySpec*>& OutAbilitySpecs) const
{
	// ignore invalid inputs
	if (InputID != INDEX_NONE)
	{
		// iterate through all abilities
		for (const FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
		{
			// add maching abilities to the list
			if (Spec.InputID == InputID)
			{
				OutAbilitySpecs.Add(&Spec);
			}
		}
	}
}

FGameplayAbilitySpec UAbilitySystemComponent::BuildAbilitySpecFromClass(TSubclassOf<UGameplayAbility> AbilityClass, int32 Level /*= 0*/, int32 InputID /*= -1*/)
{
	// validate the class
	if (!ensure(AbilityClass))
	{
		ABILITY_LOG(Error, TEXT("BuildAbilitySpecFromClass called with an invalid Ability Class."));

		return FGameplayAbilitySpec();
	}

	// get the CDO. GiveAbility will validate so we don't need to
	UGameplayAbility* AbilityCDO = AbilityClass->GetDefaultObject<UGameplayAbility>();

	// build the ability spec
	// we need to initialize this through the constructor,
	// or the Handle won't be properly set and will cause errors further down the line
	return FGameplayAbilitySpec(AbilityClass, Level, InputID);
}

void UAbilitySystemComponent::GetAllAbilities(OUT TArray<FGameplayAbilitySpecHandle>& OutAbilityHandles) const
{
	// ensure the output array is empty
	OutAbilityHandles.Empty(ActivatableAbilities.Items.Num());

	// iterate through all activatable abilities
	// NOTE: currently this doesn't include abilities that are mid-activation
	for (const FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		// add the spec handle to the list
		OutAbilityHandles.Add(Spec.Handle);
	}
}

void UAbilitySystemComponent::FindAllAbilitiesWithTags(OUT TArray<FGameplayAbilitySpecHandle>& OutAbilityHandles, FGameplayTagContainer Tags, bool bExactMatch /* = true */) const
{
	// ensure the output array is empty
	OutAbilityHandles.Empty();

	// iterate through all Ability Specs
	for (const FGameplayAbilitySpec& CurrentSpec : ActivatableAbilities.Items)
	{
		if (!CurrentSpec.Ability)
		{
			continue;
		}

		// try to get the ability instance (if instanced per actor)
		UGameplayAbility* AbilityInstance = CurrentSpec.GetPrimaryInstance();

		// default to the CDO if we can't
		if (!AbilityInstance)
		{
			AbilityInstance = CurrentSpec.Ability;
		}

		// ensure the ability instance is valid
		if (IsValid(AbilityInstance))
		{
			// do we want an exact match?
			if (bExactMatch)
			{
				// check if we match all tags
				if (AbilityInstance->GetAssetTags().HasAll(Tags))
				{
					// add the matching handle
					OutAbilityHandles.Add(CurrentSpec.Handle);
				}
			}
			else
			{

				// check if we match any tags
				if (AbilityInstance->GetAssetTags().HasAny(Tags))
				{
					// add the matching handle
					OutAbilityHandles.Add(CurrentSpec.Handle);
				}
			}
		}
	}
}

void UAbilitySystemComponent::FindAllAbilitiesMatchingQuery(OUT TArray<FGameplayAbilitySpecHandle>& OutAbilityHandles, FGameplayTagQuery Query) const
{
	// ensure the output array is empty
	OutAbilityHandles.Empty();

	// iterate through all Ability Specs
	for (const FGameplayAbilitySpec& CurrentSpec : ActivatableAbilities.Items)
	{
		// try to get the ability instance
		UGameplayAbility* AbilityInstance = CurrentSpec.GetPrimaryInstance();

		// default to the CDO if we can't
		if (!AbilityInstance)
		{
			AbilityInstance = CurrentSpec.Ability;
		}

		// ensure the ability instance is valid
		if (IsValid(AbilityInstance))
		{
			if (AbilityInstance->GetAssetTags().MatchesQuery(Query))
			{
				// add the matching handle
				OutAbilityHandles.Add(CurrentSpec.Handle);
			}
		}
	}
}

void UAbilitySystemComponent::FindAllAbilitiesWithInputID(OUT TArray<FGameplayAbilitySpecHandle>& OutAbilityHandles, int32 InputID /*= 0*/) const
{
	// ensure the output array is empty
	OutAbilityHandles.Empty();

	// find all ability specs matching the Input ID
	TArray<const FGameplayAbilitySpec*> MatchingSpecs;
	FindAllAbilitySpecsFromInputID(InputID, MatchingSpecs);

	// add all matching specs to the out array
	for (const FGameplayAbilitySpec* CurrentSpec : MatchingSpecs)
	{
		OutAbilityHandles.Add(CurrentSpec->Handle);
	}
}

FGameplayEffectContextHandle UAbilitySystemComponent::GetEffectContextFromActiveGEHandle(FActiveGameplayEffectHandle Handle)
{
	FActiveGameplayEffect* ActiveGE = ActiveGameplayEffects.GetActiveGameplayEffect(Handle);
	if (ActiveGE)
	{
		return ActiveGE->Spec.GetEffectContext();
	}

	return FGameplayEffectContextHandle();
}

UGameplayAbility* UAbilitySystemComponent::CreateNewInstanceOfAbility(FGameplayAbilitySpec& Spec, const UGameplayAbility* Ability)
{
	check(Ability);
	check(Ability->HasAllFlags(RF_ClassDefaultObject));

	AActor* Owner = GetOwner();
	check(Owner);

	UGameplayAbility * AbilityInstance = NewObject<UGameplayAbility>(Owner, Ability->GetClass());
	check(AbilityInstance);

#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_LOGF(LogAbilitySystem, Verbose, "%s: %ls (%ls)", __FUNCTION__, *GetNameSafe(AbilityInstance), *FRemoteObjectId(AbilityInstance).ToString());
#endif

	// Add it to one of our instance lists so that it doesn't GC.
	if (AbilityInstance->GetReplicationPolicy() != EGameplayAbilityReplicationPolicy::ReplicateNo)
	{
		Spec.ReplicatedInstances.Add(AbilityInstance);
		AddReplicatedInstancedAbility(AbilityInstance);
	}
	else
	{
		Spec.NonReplicatedInstances.Add(AbilityInstance);
	}
	
	return AbilityInstance;
}

void UAbilitySystemComponent::NotifyAbilityEnded(FGameplayAbilitySpecHandle Handle, UGameplayAbility* Ability, bool bWasCancelled)
{
	check(Ability);
	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(Handle);
	if (Spec == nullptr)
	{
		// The ability spec may have been removed while we were ending. We can assume everything was cleaned up if the spec isnt here.
		return;
	}

	UE_LOGF(LogAbilitySystem, Log, "%ls: Ended [%ls] %ls. Level: %d. WasCancelled: %d.", *GetNameSafe(GetOwner()), *Handle.ToString(), Spec->GetPrimaryInstance() ? *Spec->GetPrimaryInstance()->GetName() : *Ability->GetName(), Spec->Level, bWasCancelled);
	UE_VLOG(GetOwner(), VLogAbilitySystem, Log, TEXT("Ended [%s] %s. Level: %d. WasCancelled: %d."), *Handle.ToString(), Spec->GetPrimaryInstance() ? *Spec->GetPrimaryInstance()->GetName() : *Ability->GetName(), Spec->Level, bWasCancelled);

	ENetRole OwnerRole = GetOwnerRole();

	// If AnimatingAbility ended, clear the pointer
	if (LocalAnimMontageInfo.AnimatingAbility.Get() == Ability)
	{
		ClearAnimatingAbility(Ability);
	}

	// check to make sure we do not cause a roll over to uint8 by decrementing when it is 0
	if (ensureMsgf(Spec->ActiveCount > 0, TEXT("NotifyAbilityEnded called when the Spec->ActiveCount <= 0 for ability %s"), *Ability->GetName()))
	{
		Spec->ActiveCount--;
	}

	// Broadcast that the ability ended
	AbilityEndedCallbacks.Broadcast(Ability);
	OnAbilityEnded.Broadcast(FAbilityEndedData(Ability, Handle, false, bWasCancelled));
	
	// Above callbacks could have invalidated the Spec pointer, so find it again
	Spec = FindAbilitySpecFromHandle(Handle);
	if (!Spec)
	{
		ABILITY_LOG(Error, TEXT("%hs(%s): %s lost its active handle halfway through the function."), __func__, *GetNameSafe(Ability), *Handle.ToString());
		return;
	}

	/** If this is instanced per execution or flagged for cleanup, mark pending kill and remove it from our instanced lists if we are the authority */
	if (Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerExecution)
	{
		check(Ability->HasAnyFlags(RF_ClassDefaultObject) == false);	// Should never be calling this on a CDO for an instanced ability!

		if (Ability->GetReplicationPolicy() != EGameplayAbilityReplicationPolicy::ReplicateNo)
		{
			if (OwnerRole == ROLE_Authority)
			{
				Spec->ReplicatedInstances.Remove(Ability);
				RemoveReplicatedInstancedAbility(Ability);
			}
		}
		else
		{
			Spec->NonReplicatedInstances.Remove(Ability);
		}

		MarkAbilityAsGarbage(Ability);
	}

	if (OwnerRole == ROLE_Authority)
	{
		if (Spec->RemoveAfterActivation && !Spec->IsActive())
		{
			// If we should remove after activation and there are no more active instances, kill it now
			ClearAbility(Handle);
		}
		else
		{
			MarkAbilitySpecDirty(*Spec);
		}
	}
}

void UAbilitySystemComponent::ClearAbilityReplicatedDataCache(FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActivationInfo& ActivationInfo)
{
	AbilityTargetDataMap.Remove( FGameplayAbilitySpecHandleAndPredictionKey(Handle, ActivationInfo.GetActivationPredictionKey()) );
}

void UAbilitySystemComponent::CancelAbility(UGameplayAbility* Ability)
{
	ABILITYLIST_SCOPE_LOCK();
	for (FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		if (Spec.Ability == Ability)
		{
			CancelAbilitySpec(Spec, nullptr);
		}
	}
}

void UAbilitySystemComponent::CancelAbilityHandle(const FGameplayAbilitySpecHandle& AbilityHandle)
{
	ABILITYLIST_SCOPE_LOCK();
	for (FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		if (Spec.Handle == AbilityHandle)
		{
			CancelAbilitySpec(Spec, nullptr);
			return;
		}
	}
}

void UAbilitySystemComponent::CancelAbilities(const FGameplayTagContainer* WithTags, const FGameplayTagContainer* WithoutTags, UGameplayAbility* Ignore)
{
	ABILITYLIST_SCOPE_LOCK();
	for (FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		if (!Spec.IsActive() || Spec.Ability == nullptr)
		{
			continue;
		}

		const FGameplayTagContainer& AbilityTags = Spec.Ability->GetAssetTags();
		bool WithTagPass = (!WithTags || AbilityTags.HasAny(*WithTags));
		bool WithoutTagPass = (!WithoutTags || !AbilityTags.HasAny(*WithoutTags));

		if (WithTagPass && WithoutTagPass)
		{
			CancelAbilitySpec(Spec, Ignore);
		}
	}
}

void UAbilitySystemComponent::CancelAbilitySpec(FGameplayAbilitySpec& Spec, UGameplayAbility* Ignore)
{
	FGameplayAbilityActorInfo* ActorInfo = AbilityActorInfo.Get();

	if (Spec.Ability->GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced)
	{
		// We need to cancel spawned instance, not the CDO
		TArray<UGameplayAbility*> AbilitiesToCancel = Spec.GetAbilityInstances();
		for (UGameplayAbility* InstanceAbility : AbilitiesToCancel)
		{
			if (InstanceAbility && Ignore != InstanceAbility)
			{
				InstanceAbility->CancelAbility(Spec.Handle, ActorInfo, InstanceAbility->GetCurrentActivationInfoRef(), !bDestroyActiveStateInitiated);
			}
		}
	}
	else
	{
		// Try to cancel the non instanced, this may not necessarily work
		Spec.Ability->CancelAbility(Spec.Handle, ActorInfo, FGameplayAbilityActivationInfo(), !bDestroyActiveStateInitiated);
	}
	MarkAbilitySpecDirty(Spec);
}

void UAbilitySystemComponent::CancelAllAbilities(UGameplayAbility* Ignore)
{
	ABILITYLIST_SCOPE_LOCK();
	for (FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		if (Spec.IsActive())
		{
			CancelAbilitySpec(Spec, Ignore);
		}
	}
}

void UAbilitySystemComponent::DestroyActiveState()
{
	// If we haven't already begun being destroyed
	if (!bDestroyActiveStateInitiated && ((GetFlags() & RF_BeginDestroyed) == 0))
	{
		// Avoid re-entrancy (ie if during CancelAbilities() an EndAbility callback destroys the Actor owning this ability system)
		bDestroyActiveStateInitiated = true;

		// Cancel all abilities before we are destroyed.
		FGameplayAbilityActorInfo* ActorInfo = AbilityActorInfo.Get();
		
		// condition needed since in edge cases canceling abilities
		// while not having valid owner/ability component can crash
		if (ActorInfo && ActorInfo->OwnerActor.IsValid(true) && ActorInfo->AbilitySystemComponent.IsValid(true))
		{
			CancelAbilities();
		}

		if (IsOwnerActorAuthoritative())
		{
			// We should now ClearAllAbilities because not all abilities CanBeCanceled().
			// This will gracefully call EndAbility and clean-up all instances of the abilities.
			ClearAllAbilities();
		}
		else
		{
			// If we're a client, ClearAllAbilities won't execute and we should clean up these instances manually.
			// CancelAbilities() will only MarkPending kill InstancedPerExecution abilities.
			// TODO: Is it correct to simply mark these as Garbage rather than EndAbility?  I suspect not, but this
			// is ingrained behavior (circa 2015). Perhaps better to allow ClearAllAbilities on client if bDestroyActiveStateInitiated (Nov 2023).
			for (FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
			{
				TArray<UGameplayAbility*> AbilitiesToCancel = Spec.GetAbilityInstances();
				for (UGameplayAbility* InstanceAbility : AbilitiesToCancel)
				{
					MarkAbilityAsGarbage(InstanceAbility);
				}

				Spec.ReplicatedInstances.Empty();
				Spec.NonReplicatedInstances.Empty();
			}
		}
	}
}

void UAbilitySystemComponent::ApplyAbilityBlockAndCancelTags(const FGameplayTagContainer& AbilityTags, UGameplayAbility* RequestingAbility, bool bEnableBlockTags, const FGameplayTagContainer& BlockTags, bool bExecuteCancelTags, const FGameplayTagContainer& CancelTags)
{
	if (bEnableBlockTags)
	{
		BlockAbilitiesWithTags(BlockTags);
	}
	else
	{
		UnBlockAbilitiesWithTags(BlockTags);
	}

	if (bExecuteCancelTags)
	{
		CancelAbilities(&CancelTags, nullptr, RequestingAbility);
	}
}

bool UAbilitySystemComponent::AreAbilityTagsBlocked(const FGameplayTagContainer& Tags) const
{
	// Expand the passed in tags to get parents, not the blocked tags
	return Tags.HasAny(BlockedAbilityTags.GetExplicitGameplayTags());
}

void UAbilitySystemComponent::BlockAbilitiesWithTags(const FGameplayTagContainer& Tags)
{
	BlockedAbilityTags.UpdateTagCount(Tags, 1);
}

void UAbilitySystemComponent::UnBlockAbilitiesWithTags(const FGameplayTagContainer& Tags)
{
	BlockedAbilityTags.UpdateTagCount(Tags, -1);
}

void UAbilitySystemComponent::BlockAbilityByInputID(int32 InputID)
{
	const TArray<uint8>& ConstBlockedAbilityBindings = GetBlockedAbilityBindings();
	if (InputID >= 0 && InputID < ConstBlockedAbilityBindings.Num())
	{
		++GetBlockedAbilityBindings_Mutable()[InputID];
	}
}

void UAbilitySystemComponent::UnBlockAbilityByInputID(int32 InputID)
{
	const TArray<uint8>& ConstBlockedAbilityBindings = GetBlockedAbilityBindings();
	if (InputID >= 0 && InputID < ConstBlockedAbilityBindings.Num() && ConstBlockedAbilityBindings[InputID] > 0)
	{
		--GetBlockedAbilityBindings_Mutable()[InputID];
	}
}

#if !UE_BUILD_SHIPPING
int32 DenyClientActivation = 0;
static FAutoConsoleVariableRef CVarDenyClientActivation(
TEXT("AbilitySystem.DenyClientActivations"),
	DenyClientActivation,
	TEXT("Make server deny the next X ability activations from clients. For testing misprediction."),
	ECVF_Default
	);
#endif // !UE_BUILD_SHIPPING

void UAbilitySystemComponent::OnRep_ActivateAbilities()
{
	for (FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		const UGameplayAbility* SpecAbility = Spec.Ability;
		if (!SpecAbility)
		{
			// Queue up another call to make sure this gets run again, as our abilities haven't replicated yet
			GetWorld()->GetTimerManager().SetTimer(OnRep_ActivateAbilitiesTimerHandle, this, &UAbilitySystemComponent::OnRep_ActivateAbilities, 0.5);
			return;
		}
	}

	CheckForClearedAbilities();

	// Make a copy in case a pending ability alters the array while iterating
	TArray<FPendingAbilityInfo> PendingCopy = PendingServerActivatedAbilities;
	PendingServerActivatedAbilities.Empty();

	// Try to run any pending activations that couldn't run before. If they don't work now, kill them
	for (const FPendingAbilityInfo& PendingAbilityInfo : PendingCopy)
	{
		if (PendingAbilityInfo.bPartiallyActivated)
		{
			ClientActivateAbilitySucceedWithEventData_Implementation(PendingAbilityInfo.Handle, PendingAbilityInfo.PredictionKey, PendingAbilityInfo.TriggerEventData);
		}
		else
		{
			ClientTryActivateAbility(PendingAbilityInfo.Handle);
		}

		// Do some warning if we're about to drop this ability activation
		if (!PendingServerActivatedAbilities.IsEmpty())
		{
			const bool bIsSame = (PendingServerActivatedAbilities[0] == PendingAbilityInfo);
			const bool bHasNewItem = !bIsSame || (PendingServerActivatedAbilities.Num() > 1);
			UE_CLOGF(bIsSame, LogAbilitySystem, Warning, "Failed to execute Pending Ability %ls (Handle %ls) because it was not replicated in time for second-chance activation (it will be ignored)", *PendingAbilityInfo.PredictionKey.ToString(), *PendingAbilityInfo.Handle.ToString());
			UE_CLOGF(bHasNewItem, LogAbilitySystem, Warning, "New Pending Ability added during existing execution of %ls (Handle %ls). New ability will be ignored.", *PendingAbilityInfo.PredictionKey.ToString(), *PendingAbilityInfo.Handle.ToString());

			// Empty again, so we can test again on the next item.
			PendingServerActivatedAbilities.Empty();
		}
	}

	// This is redundant but helps signal we're leaving this function with an empty pending ability list
	PendingServerActivatedAbilities.Empty();
}

void UAbilitySystemComponent::GetActivatableGameplayAbilitySpecsByAllMatchingTags(const FGameplayTagContainer& GameplayTagContainer, TArray < struct FGameplayAbilitySpec* >& MatchingGameplayAbilities, bool bOnlyAbilitiesThatSatisfyTagRequirements) const
{
	if (!GameplayTagContainer.IsValid())
	{
		return;
	}

	for (const FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		if (Spec.Ability && Spec.Ability->GetAssetTags().HasAll(GameplayTagContainer))
		{
			// Consider abilities that are blocked by tags currently if we're supposed to (default behavior).  
			// That way, we can use the blocking to find an appropriate ability based on tags when we have more than 
			// one ability that match the GameplayTagContainer.
			if (!bOnlyAbilitiesThatSatisfyTagRequirements || Spec.Ability->DoesAbilitySatisfyTagRequirements(*this))
			{
				MatchingGameplayAbilities.Add(const_cast<FGameplayAbilitySpec*>(&Spec));
			}
		}
	}
}

bool UAbilitySystemComponent::TryActivateAbilitiesByTag(const FGameplayTagContainer& GameplayTagContainer, bool bAllowRemoteActivation)
{
	TArray<FGameplayAbilitySpec*> AbilitiesToActivatePtrs;
	GetActivatableGameplayAbilitySpecsByAllMatchingTags(GameplayTagContainer, AbilitiesToActivatePtrs);
	if (AbilitiesToActivatePtrs.Num() < 1)
	{
		return false;
	}

	// Convert from pointers (which can be reallocated, since they point to internal data) to copies of that data
	TArray<FGameplayAbilitySpec> AbilitiesToActivate;
	AbilitiesToActivate.Reserve(AbilitiesToActivatePtrs.Num());
	Algo::Transform(AbilitiesToActivatePtrs, AbilitiesToActivate, [](FGameplayAbilitySpec* SpecPtr) { return *SpecPtr; });

	bool bSuccess = false;
	for (const FGameplayAbilitySpec& GameplayAbilitySpec : AbilitiesToActivate)
	{
		ensure(IsValid(GameplayAbilitySpec.Ability));
		bSuccess |= TryActivateAbility(GameplayAbilitySpec.Handle, bAllowRemoteActivation);
	}

	return bSuccess;
}

bool UAbilitySystemComponent::TryActivateAbilityByClass(TSubclassOf<UGameplayAbility> InAbilityToActivate, bool bAllowRemoteActivation)
{
	bool bSuccess = false;

	const UGameplayAbility* const InAbilityCDO = InAbilityToActivate.GetDefaultObject();

	for (const FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		if (Spec.Ability == InAbilityCDO)
		{
			bSuccess |= TryActivateAbility(Spec.Handle, bAllowRemoteActivation);
			break;
		}
	}

	return bSuccess;
}

bool UAbilitySystemComponent::TryActivateAbility(FGameplayAbilitySpecHandle AbilityToActivate, bool bAllowRemoteActivation)
{
	FGameplayTagContainer FailureTags;
	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityToActivate);
	if (!Spec)
	{
		ABILITY_LOG(Warning, TEXT("TryActivateAbility called with invalid Handle"));
		return false;
	}

	// don't activate abilities that are waiting to be removed
	if (Spec->PendingRemove || Spec->RemoveAfterActivation)
	{
		return false;
	}

	UGameplayAbility* Ability = Spec->Ability;

	if (!Ability)
	{
		ABILITY_LOG(Warning, TEXT("TryActivateAbility called with invalid Ability"));
		return false;
	}

	const FGameplayAbilityActorInfo* ActorInfo = AbilityActorInfo.Get();

	// make sure the ActorInfo and then Actor on that FGameplayAbilityActorInfo are valid, if not bail out.
	if (ActorInfo == nullptr || !ActorInfo->OwnerActor.IsValid() || !ActorInfo->AvatarActor.IsValid())
	{
		return false;
	}

		
	const ENetRole NetMode = ActorInfo->AvatarActor->GetLocalRole();

	// This should only come from button presses/local instigation (AI, etc).
	if (NetMode == ROLE_SimulatedProxy)
	{
		return false;
	}

	bool bIsLocal = AbilityActorInfo->IsLocallyControlled();

	// Check to see if this a local only or server only ability, if so either remotely execute or fail
	if (!bIsLocal && (Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalOnly || Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalPredicted))
	{
		if (bAllowRemoteActivation)
		{
			ClientTryActivateAbility(AbilityToActivate);
			return true;
		}

		ABILITY_LOG(Log, TEXT("Can't activate LocalOnly or LocalPredicted ability %s when not local."), *Ability->GetName());
		return false;
	}

	if (NetMode != ROLE_Authority && (Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::ServerOnly || Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::ServerInitiated))
	{
		if (bAllowRemoteActivation)
		{
			FScopedCanActivateAbilityLogEnabler LogEnabler;
			if (Ability->CanActivateAbility(AbilityToActivate, ActorInfo, nullptr, nullptr, &FailureTags))
			{
				// No prediction key, server will assign a server-generated key
				CallServerTryActivateAbility(AbilityToActivate, Spec->InputPressed, FPredictionKey());
				return true;
			}
			else
			{
				NotifyAbilityFailed(AbilityToActivate, Ability, FailureTags);
				return false;
			}
		}

		ABILITY_LOG(Log, TEXT("Can't activate ServerOnly or ServerInitiated ability %s when not the server."), *Ability->GetName());
		return false;
	}

	return InternalTryActivateAbility(AbilityToActivate);
}

bool UAbilitySystemComponent::IsAbilityInputBlocked(int32 InputID) const
{
	// Check if this ability's input binding is currently blocked
	const TArray<uint8>& ConstBlockedAbilityBindings = GetBlockedAbilityBindings();
	if (InputID >= 0 && InputID < ConstBlockedAbilityBindings.Num() && ConstBlockedAbilityBindings[InputID] > 0)
	{
		return true;
	}

	return false;
}

/**
 * Attempts to activate the ability.
 *	-This function calls CanActivateAbility
 *	-This function handles instancing
 *	-This function handles networking and prediction
 *	-If all goes well, CallActivateAbility is called next.
 */
bool UAbilitySystemComponent::InternalTryActivateAbility(FGameplayAbilitySpecHandle Handle, FPredictionKey InPredictionKey, UGameplayAbility** OutInstancedAbility, FOnGameplayAbilityEnded::FDelegate* OnGameplayAbilityEndedDelegate, const FGameplayEventData* TriggerEventData)
{
	const FGameplayTag& NetworkFailTag = UAbilitySystemGlobals::Get().ActivateFailNetworkingTag;
	
	InternalTryActivateAbilityFailureTags.Reset();

	if (Handle.IsValid() == false)
	{
		ABILITY_LOG(Warning, TEXT("InternalTryActivateAbility called with invalid Handle! ASC: %s. AvatarActor: %s"), *GetPathName(), *GetNameSafe(GetAvatarActor_Direct()));
		return false;
	}

	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(Handle);
	if (!Spec)
	{
		ABILITY_LOG(Warning, TEXT("InternalTryActivateAbility called with a valid handle but no matching ability was found. Handle: %s ASC: %s. AvatarActor: %s"), *Handle.ToString(), *GetPathName(), *GetNameSafe(GetAvatarActor_Direct()));
		return false;
	}

	// Lock ability list so our Spec doesn't get destroyed while activating
	ABILITYLIST_SCOPE_LOCK();

	const FGameplayAbilityActorInfo* ActorInfo = AbilityActorInfo.Get();

	// make sure the ActorInfo and then Actor on that FGameplayAbilityActorInfo are valid, if not bail out.
	if (ActorInfo == nullptr || !ActorInfo->OwnerActor.IsValid() || !ActorInfo->AvatarActor.IsValid())
	{
		return false;
	}

	// This should only come from button presses/local instigation (AI, etc)
	ENetRole NetMode = ROLE_SimulatedProxy;

	// Use PC netmode if its there
	if (APlayerController* PC = ActorInfo->PlayerController.Get())
	{
		NetMode = PC->GetLocalRole();
	}
	// Fallback to avataractor otherwise. Edge case: avatar "dies" and becomes torn off and ROLE_Authority. We don't want to use this case (use PC role instead).
	else if (AActor* LocalAvatarActor = GetAvatarActor_Direct())
	{
		NetMode = LocalAvatarActor->GetLocalRole();
	}

	if (NetMode == ROLE_SimulatedProxy)
	{
		return false;
	}

	bool bIsLocal = AbilityActorInfo->IsLocallyControlled();

	UGameplayAbility* Ability = Spec->Ability;

	if (!Ability)
	{
		ABILITY_LOG(Warning, TEXT("InternalTryActivateAbility called with invalid Ability"));
		return false;
	}

	// Check to see if this a local only or server only ability, if so don't execute
	if (!bIsLocal)
	{
		if (Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalOnly || (Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalPredicted && !InPredictionKey.IsValidKey()))
		{
			// If we have a valid prediction key, the ability was started on the local client so it's okay
			UE_LOGF(LogAbilitySystem, Warning, "%ls: Can't activate %ls ability %ls when not local", *GetNameSafe(GetOwner()), *UEnum::GetValueAsString<EGameplayAbilityNetExecutionPolicy::Type>(Ability->GetNetExecutionPolicy()), *Ability->GetName());
			UE_VLOG(GetOwner(), VLogAbilitySystem, Warning, TEXT("Can't activate %s ability %s when not local"), *UEnum::GetValueAsString<EGameplayAbilityNetExecutionPolicy::Type>(Ability->GetNetExecutionPolicy()), *Ability->GetName());

			if (NetworkFailTag.IsValid())
			{
				InternalTryActivateAbilityFailureTags.AddTag(NetworkFailTag);
				NotifyAbilityFailed(Handle, Ability, InternalTryActivateAbilityFailureTags);
			}

			return false;
		}		
	}

	if (NetMode != ROLE_Authority && (Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::ServerOnly || Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::ServerInitiated))
	{
		UE_LOGF(LogAbilitySystem, Warning, "%ls: Can't activate %ls ability %ls when not the server", *GetNameSafe(GetOwner()), *UEnum::GetValueAsString<EGameplayAbilityNetExecutionPolicy::Type>(Ability->GetNetExecutionPolicy()), *Ability->GetName());
		UE_VLOG(GetOwner(), VLogAbilitySystem, Warning, TEXT("Can't activate %s ability %s when not the server"), *UEnum::GetValueAsString<EGameplayAbilityNetExecutionPolicy::Type>(Ability->GetNetExecutionPolicy()), *Ability->GetName());

		if (NetworkFailTag.IsValid())
		{
			InternalTryActivateAbilityFailureTags.AddTag(NetworkFailTag);
			NotifyAbilityFailed(Handle, Ability, InternalTryActivateAbilityFailureTags);
		}

		return false;
	}

	// If it's an instanced one, the instanced ability will be set, otherwise it will be null
	UGameplayAbility* InstancedAbility = Spec->GetPrimaryInstance();
	UGameplayAbility* AbilitySource = InstancedAbility ? InstancedAbility : Ability;

	if (TriggerEventData)
	{
		if (!AbilitySource->ShouldAbilityRespondToEvent(ActorInfo, TriggerEventData))
		{
			UE_LOGF(LogAbilitySystem, Verbose, "%ls: Can't activate %ls because ShouldAbilityRespondToEvent was false.", *GetNameSafe(GetOwner()), *Ability->GetName());
			UE_VLOG(GetOwner(), VLogAbilitySystem, Verbose, TEXT("Can't activate %s because ShouldAbilityRespondToEvent was false."), *Ability->GetName());

			NotifyAbilityFailed(Handle, AbilitySource, InternalTryActivateAbilityFailureTags);
			return false;
		}
	}

	{
		const FGameplayTagContainer* SourceTags = TriggerEventData ? &TriggerEventData->InstigatorTags : nullptr;
		const FGameplayTagContainer* TargetTags = TriggerEventData ? &TriggerEventData->TargetTags : nullptr;

		FScopedCanActivateAbilityLogEnabler LogEnabler;
		if (!AbilitySource->CanActivateAbility(Handle, ActorInfo, SourceTags, TargetTags, &InternalTryActivateAbilityFailureTags))
		{
			// At least let the user know that the native CanActivateAbility rejected it
			if (InternalTryActivateAbilityFailureTags.IsEmpty())
			{
				InternalTryActivateAbilityFailureTags.AddTag(GetDefault<UGameplayAbilitiesDeveloperSettings>()->ActivateFailCanActivateAbilityTag);
			}

			// CanActivateAbility with LogEnabler will have UE_LOG/UE_VLOG so don't add more failure logs here
			NotifyAbilityFailed(Handle, AbilitySource, InternalTryActivateAbilityFailureTags);
			return false;
		}
	}

	// If we're InstancedPerActor and we're already active, don't let us activate again as this breaks the graph
	if (Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerActor)
	{
		if (Spec->IsActive())
		{
			if (Ability->bRetriggerInstancedAbility && InstancedAbility)
			{
				UE_LOGF(LogAbilitySystem, Verbose, "%ls: Ending %ls prematurely to retrigger.", *GetNameSafe(GetOwner()), *Ability->GetName());
				UE_VLOG(GetOwner(), VLogAbilitySystem, Verbose, TEXT("Ending %s prematurely to retrigger."), *Ability->GetName());

				constexpr bool bReplicateEndAbility = true;
				constexpr bool bWasCancelled = false;
				const FGameplayAbilityActivationInfo& ActivationInfo = InstancedAbility->GetCurrentActivationInfoRef();
				InstancedAbility->EndAbility(Handle, ActorInfo, ActivationInfo, bReplicateEndAbility, bWasCancelled);
			}
			else
			{
				UE_LOGF(LogAbilitySystem, Verbose, "Can't activate instanced per actor ability %ls when their is already a currently active instance for this actor.", *Ability->GetName());
				return false;
			}
		}
	}

	// Make sure we have a primary
	if (Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerActor && !InstancedAbility)
	{
		UE_LOGF(LogAbilitySystem, Warning, "InternalTryActivateAbility called but instanced ability is missing! NetMode: %d. Ability: %ls", (int32)NetMode, *Ability->GetName());
		return false;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// We have deprecated NonInstanced and Spec.ActivationInfo but keep backwards compatibility
	const bool bNonInstanced = Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::NonInstanced;
	FGameplayAbilityActivationInfo NewActivationInfo;
	FGameplayAbilityActivationInfo& ActivationInfo = bNonInstanced ? Spec->ActivationInfo : NewActivationInfo;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Setup a fresh ActivationInfo (possibly overwriting the Spec's ActivationInfo if non-instanced)
	ActivationInfo = FGameplayAbilityActivationInfo(ActorInfo->OwnerActor.Get());

	// If we are the server or this is local only
	if (Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalOnly || (NetMode == ROLE_Authority))
	{
		// if we're the server and don't have a valid key or this ability should be started on the server create a new activation key
		bool bCreateNewServerKey = NetMode == ROLE_Authority &&
			(!InPredictionKey.IsValidKey() ||
			 (Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::ServerInitiated ||
			  Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::ServerOnly));
		if (bCreateNewServerKey)
		{
			ActivationInfo.ServerSetActivationPredictionKey(FPredictionKey::CreateNewServerInitiatedKey(this));
		}
		else if (InPredictionKey.IsValidKey())
		{
			// Otherwise if available, set the prediction key to what was passed up
			ActivationInfo.ServerSetActivationPredictionKey(InPredictionKey);
		}

		// we may have changed the prediction key so we need to update the scoped key to match
		FScopedPredictionWindow ScopedPredictionWindow(this, ActivationInfo.GetActivationPredictionKey());

		// ----------------------------------------------
		// Tell the client that you activated it (if we're not local and not server only)
		// ----------------------------------------------
		if (!bIsLocal && Ability->GetNetExecutionPolicy() != EGameplayAbilityNetExecutionPolicy::ServerOnly)
		{
			if (TriggerEventData)
			{
				ClientActivateAbilitySucceedWithEventData(Handle, ActivationInfo.GetActivationPredictionKey(), *TriggerEventData);
			}
			else
			{
				ClientActivateAbilitySucceed(Handle, ActivationInfo.GetActivationPredictionKey());
			}
			
			// This will get copied into the instanced abilities
			ActivationInfo.bCanBeEndedByOtherInstance = Ability->bServerRespectsRemoteAbilityCancellation;
		}

		// ----------------------------------------------
		//	Call ActivateAbility (note this could end the ability too!)
		// ----------------------------------------------

		// Create instance of this ability if necessary
		if (Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerExecution)
		{
			InstancedAbility = CreateNewInstanceOfAbility(*Spec, Ability);
			InstancedAbility->CallActivateAbility(Handle, ActorInfo, ActivationInfo, OnGameplayAbilityEndedDelegate, TriggerEventData);
		}
		else
		{
			AbilitySource->CallActivateAbility(Handle, ActorInfo, ActivationInfo, OnGameplayAbilityEndedDelegate, TriggerEventData);
		}
	}
	else if (Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalPredicted)
	{
		PrePredictionActivation(Ability);

		// This execution is now officially EGameplayAbilityActivationMode:Predicting and has a PredictionKey
		FScopedPredictionWindow ScopedPredictionWindow(this, true);

		ActivationInfo.SetPredicting(ScopedPredictionKey);
		
		// This must be called immediately after GeneratePredictionKey to prevent problems with recursively activating abilities
		if (TriggerEventData)
		{
			ServerTryActivateAbilityWithEventData(Handle, Spec->InputPressed, ScopedPredictionKey, *TriggerEventData);
		}
		else
		{
			CallServerTryActivateAbility(Handle, Spec->InputPressed, ScopedPredictionKey);
		}

		// When this prediction key is caught up, we better know if the ability was confirmed or rejected
		ScopedPredictionKey.NewCaughtUpDelegate().BindUObject(this, &UAbilitySystemComponent::OnClientActivateAbilityCaughtUp, Handle, ScopedPredictionKey.Current);

		if (Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerExecution)
		{
			// For now, only NonReplicated + InstancedPerExecution abilities can be Predictive.
			// We lack the code to predict spawning an instance of the execution and then merge/combine
			// with the server spawned version when it arrives.

			if (Ability->GetReplicationPolicy() == EGameplayAbilityReplicationPolicy::ReplicateNo)
			{
				InstancedAbility = CreateNewInstanceOfAbility(*Spec, Ability);
				InstancedAbility->CallActivateAbility(Handle, ActorInfo, ActivationInfo, OnGameplayAbilityEndedDelegate, TriggerEventData);
			}
			else
			{
				ABILITY_LOG(Error, TEXT("InternalTryActivateAbility called on ability %s that is InstancedPerExecution and Replicated. This is an invalid configuration."), *Ability->GetName() );
			}
		}
		else
		{
			AbilitySource->CallActivateAbility(Handle, ActorInfo, ActivationInfo, OnGameplayAbilityEndedDelegate, TriggerEventData);
		}
	}
	
	if (InstancedAbility)
	{
		if (OutInstancedAbility)
		{
			*OutInstancedAbility = InstancedAbility;
		}

		// UGameplayAbility::PreActivate actually sets this internally (via SetCurrentInfo) which happens after replication (this is only set locally).  Let's cautiously remove this code.
		if (CVarAbilitySystemSetActivationInfoMultipleTimes.GetValueOnGameThread())
		{
			InstancedAbility->SetCurrentActivationInfo(ActivationInfo);	// Need to push this to the ability if it was instanced.
		}
	}

	MarkAbilitySpecDirty(*Spec);

	const UWorld* LocalWorld = GetWorld();
	if (ensureMsgf(LocalWorld, TEXT("%hs: Could not GetWorld during activation of %s"), __func__, *GetNameSafe(Ability)))
	{
		AbilityLastActivatedTime = LocalWorld->GetTimeSeconds();
	}

	UE_LOGF(LogAbilitySystem, Log, "%ls: Activated [%ls] %ls. Level: %d. PredictionKey: %ls.", *GetNameSafe(GetOwner()), *Spec->Handle.ToString(), *GetNameSafe(AbilitySource), Spec->Level, *ActivationInfo.GetActivationPredictionKey().ToString());
	UE_VLOG(GetOwner(), VLogAbilitySystem, Log, TEXT("Activated [%s] %s. Level: %d. PredictionKey: %s."), *Spec->Handle.ToString(), *GetNameSafe(AbilitySource), Spec->Level, *ActivationInfo.GetActivationPredictionKey().ToString());
	return true;
}

void UAbilitySystemComponent::PrePredictionActivation(const UGameplayAbility* Ability)
{
	// Flush server moves that occurred before this ability activation so that the server receives the RPCs in the correct order
	// Necessary to prevent abilities that trigger animation root motion or impact movement from causing network corrections
	if (Ability && Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalPredicted)
    {
		const FGameplayAbilityActorInfo* ActorInfo = AbilityActorInfo.Get();
		if (ActorInfo && !ActorInfo->IsNetAuthority())
		{
			if (ACharacter* AvatarCharacter = Cast<ACharacter>(ActorInfo->AvatarActor.Get()))
			{
				if (UCharacterMovementComponent* AvatarCharMoveComp = Cast<UCharacterMovementComponent>(AvatarCharacter->GetMovementComponent()))
				{
					AvatarCharMoveComp->FlushServerMoves();
				}
			}
		}
	}
}

void UAbilitySystemComponent::ServerTryActivateAbility_Implementation(FGameplayAbilitySpecHandle Handle, bool InputPressed, FPredictionKey PredictionKey)
{
	InternalServerTryActivateAbility(Handle, InputPressed, PredictionKey, nullptr);
}

bool UAbilitySystemComponent::ServerTryActivateAbility_Validate(FGameplayAbilitySpecHandle Handle, bool InputPressed, FPredictionKey PredictionKey)
{
	return true;
}

void UAbilitySystemComponent::ServerTryActivateAbilityWithEventData_Implementation(FGameplayAbilitySpecHandle Handle, bool InputPressed, FPredictionKey PredictionKey, FGameplayEventData TriggerEventData)
{
	InternalServerTryActivateAbility(Handle, InputPressed, PredictionKey, &TriggerEventData);
}

bool UAbilitySystemComponent::ServerTryActivateAbilityWithEventData_Validate(FGameplayAbilitySpecHandle Handle, bool InputPressed, FPredictionKey PredictionKey, FGameplayEventData TriggerEventData)
{
	return true;
}

void UAbilitySystemComponent::ClientTryActivateAbility_Implementation(FGameplayAbilitySpecHandle Handle)
{
	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(Handle);
	if (!Spec)
	{
		// Can happen if the client gets told to activate an ability the same frame that abilities are added on the server
		FPendingAbilityInfo AbilityInfo;
		AbilityInfo.Handle = Handle;
		AbilityInfo.bPartiallyActivated = false;

		// This won't add it if we're currently being called from the pending list
		PendingServerActivatedAbilities.AddUnique(AbilityInfo);
		return;
	}

	InternalTryActivateAbility(Handle);
}

void UAbilitySystemComponent::InternalServerTryActivateAbility(FGameplayAbilitySpecHandle Handle, bool InputPressed, const FPredictionKey& PredictionKey, const FGameplayEventData* TriggerEventData)
{
#if WITH_SERVER_CODE
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (DenyClientActivation > 0)
	{
		DenyClientActivation--;
		ClientActivateAbilityFailed(Handle, PredictionKey.Current);
		return;
	}
#endif

	ABILITYLIST_SCOPE_LOCK();

	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(Handle);
	if (!Spec)
	{
		// Can potentially happen in race conditions where client tries to activate ability that is removed server side before it is received.
		UE_VLOG_UELOG(GetOwner(), LogAbilitySystem, Display, TEXT("%hs: Rejecting ClientActivation of ability with invalid SpecHandle!"), __func__);
		ClientActivateAbilityFailed(Handle, PredictionKey.Current);
		return;
	}

	const UGameplayAbility* AbilityToActivate = Spec->Ability;

	if (!ensure(AbilityToActivate))
	{
		UE_VLOG_UELOG(GetOwner(), LogAbilitySystem, Error, TEXT("%hs: Rejecting ClientActivation of unconfigured spec ability!"), __func__);
		ClientActivateAbilityFailed(Handle, PredictionKey.Current);
		return;
	}

	// Ignore a client trying to activate an ability requiring server execution
	if (AbilityToActivate->GetNetSecurityPolicy() == EGameplayAbilityNetSecurityPolicy::ServerOnlyExecution ||
		AbilityToActivate->GetNetSecurityPolicy() == EGameplayAbilityNetSecurityPolicy::ServerOnly)
	{
		UE_VLOG_UELOG(GetOwner(), LogAbilitySystem, Display, TEXT("%hs: Rejecting ClientActivation of %s due to security policy violation."), __func__, *GetNameSafe(AbilityToActivate));
		ClientActivateAbilityFailed(Handle, PredictionKey.Current);
		return;
	}

	// Consume any pending target info, to clear out cancels from old executions
	ConsumeAllReplicatedData(Handle, PredictionKey);

	FScopedPredictionWindow ScopedPredictionWindow(this, PredictionKey);

	ensure(AbilityActorInfo.IsValid());

	SCOPE_CYCLE_COUNTER(STAT_AbilitySystemComp_ServerTryActivate);
	SCOPE_CYCLE_UOBJECT(Ability, AbilityToActivate);

	UGameplayAbility* InstancedAbility = nullptr;
	Spec->InputPressed = true;

	// Attempt to activate the ability (server side) and tell the client if it succeeded or failed.
	if (InternalTryActivateAbility(Handle, PredictionKey, &InstancedAbility, nullptr, TriggerEventData))
	{
		// TryActivateAbility handles notifying the client of success, but let's still log it
		UE_VLOG_UELOG(GetOwner(), LogAbilitySystem, Verbose, TEXT("%hs: Accepted ClientActivation of %s with PredictionKey %s."), __func__, *GetNameSafe(Spec->Ability), *PredictionKey.ToString());
	}
	else
	{
		UE_VLOG_UELOG(GetOwner(), LogAbilitySystem, Display, TEXT("%hs: Rejecting ClientActivation of %s with PredictionKey %s. InternalTryActivateAbility failed: %s"),
			__func__, *GetNameSafe(Spec->Ability), *PredictionKey.ToString(), *InternalTryActivateAbilityFailureTags.ToStringSimple() );

		ClientActivateAbilityFailed(Handle, PredictionKey.Current);
		Spec->InputPressed = false;

		MarkAbilitySpecDirty(*Spec);
	}
#endif
}

void UAbilitySystemComponent::ReplicateEndOrCancelAbility(FGameplayAbilitySpecHandle Handle, FGameplayAbilityActivationInfo ActivationInfo, UGameplayAbility* Ability, bool bWasCanceled)
{
	if (Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::LocalPredicted || Ability->GetNetExecutionPolicy() == EGameplayAbilityNetExecutionPolicy::ServerInitiated)
	{
		// Only replicate ending if policy is predictive
		if (GetOwnerRole() == ROLE_Authority)
		{
			if (!AbilityActorInfo->IsLocallyControlled())
			{
				// Only tell the client about the end/cancel ability if we're not the local controller
				if (bWasCanceled)
				{
					ClientCancelAbility(Handle, ActivationInfo);
				}
				else
				{
					ClientEndAbility(Handle, ActivationInfo);
				}
			}
		}
		else if(Ability->GetNetSecurityPolicy() != EGameplayAbilityNetSecurityPolicy::ServerOnlyTermination && Ability->GetNetSecurityPolicy() != EGameplayAbilityNetSecurityPolicy::ServerOnly)
		{
			// This passes up the current prediction key if we have one
			if (bWasCanceled)
			{
				ServerCancelAbility(Handle, ActivationInfo);
			}
			else
			{
				CallServerEndAbility(Handle, ActivationInfo, ScopedPredictionKey);
			}
		}
	}
}

// This is only called when ending or canceling an ability in response to a remote instruction.
void UAbilitySystemComponent::RemoteEndOrCancelAbility(FGameplayAbilitySpecHandle AbilityToEnd, FGameplayAbilityActivationInfo ActivationInfo, bool bWasCanceled)
{
	FGameplayAbilitySpec* AbilitySpec = FindAbilitySpecFromHandle(AbilityToEnd);
	if (AbilitySpec && AbilitySpec->Ability && AbilitySpec->IsActive())
	{
		// Handle non-instanced case, which cannot perform prediction key validation
		if (AbilitySpec->Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::NonInstanced)
		{
			// End/Cancel the ability but don't replicate it back to whoever called us
			if (bWasCanceled)
			{
				AbilitySpec->Ability->CancelAbility(AbilityToEnd, AbilityActorInfo.Get(), ActivationInfo, false);
			}
			else
			{
				AbilitySpec->Ability->EndAbility(AbilityToEnd, AbilityActorInfo.Get(), ActivationInfo, false, bWasCanceled);
			}
		}
		else
		{
			TArray<UGameplayAbility*> Instances = AbilitySpec->GetAbilityInstances();

			for (auto Instance : Instances)
			{
				UE_CLOGF(Instance == nullptr, LogAbilitySystem, Fatal, "UAbilitySystemComponent::RemoteEndOrCancelAbility null instance for %ls", *GetNameSafe(AbilitySpec->Ability));

				// Check if the ability is the same prediction key (can both by 0) and has been confirmed. If so cancel it.
				if (Instance->GetCurrentActivationInfoRef().GetActivationPredictionKey() == ActivationInfo.GetActivationPredictionKey())
				{
					// Let the ability know that the remote instance has ended, even if we aren't about to end it here.
					Instance->SetRemoteInstanceHasEnded();

					if (Instance->GetCurrentActivationInfoRef().bCanBeEndedByOtherInstance)
					{
						// End/Cancel the ability but don't replicate it back to whoever called us
						if (bWasCanceled)
						{
							ForceCancelAbilityDueToReplication(Instance);
						}
						else
						{
							Instance->EndAbility(Instance->CurrentSpecHandle, Instance->CurrentActorInfo, Instance->CurrentActivationInfo, false, bWasCanceled);
						}
					}
				}
			}
		}
	}
}

void UAbilitySystemComponent::ForceCancelAbilityDueToReplication(UGameplayAbility* Instance)
{
	check(Instance);

	// Since this was a remote cancel, we should force it through. We do not support 'server says ability was cancelled but client disagrees that it can be'.
	Instance->SetCanBeCanceled(true);
	Instance->CancelAbility(Instance->CurrentSpecHandle, Instance->CurrentActorInfo, Instance->CurrentActivationInfo, false);
}

void UAbilitySystemComponent::ServerEndAbility_Implementation(FGameplayAbilitySpecHandle AbilityToEnd, FGameplayAbilityActivationInfo ActivationInfo, FPredictionKey PredictionKey)
{
	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityToEnd);

	if (Spec && Spec->Ability &&
		Spec->Ability->GetNetSecurityPolicy() != EGameplayAbilityNetSecurityPolicy::ServerOnlyTermination &&
		Spec->Ability->GetNetSecurityPolicy() != EGameplayAbilityNetSecurityPolicy::ServerOnly)
	{
		SCOPE_CYCLE_COUNTER(STAT_AbilitySystemComp_ServerEndAbility);

		FScopedPredictionWindow ScopedPrediction(this, PredictionKey);

		RemoteEndOrCancelAbility(AbilityToEnd, ActivationInfo, false);
	}
}

bool UAbilitySystemComponent::ServerEndAbility_Validate(FGameplayAbilitySpecHandle AbilityToEnd, FGameplayAbilityActivationInfo ActivationInfo, FPredictionKey PredictionKey)
{
	return true;
}

void UAbilitySystemComponent::ClientEndAbility_Implementation(FGameplayAbilitySpecHandle AbilityToEnd, FGameplayAbilityActivationInfo ActivationInfo)
{
	RemoteEndOrCancelAbility(AbilityToEnd, ActivationInfo, false);
}

void UAbilitySystemComponent::ServerCancelAbility_Implementation(FGameplayAbilitySpecHandle AbilityToCancel, FGameplayAbilityActivationInfo ActivationInfo)
{
	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityToCancel);

	if (Spec && Spec->Ability &&
		Spec->Ability->GetNetSecurityPolicy() != EGameplayAbilityNetSecurityPolicy::ServerOnlyTermination &&
		Spec->Ability->GetNetSecurityPolicy() != EGameplayAbilityNetSecurityPolicy::ServerOnly)
	{
		RemoteEndOrCancelAbility(AbilityToCancel, ActivationInfo, true);
	}
}

bool UAbilitySystemComponent::ServerCancelAbility_Validate(FGameplayAbilitySpecHandle AbilityToCancel, FGameplayAbilityActivationInfo ActivationInfo)
{
	return true;
}

void UAbilitySystemComponent::ClientCancelAbility_Implementation(FGameplayAbilitySpecHandle AbilityToCancel, FGameplayAbilityActivationInfo ActivationInfo)
{
	RemoteEndOrCancelAbility(AbilityToCancel, ActivationInfo, true);
}

static_assert(sizeof(int16) == sizeof(FPredictionKey::KeyType), "Sizeof PredictionKey::KeyType does not match RPC parameters in AbilitySystemComponent ClientActivateAbilityFailed_Implementation");


int32 ClientActivateAbilityFailedPrintDebugThreshhold = -1;
static FAutoConsoleVariableRef CVarClientActivateAbilityFailedPrintDebugThreshhold(TEXT("AbilitySystem.ClientActivateAbilityFailedPrintDebugThreshhold"), ClientActivateAbilityFailedPrintDebugThreshhold, TEXT(""), ECVF_Default );

float ClientActivateAbilityFailedPrintDebugThreshholdTime = 3.f;
static FAutoConsoleVariableRef CVarClientActivateAbilityFailedPrintDebugThreshholdTime(TEXT("AbilitySystem.ClientActivateAbilityFailedPrintDebugThreshholdTime"), ClientActivateAbilityFailedPrintDebugThreshholdTime, TEXT(""), ECVF_Default );

void UAbilitySystemComponent::ClientActivateAbilityFailed_Implementation(FGameplayAbilitySpecHandle Handle, int16 PredictionKey)
{
	// Tell anything else listening that this was rejected
	if (PredictionKey > 0)
	{
		FPredictionKeyDelegates::BroadcastRejectedDelegate(PredictionKey);
	}

	// Find the actual UGameplayAbility		
	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(Handle);
	if (Spec == nullptr)
	{
		ABILITY_LOG(Display, TEXT("ClientActivateAbilityFailed_Implementation. PredictionKey: %d Ability: Could not find!"), PredictionKey);
		return;
	}

	ABILITY_LOG(Display, TEXT("ClientActivateAbilityFailed_Implementation. PredictionKey :%d Ability: %s"), PredictionKey, *GetNameSafe(Spec->Ability));
	
	if (ClientActivateAbilityFailedPrintDebugThreshhold > 0)
	{
		if ((ClientActivateAbilityFailedStartTime <= 0.f) || ((GetWorld()->GetTimeSeconds() - ClientActivateAbilityFailedStartTime) > ClientActivateAbilityFailedPrintDebugThreshholdTime))
		{
			ClientActivateAbilityFailedStartTime = GetWorld()->GetTimeSeconds();
			ClientActivateAbilityFailedCountRecent = 0;
		}
		
		
		if (++ClientActivateAbilityFailedCountRecent > ClientActivateAbilityFailedPrintDebugThreshhold)
		{
			ABILITY_LOG(Display, TEXT("Threshold hit! Printing debug information"));
			PrintDebug();
			ClientActivateAbilityFailedCountRecent = 0;
			ClientActivateAbilityFailedStartTime = 0.f;
		}
	}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// The ability should be either confirmed or rejected by the time we get here
	if (Spec->ActivationInfo.GetActivationPredictionKey().Current == PredictionKey)
	{
		Spec->ActivationInfo.SetActivationRejected();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TArray<UGameplayAbility*> Instances = Spec->GetAbilityInstances();
	for (UGameplayAbility* Ability : Instances)
	{
		if (Ability->CurrentActivationInfo.GetActivationPredictionKey().Current == PredictionKey)
		{
			Ability->CurrentActivationInfo.SetActivationRejected();
			Ability->K2_EndAbility();
		}
	}
}

void UAbilitySystemComponent::OnClientActivateAbilityCaughtUp(FGameplayAbilitySpecHandle Handle, FPredictionKey::KeyType PredictionKey)
{
	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(Handle);
	if (Spec && Spec->IsActive())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FGameplayAbilityActivationInfo* ActivationInfo = (Spec->ActivationInfo.GetActivationPredictionKey().Current == PredictionKey) ? &Spec->ActivationInfo : nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		for (const UGameplayAbility* Ability : Spec->GetAbilityInstances())
		{
			if (Ability->CurrentActivationInfo.GetActivationPredictionKey().Current == PredictionKey)
			{
				ActivationInfo = &Ability->CurrentActivationInfo;
				break;
			}
		}

		// The ability should be either confirmed or rejected by the time we get here
		if (ActivationInfo && ActivationInfo->ActivationMode == EGameplayAbilityActivationMode::Predicting)
		{
			// It is possible to have this happen under bad network conditions. (Reliable Confirm/Reject RPC is lost, but separate property bunch makes it through before the reliable resend happens)
			ABILITY_LOG(Display, TEXT("%hs. Ability %s caught up to PredictionKey %d but instance is still active and in predicting state."), __func__, *GetNameSafe(Spec->Ability), PredictionKey);
		}
	}
}

void UAbilitySystemComponent::ClientActivateAbilitySucceed_Implementation(FGameplayAbilitySpecHandle Handle, FPredictionKey PredictionKey)
{
	ClientActivateAbilitySucceedWithEventData_Implementation(Handle, PredictionKey, FGameplayEventData());
}

void UAbilitySystemComponent::ClientActivateAbilitySucceedWithEventData_Implementation(FGameplayAbilitySpecHandle Handle, FPredictionKey PredictionKey, FGameplayEventData TriggerEventData)
{
	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(Handle);
	if (!Spec)
	{
		// Can happen if the client gets told to activate an ability the same frame that abilities are added on the server
		FPendingAbilityInfo AbilityInfo;
		AbilityInfo.PredictionKey = PredictionKey;
		AbilityInfo.Handle = Handle;
		AbilityInfo.TriggerEventData = TriggerEventData;
		AbilityInfo.bPartiallyActivated = true;

		// This won't add it if we're currently being called from the pending list
		PendingServerActivatedAbilities.AddUnique(AbilityInfo);
		return;
	}

	UGameplayAbility* AbilityToActivate = Spec->Ability;

	check(AbilityToActivate);
	ensure(AbilityActorInfo.IsValid());
	
	if (!AbilityActorInfo->OwnerActor.IsValid())
	{
		// When the owner is explicitly switched and then gets destroyed, the OwnerActor could become invalid
		UE_LOGF(LogAbilitySystem, Verbose, "%ls: server confirmed [%ls] %ls (Prediction key %ls), but failed with invalid OwnerActor", *GetNameSafe(GetOwner()), *Handle.ToString(), *GetNameSafe(AbilityToActivate), *PredictionKey.ToString());
		return;
	}

	UE_LOGF(LogAbilitySystem, Verbose, "%ls: Server Confirmed [%ls] %ls. PredictionKey: %ls", *GetNameSafe(GetOwner()), *Handle.ToString(), *GetNameSafe(AbilityToActivate), *PredictionKey.ToString());
	UE_VLOG(GetOwner(), VLogAbilitySystem, Verbose, TEXT("Server Confirmed [%s] %s. PredictionKey: %s"), *Handle.ToString(), *GetNameSafe(AbilityToActivate), *PredictionKey.ToString());

	// Fixme: We need a better way to link up/reconcile predictive replicated abilities. It would be ideal if we could predictively spawn an
	// ability and then replace/link it with the server spawned one once the server has confirmed it.

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bNonInstanced = AbilityToActivate->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::NonInstanced;
	FGameplayAbilityActivationInfo NewActivationInfo{ AbilityActorInfo->OwnerActor.Get() };
	FGameplayAbilityActivationInfo& ActivationInfo = bNonInstanced ? Spec->ActivationInfo : NewActivationInfo;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Confirm and allow the remote ending of the ability
	ActivationInfo.SetActivationConfirmed();

	const bool bLocallyPredicted = AbilityToActivate->NetExecutionPolicy == EGameplayAbilityNetExecutionPolicy::LocalPredicted;
	if (bLocallyPredicted)
	{
		if (bNonInstanced)
		{
			// AbilityToActivate->ConfirmActivateSucceed(); // This doesn't do anything for non instanced
		}
		else
		{
			// Find the one we predictively spawned, tell them we are confirmed
			bool found = false;
			TArray<UGameplayAbility*> Instances = Spec->GetAbilityInstances();
			for (UGameplayAbility* LocalAbility : Instances)
			{
				if (LocalAbility != nullptr && LocalAbility->GetCurrentActivationInfo().GetActivationPredictionKey() == PredictionKey)
				{
					LocalAbility->ConfirmActivateSucceed();
					found = true;
					break;
				}
			}

			if (!found)
			{
				ABILITY_LOG(Verbose, TEXT("Ability %s was confirmed by server but no longer exists on client (replication key: %s)"), *AbilityToActivate->GetName(), *PredictionKey.ToString());
			}
		}
	}
	else
	{

		// We haven't already executed this ability at all, so kick it off.
		if (PredictionKey.bIsServerInitiated)
		{
			// We have an active server key, set our key equal to it
			ActivationInfo.ServerSetActivationPredictionKey(PredictionKey);
		}
		
		if (AbilityToActivate->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerExecution)
		{
			// Need to instantiate this in order to execute
			UGameplayAbility* InstancedAbility = CreateNewInstanceOfAbility(*Spec, AbilityToActivate);
			InstancedAbility->CallActivateAbility(Handle, AbilityActorInfo.Get(), ActivationInfo, nullptr, TriggerEventData.EventTag.IsValid() ?  &TriggerEventData : nullptr);
		}
		else if (AbilityToActivate->GetInstancingPolicy() != EGameplayAbilityInstancingPolicy::NonInstanced)
		{
			UGameplayAbility* InstancedAbility = Spec->GetPrimaryInstance();

			if (!InstancedAbility)
			{
				ABILITY_LOG(Warning, TEXT("Ability %s cannot be activated on the client because it's missing a primary instance!"), *AbilityToActivate->GetName());
				return;
			}
			InstancedAbility->CallActivateAbility(Handle, AbilityActorInfo.Get(), ActivationInfo, nullptr, TriggerEventData.EventTag.IsValid() ? &TriggerEventData : nullptr);
		}
		else
		{
			AbilityToActivate->CallActivateAbility(Handle, AbilityActorInfo.Get(), ActivationInfo, nullptr, TriggerEventData.EventTag.IsValid() ? &TriggerEventData : nullptr);
		}
	}
}

bool UAbilitySystemComponent::HasActivatableTriggeredAbility(FGameplayTag Tag)
{
	TArray<FGameplayAbilitySpec> Specs = GetActivatableAbilities();
	const FGameplayAbilityActorInfo* ActorInfo = AbilityActorInfo.Get();
	for (const FGameplayAbilitySpec& Spec : Specs)
	{
		if (Spec.Ability == nullptr)
		{
			continue;
		}

		TArray<FAbilityTriggerData>& Triggers = Spec.Ability->AbilityTriggers;
		for (const FAbilityTriggerData& Data : Triggers)
		{
			if (Data.TriggerTag == Tag && Spec.Ability->CanActivateAbility(Spec.Handle, ActorInfo))
			{
				return true;
			}
		}
	}
	return false;
}

bool UAbilitySystemComponent::TriggerAbilityFromGameplayEvent(FGameplayAbilitySpecHandle Handle, FGameplayAbilityActorInfo* ActorInfo, FGameplayTag EventTag, const FGameplayEventData* Payload, UAbilitySystemComponent& Component)
{
	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(Handle);
	if (!ensureMsgf(Spec, TEXT("Failed to find gameplay ability spec %s"), *EventTag.ToString()))
	{
		return false;
	}

	const UGameplayAbility* InstancedAbility = Spec->GetPrimaryInstance();
	const UGameplayAbility* Ability = InstancedAbility ? InstancedAbility : Spec->Ability.Get();
	if (!ensure(Ability))
	{
		return false;
	}

	if (!ensure(Payload))
	{
		return false;
	}

	if (!HasNetworkAuthorityToActivateTriggeredAbility(*Spec))
	{
		// The server or client will handle activating the trigger
		return false;
	}

	// Make a temp copy of the payload, and copy the event tag into it
	FGameplayEventData TempEventData = *Payload;
	TempEventData.EventTag = EventTag;

	// Run on the non-instanced ability
	return InternalTryActivateAbility(Handle, ScopedPredictionKey, nullptr, nullptr, &TempEventData);
}

bool UAbilitySystemComponent::GetUserAbilityActivationInhibited() const
{
	return UserAbilityActivationInhibited;
}

void UAbilitySystemComponent::SetUserAbilityActivationInhibited(bool NewInhibit)
{
	if(AbilityActorInfo->IsLocallyControlled())
	{
		if (NewInhibit && UserAbilityActivationInhibited)
		{
			// This could cause problems if two sources try to inhibit ability activation, it is not clear when the ability should be uninhibited
			ABILITY_LOG(Warning, TEXT("Call to SetUserAbilityActivationInhibited(true) when UserAbilityActivationInhibited was already true"));
		}

		UserAbilityActivationInhibited = NewInhibit;
	}
}

void UAbilitySystemComponent::NotifyAbilityCommit(UGameplayAbility* Ability)
{
	AbilityCommittedCallbacks.Broadcast(Ability);
}

void UAbilitySystemComponent::NotifyAbilityActivated(FGameplayAbilitySpecHandle Handle, UGameplayAbility* Ability)
{
	AbilityActivatedCallbacks.Broadcast(Ability);
}

void UAbilitySystemComponent::NotifyAbilityFailed(const FGameplayAbilitySpecHandle Handle, UGameplayAbility* Ability, const FGameplayTagContainer& FailureReason)
{
	AbilityFailedCallbacks.Broadcast(Ability, FailureReason);
}

int32 UAbilitySystemComponent::HandleGameplayEvent(FGameplayTag EventTag, const FGameplayEventData* Payload)
{
	int32 TriggeredCount = 0;
	FGameplayTag CurrentTag = EventTag;
	ABILITYLIST_SCOPE_LOCK();
	while (CurrentTag.IsValid())
	{
		if (GameplayEventTriggeredAbilities.Contains(CurrentTag))
		{
			TArray<FGameplayAbilitySpecHandle> TriggeredAbilityHandles = GameplayEventTriggeredAbilities[CurrentTag];

			for (const FGameplayAbilitySpecHandle& AbilityHandle : TriggeredAbilityHandles)
			{
				if (TriggerAbilityFromGameplayEvent(AbilityHandle, AbilityActorInfo.Get(), EventTag, Payload, *this))
				{
					TriggeredCount++;
				}
			}
		}

		CurrentTag = CurrentTag.RequestDirectParent();
	}

	if (FGameplayEventMulticastDelegate* Delegate = GenericGameplayEventCallbacks.Find(EventTag))
	{
		// Make a copy before broadcasting to prevent memory stomping
		FGameplayEventMulticastDelegate DelegateCopy = *Delegate;
		DelegateCopy.Broadcast(Payload);
	}

	// Make a copy in case it changes due to callbacks
	TArray<TPair<FGameplayTagContainer, FGameplayEventTagMulticastDelegate>> LocalGameplayEventTagContainerDelegates = GameplayEventTagContainerDelegates;
	for (TPair<FGameplayTagContainer, FGameplayEventTagMulticastDelegate>& SearchPair : LocalGameplayEventTagContainerDelegates)
	{
		if (SearchPair.Key.IsEmpty() || EventTag.MatchesAny(SearchPair.Key))
		{
			SearchPair.Value.Broadcast(EventTag, Payload);
		}
	}

	return TriggeredCount;
}

FDelegateHandle UAbilitySystemComponent::AddGameplayEventTagContainerDelegate(const FGameplayTagContainer& TagFilter, const FGameplayEventTagMulticastDelegate::FDelegate& Delegate)
{
	TPair<FGameplayTagContainer, FGameplayEventTagMulticastDelegate>* FoundPair = nullptr;

	for (TPair<FGameplayTagContainer, FGameplayEventTagMulticastDelegate>& SearchPair : GameplayEventTagContainerDelegates)
	{
		if (TagFilter == SearchPair.Key)
		{
			FoundPair = &SearchPair;
			break;
		}
	}

	if (!FoundPair)
	{
		FoundPair = new(GameplayEventTagContainerDelegates) TPair<FGameplayTagContainer, FGameplayEventTagMulticastDelegate>(TagFilter, FGameplayEventTagMulticastDelegate());
	}

	return FoundPair->Value.Add(Delegate);
}

void UAbilitySystemComponent::RemoveGameplayEventTagContainerDelegate(const FGameplayTagContainer& TagFilter, FDelegateHandle DelegateHandle)
{
	// Look for and remove delegate, remove from array if no more delegates are bound
	for (int32 Index = 0; Index < GameplayEventTagContainerDelegates.Num(); Index++)
	{
		TPair<FGameplayTagContainer, FGameplayEventTagMulticastDelegate>& SearchPair = GameplayEventTagContainerDelegates[Index];
		if (TagFilter == SearchPair.Key)
		{
			SearchPair.Value.Remove(DelegateHandle);
			if (!SearchPair.Value.IsBound())
			{
				GameplayEventTagContainerDelegates.RemoveAt(Index);
			}
			break;
		}
	}
}

void UAbilitySystemComponent::MonitoredTagChanged(const FGameplayTag Tag, int32 NewCount)
{
	ABILITYLIST_SCOPE_LOCK();

	int32 TriggeredCount = 0;
	if (OwnedTagTriggeredAbilities.Contains(Tag))
	{
		TArray<FGameplayAbilitySpecHandle> TriggeredAbilityHandles = OwnedTagTriggeredAbilities[Tag];

		for (auto AbilityHandle : TriggeredAbilityHandles)
		{
			FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityHandle);

			if (!Spec || !HasNetworkAuthorityToActivateTriggeredAbility(*Spec))
			{
				continue;
			}

			if (Spec->Ability)
			{
				TArray<FAbilityTriggerData> AbilityTriggers = Spec->Ability->AbilityTriggers;
				AbilityTriggers.Append(Spec->DynamicAbilityTriggers);
				for (const FAbilityTriggerData& TriggerData : AbilityTriggers)
				{
					FGameplayTag EventTag = TriggerData.TriggerTag;

					if (EventTag == Tag)
					{
						if (NewCount > 0)
						{
							// Populate event data so this will use the same blueprint node to activate as gameplay triggers
							FGameplayEventData EventData;
							EventData.EventMagnitude = NewCount;
							EventData.EventTag = EventTag;
							EventData.Instigator = GetOwnerActor();
							EventData.Target = EventData.Instigator;
							// Try to activate it
							InternalTryActivateAbility(Spec->Handle, FPredictionKey(), nullptr, nullptr, &EventData);

							// TODO: Check client/server type
						}
						else if (NewCount == 0 && TriggerData.TriggerSource == EGameplayAbilityTriggerSource::OwnedTagPresent)
						{
							// Try to cancel, but only if the type is right
							CancelAbilitySpec(*Spec, nullptr);
						}
					}
				}
			}
		}
	}
}

bool UAbilitySystemComponent::HasNetworkAuthorityToActivateTriggeredAbility(const FGameplayAbilitySpec &Spec) const
{
	bool bIsAuthority = IsOwnerActorAuthoritative();
	bool bIsLocal = AbilityActorInfo->IsLocallyControlled();

	switch (Spec.Ability->GetNetExecutionPolicy())
	{
	case EGameplayAbilityNetExecutionPolicy::LocalOnly:
	case EGameplayAbilityNetExecutionPolicy::LocalPredicted:
		return bIsLocal;
	case EGameplayAbilityNetExecutionPolicy::ServerOnly:
	case EGameplayAbilityNetExecutionPolicy::ServerInitiated:
		return bIsAuthority;
	}

	return false;
}

void UAbilitySystemComponent::BindToInputComponent(UInputComponent* InputComponent)
{
	static const FName ConfirmBindName(TEXT("AbilityConfirm"));
	static const FName CancelBindName(TEXT("AbilityCancel"));

	// Pressed event
	{
		FInputActionBinding AB(ConfirmBindName, IE_Pressed);
		AB.ActionDelegate.GetDelegateForManualSet().BindUObject(this, &UAbilitySystemComponent::LocalInputConfirm);
		InputComponent->AddActionBinding(AB);
	}

	// 
	{
		FInputActionBinding AB(CancelBindName, IE_Pressed);
		AB.ActionDelegate.GetDelegateForManualSet().BindUObject(this, &UAbilitySystemComponent::LocalInputCancel);
		InputComponent->AddActionBinding(AB);
	}
}

void UAbilitySystemComponent::BindAbilityActivationToInputComponent(UInputComponent* InputComponent, FGameplayAbilityInputBinds BindInfo)
{
	UEnum* EnumBinds = BindInfo.GetBindEnum();

	SetBlockAbilityBindingsArray(BindInfo);

	for(int32 idx=0; idx < EnumBinds->NumEnums(); ++idx)
	{
		const FString FullStr = EnumBinds->GetNameStringByIndex(idx);
		
		// Pressed event
		{
			FInputActionBinding AB(FName(*FullStr), IE_Pressed);
			AB.ActionDelegate.GetDelegateForManualSet().BindUObject(this, &UAbilitySystemComponent::AbilityLocalInputPressed, idx);
			InputComponent->AddActionBinding(AB);
		}

		// Released event
		{
			FInputActionBinding AB(FName(*FullStr), IE_Released);
			AB.ActionDelegate.GetDelegateForManualSet().BindUObject(this, &UAbilitySystemComponent::AbilityLocalInputReleased, idx);
			InputComponent->AddActionBinding(AB);
		}
	}

	// Bind Confirm/Cancel. Note: these have to come last!
	if (BindInfo.ConfirmTargetCommand.IsEmpty() == false)
	{
		FInputActionBinding AB(FName(*BindInfo.ConfirmTargetCommand), IE_Pressed);
		AB.ActionDelegate.GetDelegateForManualSet().BindUObject(this, &UAbilitySystemComponent::LocalInputConfirm);
		InputComponent->AddActionBinding(AB);
	}
	
	if (BindInfo.CancelTargetCommand.IsEmpty() == false)
	{
		FInputActionBinding AB(FName(*BindInfo.CancelTargetCommand), IE_Pressed);
		AB.ActionDelegate.GetDelegateForManualSet().BindUObject(this, &UAbilitySystemComponent::LocalInputCancel);
		InputComponent->AddActionBinding(AB);
	}

	if (BindInfo.CancelTargetInputID >= 0)
	{
		GenericCancelInputID = BindInfo.CancelTargetInputID;
	}
	if (BindInfo.ConfirmTargetInputID >= 0)
	{
		GenericConfirmInputID = BindInfo.ConfirmTargetInputID;
	}
}

void UAbilitySystemComponent::SetBlockAbilityBindingsArray(FGameplayAbilityInputBinds BindInfo)
{
	UEnum* EnumBinds = BindInfo.GetBindEnum();
	GetBlockedAbilityBindings_Mutable().SetNumZeroed(EnumBinds->NumEnums());
}

void UAbilitySystemComponent::AbilityLocalInputPressed(int32 InputID)
{
	// Consume the input if this InputID is overloaded with GenericConfirm/Cancel and the GenericConfim/Cancel callback is bound
	if (IsGenericConfirmInputBound(InputID))
	{
		LocalInputConfirm();
		return;
	}

	if (IsGenericCancelInputBound(InputID))
	{
		LocalInputCancel();
		return;
	}

	// ---------------------------------------------------------

	ABILITYLIST_SCOPE_LOCK();
	for (FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		if (Spec.InputID == InputID)
		{
			if (Spec.Ability)
			{
				Spec.InputPressed = true;
				if (Spec.IsActive())
				{
					if (Spec.Ability->bReplicateInputDirectly && IsOwnerActorAuthoritative() == false)
					{
						ServerSetInputPressed(Spec.Handle);
					}

					AbilitySpecInputPressed(Spec);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
					// Fixing this up to use the instance activation, but this function should be deprecated as it cannot work with InstancedPerExecution
					UE_CLOGF(Spec.Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerExecution, LogAbilitySystem, Warning, "%s: %ls is InstancedPerExecution. This is unreliable for Input as you may only interact with the latest spawned Instance", __func__, *GetNameSafe(Spec.Ability));
					TArray<UGameplayAbility*> Instances = Spec.GetAbilityInstances();
					const FGameplayAbilityActivationInfo& ActivationInfo = Instances.IsEmpty() ? Spec.ActivationInfo : Instances.Last()->GetCurrentActivationInfoRef();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
					// Invoke the InputPressed event. This is not replicated here. If someone is listening, they may replicate the InputPressed event to the server.
					InvokeReplicatedEvent(EAbilityGenericReplicatedEvent::InputPressed, Spec.Handle, ActivationInfo.GetActivationPredictionKey());					
				}
				else
				{
					// Ability is not active, so try to activate it
					TryActivateAbility(Spec.Handle);
				}
			}
		}
	}
}

void UAbilitySystemComponent::AbilityLocalInputReleased(int32 InputID)
{
	ABILITYLIST_SCOPE_LOCK();
	for (FGameplayAbilitySpec& Spec : ActivatableAbilities.Items)
	{
		if (Spec.InputID == InputID)
		{
			Spec.InputPressed = false;
			if (Spec.Ability && Spec.IsActive())
			{
				if (Spec.Ability->bReplicateInputDirectly && IsOwnerActorAuthoritative() == false)
				{
					ServerSetInputReleased(Spec.Handle);
				}

				AbilitySpecInputReleased(Spec);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
				// Fixing this up to use the instance activation, but this function should be deprecated as it cannot work with InstancedPerExecution
				UE_CLOGF(Spec.Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerExecution, LogAbilitySystem, Warning, "%s: %ls is InstancedPerExecution. This is unreliable for Input as you may only interact with the latest spawned Instance", __func__, *GetNameSafe(Spec.Ability));
				TArray<UGameplayAbility*> Instances = Spec.GetAbilityInstances();
				const FGameplayAbilityActivationInfo& ActivationInfo = Instances.IsEmpty() ? Spec.ActivationInfo : Instances.Last()->GetCurrentActivationInfoRef();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				InvokeReplicatedEvent(EAbilityGenericReplicatedEvent::InputReleased, Spec.Handle, ActivationInfo.GetActivationPredictionKey());
			}
		}
	}
}

void UAbilitySystemComponent::PressInputID(int32 InputID)
{
	AbilityLocalInputPressed(InputID);
}

void UAbilitySystemComponent::ReleaseInputID(int32 InputID)
{
	AbilityLocalInputReleased(InputID);
}

void UAbilitySystemComponent::ServerSetInputPressed_Implementation(FGameplayAbilitySpecHandle AbilityHandle)
{
	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityHandle);
	if (Spec)
	{
		AbilitySpecInputPressed(*Spec);
	}

}

void UAbilitySystemComponent::ServerSetInputReleased_Implementation(FGameplayAbilitySpecHandle AbilityHandle)
{
	FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityHandle);
	if (Spec)
	{
		AbilitySpecInputReleased(*Spec);
	}
}

bool UAbilitySystemComponent::ServerSetInputPressed_Validate(FGameplayAbilitySpecHandle AbilityHandle)
{
	return true;
}

bool UAbilitySystemComponent::ServerSetInputReleased_Validate(FGameplayAbilitySpecHandle AbilityHandle)
{
	return true;
}

void UAbilitySystemComponent::AbilitySpecInputPressed(FGameplayAbilitySpec& Spec)
{
	Spec.InputPressed = true;
	if (Spec.IsActive())
	{
		// The ability is active, so just pipe the input event to it
		if (Spec.Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::NonInstanced)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Spec.Ability->InputPressed(Spec.Handle, AbilityActorInfo.Get(), Spec.ActivationInfo);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
			TArray<UGameplayAbility*> Instances = Spec.GetAbilityInstances();
			for (UGameplayAbility* Instance : Instances)
			{
				Instance->InputPressed(Spec.Handle, AbilityActorInfo.Get(), Instance->CurrentActivationInfo);
			}
		}
	}
}

void UAbilitySystemComponent::AbilitySpecInputReleased(FGameplayAbilitySpec& Spec)
{
	Spec.InputPressed = false;
	if (Spec.IsActive())
	{
		// The ability is active, so just pipe the input event to it
		if (Spec.Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::NonInstanced)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Spec.Ability->InputReleased(Spec.Handle, AbilityActorInfo.Get(), Spec.ActivationInfo);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
			TArray<UGameplayAbility*> Instances = Spec.GetAbilityInstances();
			for (UGameplayAbility* Instance : Instances)
			{
				Instance->InputReleased(Spec.Handle, AbilityActorInfo.Get(), Instance->CurrentActivationInfo);
			}
		}
	}
}

void UAbilitySystemComponent::LocalInputConfirm()
{
	FAbilityConfirmOrCancel Temp = GenericLocalConfirmCallbacks;
	GenericLocalConfirmCallbacks.Clear();
	Temp.Broadcast();
}

void UAbilitySystemComponent::LocalInputCancel()
{	
	FAbilityConfirmOrCancel Temp = GenericLocalCancelCallbacks;
	GenericLocalCancelCallbacks.Clear();
	Temp.Broadcast();
}

void UAbilitySystemComponent::InputConfirm()
{
	LocalInputConfirm();
}

void UAbilitySystemComponent::InputCancel()
{
	LocalInputCancel();
}

void UAbilitySystemComponent::TargetConfirm()
{
	// Callbacks may modify the spawned target actor array so iterate over a copy instead
	TArray<AGameplayAbilityTargetActor*> LocalTargetActors = SpawnedTargetActors;
	SpawnedTargetActors.Reset();
	for (AGameplayAbilityTargetActor* TargetActor : LocalTargetActors)
	{
		if (TargetActor)
		{
			if (TargetActor->IsConfirmTargetingAllowed())
			{
				//TODO: There might not be any cases where this bool is false
				if (!TargetActor->bDestroyOnConfirmation)
				{
					SpawnedTargetActors.Add(TargetActor);
				}
				TargetActor->ConfirmTargeting();
			}
			else
			{
				SpawnedTargetActors.Add(TargetActor);
			}
		}
	}
}

void UAbilitySystemComponent::TargetCancel()
{
	// Callbacks may modify the spawned target actor array so iterate over a copy instead
	TArray<AGameplayAbilityTargetActor*> LocalTargetActors = SpawnedTargetActors;
	SpawnedTargetActors.Reset();
	for (AGameplayAbilityTargetActor* TargetActor : LocalTargetActors)
	{
		if (TargetActor)
		{
			TargetActor->CancelTargeting();
		}
	}
}

// --------------------------------------------------------------------------

#if ENABLE_VISUAL_LOG
void UAbilitySystemComponent::ClearDebugInstantEffects()
{
	ActiveGameplayEffects.DebugExecutedGameplayEffects.Empty();
}
#endif // ENABLE_VISUAL_LOG

// ---------------------------------------------------------------------------

float UAbilitySystemComponent::PlayMontage(UGameplayAbility* InAnimatingAbility, FGameplayAbilityActivationInfo ActivationInfo, UAnimMontage* NewAnimMontage, float InPlayRate, FName StartSectionName, float StartTimeSeconds)
{
	return PlayMontageInternal(InAnimatingAbility, ActivationInfo, NewAnimMontage, InPlayRate, StartSectionName, StartTimeSeconds, [](FGameplayAbilityRepAnimMontage&){});
}

float UAbilitySystemComponent::PlayMontageInternal(UGameplayAbility* InAnimatingAbility, FGameplayAbilityActivationInfo ActivationInfo, UAnimMontage* NewAnimMontage, float InPlayRate, FName StartSectionName, float StartTimeSeconds, TFunctionRef<void(FGameplayAbilityRepAnimMontage&)> MutateRepAnimMontageFunction)
{
	float Duration = -1.f;

	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if (AnimInstance && NewAnimMontage)
	{
		Duration = AnimInstance->Montage_Play(NewAnimMontage, InPlayRate, EMontagePlayReturnType::MontageLength, StartTimeSeconds);
		if (Duration > 0.f)
		{
			if (const UGameplayAbility* RawAnimatingAbility = LocalAnimMontageInfo.AnimatingAbility.Get())
			{
				if (RawAnimatingAbility != InAnimatingAbility)
				{
					// The ability that was previously animating will have already gotten the 'interrupted' callback.
					// It may be a good idea to make this a global policy and 'cancel' the ability.
					// 
					// For now, we expect it to end itself when this happens.
				}
			}

			UAnimSequenceBase* Animation = NewAnimMontage->IsDynamicMontage() ? NewAnimMontage->GetFirstAnimReference() : NewAnimMontage;

			if (NewAnimMontage->HasRootMotion() && AnimInstance->GetOwningActor())
			{
				UE_LOGF(LogRootMotion, Log, "UAbilitySystemComponent::PlayMontage %ls, Role: %ls"
					, *GetNameSafe(Animation)
					, *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), AnimInstance->GetOwningActor()->GetLocalRole())
					);
			}

			LocalAnimMontageInfo.AnimMontage = NewAnimMontage;
			LocalAnimMontageInfo.AnimatingAbility = InAnimatingAbility;
			LocalAnimMontageInfo.PlayInstanceId = (LocalAnimMontageInfo.PlayInstanceId < UINT8_MAX ? LocalAnimMontageInfo.PlayInstanceId + 1 : 0);
			
			if (InAnimatingAbility)
			{
				InAnimatingAbility->SetCurrentMontage(NewAnimMontage);
			}
			
			// Start at a given Section.
			if (StartSectionName != NAME_None)
			{
				AnimInstance->Montage_JumpToSection(StartSectionName, NewAnimMontage);
			}

			// Replicate for non-owners and for replay recordings
			// The data we set from GetRepAnimMontageInfo_Mutable() is used both by the server to replicate to clients and by clients to record replays.
			// We need to set this data for recording clients because there exists network configurations where an abilities montage data will not replicate to some clients (for example: if the client is an autonomous proxy.)
			if (ShouldRecordMontageReplication())
			{
				FGameplayAbilityRepAnimMontage& MutableRepAnimMontageInfo = GetRepAnimMontageInfo_Mutable();

				// Those are static parameters, they are only set when the montage is played. They are not changed after that.
				MutableRepAnimMontageInfo.Animation = Animation;
				MutableRepAnimMontageInfo.PlayInstanceId = (MutableRepAnimMontageInfo.PlayInstanceId < UINT8_MAX ? MutableRepAnimMontageInfo.PlayInstanceId + 1 : 0);

				MutableRepAnimMontageInfo.SectionIdToPlay = 0;
				if (MutableRepAnimMontageInfo.Animation && StartSectionName != NAME_None)
				{
					// we add one so INDEX_NONE can be used in the on rep
					MutableRepAnimMontageInfo.SectionIdToPlay = NewAnimMontage->GetSectionIndex(StartSectionName) + 1;
				}

				if (NewAnimMontage->IsDynamicMontage())
				{
					check(!NewAnimMontage->SlotAnimTracks.IsEmpty());
					MutableRepAnimMontageInfo.SlotName = NewAnimMontage->SlotAnimTracks[0].SlotName;
					MutableRepAnimMontageInfo.BlendOutTime = NewAnimMontage->GetDefaultBlendInTime();
				}

				// Give the caller the opportunity to modify the rep info for this montage, used by "slot animation as dynamic montage" to replicate play count parameter
				MutateRepAnimMontageFunction(MutableRepAnimMontageInfo);

				// Update parameters that change during Montage life time.
				AnimMontage_UpdateReplicatedData();
			}

			// Replicate to non-owners
			if (IsOwnerActorAuthoritative())
			{
				// Force net update on our avatar actor.
				if (AbilityActorInfo->AvatarActor != nullptr)
				{
					AbilityActorInfo->AvatarActor->ForceNetUpdate();
				}
			}
			else
			{
				// If this prediction key is rejected, we need to end the preview
				FPredictionKey PredictionKey = GetPredictionKeyForNewAction();
				if (PredictionKey.IsValidKey())
				{
					PredictionKey.NewRejectedDelegate().BindUObject(this, &UAbilitySystemComponent::OnPredictiveMontageRejected, NewAnimMontage);
				}
			}
		}
	}

	return Duration;
}

float UAbilitySystemComponent::PlayMontageSimulated(UAnimMontage* NewAnimMontage, float InPlayRate, FName StartSectionName)
{
	float Duration = -1.f;
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if (AnimInstance && NewAnimMontage)
	{
		Duration = AnimInstance->Montage_Play(NewAnimMontage, InPlayRate);
		if (Duration > 0.f)
		{
			LocalAnimMontageInfo.AnimMontage = NewAnimMontage;
		}
	}

	return Duration;
}

void UAbilitySystemComponent::AnimMontage_UpdateReplicatedData()
{
	check(ShouldRecordMontageReplication());

	AnimMontage_UpdateReplicatedData(GetRepAnimMontageInfo_Mutable());
}

void UAbilitySystemComponent::AnimMontage_UpdateReplicatedData(FGameplayAbilityRepAnimMontage& OutRepAnimMontageInfo)
{
	const UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if (AnimInstance && LocalAnimMontageInfo.AnimMontage)
	{
		if (LocalAnimMontageInfo.AnimMontage->IsDynamicMontage())
		{
			OutRepAnimMontageInfo.Animation = LocalAnimMontageInfo.AnimMontage->GetFirstAnimReference();
			OutRepAnimMontageInfo.BlendOutTime = LocalAnimMontageInfo.AnimMontage->GetDefaultBlendOutTime();
		}
		else
		{
			OutRepAnimMontageInfo.Animation = LocalAnimMontageInfo.AnimMontage;
			OutRepAnimMontageInfo.BlendOutTime = 0.0f;
		}

		// Compressed Flags
		const bool bIsStopped = AnimInstance->Montage_GetIsStopped(LocalAnimMontageInfo.AnimMontage);

		if (!bIsStopped)
		{
			OutRepAnimMontageInfo.PlayRate = AnimInstance->Montage_GetPlayRate(LocalAnimMontageInfo.AnimMontage);
			OutRepAnimMontageInfo.Position = AnimInstance->Montage_GetPosition(LocalAnimMontageInfo.AnimMontage);
			OutRepAnimMontageInfo.BlendTime = AnimInstance->Montage_GetBlendTime(LocalAnimMontageInfo.AnimMontage);
		}

		if (OutRepAnimMontageInfo.IsStopped != bIsStopped)
		{
			// Set this prior to calling UpdateShouldTick, so we start ticking if we are playing a Montage
			OutRepAnimMontageInfo.IsStopped = bIsStopped;

			if (bIsStopped)
			{
				// Use AnyThread because GetValueOnGameThread will fail check() when doing replays
				constexpr bool bForceGameThreadValue = true;
				if (CVarGasFixClientSideMontageBlendOutTime.GetValueOnAnyThread(bForceGameThreadValue))
				{
					// Replicate blend out time. This requires a manual search since Montage_GetBlendTime will fail
					// in GetActiveInstanceForMontage for Montages that are stopped.
					for (const FAnimMontageInstance* MontageInstance : AnimInstance->MontageInstances)
					{
						if (MontageInstance->Montage == LocalAnimMontageInfo.AnimMontage)
						{
							OutRepAnimMontageInfo.BlendTime = MontageInstance->GetBlendTime();
							break;
						}
					}
				}
			}

			// When we start or stop an animation, update the clients right away for the Avatar Actor
			if (AbilityActorInfo->AvatarActor != nullptr)
			{
				AbilityActorInfo->AvatarActor->ForceNetUpdate();
			}

			// When this changes, we should update whether or not we should be ticking
			UpdateShouldTick();
		}

		// Replicate NextSectionID to keep it in sync.
		// We actually replicate NextSectionID+1 on a BYTE to put INDEX_NONE in there.
		int32 CurrentSectionID = LocalAnimMontageInfo.AnimMontage->GetSectionIndexFromPosition(OutRepAnimMontageInfo.Position);
		if (CurrentSectionID != INDEX_NONE)
		{
			constexpr bool bForceGameThreadValue = true;
			if (CVarUpdateMontageSectionIdToPlay.GetValueOnAnyThread(bForceGameThreadValue))
			{
				OutRepAnimMontageInfo.SectionIdToPlay = uint8(CurrentSectionID + 1);
			}

			int32 NextSectionID = AnimInstance->Montage_GetNextSectionID(LocalAnimMontageInfo.AnimMontage, CurrentSectionID);
			if (NextSectionID >= (256 - 1))
			{
				ABILITY_LOG( Error, TEXT("AnimMontage_UpdateReplicatedData. NextSectionID = %d.  RepAnimMontageInfo.Position: %.2f, CurrentSectionID: %d. LocalAnimMontageInfo.AnimMontage %s"), 
					NextSectionID, OutRepAnimMontageInfo.Position, CurrentSectionID, *GetNameSafe(LocalAnimMontageInfo.AnimMontage) );
				ensure(NextSectionID < (256 - 1));
			}
			OutRepAnimMontageInfo.NextSectionID = uint8(NextSectionID + 1);
		}
		else
		{
			OutRepAnimMontageInfo.NextSectionID = 0;
		}
	}
}

void UAbilitySystemComponent::AnimMontage_UpdateForcedPlayFlags(FGameplayAbilityRepAnimMontage& OutRepAnimMontageInfo)
{
	OutRepAnimMontageInfo.PlayInstanceId = LocalAnimMontageInfo.PlayInstanceId;
}

void UAbilitySystemComponent::OnPredictiveMontageRejected(UAnimMontage* PredictiveMontage)
{
	static const float MONTAGE_PREDICTION_REJECT_FADETIME = 0.25f;

	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if (AnimInstance && PredictiveMontage)
	{
		// If this montage is still playing: kill it
		if (AnimInstance->Montage_IsPlaying(PredictiveMontage))
		{
			AnimInstance->Montage_Stop(MONTAGE_PREDICTION_REJECT_FADETIME, PredictiveMontage);
		}
	}
}

bool UAbilitySystemComponent::IsReadyForReplicatedMontage()
{
	/** Children may want to override this for additional checks (e.g, "has skin been applied") */
	return true;
}

/**	Replicated Event for AnimMontages */
void UAbilitySystemComponent::OnRep_ReplicatedAnimMontage()
{
	UWorld* World = GetWorld();

	const FGameplayAbilityRepAnimMontage& ConstRepAnimMontageInfo = GetRepAnimMontageInfo_Mutable();

	if (ConstRepAnimMontageInfo.bSkipPlayRate)
	{
		GetRepAnimMontageInfo_Mutable().PlayRate = 1.f;
	}

	const bool bIsPlayingReplay = World && World->IsPlayingReplay();

	const float MONTAGE_REP_POS_ERR_THRESH = bIsPlayingReplay ? CVarReplayMontageErrorThreshold.GetValueOnGameThread() : 0.1f;

	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if (AnimInstance == nullptr || !IsReadyForReplicatedMontage())
	{
		// We can't handle this yet
		bPendingMontageRep = true;
		return;
	}
	bPendingMontageRep = false;

	if (!AbilityActorInfo->IsLocallyControlled())
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("net.Montage.Debug"));
		bool DebugMontage = (CVar && CVar->GetValueOnGameThread() == 1);
		if (DebugMontage)
		{
			ABILITY_LOG( Warning, TEXT("\n\nOnRep_ReplicatedAnimMontage, %s"), *GetNameSafe(this));
			ABILITY_LOG( Warning, TEXT("\tAnimMontage: %s\n\tPlayRate: %f\n\tPosition: %f\n\tBlendTime: %f\n\tNextSectionID: %d\n\tIsStopped: %d\n\tPlayInstanceId: %d"),
				*GetNameSafe(ConstRepAnimMontageInfo.Animation),
				ConstRepAnimMontageInfo.PlayRate,
				ConstRepAnimMontageInfo.Position,
				ConstRepAnimMontageInfo.BlendTime,
				ConstRepAnimMontageInfo.NextSectionID,
				ConstRepAnimMontageInfo.IsStopped,
				ConstRepAnimMontageInfo.PlayInstanceId);
			ABILITY_LOG( Warning, TEXT("\tLocalAnimMontageInfo.AnimMontage: %s\n\tPosition: %f"),
				*GetNameSafe(LocalAnimMontageInfo.AnimMontage), AnimInstance->Montage_GetPosition(LocalAnimMontageInfo.AnimMontage));
		}

		if(ConstRepAnimMontageInfo.Animation)
		{
			// New Montage to play
			UAnimSequenceBase* LocalAnimation = LocalAnimMontageInfo.AnimMontage && LocalAnimMontageInfo.AnimMontage->IsDynamicMontage() ? LocalAnimMontageInfo.AnimMontage->GetFirstAnimReference() : LocalAnimMontageInfo.AnimMontage.Get();
			
			//Animations on sim clients may time out due to poor network conditions; so if the client triggering an ability continues to do so and the montage is stopped for sim clients, we restart the montage on sim clients.
			const bool bSimulatedMontageRequiresRestart = LocalAnimMontageInfo.AnimMontage && !ConstRepAnimMontageInfo.IsStopped && AnimInstance->Montage_GetIsStopped(LocalAnimMontageInfo.AnimMontage) 
				&& CVarAllowSimulatedMontageRestart.GetValueOnGameThread();

			if ((LocalAnimation != ConstRepAnimMontageInfo.Animation) ||
				(LocalAnimMontageInfo.PlayInstanceId != ConstRepAnimMontageInfo.PlayInstanceId) ||
				bSimulatedMontageRequiresRestart)
			{
				LocalAnimMontageInfo.PlayInstanceId = ConstRepAnimMontageInfo.PlayInstanceId;

				if (UAnimMontage* MontageToPlay = Cast<UAnimMontage>(ConstRepAnimMontageInfo.Animation))
				{
					PlayMontageSimulated(MontageToPlay, ConstRepAnimMontageInfo.PlayRate);
				}
				else
				{
					if (ConstRepAnimMontageInfo.PlayCount == 1.0f)
					{
						PlaySlotAnimationAsDynamicMontageSimulated(
							ConstRepAnimMontageInfo.Animation,
							ConstRepAnimMontageInfo.SlotName,
							ConstRepAnimMontageInfo.BlendTime,
							ConstRepAnimMontageInfo.BlendOutTime,
							ConstRepAnimMontageInfo.PlayRate);
					}
					else
					{
						PlaySlotAnimationAsDynamicMontage_WithFractionalLoopsSimulated(
							ConstRepAnimMontageInfo.Animation,
							ConstRepAnimMontageInfo.SlotName,
							ConstRepAnimMontageInfo.BlendTime,
							ConstRepAnimMontageInfo.BlendOutTime,
							ConstRepAnimMontageInfo.PlayRate,
							ConstRepAnimMontageInfo.PlayCount
						);
					}
				}
			}

			if (LocalAnimMontageInfo.AnimMontage == nullptr)
			{ 
				ABILITY_LOG(Warning, TEXT("OnRep_ReplicatedAnimMontage: PlayMontageSimulated failed. Name: %s, Animation: %s"), *GetNameSafe(this), *GetNameSafe(ConstRepAnimMontageInfo.Animation));
				return;
			}

			// Play Rate has changed
			if (AnimInstance->Montage_GetPlayRate(LocalAnimMontageInfo.AnimMontage) != ConstRepAnimMontageInfo.PlayRate)
			{
				AnimInstance->Montage_SetPlayRate(LocalAnimMontageInfo.AnimMontage, ConstRepAnimMontageInfo.PlayRate);
			}

			// Compressed Flags
			const bool bIsStopped = AnimInstance->Montage_GetIsStopped(LocalAnimMontageInfo.AnimMontage);
			const bool bReplicatedIsStopped = bool(ConstRepAnimMontageInfo.IsStopped);

			// Process stopping first, so we don't change sections and cause blending to pop.
			if (bReplicatedIsStopped)
			{
				if (!bIsStopped)
				{
					CurrentMontageStop(ConstRepAnimMontageInfo.BlendTime);
				}
			}
			else if (!ConstRepAnimMontageInfo.SkipPositionCorrection)
			{
				const int32 RepSectionID = LocalAnimMontageInfo.AnimMontage->GetSectionIndexFromPosition(ConstRepAnimMontageInfo.Position);
				const int32 RepNextSectionID = int32(ConstRepAnimMontageInfo.NextSectionID) - 1;
		
				// And NextSectionID for the replicated SectionID.
				if( RepSectionID != INDEX_NONE )
				{
					const int32 NextSectionID = AnimInstance->Montage_GetNextSectionID(LocalAnimMontageInfo.AnimMontage, RepSectionID);

					// If NextSectionID is different than the replicated one, then set it.
					if( NextSectionID != RepNextSectionID )
					{
						AnimInstance->Montage_SetNextSection(LocalAnimMontageInfo.AnimMontage->GetSectionName(RepSectionID), LocalAnimMontageInfo.AnimMontage->GetSectionName(RepNextSectionID), LocalAnimMontageInfo.AnimMontage);
						ABILITY_LOG(Verbose, TEXT("OnRep_ReplicatedAnimMontage: Setting NextSection of %s to %s"), *GetNameSafe(LocalAnimMontageInfo.AnimMontage), *LocalAnimMontageInfo.AnimMontage->GetSectionName(RepNextSectionID).ToString()); 
					}

					// Make sure we haven't received that update too late and the client hasn't already jumped to another section. 
					const int32 CurrentSectionID = LocalAnimMontageInfo.AnimMontage->GetSectionIndexFromPosition(AnimInstance->Montage_GetPosition(LocalAnimMontageInfo.AnimMontage));
					if ((CurrentSectionID != RepSectionID) && (CurrentSectionID != RepNextSectionID))
					{
						// Client is in a wrong section, teleport it into the beginning of the right section
						const float SectionStartTime = LocalAnimMontageInfo.AnimMontage->GetAnimCompositeSection(RepSectionID).GetTime();
						AnimInstance->Montage_SetPosition(LocalAnimMontageInfo.AnimMontage, SectionStartTime);
						ABILITY_LOG(Verbose, TEXT("OnRep_ReplicatedAnimMontage: Setting Section Start Position of %s to %.3f"), *GetNameSafe(LocalAnimMontageInfo.AnimMontage), SectionStartTime); 
					}
				}

				// Update Position. If error is too great, jump to replicated position.
				const float CurrentPosition = AnimInstance->Montage_GetPosition(LocalAnimMontageInfo.AnimMontage);
				const int32 CurrentSectionID = LocalAnimMontageInfo.AnimMontage->GetSectionIndexFromPosition(CurrentPosition);
				const float DeltaPosition = ConstRepAnimMontageInfo.Position - CurrentPosition;

				// Only check threshold if we are located in the same section. Different sections require a bit more work as we could be jumping around the timeline.
				// And therefore DeltaPosition is not as trivial to determine.
				if ((CurrentSectionID == RepSectionID) && (FMath::Abs(DeltaPosition) > MONTAGE_REP_POS_ERR_THRESH) && (ConstRepAnimMontageInfo.IsStopped == 0))
				{
					// fast forward to server position and trigger notifies
					if (FAnimMontageInstance* MontageInstance = AnimInstance->GetActiveInstanceForMontage(LocalAnimMontageInfo.AnimMontage))
					{
						// Skip triggering notifies if we're going backwards in time, we've already triggered them.
						const float DeltaTime = !FMath::IsNearlyZero(ConstRepAnimMontageInfo.PlayRate) ? (DeltaPosition / ConstRepAnimMontageInfo.PlayRate) : 0.f;
						if (DeltaTime >= 0.f)
						{
							MontageInstance->UpdateWeight(DeltaTime);
							MontageInstance->HandleEvents(CurrentPosition, ConstRepAnimMontageInfo.Position, nullptr);
							
							// @todo: Not sure how we haven't noticed missed notifies, since we are queueing AND firing notifies from the UAnimInstance we need to manually handle the unfiltered notifies.
							
							// To keep the anim graph notify states alive, we force to only trigger montage notifies.
							AnimInstance->TriggerAnimNotifies(DeltaTime, UAnimInstance::ENotifyTriggerMode::ForceMontageOnly);
						}
					}
					// @todo We should probably instead introduce the delta offset into the anim instance and have it trickle it out over several frames,
					// as it computes deltas to maintain smoothness, and to handle anim notifies automatically.
					// Forcing the position here is not a good way to do replication syncing for animations. -jwf
					AnimInstance->Montage_SetPosition(LocalAnimMontageInfo.AnimMontage, ConstRepAnimMontageInfo.Position);
					ABILITY_LOG(Verbose, TEXT("OnRep_ReplicatedAnimMontage: Setting Position of %s to %.3f"), *GetNameSafe(LocalAnimMontageInfo.AnimMontage), ConstRepAnimMontageInfo.Position); 
				}
			}
			// Update current and next section if not replicating position
			else
			{
				const float CurrentPosition = AnimInstance->Montage_GetPosition(LocalAnimMontageInfo.AnimMontage);
				int32 CurrentSectionID = LocalAnimMontageInfo.AnimMontage->GetSectionIndexFromPosition(CurrentPosition);
				const int32 RepSectionIdToPlay = (static_cast<int32>(ConstRepAnimMontageInfo.SectionIdToPlay) - 1);
				FName CurrentSectionName = LocalAnimMontageInfo.AnimMontage->GetSectionName(CurrentSectionID);

				// If RepSectionIdToPlay is valid and different from the current section, then jump to it
				if (RepSectionIdToPlay != INDEX_NONE && RepSectionIdToPlay != CurrentSectionID )
				{
					CurrentSectionName = LocalAnimMontageInfo.AnimMontage->GetSectionName(RepSectionIdToPlay);
					if (CurrentSectionName != NAME_None)
					{
						ABILITY_LOG(Verbose, TEXT("OnRep_ReplicatedAnimMontage: %s jumping to section %s"), *GetNameSafe(LocalAnimMontageInfo.AnimMontage), *CurrentSectionName.ToString());
						AnimInstance->Montage_JumpToSection(CurrentSectionName);
						CurrentSectionID = RepSectionIdToPlay;
					}
					else
					{
						ABILITY_LOG(Warning, TEXT("OnRep_ReplicatedAnimMontage: Failed to replicate current section due to invalid name. Name: %s, Section ID: %i"), 
						*GetNameSafe(this), 
						CurrentSectionID);
					}
				}

				constexpr bool bForceGameThreadValue = true;
				if (CVarReplicateMontageNextSectionId.GetValueOnAnyThread(bForceGameThreadValue))
				{
					const int32 NextSectionID = AnimInstance->Montage_GetNextSectionID(LocalAnimMontageInfo.AnimMontage, CurrentSectionID);
					const int32 RepNextSectionID = int32(ConstRepAnimMontageInfo.NextSectionID) - 1;

					// If NextSectionID is different than the replicated one, then set it.
					if (RepNextSectionID != INDEX_NONE && NextSectionID != RepNextSectionID)
					{
						const FName NextSectionName = LocalAnimMontageInfo.AnimMontage->GetSectionName(RepNextSectionID);
						if (CurrentSectionName != NAME_None && NextSectionName != NAME_None)
						{
							AnimInstance->Montage_SetNextSection(CurrentSectionName, NextSectionName, LocalAnimMontageInfo.AnimMontage);
						}
						else
						{
							ABILITY_LOG(Warning, TEXT("OnRep_ReplicatedAnimMontage: Failed to replicate next section due to invalid name. Name: %s, Current Section ID: %i, Next Section ID: %i"), 
							*GetNameSafe(this), 
							CurrentSectionID, 
							RepNextSectionID);
						}
					}
				}
			}
		}
	}
}

void UAbilitySystemComponent::CurrentMontageStop(float OverrideBlendOutTime)
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	UAnimMontage* MontageToStop = LocalAnimMontageInfo.AnimMontage;
	bool bShouldStopMontage = AnimInstance && MontageToStop && !AnimInstance->Montage_GetIsStopped(MontageToStop);

	if (bShouldStopMontage)
	{
		const float BlendOutTime = (OverrideBlendOutTime >= 0.0f ? OverrideBlendOutTime : MontageToStop->BlendOut.GetBlendTime());

		AnimInstance->Montage_Stop(BlendOutTime, MontageToStop);

		if (IsOwnerActorAuthoritative())
		{
			AnimMontage_UpdateReplicatedData();
		}
	}
}

void UAbilitySystemComponent::StopMontageIfCurrent(const UAnimMontage& Montage, float OverrideBlendOutTime)
{
	if (&Montage == LocalAnimMontageInfo.AnimMontage)
	{
		CurrentMontageStop(OverrideBlendOutTime);
	}
}

void UAbilitySystemComponent::ClearAnimatingAbility(UGameplayAbility* Ability)
{
	if (LocalAnimMontageInfo.AnimatingAbility.Get() == Ability)
	{
		Ability->SetCurrentMontage(nullptr);
		LocalAnimMontageInfo.AnimatingAbility = nullptr;
	}
}

void UAbilitySystemComponent::CurrentMontageJumpToSection(FName SectionName)
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if ((SectionName != NAME_None) && AnimInstance && LocalAnimMontageInfo.AnimMontage)
	{
		AnimInstance->Montage_JumpToSection(SectionName, LocalAnimMontageInfo.AnimMontage);

		// This data is needed for replication on the server and recording replays on clients.
		// We need to set GetRepAnimMontageInfo_Mutable on replay recording clients because this data is NOT replicated to all clients (for example, it is NOT replicated to autonomous proxy clients.)
		if (ShouldRecordMontageReplication())
		{
			FGameplayAbilityRepAnimMontage& MutableRepAnimMontageInfo = GetRepAnimMontageInfo_Mutable();

			MutableRepAnimMontageInfo.SectionIdToPlay = 0;
			if (MutableRepAnimMontageInfo.Animation)
			{
				// Only change SectionIdToPlay if the anim montage's source is a montage. Dynamic montages have no sections.
				if (const UAnimMontage* RepAnimMontage = Cast<UAnimMontage>(MutableRepAnimMontageInfo.Animation))
				{
					// we add one so INDEX_NONE can be used in the on rep
					MutableRepAnimMontageInfo.SectionIdToPlay = RepAnimMontage->GetSectionIndex(SectionName) + 1;
				}
			}

			AnimMontage_UpdateReplicatedData();
		}
		
		// If we are NOT the authority, then let the server handling jumping the montage.
		if (!IsOwnerActorAuthoritative())
		{	
			UAnimSequenceBase* Animation = LocalAnimMontageInfo.AnimMontage->IsDynamicMontage() ? LocalAnimMontageInfo.AnimMontage->GetFirstAnimReference() : LocalAnimMontageInfo.AnimMontage.Get();
			ServerCurrentMontageJumpToSectionName(Animation, SectionName);
		}
	}
}

void UAbilitySystemComponent::CurrentMontageSetNextSectionName(FName FromSectionName, FName ToSectionName)
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if (LocalAnimMontageInfo.AnimMontage && AnimInstance)
	{
		// Set Next Section Name.
		AnimInstance->Montage_SetNextSection(FromSectionName, ToSectionName, LocalAnimMontageInfo.AnimMontage);

		// Update replicated version for Simulated Proxies if we are on the server.
		if( IsOwnerActorAuthoritative() )
		{
			AnimMontage_UpdateReplicatedData();
		}
		else
		{
			float CurrentPosition = AnimInstance->Montage_GetPosition(LocalAnimMontageInfo.AnimMontage);
			UAnimSequenceBase* Animation = LocalAnimMontageInfo.AnimMontage->IsDynamicMontage() ? LocalAnimMontageInfo.AnimMontage->GetFirstAnimReference() : LocalAnimMontageInfo.AnimMontage.Get();
			ServerCurrentMontageSetNextSectionName(Animation, CurrentPosition, FromSectionName, ToSectionName);
		}
	}
}

FName UAbilitySystemComponent::CurrentMontageGetNextSectionName() const
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if (LocalAnimMontageInfo.AnimMontage && AnimInstance)
	{
		FAnimMontageInstance* MontageInstance = AnimInstance->GetActiveInstanceForMontage(LocalAnimMontageInfo.AnimMontage);
		if (MontageInstance)
		{
			return MontageInstance->GetNextSection();
		}
	}

	return NAME_None;
}

void UAbilitySystemComponent::CurrentMontageSetPlayRate(float InPlayRate)
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if (LocalAnimMontageInfo.AnimMontage && AnimInstance)
	{
		// Set Play Rate
		AnimInstance->Montage_SetPlayRate(LocalAnimMontageInfo.AnimMontage, InPlayRate);

		// Update replicated version for Simulated Proxies if we are on the server.
		if (IsOwnerActorAuthoritative())
		{
			AnimMontage_UpdateReplicatedData();
		}
		else
		{
			UAnimSequenceBase* Animation = LocalAnimMontageInfo.AnimMontage->IsDynamicMontage() ? LocalAnimMontageInfo.AnimMontage->GetFirstAnimReference() : LocalAnimMontageInfo.AnimMontage.Get();
			ServerCurrentMontageSetPlayRate(Animation, InPlayRate);
		}
	}
}

bool UAbilitySystemComponent::ServerCurrentMontageSetNextSectionName_Validate(UAnimSequenceBase* ClientAnimation, float ClientPosition, FName SectionName, FName NextSectionName)
{
	return true;
}

void UAbilitySystemComponent::ServerCurrentMontageSetNextSectionName_Implementation(UAnimSequenceBase* ClientAnimation, float ClientPosition, FName SectionName, FName NextSectionName)
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if (AnimInstance && LocalAnimMontageInfo.AnimMontage)
	{
		UAnimSequenceBase* CurrentAnimation = LocalAnimMontageInfo.AnimMontage->IsDynamicMontage() ? LocalAnimMontageInfo.AnimMontage->GetFirstAnimReference() : LocalAnimMontageInfo.AnimMontage.Get();
		if (ClientAnimation == CurrentAnimation)
		{
			UAnimMontage* CurrentAnimMontage = LocalAnimMontageInfo.AnimMontage;

			// Set NextSectionName
			AnimInstance->Montage_SetNextSection(SectionName, NextSectionName, CurrentAnimMontage);

			// Correct position if we are in an invalid section
			float CurrentPosition = AnimInstance->Montage_GetPosition(CurrentAnimMontage);
			int32 CurrentSectionID = CurrentAnimMontage->GetSectionIndexFromPosition(CurrentPosition);
			FName CurrentSectionName = CurrentAnimMontage->GetSectionName(CurrentSectionID);

			int32 ClientSectionID = CurrentAnimMontage->GetSectionIndexFromPosition(ClientPosition);
			FName ClientCurrentSectionName = CurrentAnimMontage->GetSectionName(ClientSectionID);
			if ((CurrentSectionName != ClientCurrentSectionName) || (CurrentSectionName != SectionName))
			{
				// We are in an invalid section, jump to client's position.
				AnimInstance->Montage_SetPosition(CurrentAnimMontage, ClientPosition);
			}

			// Update replicated version for Simulated Proxies if we are on the server.
			if (IsOwnerActorAuthoritative())
			{
				AnimMontage_UpdateReplicatedData();
			}
		}
	}
}

bool UAbilitySystemComponent::ServerCurrentMontageJumpToSectionName_Validate(UAnimSequenceBase* ClientAnimation, FName SectionName)
{
	return true;
}

void UAbilitySystemComponent::ServerCurrentMontageJumpToSectionName_Implementation(UAnimSequenceBase* ClientAnimation, FName SectionName)
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if (AnimInstance && LocalAnimMontageInfo.AnimMontage)
	{
		UAnimSequenceBase* CurrentAnimation = LocalAnimMontageInfo.AnimMontage->IsDynamicMontage() ? LocalAnimMontageInfo.AnimMontage->GetFirstAnimReference() : LocalAnimMontageInfo.AnimMontage.Get();
		if (ClientAnimation == CurrentAnimation)
		{
			UAnimMontage* CurrentAnimMontage = LocalAnimMontageInfo.AnimMontage;

			// Set NextSectionName
			AnimInstance->Montage_JumpToSection(SectionName, CurrentAnimMontage);

			// Update replicated version for Simulated Proxies if we are on the server.
			if (IsOwnerActorAuthoritative())
			{
				FGameplayAbilityRepAnimMontage& MutableRepAnimMontageInfo = GetRepAnimMontageInfo_Mutable();

				MutableRepAnimMontageInfo.SectionIdToPlay = 0;
				if (MutableRepAnimMontageInfo.Animation && SectionName != NAME_None)
				{
					// Only change SectionIdToPlay if the anim montage's source is a montage. Dynamic montages have no sections.
					if (const UAnimMontage* RepAnimMontage = Cast<UAnimMontage>(MutableRepAnimMontageInfo.Animation))
					{
						// we add one so INDEX_NONE can be used in the on rep
						MutableRepAnimMontageInfo.SectionIdToPlay = RepAnimMontage->GetSectionIndex(SectionName) + 1;
					}
				}

				AnimMontage_UpdateReplicatedData();
			}
		}
	}
}

bool UAbilitySystemComponent::ServerCurrentMontageSetPlayRate_Validate(UAnimSequenceBase* ClientAnimation, float InPlayRate)
{
	return true;
}

void UAbilitySystemComponent::ServerCurrentMontageSetPlayRate_Implementation(UAnimSequenceBase* ClientAnimation, float InPlayRate)
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if (AnimInstance && LocalAnimMontageInfo.AnimMontage)
	{
		UAnimSequenceBase* CurrentAnimation = LocalAnimMontageInfo.AnimMontage->IsDynamicMontage() ? LocalAnimMontageInfo.AnimMontage->GetFirstAnimReference() : LocalAnimMontageInfo.AnimMontage.Get();
		if (ClientAnimation == CurrentAnimation)
		{
			UAnimMontage* CurrentAnimMontage = LocalAnimMontageInfo.AnimMontage;

			// Set PlayRate
			AnimInstance->Montage_SetPlayRate(CurrentAnimMontage, InPlayRate);

			// Update replicated version for Simulated Proxies if we are on the server.
			if (IsOwnerActorAuthoritative())
			{
				AnimMontage_UpdateReplicatedData();
			}
		}
	}
}

UAnimMontage* UAbilitySystemComponent::PlaySlotAnimationAsDynamicMontage(UGameplayAbility* AnimatingAbility, FGameplayAbilityActivationInfo ActivationInfo, UAnimSequenceBase* AnimAsset, FName SlotName, float BlendInTime, float BlendOutTime, float InPlayRate, float StartTimeSeconds)
{
	UAnimMontage* DynamicMontage = UAnimMontage::CreateSlotAnimationAsDynamicMontage(AnimAsset, SlotName, BlendInTime, BlendOutTime, InPlayRate, 1, -1.0f, 0.0f);
	PlayMontage(AnimatingAbility, ActivationInfo, DynamicMontage, InPlayRate, NAME_None, StartTimeSeconds);
	return DynamicMontage;
}

UAnimMontage* UAbilitySystemComponent::PlaySlotAnimationAsDynamicMontageSimulated(UAnimSequenceBase* AnimAsset, FName SlotName, float BlendInTime, float BlendOutTime, float InPlayRate)
{
	UAnimMontage* DynamicMontage = UAnimMontage::CreateSlotAnimationAsDynamicMontage(AnimAsset, SlotName, BlendInTime, BlendOutTime, InPlayRate, 1, -1.0f, 0.0f);
	PlayMontageSimulated(DynamicMontage, InPlayRate, NAME_None);
	return DynamicMontage;
}

UAnimMontage* UAbilitySystemComponent::PlaySlotAnimationAsDynamicMontage_WithFractionalLoopsSimulated(UAnimSequenceBase* AnimAsset, FName SlotName, float BlendInTime, float BlendOutTime, float InPlayRate, float PlayCount)
{
	UAnimMontage* DynamicMontage = UAnimMontage::CreateSlotAnimationAsDynamicMontage_WithFractionalLoops(AnimAsset, SlotName, BlendInTime, BlendOutTime, PlayCount);
	PlayMontageSimulated(DynamicMontage, InPlayRate, NAME_None);
	return DynamicMontage;
}

UAnimMontage* UAbilitySystemComponent::PlaySlotAnimationAsDynamicMontage_WithFractionalLoops(UGameplayAbility* AnimatingAbility, FGameplayAbilityActivationInfo ActivationInfo, UAnimSequenceBase* AnimAsset, FName SlotName, float BlendInTime, float BlendOutTime, float InPlayRate, float StartTimeSeconds, float PlayCount)
{
	UAnimMontage* DynamicMontage = UAnimMontage::CreateSlotAnimationAsDynamicMontage_WithFractionalLoops(AnimAsset, SlotName, BlendInTime, BlendOutTime, PlayCount);
	PlayMontageInternal(AnimatingAbility, ActivationInfo, DynamicMontage, InPlayRate, NAME_None, StartTimeSeconds, [PlayCount](FGameplayAbilityRepAnimMontage& MutableRepAnimMontageInfo)
		{
			MutableRepAnimMontageInfo.PlayCount = PlayCount;
		});
	return DynamicMontage;
}

UAnimMontage* UAbilitySystemComponent::GetCurrentMontage() const
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	if (LocalAnimMontageInfo.AnimMontage && AnimInstance && AnimInstance->Montage_IsActive(LocalAnimMontageInfo.AnimMontage))
	{
		return LocalAnimMontageInfo.AnimMontage;
	}

	return nullptr;
}

int32 UAbilitySystemComponent::GetCurrentMontageSectionID() const
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	UAnimMontage* CurrentAnimMontage = GetCurrentMontage();

	if (CurrentAnimMontage && AnimInstance)
	{
		float MontagePosition = AnimInstance->Montage_GetPosition(CurrentAnimMontage);
		return CurrentAnimMontage->GetSectionIndexFromPosition(MontagePosition);
	}

	return INDEX_NONE;
}

FName UAbilitySystemComponent::GetCurrentMontageSectionName() const
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	UAnimMontage* CurrentAnimMontage = GetCurrentMontage();

	if (CurrentAnimMontage && AnimInstance)
	{
		float MontagePosition = AnimInstance->Montage_GetPosition(CurrentAnimMontage);
		int32 CurrentSectionID = CurrentAnimMontage->GetSectionIndexFromPosition(MontagePosition);

		return CurrentAnimMontage->GetSectionName(CurrentSectionID);
	}

	return NAME_None;
}

float UAbilitySystemComponent::GetCurrentMontageSectionLength() const
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	UAnimMontage* CurrentAnimMontage = GetCurrentMontage();
	if (CurrentAnimMontage && AnimInstance)
	{
		int32 CurrentSectionID = GetCurrentMontageSectionID();
		if (CurrentSectionID != INDEX_NONE)
		{
			TArray<FCompositeSection>& CompositeSections = CurrentAnimMontage->CompositeSections;

			// If we have another section after us, then take delta between both start times.
			if (CurrentSectionID < (CompositeSections.Num() - 1))
			{
				return (CompositeSections[CurrentSectionID + 1].GetTime() - CompositeSections[CurrentSectionID].GetTime());
			}
			// Otherwise we are the last section, so take delta with Montage total time.
			else
			{
				return (CurrentAnimMontage->GetPlayLength() - CompositeSections[CurrentSectionID].GetTime());
			}
		}

		// if we have no sections, just return total length of Montage.
		return CurrentAnimMontage->GetPlayLength();
	}

	return 0.f;
}

float UAbilitySystemComponent::GetCurrentMontageSectionTimeLeft() const
{
	UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;
	UAnimMontage* CurrentAnimMontage = GetCurrentMontage();
	if (CurrentAnimMontage && AnimInstance && AnimInstance->Montage_IsActive(CurrentAnimMontage))
	{
		const float CurrentPosition = AnimInstance->Montage_GetPosition(CurrentAnimMontage);
		return CurrentAnimMontage->GetSectionTimeLeftFromPos(CurrentPosition);
	}

	return -1.f;
}

void UAbilitySystemComponent::SetMontageRepAnimPositionMethod(ERepAnimPositionMethod InMethod)
{
	UE_LOGF(LogAbilitySystem, Verbose, "SetMontageRepAnimPositionMethod %ls to %ls", *GetNameSafe(GetOwner()), *UEnum::GetValueAsString(InMethod));
	GetRepAnimMontageInfo_Mutable().SetRepAnimPositionMethod(InMethod);
}

bool UAbilitySystemComponent::IsAnimatingAbility(UGameplayAbility* InAbility) const
{
	return (LocalAnimMontageInfo.AnimatingAbility.Get() == InAbility);
}

UGameplayAbility* UAbilitySystemComponent::GetAnimatingAbility() const
{
	return LocalAnimMontageInfo.AnimatingAbility.Get();
}

void UAbilitySystemComponent::ConfirmAbilityTargetData(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, const FGameplayAbilityTargetDataHandle& TargetData, const FGameplayTag& ApplicationTag)
{
	TSharedPtr<FAbilityReplicatedDataCache> CachedData = AbilityTargetDataMap.Find(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));
	if (CachedData.IsValid())
	{
		CachedData->TargetSetDelegate.Broadcast(TargetData, ApplicationTag);
	}
}

void UAbilitySystemComponent::CancelAbilityTargetData(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey)
{
	TSharedPtr<FAbilityReplicatedDataCache> CachedData = AbilityTargetDataMap.Find(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));
	if (CachedData.IsValid())
	{
		CachedData->Reset();
		CachedData->TargetCancelledDelegate.Broadcast();
	}
}

void UAbilitySystemComponent::ConsumeAllReplicatedData(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey)
{
	TSharedPtr<FAbilityReplicatedDataCache> CachedData = AbilityTargetDataMap.Find(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));
	if (CachedData.IsValid())
	{
		CachedData->Reset();
	}
}

void UAbilitySystemComponent::ConsumeClientReplicatedTargetData(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey)
{
	TSharedPtr<FAbilityReplicatedDataCache> CachedData = AbilityTargetDataMap.Find(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));
	if (CachedData.IsValid())
	{
		CachedData->TargetData.Clear();
		CachedData->bTargetConfirmed = false;
		CachedData->bTargetCancelled = false;
	}
}

void UAbilitySystemComponent::ConsumeGenericReplicatedEvent(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey)
{
	TSharedPtr<FAbilityReplicatedDataCache> CachedData = AbilityTargetDataMap.Find(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));
	if (CachedData.IsValid())
	{
		CachedData->GenericEvents[EventType].bTriggered = false;
	}
}

FAbilityReplicatedData UAbilitySystemComponent::GetReplicatedDataOfGenericReplicatedEvent(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey)
{
	FAbilityReplicatedData ReturnData;

	TSharedPtr<FAbilityReplicatedDataCache> CachedData = AbilityTargetDataMap.Find(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));
	if (CachedData.IsValid())
	{
		ReturnData.bTriggered = CachedData->GenericEvents[EventType].bTriggered;
		ReturnData.VectorPayload = CachedData->GenericEvents[EventType].VectorPayload;
	}

	return ReturnData;
}

void UAbilitySystemComponent::ServerSetReplicatedEvent_Implementation(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey,  FPredictionKey CurrentPredictionKey)
{
	FScopedPredictionWindow ScopedPrediction(this, CurrentPredictionKey);

	InvokeReplicatedEvent(EventType, AbilityHandle, AbilityOriginalPredictionKey, CurrentPredictionKey);
}

void UAbilitySystemComponent::ServerSetReplicatedEventWithPayload_Implementation(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey,  FPredictionKey CurrentPredictionKey, FVector_NetQuantize100 VectorPayload)
{
	FScopedPredictionWindow ScopedPrediction(this, CurrentPredictionKey);

	InvokeReplicatedEventWithPayload(EventType, AbilityHandle, AbilityOriginalPredictionKey, CurrentPredictionKey, VectorPayload);
}

bool UAbilitySystemComponent::InvokeReplicatedEvent(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, FPredictionKey CurrentPredictionKey)
{
	TSharedRef<FAbilityReplicatedDataCache> ReplicatedData = AbilityTargetDataMap.FindOrAdd(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));

	ReplicatedData->GenericEvents[(uint8)EventType].bTriggered = true;
	ReplicatedData->PredictionKey = CurrentPredictionKey;

	if (ReplicatedData->GenericEvents[EventType].Delegate.IsBound())
	{
		ReplicatedData->GenericEvents[EventType].Delegate.Broadcast();
		return true;
	}
	else
	{
		return false;
	}
}

bool UAbilitySystemComponent::InvokeReplicatedEventWithPayload(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, FPredictionKey CurrentPredictionKey, FVector_NetQuantize100 VectorPayload)
{
	TSharedRef<FAbilityReplicatedDataCache> ReplicatedData = AbilityTargetDataMap.FindOrAdd(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));
	ReplicatedData->GenericEvents[(uint8)EventType].bTriggered = true;
	ReplicatedData->GenericEvents[(uint8)EventType].VectorPayload = VectorPayload;
	ReplicatedData->PredictionKey = CurrentPredictionKey;

	if (ReplicatedData->GenericEvents[EventType].Delegate.IsBound())
	{
		ReplicatedData->GenericEvents[EventType].Delegate.Broadcast();
		return true;
	}
	else
	{
		return false;
	}
}

bool UAbilitySystemComponent::ServerSetReplicatedEvent_Validate(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey,  FPredictionKey CurrentPredictionKey)
{
	if (EventType >= EAbilityGenericReplicatedEvent::MAX)
	{
		return false;
	}
	return true;
}

bool UAbilitySystemComponent::ServerSetReplicatedEventWithPayload_Validate(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey,  FPredictionKey CurrentPredictionKey, FVector_NetQuantize100 VectorPayload)
{
	if (EventType >= EAbilityGenericReplicatedEvent::MAX)
	{
		return false;
	}
	return true;
}

void UAbilitySystemComponent::ClientSetReplicatedEvent_Implementation(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey)
{
	InvokeReplicatedEvent(EventType, AbilityHandle, AbilityOriginalPredictionKey, ScopedPredictionKey);
}

void UAbilitySystemComponent::ServerSetReplicatedTargetData_Implementation(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, const FGameplayAbilityTargetDataHandle& ReplicatedTargetDataHandle, FGameplayTag ApplicationTag, FPredictionKey CurrentPredictionKey)
{
	FScopedPredictionWindow ScopedPrediction(this, CurrentPredictionKey);

	// Always adds to cache to store the new data
	TSharedRef<FAbilityReplicatedDataCache> ReplicatedData = AbilityTargetDataMap.FindOrAdd(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));

	if (ReplicatedData->TargetData.Num() > 0)
	{
		FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityHandle);
		if (Spec && Spec->Ability)
		{
			// Can happen under normal circumstances if ServerForceClientTargetData is hit
			ABILITY_LOG(Display, TEXT("Ability %s is overriding pending replicated target data."), *Spec->Ability->GetName());
		}
	}

	ReplicatedData->TargetData = ReplicatedTargetDataHandle;
	ReplicatedData->ApplicationTag = ApplicationTag;
	ReplicatedData->bTargetConfirmed = true;
	ReplicatedData->bTargetCancelled = false;
	ReplicatedData->PredictionKey = CurrentPredictionKey;

	ReplicatedData->TargetSetDelegate.Broadcast(ReplicatedTargetDataHandle, ReplicatedData->ApplicationTag);
}

bool UAbilitySystemComponent::ServerSetReplicatedTargetData_Validate(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, const FGameplayAbilityTargetDataHandle& ReplicatedTargetDataHandle, FGameplayTag ApplicationTag, FPredictionKey CurrentPredictionKey)
{
	// check the data coming from the client to ensure it's valid
	for (const TSharedPtr<FGameplayAbilityTargetData>& TgtData : ReplicatedTargetDataHandle.Data)
	{
		if (!ensure(TgtData.IsValid()))
		{
			return false;
		}
	}

	return true;
}

void UAbilitySystemComponent::ServerSetReplicatedTargetDataCancelled_Implementation(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, FPredictionKey CurrentPredictionKey)
{
	FScopedPredictionWindow ScopedPrediction(this, CurrentPredictionKey);

	// Always adds to cache to store the new data
	TSharedRef<FAbilityReplicatedDataCache> ReplicatedData = AbilityTargetDataMap.FindOrAdd(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));

	ReplicatedData->Reset();
	ReplicatedData->bTargetCancelled = true;
	ReplicatedData->PredictionKey = CurrentPredictionKey;
	ReplicatedData->TargetCancelledDelegate.Broadcast();
}

bool UAbilitySystemComponent::ServerSetReplicatedTargetDataCancelled_Validate(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, FPredictionKey CurrentPredictionKey)
{
	return true;
}

void UAbilitySystemComponent::CallAllReplicatedDelegatesIfSet(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey)
{
	TSharedPtr<FAbilityReplicatedDataCache> CachedData = AbilityTargetDataMap.Find(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));
	if (CachedData.IsValid())
	{
		FScopedPredictionWindow ScopedWindow(this, CachedData->PredictionKey, false);
		if (CachedData->bTargetConfirmed)
		{
			CachedData->TargetSetDelegate.Broadcast(CachedData->TargetData, CachedData->ApplicationTag);
		}
		else if (CachedData->bTargetCancelled)
		{
			CachedData->TargetCancelledDelegate.Broadcast();
		}

		for (int32 idx=0; idx < EAbilityGenericReplicatedEvent::MAX; ++idx)
		{
			if (CachedData->GenericEvents[idx].bTriggered)
			{
				CachedData->GenericEvents[idx].Delegate.Broadcast();
			}
		}
	}
}

bool UAbilitySystemComponent::CallReplicatedTargetDataDelegatesIfSet(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey)
{
	bool CalledDelegate = false;
	TSharedPtr<FAbilityReplicatedDataCache> CachedData = AbilityTargetDataMap.Find(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));
	if (CachedData.IsValid())
	{
		// Use prediction key that was sent to us
		FScopedPredictionWindow ScopedWindow(this, CachedData->PredictionKey, false);

		if (CachedData->bTargetConfirmed)
		{
			CachedData->TargetSetDelegate.Broadcast(CachedData->TargetData, CachedData->ApplicationTag);
			CalledDelegate = true;
		}
		else if (CachedData->bTargetCancelled)
		{
			CachedData->TargetCancelledDelegate.Broadcast();
			CalledDelegate = true;
		}
	}

	return CalledDelegate;
}

bool UAbilitySystemComponent::CallReplicatedEventDelegateIfSet(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey)
{
	TSharedPtr<FAbilityReplicatedDataCache> CachedData = AbilityTargetDataMap.Find(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));
	if (CachedData.IsValid() && CachedData->GenericEvents[EventType].bTriggered)
	{
		FScopedPredictionWindow ScopedWindow(this, CachedData->PredictionKey, false);

		// Already triggered, fire off delegate
		CachedData->GenericEvents[EventType].Delegate.Broadcast();
		return true;
	}
	return false;
}

bool UAbilitySystemComponent::CallOrAddReplicatedDelegate(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, FSimpleMulticastDelegate::FDelegate Delegate)
{
	TSharedRef<FAbilityReplicatedDataCache> CachedData = AbilityTargetDataMap.FindOrAdd(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey));
	if (CachedData->GenericEvents[EventType].bTriggered)
	{
		FScopedPredictionWindow ScopedWindow(this, CachedData->PredictionKey, false);

		// Already triggered, fire off delegate
		Delegate.Execute();
		return true;
	}
	
	// Not triggered yet, so just add the delegate
	CachedData->GenericEvents[EventType].Delegate.Add(Delegate);
	return false;
}

FAbilityTargetDataSetDelegate& UAbilitySystemComponent::AbilityTargetDataSetDelegate(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey)
{
	return AbilityTargetDataMap.FindOrAdd(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey))->TargetSetDelegate;
}

FSimpleMulticastDelegate& UAbilitySystemComponent::AbilityTargetDataCancelledDelegate(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey)
{
	return AbilityTargetDataMap.FindOrAdd(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey))->TargetCancelledDelegate;
}

FSimpleMulticastDelegate& UAbilitySystemComponent::AbilityReplicatedEventDelegate(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey)
{
	return AbilityTargetDataMap.FindOrAdd(FGameplayAbilitySpecHandleAndPredictionKey(AbilityHandle, AbilityOriginalPredictionKey))->GenericEvents[EventType].Delegate;
}

int32 AbilitySystemLogServerRPCBatching = 0;
static FAutoConsoleVariableRef CVarAbilitySystemLogServerRPCBatching(TEXT("AbilitySystem.ServerRPCBatching.Log"), AbilitySystemLogServerRPCBatching, TEXT(""), ECVF_Default	);

FScopedServerAbilityRPCBatcher::FScopedServerAbilityRPCBatcher(UAbilitySystemComponent* InASC, FGameplayAbilitySpecHandle InAbilityHandle) 
	: ASC(InASC), AbilityHandle(InAbilityHandle), ScopedPredictionWindow(InASC)
{
	if (ASC && AbilityHandle.IsValid() && ASC->ShouldDoServerAbilityRPCBatch())
	{
		ASC->BeginServerAbilityRPCBatch(InAbilityHandle);
	}
	else
	{
		ASC = nullptr;
	}
}

FScopedServerAbilityRPCBatcher::~FScopedServerAbilityRPCBatcher()
{
	if (ASC)
	{
		ASC->EndServerAbilityRPCBatch(AbilityHandle);
	}
}

bool UAbilitySystemComponent::ServerAbilityRPCBatch_Validate(FServerAbilityRPCBatch BatchInfo)
{
	return true;
}

void UAbilitySystemComponent::ServerAbilityRPCBatch_Implementation(FServerAbilityRPCBatch BatchInfo)
{
	ServerAbilityRPCBatch_Internal(BatchInfo);
}

void UAbilitySystemComponent::ServerAbilityRPCBatch_Internal(FServerAbilityRPCBatch& BatchInfo)
{
	UE_CLOGF(AbilitySystemLogServerRPCBatching, LogAbilitySystem, Display, "::ServerAbilityRPCBatch_Implementation %ls %ls", *BatchInfo.AbilitySpecHandle.ToString(), *BatchInfo.PredictionKey.ToString());

	ServerTryActivateAbility_Implementation(BatchInfo.AbilitySpecHandle, BatchInfo.InputPressed, BatchInfo.PredictionKey);
	ServerSetReplicatedTargetData_Implementation(BatchInfo.AbilitySpecHandle, BatchInfo.PredictionKey, BatchInfo.TargetData, FGameplayTag(), BatchInfo.PredictionKey);

	if (BatchInfo.Ended)
	{
		// This FakeInfo is probably bogus for the general case but should work for the limited use of batched RPCs
		FGameplayAbilityActivationInfo FakeInfo;
		FakeInfo.ServerSetActivationPredictionKey(BatchInfo.PredictionKey);
		ServerEndAbility(BatchInfo.AbilitySpecHandle, FakeInfo, BatchInfo.PredictionKey);
	}
}

void UAbilitySystemComponent::BeginServerAbilityRPCBatch(FGameplayAbilitySpecHandle AbilityHandle)
{
	UE_CLOGF(AbilitySystemLogServerRPCBatching, LogAbilitySystem, Display, "::BeginServerAbilityRPCBatch %ls", *AbilityHandle.ToString());

	if (FServerAbilityRPCBatch* ExistingBatchData = LocalServerAbilityRPCBatchData.FindByKey(AbilityHandle))
	{
		FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityHandle);
		ABILITY_LOG(Warning, TEXT("::BeginServerAbilityRPCBatch called when there is already a batch started for ability (%s)"), Spec ? *GetNameSafe(Spec->Ability) : TEXT("INVALID"));
		return;
	}
	
	/** Create new FServerAbilityRPCBatch and initiailze it to this AbilityHandle */
	FServerAbilityRPCBatch& NewBatchData = LocalServerAbilityRPCBatchData[LocalServerAbilityRPCBatchData.AddDefaulted()];
	NewBatchData.AbilitySpecHandle = AbilityHandle;
}

void UAbilitySystemComponent::EndServerAbilityRPCBatch(FGameplayAbilitySpecHandle AbilityHandle)
{
	/** See if we have batch data for this ability (we should!) and call the ServerAbilityRPCBatch rpc (send what we batched up to the server) */
	int32 idx = LocalServerAbilityRPCBatchData.IndexOfByKey(AbilityHandle);
	if (idx != INDEX_NONE)
	{
		UE_CLOGF(AbilitySystemLogServerRPCBatching, LogAbilitySystem, Display, "::EndServerAbilityRPCBatch. Calling ServerAbilityRPCBatch. Handle: %ls. PredictionKey: %ls.", *LocalServerAbilityRPCBatchData[idx].AbilitySpecHandle.ToString(), *LocalServerAbilityRPCBatchData[idx].PredictionKey.ToString());

		FServerAbilityRPCBatch& ThisBatch = LocalServerAbilityRPCBatchData[idx];
		if (ThisBatch.Started)
		{
			if (ThisBatch.PredictionKey.IsValidKey() == false)
			{
				ABILITY_LOG(Warning, TEXT("::EndServerAbilityRPCBatch was started but has an invalid prediction key. Handle: %s. PredictionKey: %s."), *ThisBatch.AbilitySpecHandle.ToString(), *ThisBatch.PredictionKey.ToString());
			}

			ServerAbilityRPCBatch(ThisBatch);
		}

		LocalServerAbilityRPCBatchData.RemoveAt(idx, EAllowShrinking::No);
	}
	else
	{
		FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityHandle);
		ABILITY_LOG(Warning, TEXT("::EndServerAbilityRPCBatch called on ability %s when no batch has been started."), Spec ? *GetNameSafe(Spec->Ability) : TEXT("INVALID"));
	}
}

void UAbilitySystemComponent::CallServerTryActivateAbility(FGameplayAbilitySpecHandle AbilityHandle, bool InputPressed, FPredictionKey PredictionKey)
{
	UE_CLOGF(AbilitySystemLogServerRPCBatching, LogAbilitySystem, Display, "::CallServerTryActivateAbility %ls %d %ls", *AbilityHandle.ToString(), InputPressed, *PredictionKey.ToString());

	/** Queue this call up if we are in  a batch window, otherwise just push it through now */
	if (FServerAbilityRPCBatch* ExistingBatchData = LocalServerAbilityRPCBatchData.FindByKey(AbilityHandle))
	{
		if (ExistingBatchData->Started)
		{
			FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityHandle);
			ABILITY_LOG(Warning, TEXT("::CallServerTryActivateAbility called multiple times for ability (%s) during a single batch."), Spec ? *GetNameSafe(Spec->Ability) : TEXT("INVALID"));
			return;
		}

		ExistingBatchData->Started = true;
		ExistingBatchData->InputPressed = InputPressed;
		ExistingBatchData->PredictionKey = PredictionKey;
	}
	else
	{
		UE_CLOGF(AbilitySystemLogServerRPCBatching, LogAbilitySystem, Display, "    NO BATCH IN SCOPE");
		ServerTryActivateAbility(AbilityHandle, InputPressed, PredictionKey);
	}
}

void UAbilitySystemComponent::CallServerSetReplicatedTargetData(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, const FGameplayAbilityTargetDataHandle& ReplicatedTargetDataHandle, FGameplayTag ApplicationTag, FPredictionKey CurrentPredictionKey)
{
	UE_CLOGF(AbilitySystemLogServerRPCBatching, LogAbilitySystem, Display, "::CallServerSetReplicatedTargetData %ls %ls %ls %ls %ls", 
		*AbilityHandle.ToString(), *AbilityOriginalPredictionKey.ToString(), ReplicatedTargetDataHandle.IsValid(0) ? *ReplicatedTargetDataHandle.Get(0)->ToString() : TEXT("NULL"), *ApplicationTag.ToString(), *CurrentPredictionKey.ToString());

	/** Queue this call up if we are in  a batch window, otherwise just push it through now */
	if (FServerAbilityRPCBatch* ExistingBatchData = LocalServerAbilityRPCBatchData.FindByKey(AbilityHandle))
	{
		if (!ExistingBatchData->Started)
		{
			// A batch window was setup but we didn't see the normal try activate -> target data -> end path. So let this unbatched rpc through.
			FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityHandle);
			UE_CLOGF(AbilitySystemLogServerRPCBatching, LogAbilitySystem, Display, "::CallServerSetReplicatedTargetData called for ability (%ls) when CallServerTryActivateAbility has not been called", Spec ? *GetNameSafe(Spec->Ability) : TEXT("INVALID"));
			ServerSetReplicatedTargetData(AbilityHandle, AbilityOriginalPredictionKey, ReplicatedTargetDataHandle, ApplicationTag, CurrentPredictionKey);
			return;
		}

		if (ExistingBatchData->PredictionKey.IsValidKey() == false)
		{
			FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityHandle);
			ABILITY_LOG(Warning, TEXT("::CallServerSetReplicatedTargetData called for ability (%s) when the prediction key is not valid."), Spec ? *GetNameSafe(Spec->Ability) : TEXT("INVALID"));
		}


		ExistingBatchData->TargetData = ReplicatedTargetDataHandle;
	}
	else
	{
		ServerSetReplicatedTargetData(AbilityHandle, AbilityOriginalPredictionKey, ReplicatedTargetDataHandle, ApplicationTag, CurrentPredictionKey);
	}

}

void UAbilitySystemComponent::CallServerEndAbility(FGameplayAbilitySpecHandle AbilityHandle, FGameplayAbilityActivationInfo ActivationInfo, FPredictionKey PredictionKey)
{
	UE_CLOGF(AbilitySystemLogServerRPCBatching, LogAbilitySystem, Display, "::CallServerEndAbility %ls (%d %ls) %ls", *AbilityHandle.ToString(), ActivationInfo.bCanBeEndedByOtherInstance, *ActivationInfo.GetActivationPredictionKey().ToString(), *PredictionKey.ToString());

	/** Queue this call up if we are in  a batch window, otherwise just push it through now */
	if (FServerAbilityRPCBatch* ExistingBatchData = LocalServerAbilityRPCBatchData.FindByKey(AbilityHandle))
	{
		if (!ExistingBatchData->Started)
		{
			// A batch window was setup but we didn't see the normal try activate -> target data -> end path. So let this unbatched rpc through.
			FGameplayAbilitySpec* Spec = FindAbilitySpecFromHandle(AbilityHandle);
			UE_CLOGF(AbilitySystemLogServerRPCBatching, LogAbilitySystem, Display, "::CallServerEndAbility called for ability (%ls) when CallServerTryActivateAbility has not been called", Spec ? *GetNameSafe(Spec->Ability) : TEXT("INVALID"));
			ServerEndAbility(AbilityHandle, ActivationInfo, PredictionKey);
			return;
		}

		ExistingBatchData->Ended = true;
	}
	else
	{
		ServerEndAbility(AbilityHandle, ActivationInfo, PredictionKey);
	}
}

#undef LOCTEXT_NAMESPACE
