// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbilitySystemDebugHUD.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Debug/DebugDrawService.h"
#include "Engine/LocalPlayer.h"
#include "EngineUtils.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "GameplayEffect.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilitySystemDebugHUD)

namespace
{
	static float DebugDrawMaxDistance = 2048.f;
	static FAutoConsoleVariableRef CVarDebugDrawMaxDistance(
		TEXT("AbilitySystem.DebugDrawMaxDistance"),
		DebugDrawMaxDistance,
		TEXT("Set the maximum camera distance allowed for Debug Drawing by the Ability System.")
	);
}

bool AAbilitySystemDebugHUD::bEnableBasicHUD = false;

FDelegateHandle AAbilitySystemDebugHUD::DrawDebugDelegateHandle;

TSubclassOf<AAbilitySystemDebugHUD> AAbilitySystemDebugHUD::HUDClass = AAbilitySystemDebugHUD::StaticClass();

void UAbilitySystemDebugHUDExtension_Tags::ToggleExtension(const TArray<FString>& Args, UWorld* World)
{
	UAbilitySystemDebugHUDExtension_Tags* Extension = GetMutableDefault<UAbilitySystemDebugHUDExtension_Tags>();
	if (Args.IsEmpty())
	{
		const bool bIsCurrentlyEnabled = Extension->bEnabled || !Extension->TagsToDisplay.IsEmpty();

		if (bIsCurrentlyEnabled)
		{
			Extension->TagsToDisplay.Empty();
		}

		Extension->bEnabled = !bIsCurrentlyEnabled;
	}
	else
	{
		for (const FString& Tag : Args)
		{
			FSetElementId Element = Extension->TagsToDisplay.FindId(Tag);
			if (Element.IsValidId())
			{
				Extension->TagsToDisplay.Remove(Element);
			}
			else
			{
				Extension->TagsToDisplay.Add(Tag);
			}
		}
	}

	AAbilitySystemDebugHUD::NotifyExtensionEnableChanged(World);
}

bool UAbilitySystemDebugHUDExtension_Tags::IsEnabled() const
{
	return bEnabled || !TagsToDisplay.IsEmpty();
}

void UAbilitySystemDebugHUDExtension_Tags::GetDebugStrings(const AActor* Actor, const UAbilitySystemComponent* Comp, OUT TArray<FString>& OutDebugStrings) const
{
	if (!Comp)
	{
		return;
	}

	// Count up how many GameplayEffects granted the ASC each tag
	TMap<FGameplayTag, uint32> TagCounts;
	for (FActiveGameplayEffectsContainer::ConstIterator ActiveEffectIt = Comp->GetActiveGameplayEffects().CreateConstIterator(); ActiveEffectIt; ++ActiveEffectIt)
	{
		FGameplayTagContainer EffectTagContainer;
		ActiveEffectIt->Spec.GetAllGrantedTags(EffectTagContainer);

		for (auto EffectTagIt = EffectTagContainer.CreateConstIterator(); EffectTagIt; ++EffectTagIt)
		{
			++TagCounts.FindOrAdd(*EffectTagIt, 0);
		}
	}

	FGameplayTagContainer Container;
	Comp->GetOwnedGameplayTags(Container);
	for (auto ContainerIt = Container.CreateConstIterator(); ContainerIt; ++ContainerIt)
	{
		if (TagsToDisplay.IsEmpty() || TagsToDisplay.Contains(ContainerIt->ToString()))
		{
			if (const uint32* TagCount = TagCounts.Find(*ContainerIt))
			{
				OutDebugStrings.Add(FString::Printf(TEXT("%s x%d"), *ContainerIt->ToString(), *TagCount));
			}
			else
			{
				OutDebugStrings.Add(ContainerIt->ToString());
			}
		}
	}
}

void UAbilitySystemDebugHUDExtension_Attributes::ToggleExtension(const TArray<FString>& Args, UWorld* World)
{
	UAbilitySystemDebugHUDExtension_Attributes* Extension = GetMutableDefault<UAbilitySystemDebugHUDExtension_Attributes>();

	for (const FString& Arg : Args)
	{
		FSetElementId FoundId = Extension->AttributesToDisplay.FindId(Arg);
		if (FoundId.IsValidId())
		{
			Extension->AttributesToDisplay.Remove(FoundId);
		}
		else
		{
			Extension->AttributesToDisplay.Add(Arg);
		}
	}

	AAbilitySystemDebugHUD::NotifyExtensionEnableChanged(World);
}

void UAbilitySystemDebugHUDExtension_Attributes::ToggleIncludeModifiers()
{
	UAbilitySystemDebugHUDExtension_Attributes* Extension = GetMutableDefault<UAbilitySystemDebugHUDExtension_Attributes>();
	Extension->bIncludeModifiers = !Extension->bIncludeModifiers;
}

void UAbilitySystemDebugHUDExtension_Attributes::ClearDisplayedAttributes(const TArray<FString>& Args, UWorld* World)
{
	UAbilitySystemDebugHUDExtension_Attributes* Extension = GetMutableDefault<UAbilitySystemDebugHUDExtension_Attributes>();
	Extension->AttributesToDisplay.Empty();

	AAbilitySystemDebugHUD::NotifyExtensionEnableChanged(World);
}

bool UAbilitySystemDebugHUDExtension_Attributes::IsEnabled() const
{
	return AttributesToDisplay.Num() > 0;
}

void UAbilitySystemDebugHUDExtension_Attributes::GetDebugStrings(const AActor* Actor, const UAbilitySystemComponent* Comp, OUT TArray<FString>& OutDebugStrings) const
{
	if (!Comp)
	{
		return;
	}

	const FActiveGameplayEffectsContainer& ActiveEffects = Comp->GetActiveGameplayEffects();

	TArray<UObject*> Objects;
	GetObjectsWithOuter(Comp->GetOwnerActor(), Objects);

	TMultiMap<FGameplayAttribute, FActiveGameplayEffectsContainer::DebugExecutedGameplayEffectData> EffectMap;

	ActiveEffects.GetActiveGameplayEffectDataByAttribute(EffectMap);

	for (UObject* Object : Objects)
	{
		UAttributeSet* AttributeSet = Cast<UAttributeSet>(Object);
		if (AttributeSet)
		{
			UClass* Class = AttributeSet->GetClass();

			for (TFieldIterator<FProperty> PropertyIt(Class, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
			{
				FProperty* Property = *PropertyIt;

				if (AttributesToDisplay.Contains(Property->GetName()))
				{
					FGameplayAttribute Attribute(Property);
					OutDebugStrings.Add(FString::Printf(TEXT("%s: %f (Base: %f)"), *Property->GetName(), Comp->GetNumericAttribute(Attribute), Comp->GetNumericAttributeBase(Attribute)));

					// if we're also showing modifiers, grab them and display them here
					if (bIncludeModifiers)
					{
						TArray<FActiveGameplayEffectsContainer::DebugExecutedGameplayEffectData> AttributeEffects;
						EffectMap.MultiFind(Attribute, AttributeEffects);

						for (const FActiveGameplayEffectsContainer::DebugExecutedGameplayEffectData& DebugData : AttributeEffects)
						{
							OutDebugStrings.Add(FString::Printf(TEXT("%s %s"), *DebugData.GameplayEffectName, *DebugData.ActivationState));
							OutDebugStrings.Add(FString::Printf(TEXT("%s %f"), *EGameplayModOpToString(DebugData.ModifierOp), DebugData.Magnitude));
							OutDebugStrings.Add(FString::Printf(TEXT("Stack Count: %i"), DebugData.StackCount));
						}
					}
				}
			}
		}
	}
}

void UAbilitySystemDebugHUDExtension_BlockedAbilityTags::ToggleExtension(const TArray<FString>& Args, UWorld* World)
{
	UAbilitySystemDebugHUDExtension_BlockedAbilityTags* Extension = GetMutableDefault<UAbilitySystemDebugHUDExtension_BlockedAbilityTags>();
	if (Args.IsEmpty())
	{
		const bool bIsCurrentlyEnabled = Extension->bEnabled || !Extension->TagsToDisplay.IsEmpty();

		if (bIsCurrentlyEnabled)
		{
			Extension->TagsToDisplay.Empty();
		}

		Extension->bEnabled = !bIsCurrentlyEnabled;
	}
	else
	{
		for (const FString& Tag : Args)
		{
			FSetElementId Element = Extension->TagsToDisplay.FindId(Tag);
			if (Element.IsValidId())
			{
				Extension->TagsToDisplay.Remove(Element);
			}
			else
			{
				Extension->TagsToDisplay.Add(Tag);
			}
		}
	}

	AAbilitySystemDebugHUD::NotifyExtensionEnableChanged(World);
}

bool UAbilitySystemDebugHUDExtension_BlockedAbilityTags::IsEnabled() const
{
	return bEnabled || !TagsToDisplay.IsEmpty();
}

void UAbilitySystemDebugHUDExtension_BlockedAbilityTags::GetDebugStrings(const AActor* Actor, const UAbilitySystemComponent* Comp, OUT TArray<FString>& OutDebugStrings) const
{
	if (!Comp)
	{
		return;
	}

	// Count up how many GameplayEffects blocked each tag
	TMap<FGameplayTag, uint32> TagCounts;
	for (FActiveGameplayEffectsContainer::ConstIterator ActiveEffectIt = Comp->GetActiveGameplayEffects().CreateConstIterator(); ActiveEffectIt; ++ActiveEffectIt)
	{
		FGameplayTagContainer EffectTagContainer;
		ActiveEffectIt->Spec.GetAllBlockedAbilityTags(EffectTagContainer);

		for (auto EffectTagIt = EffectTagContainer.CreateConstIterator(); EffectTagIt; ++EffectTagIt)
		{
			++TagCounts.FindOrAdd(*EffectTagIt, 0);
		}
	}

	FGameplayTagContainer Container;
	Comp->GetBlockedAbilityTags(Container);
	for (auto ContainerIt = Container.CreateConstIterator(); ContainerIt; ++ContainerIt)
	{
		if (TagsToDisplay.IsEmpty() || TagsToDisplay.Contains(ContainerIt->ToString()))
		{
			if (const uint32* TagCount = TagCounts.Find(*ContainerIt))
			{
				OutDebugStrings.Add(FString::Printf(TEXT("%s x%d"), *ContainerIt->ToString(), *TagCount));
			}
			else
			{
				OutDebugStrings.Add(ContainerIt->ToString());
			}
		}
	}
}

AAbilitySystemDebugHUD::AAbilitySystemDebugHUD(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{

}

void AAbilitySystemDebugHUD::ToggleBasicHUD(const TArray<FString>& Args, UWorld* World)
{
	bEnableBasicHUD = !bEnableBasicHUD;

	NotifyExtensionEnableChanged(World);
}

void AAbilitySystemDebugHUD::NotifyExtensionEnableChanged(UWorld* InWorld)
{
	if (!InWorld)
	{
		return;
	}

	AAbilitySystemDebugHUD* HUD = nullptr;
	if (TActorIterator<AAbilitySystemDebugHUD> It(InWorld); It)
	{
		HUD = *It;
	}

	bool bAnyExtensionEnabled = bEnableBasicHUD;
	for (TObjectIterator<UAbilitySystemDebugHUDExtension> ExtIt(RF_NoFlags); ExtIt; ++ExtIt)
	{
		bAnyExtensionEnabled |= ExtIt->IsEnabled();
	}

	static FDelegateHandle PostWorldInitDelegateHandle;

	if (!HUD && bAnyExtensionEnabled)
	{
		CreateHUD(InWorld);

		PostWorldInitDelegateHandle = FWorldDelegates::OnPostWorldInitialization.AddLambda([](UWorld* World, const UWorld::InitializationValues IVS)
			{
				if (World->WorldType == EWorldType::PIE || World->WorldType == EWorldType::Game)
				{
					CreateHUD(World);
				}
			});

	}
	else if (HUD && !bAnyExtensionEnabled)
	{
		FWorldDelegates::OnPostWorldInitialization.Remove(PostWorldInitDelegateHandle);
		UDebugDrawService::Unregister(DrawDebugDelegateHandle);
		HUD->Destroy();
	}
}

void AAbilitySystemDebugHUD::CreateHUD(UWorld* World)
{
	UClass* HUDClassToSpawn = HUDClass.Get();
	if (!HUDClassToSpawn)
	{
		HUDClassToSpawn = AAbilitySystemDebugHUD::StaticClass();
	}

	AAbilitySystemDebugHUD* HUD = World->SpawnActor<AAbilitySystemDebugHUD>();

	FDebugDrawDelegate DrawDebugDelegate = FDebugDrawDelegate::CreateUObject(HUD, &AAbilitySystemDebugHUD::DrawDebugHUD);
	DrawDebugDelegateHandle = UDebugDrawService::Register(TEXT("GameplayDebug"), DrawDebugDelegate);
}

void AAbilitySystemDebugHUD::DrawWithBackground(UFont* InFont, const FString& Text, const FColor& TextColor, EAlignHorizontal::Type HAlign, float& OffsetX, EAlignVertical::Type VAlign, float& OffsetY, float Alpha)
{
	float SizeX, SizeY;
	Canvas->StrLen(InFont, Text, SizeX, SizeY);

	const float PosX = (HAlign == EAlignHorizontal::Center) ? OffsetX + (Canvas->ClipX - SizeX) * 0.5f :
		(HAlign == EAlignHorizontal::Left) ? Canvas->OrgX + OffsetX :
		Canvas->ClipX - SizeX - OffsetX;

	const float PosY = (VAlign == EAlignVertical::Center) ? OffsetY + (Canvas->ClipY - SizeY) * 0.5f :
		(VAlign == EAlignVertical::Top) ? Canvas->OrgY + OffsetY :
		Canvas->ClipY - SizeY - OffsetY;

	const float BoxPadding = 5.0f;

	const float X = PosX - BoxPadding;
	const float Y = PosY - BoxPadding;
	const float Z = 0.1f;
	FCanvasTileItem TileItem(FVector2D(X, Y), FVector2D(SizeX + BoxPadding * 2.0f, SizeY + BoxPadding * 2.0f), FLinearColor(0.75f, 0.75f, 0.75f, Alpha));
	Canvas->DrawItem(TileItem);

	FLinearColor TextCol(TextColor);
	TextCol.A = Alpha;
	FCanvasTextItem TextItem(FVector2D(PosX, PosY), FText::FromString(Text), GEngine->GetSmallFont(), TextCol);
	Canvas->DrawItem(TextItem);

	OffsetY += 25;
}

void AAbilitySystemDebugHUD::DrawDebugHUD(UCanvas* InCanvas, APlayerController* Cont)
{
	Canvas = InCanvas;
	if (!Canvas)
	{
		return;
	}

	APlayerController *PC = GetWorld()->GetFirstPlayerController();
	if (!PC)
	{
		return;
	}

	if (bEnableBasicHUD)
	{
		if (UAbilitySystemComponent* AbilitySystemComponent = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(PC->GetPawn()))
		{
			DrawDebugAbilitySystemComponent(AbilitySystemComponent);
		}
	}

	DrawAbilityDebugInfo(Canvas, PC);
}

bool AAbilitySystemDebugHUD::ShouldDisplayDebugForActor(UCanvas* InCanvas, const AActor* Actor, const FVector& CameraPosition, const FVector& CameraDir) const
{
	if (!Actor)
	{
		return false;
	}

	if (!Actor->WasRecentlyRendered(0.1f))
	{
		return false;
	}

	const FVector ThisLocation = Actor->GetActorLocation();
	const FVector CameraToActorVector = (ThisLocation - CameraPosition);
	const bool bIsBehindCamera = ((CameraDir | CameraToActorVector) < 0.f);
	if (bIsBehindCamera)
	{
		return false;
	}

	const bool bBeyondMaxDistance = CameraToActorVector.SizeSquared2D() > FMath::Square(DebugDrawMaxDistance);
	if (bBeyondMaxDistance)
	{
		return false;
	}

	FVector ThisOrigin;
	FVector BoxExtent;
	Actor->GetActorBounds(true, ThisOrigin, BoxExtent);

	const FVector ScreenLoc = InCanvas->Project(ThisOrigin);
	if ((ScreenLoc.X < 0) || (ScreenLoc.X >= InCanvas->ClipX) || (ScreenLoc.Y < 0) || (ScreenLoc.Y >= InCanvas->ClipY))
	{
		return false;
	}

	return true;
}

void AAbilitySystemDebugHUD::DisplayDebugStrings(UCanvas* InCanvas, const AActor* Actor, const TArray<FString>& DebugStrings, const FVector& CameraPosition, const FVector& CameraDir, float& VerticalOffset) const
{
	if (DebugStrings.Num() == 0)
	{
		return;
	}

	struct FStringAndLineHeight
	{
		FString String;
		float LineHeight;

		FStringAndLineHeight(const FString& InString, float InLineHeight)
			: String(InString), LineHeight(InLineHeight)
		{}
	};

	UFont* Font = GEngine->GetTinyFont();
	TArray<FStringAndLineHeight> StringsToDraw;
	float MaxWidth = 0.f;
	float TotalHeight = 0.f;
	for (auto ContainerIt = DebugStrings.CreateConstIterator(); ContainerIt; ++ContainerIt)
	{
		const FString DrawString = *ContainerIt;
		float TextWidth = 0.f;
		float TextHeight = 0.f;
		InCanvas->TextSize(Font, DrawString, TextWidth, TextHeight);
		MaxWidth = FMath::Max(MaxWidth, TextWidth);
		TotalHeight += TextHeight;
		StringsToDraw.Emplace(FStringAndLineHeight(DrawString, TextHeight));
	}

	FVector ThisOrigin;
	FVector BoxExtent;
	Actor->GetActorBounds(true, ThisOrigin, BoxExtent);
	const FVector ScreenLoc = InCanvas->Project(ThisOrigin);
	const float TextLeft = ScreenLoc.X - MaxWidth * 0.5f;
	const float TextTop = ScreenLoc.Y - TotalHeight * 0.5f + VerticalOffset;

	// Draw a background
	const FLinearColor BackgroundColor(0.106f, 0.039f, 0.039f, 0.3f);
	const float BoxPadding = 2.f;
	const FVector2D BoxLocation(TextLeft - BoxPadding, TextTop - BoxPadding);
	const FVector2D BoxSize(MaxWidth + BoxPadding * 2, TotalHeight + BoxPadding * 2);
	VerticalOffset += BoxSize.Y + BoxPadding;
	FCanvasTileItem Tile(BoxLocation, BoxSize, BackgroundColor);
	InCanvas->DrawItem(Tile);

	// Debug Text
	InCanvas->SetDrawColor(FColor(242, 242, 242));
	float DrawTop = TextTop;
	for (const FStringAndLineHeight& StringAndLineHeight : StringsToDraw)
	{
		InCanvas->DrawText(Font, StringAndLineHeight.String, TextLeft, DrawTop);
		DrawTop += StringAndLineHeight.LineHeight;
	}
}

void AAbilitySystemDebugHUD::DrawDebugAbilitySystemComponent(UAbilitySystemComponent *Component)
{
	UWorld *World = GetWorld();
	float GameWorldTime = World->GetTimeSeconds();

	UFont* Font = GEngine->GetSmallFont();
	FColor Color(38, 128, 0);
	float X = 20.f;
	float Y = 20.f;

	FString String = FString::Printf(TEXT("%.2f"), Component->GetWorld()->GetTimeSeconds());
	DrawWithBackground(Font, String, Color, EAlignHorizontal::Left, X, EAlignVertical::Top, Y);

	String = FString::Printf(TEXT("%s (%d)"), *Component->GetPathName(), Component->IsDefaultSubobject());
	DrawWithBackground(Font, String, Color, EAlignHorizontal::Left, X, EAlignVertical::Top, Y);

	
	String = FString::Printf(TEXT("%s == %s"), *Component->GetArchetype()->GetPathName(), *Component->GetClass()->GetDefaultObject()->GetPathName());
	DrawWithBackground(Font, String, Color, EAlignHorizontal::Left, X, EAlignVertical::Top, Y);


	for (const UAttributeSet* Set : Component->GetSpawnedAttributes())
	{
		if (!Set)
		{
			continue;
		}

		// Draw Attribute Set
		DrawWithBackground(Font, FString::Printf(TEXT("%s (%d)"), *Set->GetName(), Set->IsDefaultSubobject()), Color, EAlignHorizontal::Left, X, EAlignVertical::Top, Y);

		String = FString::Printf(TEXT("%s == %s"), *Set->GetArchetype()->GetPathName(), *Set->GetClass()->GetDefaultObject()->GetPathName());
		DrawWithBackground(Font, String, Color, EAlignHorizontal::Left, X, EAlignVertical::Top, Y);

		for (TFieldIterator<FProperty> PropertyIt(Set->GetClass(), EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			FProperty *Prop = *PropertyIt;			

			FString ValueString;
			const void *PropertyValue = Prop->ContainerPtrToValuePtr<void>(Set);
			Prop->ExportTextItem_Direct(ValueString, PropertyValue, NULL, NULL, 0);

			String = FString::Printf(TEXT("%s: %s"), *Prop->GetName(), *ValueString);
			DrawWithBackground(Font, String, Color, EAlignHorizontal::Left, X, EAlignVertical::Top, Y);
		}

		Y+= 25;
		// Draw Active GameplayEffect
		for (FActiveGameplayEffect& Effect : &Component->ActiveGameplayEffects)
		{
			String = FString::Printf(TEXT("%s. %s %.2f"), *Effect.Spec.ToSimpleString(), *Effect.PredictionKey.ToString(), Effect.GetTimeRemaining(GameWorldTime));
			DrawWithBackground(Font, String, Color, EAlignHorizontal::Left, X, EAlignVertical::Top, Y);	
		}
	}
	
}

void AAbilitySystemDebugHUD::DrawAbilityDebugInfo(UCanvas* InCanvas, APlayerController* PC) const
{
	if (!PC)
	{
		// Need a PC to determine the camera location
		return;
	}

	UWorld* PCWorld = PC->GetWorld();

	if (!PCWorld)
	{
		// Need a world to determine which actors to draw
		return;
	}

	FVector CameraPosition;
	FRotator CameraRotation;
	PC->GetPlayerViewPoint(CameraPosition, CameraRotation);
	const FVector CameraDir = CameraRotation.Vector();

	for (TObjectIterator<UAbilitySystemComponent> ASCIt; ASCIt; ++ASCIt)
	{
		if (UAbilitySystemComponent* ASC = *ASCIt)
		{
			if (ASC->GetWorld() == PCWorld)
			{
				// We need to have actor info for valid debug info.
				if (!ASC->AbilityActorInfo.IsValid())
				{
					continue;
				}
				
				const AActor* AvatarActor = ASC->GetAvatarActor();
				if (!ShouldDisplayDebugForActor(InCanvas, AvatarActor, CameraPosition, CameraDir))
				{
					continue;
				}

				float VerticalOffset = 0.f;
				for (TObjectIterator<UAbilitySystemDebugHUDExtension> ExtIt(EObjectFlags::RF_NoFlags); ExtIt; ++ExtIt)
				{
					if (ExtIt->IsEnabled())
					{
						TArray<FString> DebugStrings;
						ExtIt->GetDebugStrings(AvatarActor, ASC, DebugStrings);
						DisplayDebugStrings(InCanvas, AvatarActor, DebugStrings, CameraPosition, CameraDir, VerticalOffset);
					}
				}
			}
		}
	}
}


#if !UE_BUILD_SHIPPING

FAutoConsoleCommandWithWorldAndArgs AbilitySystemToggleDebugHUDCommand(
	TEXT("AbilitySystem.DebugBasicHUD"),
	TEXT("Toggles Drawing a basic debug HUD for the local player's ability system component"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(AAbilitySystemDebugHUD::ToggleBasicHUD)
);

FAutoConsoleCommandWithWorldAndArgs AbilitySystemToggleDebugTagsCommand(
	TEXT("AbilitySystem.DebugAbilityTags"),
	TEXT("Usage: AbilitySystem.DebugAbilityTags [TagName] [TagName]...\nToggles Drawing Ability Tags on Actors with AbilitySystemComponents. If no tags are given, draws all owned tags."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(UAbilitySystemDebugHUDExtension_Tags::ToggleExtension)
);

FAutoConsoleCommandWithWorldAndArgs AbilitySystemToggleDebugAttributesCommand(
	TEXT("AbilitySystem.DebugAttribute"),
	TEXT("Usage: AbilitySystem.DebugAttribute [AttributeName] [AttributeName]...\nToggles Drawing the given attributes on Actors with AbilitySystemComponents"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(UAbilitySystemDebugHUDExtension_Attributes::ToggleExtension)
);

FAutoConsoleCommandWithWorldAndArgs AbilitySystemClearDebugAttributesCommand(
	TEXT("AbilitySystem.ClearDebugAttributes"),
	TEXT("Usage: AbilitySystem.ClearDebugAttributes...\nStops drawing all attributes on Actors with AbilitySystemComponents"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(UAbilitySystemDebugHUDExtension_Attributes::ClearDisplayedAttributes)
);

FAutoConsoleCommand AbilitySystemToggleIncludeModifiersCommand(
	TEXT("AbilitySystem.DebugIncludeModifiers"),
	TEXT("Usage: AbilitySystem.DebugIncludeModifiers...\nToggles whether or not modifiers are displayed alongside debugged attributes"),
	FConsoleCommandDelegate::CreateStatic(UAbilitySystemDebugHUDExtension_Attributes::ToggleIncludeModifiers)
);

FAutoConsoleCommandWithWorldAndArgs AbilitySystemToggleDebugBlockedTagsCommand(
	TEXT("AbilitySystem.DebugBlockedAbilityTags"),
	TEXT("Usage: AbilitySystem.DebugBlockedAbilityTags [TagName] [TagName]...\nToggles Drawing Blocked Ability Tags on Actors with AbilitySystemComponents. If no tags are given, draws all blocked tags."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(UAbilitySystemDebugHUDExtension_BlockedAbilityTags::ToggleExtension)
);

#endif // !UE_BUILD_SHIPPING

