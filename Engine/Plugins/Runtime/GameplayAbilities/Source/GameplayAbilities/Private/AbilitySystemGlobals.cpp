// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbilitySystemGlobals.h"
#include "Abilities/GameplayAbilityTypes.h"
#include "AbilitySystemStats.h"
#include "Engine/Blueprint.h"
#include "GameplayAbilitiesDeveloperSettings.h"
#include "GameFramework/Pawn.h"
#include "GameplayCueInterface.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemInterface.h"
#include "AbilitySystemLog.h"
#include "HAL/LowLevelMemTracker.h"
#include "GameFramework/PlayerController.h"
#include "GameplayCueManager.h"
#include "GameplayTagResponseTable.h"
#include "GameplayTagsManager.h"
#include "Engine/Engine.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"

#include "Serialization/GameplayAbilityTargetDataHandleNetSerializer.h"
#include "Serialization/GameplayEffectContextHandleNetSerializer.h"
#include "Serialization/PredictionKeyNetSerializer.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilitySystemGlobals)

namespace UE::AbilitySystemGlobals
{
	bool bIgnoreAbilitySystemCooldowns = false;
	bool bIgnoreAbilitySystemCosts = false;
	float AbilitySystemGlobalScaler = 1.f;


	FAutoConsoleVariableRef CVarAbilitySystemIgnoreCooldowns(TEXT("AbilitySystem.IgnoreCooldowns"), bIgnoreAbilitySystemCooldowns, TEXT("Ignore cooldowns for all Gameplay Abilities."), ECVF_Cheat);
	FAutoConsoleVariableRef CVarAbilitySystemIgnoreCosts(TEXT("AbilitySystem.IgnoreCosts"), bIgnoreAbilitySystemCosts, TEXT("Ignore costs for all Gameplay Abilities."), ECVF_Cheat);

	static FAutoConsoleVariableRef CVarAbilitySystemGlobalScaler(TEXT("AbilitySystem.GlobalAbilityScale"), AbilitySystemGlobalScaler, TEXT("Global rate for scaling ability stuff like montages and root motion tasks. Used only for testing/iteration, never for shipping."), ECVF_Cheat);
}

UAbilitySystemGlobals::UAbilitySystemGlobals(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	RegisteredReimportCallback = false;
#endif // #if WITH_EDITORONLY_DATA
}

bool UAbilitySystemGlobals::IsAbilitySystemGlobalsInitialized() const
{
	return bInitialized;
}

bool UAbilitySystemGlobals::ShouldUseDebugTargetFromHud()
{
	return GetDefault<UGameplayAbilitiesDeveloperSettings>()->bUseDebugTargetFromHud;
}

void UAbilitySystemGlobals::InitGlobalData()
{
	// Make sure the user didn't try to initialize the system again (we call InitGlobalData automatically in UE5.3+).
	if (IsAbilitySystemGlobalsInitialized())
	{
		return;
	}
	bInitialized = true;

	LLM_SCOPE_BYNAME(TEXT("AbilitySystem"));
	GetGlobalCurveTable();
	GetGlobalAttributeMetaDataTable();
	
	InitAttributeDefaults();
	ReloadAttributeDefaults();

	GetGameplayCueManager();
	GetGameplayTagResponseTable();
	InitGlobalTags();
	PerformDeveloperSettingsUpgrade();

	InitTargetDataScriptStructCache();

	// Register for PreloadMap so cleanup can occur on map transitions
	FCoreUObjectDelegates::PreLoadMapWithContext.AddUObject(this, &UAbilitySystemGlobals::HandlePreLoadMap);

#if WITH_EDITOR
	// Register in editor for PreBeginPlay so cleanup can occur when we start a PIE session
	if (GIsEditor)
	{
		FEditorDelegates::PreBeginPIE.AddUObject(this, &UAbilitySystemGlobals::OnPreBeginPIE);
	}
#endif
}


UCurveTable * UAbilitySystemGlobals::GetGlobalCurveTable()
{
	if (!GlobalCurveTable)
	{
		const UGameplayAbilitiesDeveloperSettings* DeveloperSettings = GetDefault<UGameplayAbilitiesDeveloperSettings>();
		if (DeveloperSettings->GlobalCurveTableName.IsValid())
		{
			GlobalCurveTable = Cast<UCurveTable>(DeveloperSettings->GlobalCurveTableName.TryLoad());
		}
	}
	return GlobalCurveTable;
}

UDataTable * UAbilitySystemGlobals::GetGlobalAttributeMetaDataTable()
{	
	if (!GlobalAttributeMetaDataTable)
	{
		const UGameplayAbilitiesDeveloperSettings* DeveloperSettings = GetDefault<UGameplayAbilitiesDeveloperSettings>();
		if (DeveloperSettings->GlobalAttributeMetaDataTableName.IsValid())
		{
			GlobalAttributeMetaDataTable = Cast<UDataTable>(DeveloperSettings->GlobalAttributeMetaDataTableName.TryLoad());
		}
	}
	return GlobalAttributeMetaDataTable;
}

bool UAbilitySystemGlobals::DeriveGameplayCueTagFromAssetName(FString AssetName, FGameplayTag& GameplayCueTag, FName& GameplayCueName)
{
	FGameplayTag OriginalTag = GameplayCueTag;
	
	// In the editor, attempt to infer GameplayCueTag from our asset name (if there is no valid GameplayCueTag already).
#if WITH_EDITOR
	if (GIsEditor)
	{
		if (GameplayCueTag.IsValid() == false)
		{
			AssetName.RemoveFromStart(TEXT("Default__"));
			AssetName.RemoveFromStart(TEXT("REINST_"));
			AssetName.RemoveFromStart(TEXT("SKEL_"));
			AssetName.RemoveFromStart(TEXT("GC_"));		// allow GC_ prefix in asset name
			AssetName.RemoveFromEnd(TEXT("_c"));

			AssetName.ReplaceInline(TEXT("_"), TEXT("."), ESearchCase::CaseSensitive);

			if (!AssetName.Contains(TEXT("GameplayCue")))
			{
				AssetName = FString(TEXT("GameplayCue.")) + AssetName;
			}

			GameplayCueTag = UGameplayTagsManager::Get().RequestGameplayTag(FName(*AssetName), false);
		}
		GameplayCueName = GameplayCueTag.GetTagName();
	}
#endif
	return (OriginalTag != GameplayCueTag);
}

bool UAbilitySystemGlobals::ShouldAllowGameplayModEvaluationChannels() const
{
	return GetDefault<UGameplayAbilitiesDeveloperSettings>()->bAllowGameplayModEvaluationChannels;
}

bool UAbilitySystemGlobals::IsGameplayModEvaluationChannelValid(EGameplayModEvaluationChannel Channel) const
{
	// Only valid if channels are allowed and the channel has a game-specific alias specified or if not using channels and the channel is Channel0
	const bool bAllowChannels = ShouldAllowGameplayModEvaluationChannels();
	return bAllowChannels ? (!GetGameplayModEvaluationChannelAlias(Channel).IsNone()) : (Channel == EGameplayModEvaluationChannel::Channel0);
}

const FName& UAbilitySystemGlobals::GetGameplayModEvaluationChannelAlias(EGameplayModEvaluationChannel Channel) const
{
	return GetGameplayModEvaluationChannelAlias(static_cast<int32>(Channel));
}

const FName& UAbilitySystemGlobals::GetGameplayModEvaluationChannelAlias(int32 Index) const
{
	const UGameplayAbilitiesDeveloperSettings* DeveloperSettings = GetDefault<UGameplayAbilitiesDeveloperSettings>();
	check(Index >= 0 && Index < UE_ARRAY_COUNT(DeveloperSettings->GameplayModEvaluationChannelAliases));
	return DeveloperSettings->GameplayModEvaluationChannelAliases[Index];
}

TArray<FString> UAbilitySystemGlobals::GetGameplayCueNotifyPaths()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	//Use set so we can append just unique paths
	TSet<FString> ReturnPaths = TSet(GameplayCueNotifyPaths);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ReturnPaths.Append(GetDefault<UGameplayAbilitiesDeveloperSettings>()->GameplayCueNotifyPaths);
	return ReturnPaths.Array();
}

void UAbilitySystemGlobals::AddGameplayCueNotifyPath(const FString& InPath)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GameplayCueNotifyPaths.AddUnique(InPath);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

int32 UAbilitySystemGlobals::RemoveGameplayCueNotifyPath(const FString& InPath)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GameplayCueNotifyPaths.Remove(InPath);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR

void UAbilitySystemGlobals::OnTableReimported(UObject* InObject)
{
	if (GIsEditor && !IsRunningCommandlet() && InObject)
	{
		UCurveTable* ReimportedCurveTable = Cast<UCurveTable>(InObject);
		if (ReimportedCurveTable && GlobalAttributeDefaultsTables.Contains(ReimportedCurveTable))
		{
			ReloadAttributeDefaults();
		}
	}	
}

#endif

FGameplayAbilityActorInfo * UAbilitySystemGlobals::AllocAbilityActorInfo() const
{
	return new FGameplayAbilityActorInfo();
}

FGameplayEffectContext* UAbilitySystemGlobals::AllocGameplayEffectContext() const
{
	return new FGameplayEffectContext();
}

/** Helping function to avoid having to manually cast */
UAbilitySystemComponent* UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(const AActor* Actor, bool LookForComponent)
{
	if (Actor == nullptr)
	{
		return nullptr;
	}

	const IAbilitySystemInterface* ASI = Cast<IAbilitySystemInterface>(Actor);
	if (ASI)
	{
		return ASI->GetAbilitySystemComponent();
	}

	if (LookForComponent)
	{
		// Fall back to a component search to better support BP-only actors
		return Actor->FindComponentByClass<UAbilitySystemComponent>();
	}

	return nullptr;
}

bool UAbilitySystemGlobals::ShouldPredictTargetGameplayEffects() const
{
	return GetDefault<UGameplayAbilitiesDeveloperSettings>()->PredictTargetGameplayEffects;
}

bool UAbilitySystemGlobals::ShouldReplicateActivationOwnedTags() const
{
	return GetDefault<UGameplayAbilitiesDeveloperSettings>()->ReplicateActivationOwnedTags;
}

// --------------------------------------------------------------------

UFunction* UAbilitySystemGlobals::GetGameplayCueFunction(const FGameplayTag& ChildTag, UClass* Class, FName &MatchedTag)
{
	SCOPE_CYCLE_COUNTER(STAT_GetGameplayCueFunction);

	// A global cached map to lookup these functions might be a good idea. Keep in mind though that FindFunctionByName
	// is fast and already gives us a reliable map lookup.
	// 
	// We would get some speed by caching off the 'fully qualified name' to 'best match' lookup. E.g. we can directly map
	// 'GameplayCue.X.Y' to the function 'GameplayCue.X' without having to look for GameplayCue.X.Y first.
	// 
	// The native remapping (Gameplay.X.Y to Gameplay_X_Y) is also annoying and slow and could be fixed by this as well.
	// 
	// Keep in mind that any UFunction* cacheing is pretty unsafe. Classes can be loaded (and unloaded) during runtime
	// and will be regenerated all the time in the editor. Just doing a single pass at startup won't be enough,
	// we'll need a mechanism for registering classes when they are loaded on demand.
	
	FGameplayTagContainer TagAndParentsContainer = ChildTag.GetGameplayTagParents();

	for (auto InnerTagIt = TagAndParentsContainer.CreateConstIterator(); InnerTagIt; ++InnerTagIt)
	{
		FName CueName = InnerTagIt->GetTagName();
		if (UFunction* Func = Class->FindFunctionByName(CueName, EIncludeSuperFlag::IncludeSuper))
		{
			MatchedTag = CueName;
			return Func;
		}

		// Native functions cant be named with ".", so look for them with _. 
		FName NativeCueFuncName = *CueName.ToString().Replace(TEXT("."), TEXT("_"));
		if (UFunction* Func = Class->FindFunctionByName(NativeCueFuncName, EIncludeSuperFlag::IncludeSuper))
		{
			MatchedTag = CueName; // purposefully returning the . qualified name.
			return Func;
		}
	}

	return nullptr;
}

void UAbilitySystemGlobals::InitGlobalTags()
{
	auto TagFromDeprecatedName = [](FGameplayTag& Tag, FName DeprecatedName)
	{
		if (!Tag.IsValid() && !DeprecatedName.IsNone())
		{
			Tag = FGameplayTag::RequestGameplayTag(DeprecatedName);
			return true;
		}

		return false;
	};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TagFromDeprecatedName(ActivateFailIsDeadTag, ActivateFailIsDeadName);
	TagFromDeprecatedName(ActivateFailCooldownTag, ActivateFailCooldownName);
	TagFromDeprecatedName(ActivateFailCostTag, ActivateFailCostName);
	TagFromDeprecatedName(ActivateFailTagsBlockedTag, ActivateFailTagsBlockedName);
	TagFromDeprecatedName(ActivateFailTagsMissingTag, ActivateFailTagsMissingName);
	TagFromDeprecatedName(ActivateFailNetworkingTag, ActivateFailNetworkingName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAbilitySystemGlobals::PerformDeveloperSettingsUpgrade()
{
	auto SyncTag = [](FGameplayTag& DestinationTag, const FGameplayTag& OurTag)
	{
		if (OurTag.IsValid() && DestinationTag != OurTag)
		{
			DestinationTag = OurTag;
			return true;
		}

		return false;
	};

	UGameplayAbilitiesDeveloperSettings* DeveloperSettings = GetMutableDefault<UGameplayAbilitiesDeveloperSettings>();

	bool bUpgraded = false;
	bUpgraded |= SyncTag(DeveloperSettings->ActivateFailCooldownTag, ActivateFailCooldownTag);
	bUpgraded |= SyncTag(DeveloperSettings->ActivateFailCostTag, ActivateFailCostTag);
	bUpgraded |= SyncTag(DeveloperSettings->ActivateFailNetworkingTag, ActivateFailNetworkingTag);
	bUpgraded |= SyncTag(DeveloperSettings->ActivateFailTagsBlockedTag, ActivateFailTagsBlockedTag);
	bUpgraded |= SyncTag(DeveloperSettings->ActivateFailTagsMissingTag, ActivateFailTagsMissingTag);

	if (bUpgraded)
	{
		UE_LOGF(LogAbilitySystem, Warning, "AbilitySystemGlobals' Tags did not agree with GameplayAbilitiesDeveloperSettings.  Updating GameplayAbilitiesDeveloperSettings Config to use Tags from AbilitySystemGlobals");

		bool bSuccess = DeveloperSettings->TryUpdateDefaultConfigFile();
		if (!bSuccess)
		{
			UE_LOGF(LogAbilitySystem, Warning, "AbilitySystemGlobals config file (DefaultGame.ini) couldn't be saved. Make sure the file is writable to update it.");
		}
	}

	// Now that the upgrade is done, copy any settings set in the DeveloperSettings back to here (so calls to UAbilitySystemGlobals::Get().SomeTag work)
	SyncTag(ActivateFailCooldownTag, DeveloperSettings->ActivateFailCooldownTag);
	SyncTag(ActivateFailCostTag, DeveloperSettings->ActivateFailCostTag);
	SyncTag(ActivateFailNetworkingTag, DeveloperSettings->ActivateFailNetworkingTag);
	SyncTag(ActivateFailTagsBlockedTag, DeveloperSettings->ActivateFailTagsBlockedTag);
	SyncTag(ActivateFailTagsMissingTag, DeveloperSettings->ActivateFailTagsMissingTag);
}

void UAbilitySystemGlobals::InitTargetDataScriptStructCache()
{
	TargetDataStructCache.InitForType(FGameplayAbilityTargetData::StaticStruct());
	EffectContextStructCache.InitForType(FGameplayEffectContext::StaticStruct());

	// Handle loading for new modules that might have new structs to add
	if (!ModulesChangedHandle.IsValid())
	{
		ModulesChangedHandle = FModuleManager::Get().OnModulesChanged().AddWeakLambda(this, [this](FName ModuleName, EModuleChangeReason Reason)
		{
			if (Reason == EModuleChangeReason::ModuleLoaded)
			{
				TargetDataStructCache.SetInitDirty();
				EffectContextStructCache.SetInitDirty();
			}
		});
	}

	// Handle module unloading to avoid holding onto released structs
	if (!ModulesUnloadedHandle.IsValid())
	{
		ModulesUnloadedHandle = FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.AddWeakLambda(this, [this](TConstArrayView<UPackage*> Packages)
		{
			TargetDataStructCache.RemoveInPackages(Packages);
			EffectContextStructCache.RemoveInPackages(Packages);
		});
	}
}

// --------------------------------------------------------------------

void UAbilitySystemGlobals::InitGameplayCueParameters(FGameplayCueParameters& CueParameters, const FGameplayEffectSpecForRPC& Spec)
{
	CueParameters.AggregatedSourceTags = Spec.AggregatedSourceTags;
	CueParameters.AggregatedTargetTags = Spec.AggregatedTargetTags;
	CueParameters.GameplayEffectLevel = Spec.GetLevel();
	CueParameters.AbilityLevel = Spec.GetAbilityLevel();
	InitGameplayCueParameters(CueParameters, Spec.GetContext());
}

void UAbilitySystemGlobals::InitGameplayCueParameters_GESpec(FGameplayCueParameters& CueParameters, const FGameplayEffectSpec& Spec)
{
	CueParameters.AggregatedSourceTags = *Spec.CapturedSourceTags.GetAggregatedTags();
	CueParameters.AggregatedTargetTags = *Spec.CapturedTargetTags.GetAggregatedTags();

	// Look for a modified attribute magnitude to pass to the CueParameters
	for (const FGameplayEffectCue& CueDef : Spec.Def->GameplayCues)
	{	
		bool FoundMatch = false;
		if (CueDef.MagnitudeAttribute.IsValid())
		{
			for (const FGameplayEffectModifiedAttribute& ModifiedAttribute : Spec.ModifiedAttributes)
			{
				if (ModifiedAttribute.Attribute == CueDef.MagnitudeAttribute)
				{
					CueParameters.RawMagnitude = ModifiedAttribute.TotalMagnitude;
					FoundMatch = true;
					break;
				}
			}
			if (FoundMatch)
			{
				break;
			}
		}
	}

	CueParameters.GameplayEffectLevel = Spec.GetLevel();
	CueParameters.AbilityLevel = Spec.GetEffectContext().GetAbilityLevel();

	InitGameplayCueParameters(CueParameters, Spec.GetContext());
}

void UAbilitySystemGlobals::InitGameplayCueParameters(FGameplayCueParameters& CueParameters, const FGameplayEffectContextHandle& EffectContext)
{
	if (EffectContext.IsValid())
	{
		// Copy Context over wholesale. Projects may want to override this and not copy over all data
		CueParameters.EffectContext = EffectContext;
	}
}

// --------------------------------------------------------------------

void UAbilitySystemGlobals::StartAsyncLoadingObjectLibraries()
{
	if (GlobalGameplayCueManager != nullptr)
	{
		GlobalGameplayCueManager->InitializeRuntimeObjectLibrary();
	}
}

// --------------------------------------------------------------------

/** Initialize FAttributeSetInitter. This is virtual so projects can override what class they use */
void UAbilitySystemGlobals::AllocAttributeSetInitter()
{
	GlobalAttributeSetInitter = TSharedPtr<FAttributeSetInitter>(new FAttributeSetInitterDiscreteLevels());
}

FAttributeSetInitter* UAbilitySystemGlobals::GetAttributeSetInitter() const
{
	check(GlobalAttributeSetInitter.IsValid());
	return GlobalAttributeSetInitter.Get();
}

void UAbilitySystemGlobals::AddAttributeDefaultTables(const FName OwnerName, const TArray<FSoftObjectPath>& AttribDefaultTableNames)
{
	bool bModified = false;
	for (const FSoftObjectPath& TableName : AttribDefaultTableNames)
	{
		if (TArray<FName>* Found = GlobalAttributeSetDefaultsTableNamesWithOwners.Find(TableName))
		{
			Found->Add(OwnerName);
		}
		else
		{
			TArray<FName> Owners = { OwnerName };
			GlobalAttributeSetDefaultsTableNamesWithOwners.Add(TableName, MoveTemp(Owners));

			UCurveTable* AttribTable = Cast<UCurveTable>(TableName.TryLoad());
			if (AttribTable)
			{
				GlobalAttributeDefaultsTables.AddUnique(AttribTable);
				if (!AttribTable->OnCurveTableChanged().IsBoundToObject(this))
				{
					AttribTable->OnCurveTableChanged().AddUObject(this, &ThisClass::OnAttributeDefaultTableChanged);
				}
				bModified = true;
			}
		}
	}

	if (bModified)
	{
		ReloadAttributeDefaults();
	}
}

void UAbilitySystemGlobals::RemoveAttributeDefaultTables(const FName OwnerName, const TArray<FSoftObjectPath>& AttribDefaultTableNames)
{
	bool bModified = false;
	const UGameplayAbilitiesDeveloperSettings* DeveloperSettings = GetDefault<UGameplayAbilitiesDeveloperSettings>();
	for (const FSoftObjectPath& TableName : AttribDefaultTableNames)
	{
		if (TableName.IsValid())
		{
			if (TArray<FName>* Found = GlobalAttributeSetDefaultsTableNamesWithOwners.Find(TableName))
			{
				Found->RemoveSingle(OwnerName);

				// If no references remain, clear the pointer in GlobalAttributeDefaultsTables to allow GC
				if (Found->IsEmpty())
				{
					GlobalAttributeSetDefaultsTableNamesWithOwners.Remove(TableName);

					// Only if not listed in config file
					if (!DeveloperSettings->GlobalAttributeSetDefaultsTableNames.Contains(TableName))
					{
						// Remove reference to allow GC so package can be unloaded
						if (UCurveTable* AttribTable = Cast<UCurveTable>(TableName.ResolveObject()))
						{
							if (GlobalAttributeDefaultsTables.Remove(AttribTable) > 0)
							{
								AttribTable->OnCurveTableChanged().RemoveAll(this);
								bModified = true;
							}
						}
					}
				}
			}
		}
	}

	if (bModified)
	{
		ReloadAttributeDefaults();
	}
}

void UAbilitySystemGlobals::OnAttributeDefaultTableChanged()
{
	ReloadAttributeDefaults();
}

TArray<FSoftObjectPath> UAbilitySystemGlobals::GetGlobalAttributeSetDefaultsTablePaths() const
{
	TArray<FSoftObjectPath> AttribSetDefaultsTables;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Handle deprecated, single global table name
	if (GlobalAttributeSetDefaultsTableName.IsValid())
	{
		AttribSetDefaultsTables.Add(GlobalAttributeSetDefaultsTableName);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const UGameplayAbilitiesDeveloperSettings* DeveloperSettings = GetDefault<UGameplayAbilitiesDeveloperSettings>();
	AttribSetDefaultsTables.Append(DeveloperSettings->GlobalAttributeSetDefaultsTableNames);

	return AttribSetDefaultsTables;
}

void UAbilitySystemGlobals::InitAttributeDefaults()
{
	TArray<FSoftObjectPath> AttribSetDefaultsTables = GetGlobalAttributeSetDefaultsTablePaths();
	for (const FSoftObjectPath& AttribSetDefaultsTablePath : AttribSetDefaultsTables)
	{
		if (AttribSetDefaultsTablePath.IsValid())
		{
			UCurveTable* AttribTable = Cast<UCurveTable>(AttribSetDefaultsTablePath.TryLoad());
			if (ensureMsgf(AttribTable, TEXT("Could not load Global AttributeSet Defaults Table: %s"), *AttribSetDefaultsTablePath.ToString()))
			{
				GlobalAttributeDefaultsTables.AddUnique(AttribTable);
				if (!AttribTable->OnCurveTableChanged().IsBoundToObject(this))
				{
					AttribTable->OnCurveTableChanged().AddUObject(this, &ThisClass::OnAttributeDefaultTableChanged);
				}
			}
		}
	}
}

void UAbilitySystemGlobals::ReloadAttributeDefaults()
{
	if (!GlobalAttributeDefaultsTables.IsEmpty())
	{
		AllocAttributeSetInitter();
		GetAttributeSetInitter()->PreloadAttributeSetData(GlobalAttributeDefaultsTables);

		// Subscribe for reimports if in the editor
#if WITH_EDITOR
		if (GIsEditor && !RegisteredReimportCallback)
		{
			GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.AddUObject(this, &UAbilitySystemGlobals::OnTableReimported);
			RegisteredReimportCallback = true;
		}
#endif
	}
}

// --------------------------------------------------------------------

UGameplayCueManager* UAbilitySystemGlobals::GetGameplayCueManager()
{
	if (GlobalGameplayCueManager == nullptr)
	{
		const UGameplayAbilitiesDeveloperSettings* DeveloperSettings = GetDefault<UGameplayAbilitiesDeveloperSettings>();
		// Loads mud specific gameplaycue manager object if specified
		if (GlobalGameplayCueManager == nullptr && DeveloperSettings->GlobalGameplayCueManagerName.IsValid())
		{
			GlobalGameplayCueManager = LoadObject<UGameplayCueManager>(nullptr, *DeveloperSettings->GlobalGameplayCueManagerName.ToString(), nullptr, LOAD_None, nullptr);
			if (GlobalGameplayCueManager == nullptr)
			{
				ABILITY_LOG(Error, TEXT("Unable to Load GameplayCueManager %s"), *DeveloperSettings->GlobalGameplayCueManagerName.ToString() );
			}
		}
		
		// Load specific gameplaycue manager class if specified
		if ( GlobalGameplayCueManager == nullptr && DeveloperSettings->GlobalGameplayCueManagerClass.IsValid() )
		{
			UClass* GCMClass = LoadClass<UObject>(nullptr, *DeveloperSettings->GlobalGameplayCueManagerClass.ToString(), nullptr, LOAD_None, nullptr);
			if (GCMClass)
			{
				GlobalGameplayCueManager = NewObject<UGameplayCueManager>(this, GCMClass, NAME_None);
			}
		}
		
		if ( GlobalGameplayCueManager == nullptr)
		{
			// Fallback to base native class
			GlobalGameplayCueManager = NewObject<UGameplayCueManager>(this, UGameplayCueManager::StaticClass(), NAME_None);
		}

		GlobalGameplayCueManager->OnCreated();

		if (GetGameplayCueNotifyPaths().IsEmpty())
		{
			AddGameplayCueNotifyPath(TEXT("/Game"));
			ABILITY_LOG(Warning, TEXT("No GameplayCueNotifyPaths were specified in DefaultGame.ini under [/Script/GameplayAbilities.AbilitySystemGlobals]. Falling back to using all of /Game/. This may be slow on large projects. Consider specifying which paths are to be searched."));
		}
		
		if (GlobalGameplayCueManager->ShouldAsyncLoadObjectLibrariesAtStart())
		{
			StartAsyncLoadingObjectLibraries();
		}
	}

	check(GlobalGameplayCueManager);
	return GlobalGameplayCueManager;
}

UGameplayTagReponseTable* UAbilitySystemGlobals::GetGameplayTagResponseTable()
{
	const UGameplayAbilitiesDeveloperSettings* DeveloperSettings = GetDefault<UGameplayAbilitiesDeveloperSettings>();
	if (GameplayTagResponseTable == nullptr && DeveloperSettings->GameplayTagResponseTableName.IsValid())
	{
		GameplayTagResponseTable = LoadObject<UGameplayTagReponseTable>(nullptr, *DeveloperSettings->GameplayTagResponseTableName.ToString(), nullptr, LOAD_None, nullptr);
	}

	return GameplayTagResponseTable;
}

void UAbilitySystemGlobals::GlobalPreGameplayEffectSpecApply(FGameplayEffectSpec& Spec, UAbilitySystemComponent* AbilitySystemComponent)
{

}

bool UAbilitySystemGlobals::ShouldIgnoreCooldowns() const
{
	return UE::AbilitySystemGlobals::bIgnoreAbilitySystemCooldowns;
}

bool UAbilitySystemGlobals::ShouldIgnoreCosts() const
{
	return UE::AbilitySystemGlobals::bIgnoreAbilitySystemCosts;
}

#if WITH_EDITOR
void UAbilitySystemGlobals::OnPreBeginPIE(const bool bIsSimulatingInEditor)
{
	ResetCachedData();
}
#endif // WITH_EDITOR

void UAbilitySystemGlobals::ResetCachedData()
{
	IGameplayCueInterface::ClearTagToFunctionMap();
}

void UAbilitySystemGlobals::HandlePreLoadMap(const FWorldContext& WorldContext, const FString& MapName)
{
	// We don't want to reset for PIE since this is shared memory (which would have received OnPreBeginPIE).
	if (WorldContext.PIEInstance > 0)
	{
		return;
	}

	// If we are preloading a map but coming from an existing map, then we should wait until the previous map is cleaned up,
	// otherwise we'll end up stomping FActiveGameplayEffectHandle map.
	if (const UWorld* InWorld = WorldContext.World())
	{
		FWorldDelegates::OnPostWorldCleanup.AddWeakLambda(InWorld, [InWorld](UWorld* WorldParam, bool bSessionEnded, bool bCleanupResources)
			{
				if (WorldParam == InWorld)
				{
					ResetCachedData();
				}
			});

		return;
	}

	ResetCachedData();
}

void UAbilitySystemGlobals::Notify_OpenAssetInEditor(FString AssetName, int AssetType)
{
	AbilityOpenAssetInEditorCallbacks.Broadcast(AssetName, AssetType);
}

void UAbilitySystemGlobals::Notify_FindAssetInEditor(FString AssetName, int AssetType)
{
	AbilityFindAssetInEditorCallbacks.Broadcast(AssetName, AssetType);
}

void UAbilitySystemGlobals::NonShipping_ApplyGlobalAbilityScaler_Rate(float& Rate)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	Rate *= UE::AbilitySystemGlobals::AbilitySystemGlobalScaler;
#endif
}

void UAbilitySystemGlobals::NonShipping_ApplyGlobalAbilityScaler_Duration(float& Duration)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (UE::AbilitySystemGlobals::AbilitySystemGlobalScaler > 0.f)
	{
		Duration /= UE::AbilitySystemGlobals::AbilitySystemGlobalScaler;
	}
#endif
}

void FNetSerializeScriptStructCache::InitForType(UScriptStruct* InScriptStruct)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNetSerializeScriptStructCache::InitForType);

	ScriptStructs.Reset();
	BaseStructType = InScriptStruct;

	// Find all script structs of this type and add them to the list
	// (not sure of a better way to do this but it should only happen once at startup)
	for (TObjectIterator<UScriptStruct> It; It; ++It)
	{
		if (It->IsChildOf(InScriptStruct) && IsValid(*It))
		{
			UE_LOGF(LogAbilitySystem, Verbose, "FNetSerializeScriptStructCache::InitForType: found struct type %ls",
				*It->GetName());
			ScriptStructs.Add(*It);
		}
	}
	
	ScriptStructs.Sort([](const UScriptStruct& A, const UScriptStruct& B) { return A.GetName().ToLower() > B.GetName().ToLower(); });

	bIsInitDirty = false;
}

void FNetSerializeScriptStructCache::RemoveInPackages(TConstArrayView<UPackage*> Packages)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNetSerializeScriptStructCache::RemoveInPackages);

	for (auto It = ScriptStructs.CreateIterator(); It; ++It)
	{
		if (Packages.Contains(It->GetPackage()))
		{
			ABILITY_LOG(Verbose, TEXT("FNetSerializeScriptStructCache::RemoveInPackages: removed %s"), *It->GetName());
			It.RemoveCurrent();
		}
	}
}

bool FNetSerializeScriptStructCache::NetSerialize(FArchive& Ar, UScriptStruct*& Struct)
{
	if (bIsInitDirty)
	{
		InitForType(BaseStructType);
	}

	if (Ar.IsSaving())
	{
		int32 idx;
		if (ScriptStructs.Find(Struct, idx))
		{
			check(idx < (1 << 8));
			uint8 b = idx;
			Ar.SerializeBits(&b, 8);
			return true;
		}
		ABILITY_LOG(Error, TEXT("Could not find %s in ScriptStructCache"), *GetNameSafe(Struct));
		return false;
	}
	else
	{
		uint8 b = 0;
		Ar.SerializeBits(&b, 8);
		if (ScriptStructs.IsValidIndex(b))
		{
			Struct = ScriptStructs[b];
			return true;
		}

		ABILITY_LOG(Error, TEXT("Could not find script struct at idx %d"), b);
		return false;
	}
}
