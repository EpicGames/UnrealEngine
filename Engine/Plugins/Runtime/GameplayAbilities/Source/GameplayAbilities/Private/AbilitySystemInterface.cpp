// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbilitySystemInterface.h"
#include "AbilitySystemReplicationProxyInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilitySystemInterface)

UAbilitySystemInterface::UAbilitySystemInterface(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

UAbilitySystemReplicationProxyInterface::UAbilitySystemReplicationProxyInterface(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

