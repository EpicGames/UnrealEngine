// Copyright Epic Games, Inc. All Rights Reserved.
#include "AbilitySystemLog.h"

DEFINE_LOG_CATEGORY(LogAbilitySystem);
DEFINE_LOG_CATEGORY(VLogAbilitySystem);
DEFINE_LOG_CATEGORY(LogGameplayEffects);

