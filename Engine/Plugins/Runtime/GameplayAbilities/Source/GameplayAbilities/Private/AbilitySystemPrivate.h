// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumClassFlags.h"
#include "HAL/IConsoleManager.h"

namespace UE::AbilitySystem::Private
{
	/** These are flags to rollback fixes in live production */
	enum class EAllowPredictiveGEFlags
	{
		None = 0,								// No intentional loopholes, try our best to enforce this rule (preferred, default).
		AllowNativeRemoveByHandle = 1 << 0,		// Some games may be relying on RemoveActiveGameplayEffect having only the UFUNCTION BlueprintAuthority check; it was previously not checked in native.
		AllowRemovalByTagRequirements = 1 << 1,	// A bug introduced in UE5.3 allowed target tag requirements to remove GE's client-side due to the above.
		AllowGameplayEventToApplyGE = 1 << 2	// Legacy bug allowed Gameplay Events to create a prediction window with which one could apply GE's locally. That causes issues since it's not guaranteed to be communicated to the server.
	};
	ENUM_CLASS_FLAGS(EAllowPredictiveGEFlags);

	// CVar that controls which fixes to the predictive GE code we want to disable (for legacy reasons)
	inline int32 CVarAllowPredictiveGEFlagsValue = 0;
	inline FAutoConsoleVariableRef CVarAllowPredictiveGEFlags(TEXT("AbilitySystem.Fix.AllowPredictiveGEFlags"), CVarAllowPredictiveGEFlagsValue, TEXT("Default: 0 (all fixes, no exceptions).\n \
		Use flag 0x1 to allow when removing by Handle in native (legacy).  Use flag 0x2 to allow RequirementsTags to remove (bug introduced in UE5.3).  Use flag 0x4 to allow Gameplay Events to predictively execute GE's (legacy bug)."));
}
