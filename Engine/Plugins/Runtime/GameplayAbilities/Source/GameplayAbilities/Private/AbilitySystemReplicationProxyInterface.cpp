// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbilitySystemReplicationProxyInterface.h"

#include "GameplayPrediction.h"
#include "GameplayEffect.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilitySystemReplicationProxyInterface)


void IAbilitySystemReplicationProxyInterface::Call_InvokeGameplayCueExecuted_FromSpec(const FGameplayEffectSpecForRPC Spec, FPredictionKey PredictionKey)
{
	NetMulticast_InvokeGameplayCueExecuted_FromSpec(Spec, PredictionKey);
}

void IAbilitySystemReplicationProxyInterface::Call_InvokeGameplayCueExecuted(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext)
{
	NetMulticast_InvokeGameplayCueExecuted(GameplayCueTag, PredictionKey, EffectContext);
}

void IAbilitySystemReplicationProxyInterface::Call_InvokeGameplayCuesExecuted(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext)
{
	NetMulticast_InvokeGameplayCuesExecuted(GameplayCueTags, PredictionKey, EffectContext);
}

void IAbilitySystemReplicationProxyInterface::Call_InvokeGameplayCueExecuted_WithParams(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters)
{
	NetMulticast_InvokeGameplayCueExecuted_WithParams(GameplayCueTag, PredictionKey, GameplayCueParameters);
}

void IAbilitySystemReplicationProxyInterface::Call_InvokeGameplayCuesExecuted_WithParams(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters)
{
	NetMulticast_InvokeGameplayCuesExecuted_WithParams(GameplayCueTags, PredictionKey, GameplayCueParameters);
}

void IAbilitySystemReplicationProxyInterface::Call_InvokeGameplayCueAdded(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext)
{
	NetMulticast_InvokeGameplayCueAdded(GameplayCueTag, PredictionKey, EffectContext);
}

void IAbilitySystemReplicationProxyInterface::Call_InvokeGameplayCueAdded_WithParams(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters Parameters)
{
	NetMulticast_InvokeGameplayCueAdded_WithParams(GameplayCueTag, PredictionKey, Parameters);
}

void IAbilitySystemReplicationProxyInterface::Call_InvokeGameplayCueAddedAndWhileActive_FromSpec(const FGameplayEffectSpecForRPC& Spec, FPredictionKey PredictionKey)
{
	NetMulticast_InvokeGameplayCueAddedAndWhileActive_FromSpec(Spec, PredictionKey);
}

void IAbilitySystemReplicationProxyInterface::Call_InvokeGameplayCueAddedAndWhileActive_WithParams(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters)
{
	NetMulticast_InvokeGameplayCueAddedAndWhileActive_WithParams(GameplayCueTag, PredictionKey, GameplayCueParameters);
}

void IAbilitySystemReplicationProxyInterface::Call_InvokeGameplayCuesAddedAndWhileActive_WithParams(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters)
{
	NetMulticast_InvokeGameplayCuesAddedAndWhileActive_WithParams(GameplayCueTags, PredictionKey, GameplayCueParameters);
}
