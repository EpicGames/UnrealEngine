// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbilitySystemStats.h"

DEFINE_STAT(STAT_GameplayEffectsGetOwnedTags);
DEFINE_STAT(STAT_GameplayEffectsHasAllTags);
DEFINE_STAT(STAT_GameplayEffectsHasAnyTag);
DEFINE_STAT(STAT_GameplayEffectsTick);
DEFINE_STAT(STAT_GameplayEffectsCanApplyAttributeModifiers);
DEFINE_STAT(STAT_GameplayEffectsGetActiveEffectsTimeRemaining);
DEFINE_STAT(STAT_GameplayEffectsGetActiveEffectsDuration);
DEFINE_STAT(STAT_GameplayEffectsGetActiveEffectsTimeRemainingAndDuration);
DEFINE_STAT(STAT_GameplayEffectsGetActiveEffects);
DEFINE_STAT(STAT_GameplayEffectsGetAllActiveEffectHandles);
DEFINE_STAT(STAT_GameplayEffectsModifyActiveEffectStartTime);
DEFINE_STAT(STAT_GameplayAbilityGetCooldownTimeRemaining);
DEFINE_STAT(STAT_GameplayAbilityGetCooldownTimeRemainingAndDuration);
DEFINE_STAT(STAT_RemoveActiveGameplayEffect);
DEFINE_STAT(STAT_ApplyGameplayEffectSpec);
DEFINE_STAT(STAT_GetGameplayCueFunction);
DEFINE_STAT(STAT_GetOutgoingSpec);
DEFINE_STAT(STAT_InitAttributeSetDefaults);
DEFINE_STAT(STAT_TickAbilityTasks);
DEFINE_STAT(STAT_FindAbilitySpecFromHandle);
DEFINE_STAT(STAT_AggregatorEvaluate);
DEFINE_STAT(STAT_AggregatorBroadcastOnDirty);
DEFINE_STAT(STAT_HasApplicationImmunityToSpec);
DEFINE_STAT(STAT_HasMatchingGameplayTag);
DEFINE_STAT(STAT_HandleGameplayCueNotifyStatic);
DEFINE_STAT(STAT_HandleGameplayCueNotifyActor);
DEFINE_STAT(STAT_ApplyGameplayEffectToTarget);
DEFINE_STAT(STAT_ExecuteActiveEffectsFrom);
DEFINE_STAT(STAT_OnActiveGameplayEffectAdded);
DEFINE_STAT(STAT_OnActiveGameplayEffectRemoved);
DEFINE_STAT(STAT_GameplayCueInterface_HandleGameplayCue);
DEFINE_STAT(STAT_AbilitySystem_TaskCount);
DEFINE_STAT(STAT_InternalExecuteMod);
DEFINE_STAT(STAT_PostGameplayEffectExecute);
