// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbilitySystemTestAttributeSet.h"
#include "GameplayTagContainer.h"
#include "GameplayEffect.h"
#include "GameplayTagsModule.h"
#include "GameplayEffectExtension.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilitySystemTestAttributeSet)


UAbilitySystemTestAttributeSet::UAbilitySystemTestAttributeSet(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	Health = MaxHealth = 100.f;
	Mana = MaxMana = 100.f;
	
	Damage = 0.f;
	CritChance = 0.f;
	SpellDamage = 0.f;
	PhysicalDamage = 0.f;
	Strength = 0.f;
	StackingAttribute1 = 0.f;
	StackingAttribute2 = 0.f;
	NoStackAttribute = 0.f;
}

bool UAbilitySystemTestAttributeSet::PreGameplayEffectExecute(struct FGameplayEffectModCallbackData &Data)
{
#if 0
	static FProperty *HealthProperty = FindFieldChecked<FProperty>(UAbilitySystemTestAttributeSet::StaticClass(), GET_MEMBER_NAME_CHECKED(UAbilitySystemTestAttributeSet, Health));
	static FProperty *DamageProperty = FindFieldChecked<FProperty>(UAbilitySystemTestAttributeSet::StaticClass(), GET_MEMBER_NAME_CHECKED(UAbilitySystemTestAttributeSet, Damage));

	// In this function, our GameplayEffect mod has been evaluated. We have a magnitude and a Tags collection that we can still modify before it is applied.
	// We also still have the Aggregation data that calculated Data.EvaluatedData. If we really needed to, we could look at this, remove or change things at the aggregator level, and reevaluate ourselves.
	// But that would be considered very advanced/rare.

	FProperty *ModifiedProperty = Data.ModifierSpec.Info.Attribute.GetUProperty();

	// Is Damage about to be applied?
	if (DamageProperty == ModifiedProperty)
	{
		// Can the target dodge this completely?
		if (DodgeChance > 0.f)
		{
			if (FMath::FRand() <= DodgeChance)
			{
				// Dodge!
				Data.EvaluatedData.Magnitude = 0.f;
				Data.EvaluatedData.Tags.AddTag(FGameplayTag::RequestGameplayTag(FName(TEXT("Dodged"))));

				// How dodge is handled will be game dependant. There are a few options I think of:
				// -We still apply 0 damage, but tag it as Dodged. The GameplayCue system could pick up on this and play a visual effect. The combat log could pick up in and print it out too.
				// -We throw out this GameplayEffect right here, and apply another GameplayEffect for 'Dodge' it wouldn't modify an attribute but could trigger gameplay cues, it could serve as a 'cooldown' for dodge
				//		if the game wanted rules like 'you can't dodge more than once every .5 seconds', etc.
			}
		}		
		
		if (Data.EvaluatedData.Magnitude > 0.f)
		{
			// Check the source - does it have Crit?
			const UAbilitySystemTestAttributeSet* SourceAttributes = Data.EffectSpec.EffectContext.GetOriginalInstigatorAbilitySystemComponent()->GetSet<UAbilitySystemTestAttributeSet>();
			if (SourceAttributes && SourceAttributes->CritChance > 0.f)
			{
				if (FMath::FRand() <= SourceAttributes->CritChance)
				{
					// Crit!
					Data.EvaluatedData.Magnitude *= SourceAttributes->CritMultiplier;
					Data.EvaluatedData.Tags.AddTag(FGameplayTag::RequestGameplayTag(FName(TEXT("Damage.Crit"))));
				}
			}

			// Now apply armor reduction
			if (Data.EvaluatedData.Tags.HasTag(FGameplayTag::RequestGameplayTag(FName(TEXT("Damage.Physical")))))
			{
				// This is a trivial/naive implementation of armor. It assumes the rmorDamageReduction is an actual % to reduce physics damage by.
				// Real games would probably use armor rating as an attribute and calculate a % here based on the damage source's level, etc.
				Data.EvaluatedData.Magnitude *= (1.f - ArmorDamageReduction);
				Data.EvaluatedData.Tags.AddTag(FGameplayTag::RequestGameplayTag(FName(TEXT("Damage.Mitigated.Armor"))));
			}
		}

	}

#endif

	return true;
}

void UAbilitySystemTestAttributeSet::PostGameplayEffectExecute(const struct FGameplayEffectModCallbackData &Data)
{
	static FProperty* HealthProperty = FindFieldChecked<FProperty>(UAbilitySystemTestAttributeSet::StaticClass(), GET_MEMBER_NAME_CHECKED(UAbilitySystemTestAttributeSet, Health));
	static FProperty* DamageProperty = FindFieldChecked<FProperty>(UAbilitySystemTestAttributeSet::StaticClass(), GET_MEMBER_NAME_CHECKED(UAbilitySystemTestAttributeSet, Damage));

	FProperty* ModifiedProperty = Data.EvaluatedData.Attribute.GetUProperty();

	// What property was modified?
	if (DamageProperty == ModifiedProperty)
	{
		// Anytime Damage is applied with 'Damage.Fire' tag, there is a chance to apply a burning DOT
		if (Data.EffectSpec.CapturedSourceTags.GetAggregatedTags()->HasTag( FGameplayTag::RequestGameplayTag(FName(TEXT("FireDamage")))))
		{
			// Logic to rand() a burning DOT, if successful, apply DOT GameplayEffect to the target
		}

		// Treat damage as minus health
		Health -= Damage;
		Damage = 0.f;

		// Check for Death?
		//  -This could be defined here or at the actor level.
		//  -Doing it here makes a lot of sense to me, but we have legacy code in ::TakeDamage function, so some games may just want to punt to that pipeline from here.
	}
}


void UAbilitySystemTestAttributeSet::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	DISABLE_ALL_CLASS_REPLICATED_PROPERTIES(UAbilitySystemTestAttributeSet, EFieldIteratorFlags::IncludeSuper);
	
	/*
	DOREPLIFETIME( UAbilitySystemTestAttributeSet, MaxHealth);
	DOREPLIFETIME( UAbilitySystemTestAttributeSet, Health);
	DOREPLIFETIME( UAbilitySystemTestAttributeSet, Mana);
	DOREPLIFETIME( UAbilitySystemTestAttributeSet, MaxMana);

	DOREPLIFETIME( UAbilitySystemTestAttributeSet, SpellDamage);
	DOREPLIFETIME( UAbilitySystemTestAttributeSet, PhysicalDamage);

	DOREPLIFETIME( UAbilitySystemTestAttributeSet, CritChance);
	DOREPLIFETIME( UAbilitySystemTestAttributeSet, CritMultiplier);
	DOREPLIFETIME( UAbilitySystemTestAttributeSet, ArmorDamageReduction);

	DOREPLIFETIME( UAbilitySystemTestAttributeSet, DodgeChance);
	DOREPLIFETIME( UAbilitySystemTestAttributeSet, LifeSteal);

	DOREPLIFETIME( UAbilitySystemTestAttributeSet, Strength);
	*/
}

