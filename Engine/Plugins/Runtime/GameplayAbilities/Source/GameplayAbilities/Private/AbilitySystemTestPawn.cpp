// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbilitySystemTestPawn.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemTestAttributeSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilitySystemTestPawn)

FName  AAbilitySystemTestPawn::AbilitySystemComponentName(TEXT("AbilitySystemComponent0"));

AAbilitySystemTestPawn::AAbilitySystemTestPawn(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	AbilitySystemComponent = CreateDefaultSubobject<UAbilitySystemComponent>(AAbilitySystemTestPawn::AbilitySystemComponentName);
	AbilitySystemComponent->SetIsReplicated(true);

	//DefaultAbilitySet = NULL;
}

void AAbilitySystemTestPawn::PostInitializeComponents()
{	
	static FProperty *DamageProperty = FindFieldChecked<FProperty>(UAbilitySystemTestAttributeSet::StaticClass(), GET_MEMBER_NAME_CHECKED(UAbilitySystemTestAttributeSet, Damage));

	Super::PostInitializeComponents();
	AbilitySystemComponent->InitStats(UAbilitySystemTestAttributeSet::StaticClass(), NULL);

	/*
	if (DefaultAbilitySet != NULL)
	{
		AbilitySystemComponent->InitializeAbilities(DefaultAbilitySet);
	}
	*/
}

UAbilitySystemComponent* AAbilitySystemTestPawn::GetAbilitySystemComponent() const
{
	return FindComponentByClass<UAbilitySystemComponent>();
}


