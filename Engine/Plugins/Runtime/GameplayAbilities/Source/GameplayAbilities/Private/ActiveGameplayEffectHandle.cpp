// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActiveGameplayEffectHandle.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActiveGameplayEffectHandle)

namespace
{
	static int32 GetNextActiveGEHandleID()
	{
		static int32 GHandleID = 0;
		int32 NewHandleID = ++GHandleID;
		if (GHandleID < 1)
		{
			GHandleID = 1;
		}

		return NewHandleID;
	}
}

FActiveGameplayEffectHandle::FActiveGameplayEffectHandle(UAbilitySystemComponent* InOwningASC)
	: WeakOwningASC(InOwningASC)
	, Handle(GetNextActiveGEHandleID())
	, bPassedFiltersAndWasExecuted(true)
{
}

FActiveGameplayEffectHandle FActiveGameplayEffectHandle::GetInstantExecutedHandle()
{
	FActiveGameplayEffectHandle InstantGEHandle;
	InstantGEHandle.bPassedFiltersAndWasExecuted = true;

	return InstantGEHandle;
}

FActiveGameplayEffectHandle FActiveGameplayEffectHandle::GenerateNewHandle(UAbilitySystemComponent* OwningComponent)
{
	return FActiveGameplayEffectHandle(OwningComponent);
}

UAbilitySystemComponent* FActiveGameplayEffectHandle::GetOwningAbilitySystemComponent() const
{
	if (UAbilitySystemComponent* OwningASC = WeakOwningASC.Get())
	{
		// Check if this ActiveGE is still valid.  Note: This will fail if PendingRemove is true
		if (const FActiveGameplayEffect* ActiveGE = OwningASC->GetActiveGameplayEffect(*this))
		{
			return OwningASC;
		}
	}

	return nullptr;
}
