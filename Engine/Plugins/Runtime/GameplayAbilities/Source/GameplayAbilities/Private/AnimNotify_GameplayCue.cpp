// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNotify_GameplayCue.h"
#include "Components/SkeletalMeshComponent.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"
#include "Animation/AnimMontage.h"
#include "GameplayCueManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotify_GameplayCue)


typedef void (*GameplayCueFunc)(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters);


static void ProcessGameplayCue(GameplayCueFunc Func, USkeletalMeshComponent* MeshComp, FGameplayTag GameplayCue, UAnimSequenceBase* Animation)
{
	if (MeshComp && GameplayCue.IsValid())
	{
		AActor* OwnerActor = MeshComp->GetOwner();

#if WITH_EDITOR
		if (GIsEditor && (OwnerActor == nullptr))
		{
			// Make preview work in anim preview window
			UGameplayCueManager::PreviewComponent = MeshComp;
			UGameplayCueManager::PreviewWorld = MeshComp->GetWorld();

			if (UGameplayCueManager* GCM = UAbilitySystemGlobals::Get().GetGameplayCueManager())
			{
				GCM->LoadNotifyForEditorPreview(GameplayCue);
			}
		}
#endif // #if WITH_EDITOR

		FGameplayCueParameters Parameters;
		Parameters.Instigator = OwnerActor;

		// Try to get the ability level. This may not be able to find the ability level
		// in cases where a blend out is happening or two abilities are trying to play animations
		if (UAbilitySystemComponent* ASC = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(OwnerActor))
		{
			if (ASC->GetCurrentMontage() == Animation)
			{
				if (const UGameplayAbility* Ability = ASC->GetAnimatingAbility())
				{
					Parameters.AbilityLevel = Ability->GetAbilityLevel();
				}
			}

			// Always use ASC's owner for instigator
			Parameters.Instigator = ASC->GetOwner();
		}

		Parameters.TargetAttachComponent = MeshComp;

		(*Func)(OwnerActor, GameplayCue, Parameters);
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		UGameplayCueManager::PreviewComponent = nullptr;
		UGameplayCueManager::PreviewWorld = nullptr;
	}
#endif // #if WITH_EDITOR
}


//////////////////////////////////////////////////////////////////////////
// UAnimNotify_GameplayCue
//////////////////////////////////////////////////////////////////////////
UAnimNotify_GameplayCue::UAnimNotify_GameplayCue()
{
}

void UAnimNotify_GameplayCue::Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference)
{
	Super::Notify(MeshComp, Animation, EventReference);

	ProcessGameplayCue(&UGameplayCueManager::ExecuteGameplayCue_NonReplicated, MeshComp, GameplayCue.GameplayCueTag, Animation);
}

FString UAnimNotify_GameplayCue::GetNotifyName_Implementation() const
{
	FString DisplayName = TEXT("GameplayCue");

	if (GameplayCue.GameplayCueTag.IsValid())
	{
		DisplayName = GameplayCue.GameplayCueTag.ToString();
		DisplayName += TEXT(" (Burst)");
	}

	return DisplayName;
}

#if WITH_EDITOR
bool UAnimNotify_GameplayCue::CanBePlaced(UAnimSequenceBase* Animation) const
{
	return (Animation && Animation->IsA(UAnimMontage::StaticClass()));
}
#endif // WITH_EDITOR


//////////////////////////////////////////////////////////////////////////
// UAnimNotify_GameplayCueState
//////////////////////////////////////////////////////////////////////////
UAnimNotify_GameplayCueState::UAnimNotify_GameplayCueState()
{
}

void UAnimNotify_GameplayCueState::NotifyBegin(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference)
{
	Super::NotifyBegin(MeshComp, Animation, TotalDuration, EventReference);

	ProcessGameplayCue(&UGameplayCueManager::AddGameplayCue_NonReplicated, MeshComp, GameplayCue.GameplayCueTag, Animation);

#if WITH_EDITORONLY_DATA
	// Grab proxy tick delegate if someone registered it.
	if (UGameplayCueManager::PreviewProxyTick.IsBound())
	{
		PreviewProxyTick = UGameplayCueManager::PreviewProxyTick;
		UGameplayCueManager::PreviewProxyTick.Unbind();
	}
#endif // #if WITH_EDITORONLY_DATA
}

void UAnimNotify_GameplayCueState::NotifyTick(USkeletalMeshComponent * MeshComp, UAnimSequenceBase * Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference)
{
	AActor* OwnerActor = MeshComp->GetOwner();

#if WITH_EDITOR
	if (GIsEditor && OwnerActor == nullptr)
	{
		// Make preview work in anim preview window.
		UGameplayCueManager::PreviewComponent = MeshComp;
		UGameplayCueManager::PreviewWorld = MeshComp->GetWorld();

#if WITH_EDITORONLY_DATA
		PreviewProxyTick.ExecuteIfBound(FrameDeltaTime);
#endif // #if WITH_EDITORONLY_DATA
	}
#endif // #if WITH_EDITOR

	Super::NotifyTick(MeshComp, Animation, FrameDeltaTime, EventReference);

#if WITH_EDITOR
	if (GIsEditor)
	{
		UGameplayCueManager::PreviewComponent = nullptr;
		UGameplayCueManager::PreviewWorld = nullptr;
	}
#endif // #if WITH_EDITOR
}

FString UAnimNotify_GameplayCueState::GetNotifyName_Implementation() const
{
	FString DisplayName = TEXT("GameplayCue");

	if (GameplayCue.GameplayCueTag.IsValid())
	{
		DisplayName = GameplayCue.GameplayCueTag.ToString();
		DisplayName += TEXT(" (Looping)");
	}

	return DisplayName;
}

void UAnimNotify_GameplayCueState::NotifyEnd(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference)
{
	Super::NotifyEnd(MeshComp, Animation, EventReference);

	ProcessGameplayCue(&UGameplayCueManager::RemoveGameplayCue_NonReplicated, MeshComp, GameplayCue.GameplayCueTag, Animation);
}

#if WITH_EDITOR
bool UAnimNotify_GameplayCueState::CanBePlaced(UAnimSequenceBase* Animation) const
{
	return (Animation && Animation->IsA(UAnimMontage::StaticClass()));
}
#endif // #if WITH_EDITOR
