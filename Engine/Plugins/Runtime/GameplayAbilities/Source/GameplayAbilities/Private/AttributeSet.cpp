// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttributeSet.h"
#include "Stats/StatsMisc.h"
#include "EngineDefines.h"
#include "Engine/ObjectLibrary.h"
#include "Engine/World.h"
#include "VisualLogger/VisualLogger.h"
#include "AbilitySystemLog.h"
#include "GameplayEffectAggregator.h"
#include "AbilitySystemStats.h"
#include "UObject/UObjectIterator.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemTestAttributeSet.h"
#include "Net/Core/PushModel/PushModel.h"
#include "UObject/UObjectThreadContext.h"

#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AttributeSet)

#if ENABLE_VISUAL_LOG
namespace
{
	int32 bDoAttributeGraphVLogging = 1;
	FAutoConsoleVariableRef CVarDoAttributeGraphVLogging(TEXT("g.debug.vlog.AttributeGraph")
		, bDoAttributeGraphVLogging, TEXT("Controlls whether Attribute changes are being recorded by VisLog"), ECVF_Cheat);
}
#endif

float FGameplayAttributeData::GetCurrentValue() const
{
	return CurrentValue;
}

void FGameplayAttributeData::SetCurrentValue(float NewValue)
{
	CurrentValue = NewValue;
}

float FGameplayAttributeData::GetBaseValue() const
{
	return BaseValue;
}

void FGameplayAttributeData::SetBaseValue(float NewValue)
{
	BaseValue = NewValue;
}


FGameplayAttribute::FGameplayAttribute(FProperty *NewProperty)
{
	if (FGameplayAttribute::IsSupportedProperty(NewProperty))
	{
		Attribute = NewProperty;
		AttributeOwner = Attribute->GetOwnerStruct();
		Attribute->GetName(AttributeName);
	}
	else
	{
		// Non-null property passed in that's incompatible.
		ensureMsgf(!NewProperty, TEXT("Tried to construct FGameplayAttribute with invalid property '%s' of type '%s'. Treating as invalid attribute."), *NewProperty->GetName(), *NewProperty->GetClass()->GetName());

		Attribute = nullptr;
		AttributeOwner = nullptr;
		AttributeName = "";
	}
}

void FGameplayAttribute::SetNumericValueChecked(float& NewValue, class UAttributeSet* Dest) const
{
	check(Dest);

	FNumericProperty* NumericProperty = CastField<FNumericProperty>(Attribute.Get());
	float OldValue = 0.f;
	if (NumericProperty)
	{
		void* ValuePtr = NumericProperty->ContainerPtrToValuePtr<void>(Dest);
		OldValue = *static_cast<float*>(ValuePtr);
		Dest->PreAttributeChange(*this, NewValue);
		NumericProperty->SetFloatingPointPropertyValue(ValuePtr, NewValue);
		Dest->PostAttributeChange(*this, OldValue, NewValue);

		MARK_PROPERTY_DIRTY(Dest, NumericProperty);
	}
	else if (IsGameplayAttributeDataProperty(Attribute.Get()))
	{
		FStructProperty* StructProperty = CastField<FStructProperty>(Attribute.Get());
		check(StructProperty);
		FGameplayAttributeData* DataPtr = StructProperty->ContainerPtrToValuePtr<FGameplayAttributeData>(Dest);
		check(DataPtr);
		OldValue = DataPtr->GetCurrentValue();
		Dest->PreAttributeChange(*this, NewValue);
		DataPtr->SetCurrentValue(NewValue);
		Dest->PostAttributeChange(*this, OldValue, DataPtr->GetCurrentValue());

		MARK_PROPERTY_DIRTY(Dest, StructProperty);
	}
	else
	{
		check(false);
	}

#if ENABLE_VISUAL_LOG
	// draw a graph of the changes to the attribute in the visual logger
	if (bDoAttributeGraphVLogging && FVisualLogger::IsRecording())
	{
		AActor* OwnerActor = Dest->GetOwningActor();
		if (OwnerActor)
		{
			ABILITY_VLOG_ATTRIBUTE_GRAPH(OwnerActor, Log, GetName(), OldValue, NewValue);
		}
	}
#endif
}

float FGameplayAttribute::GetNumericValue(const UAttributeSet* Src) const
{
	const FNumericProperty* const NumericProperty = CastField<FNumericProperty>(Attribute.Get());
	if (NumericProperty)
	{
		const void* ValuePtr = NumericProperty->ContainerPtrToValuePtr<void>(Src);
		return NumericProperty->GetFloatingPointPropertyValue(ValuePtr);
	}
	else if (IsGameplayAttributeDataProperty(Attribute.Get()))
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(Attribute.Get());
		check(StructProperty);
		const FGameplayAttributeData* DataPtr = StructProperty->ContainerPtrToValuePtr<FGameplayAttributeData>(Src);
		if (ensure(DataPtr))
		{
			return DataPtr->GetCurrentValue();
		}
	}

	return 0.f;
}

float FGameplayAttribute::GetNumericValueChecked(const UAttributeSet* Src) const
{
	FNumericProperty* NumericProperty = CastField<FNumericProperty>(Attribute.Get());
	if (NumericProperty)
	{
		const void* ValuePtr = NumericProperty->ContainerPtrToValuePtr<void>(Src);
		return NumericProperty->GetFloatingPointPropertyValue(ValuePtr);
	}
	else if (IsGameplayAttributeDataProperty(Attribute.Get()))
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(Attribute.Get());
		check(StructProperty);
		const FGameplayAttributeData* DataPtr = StructProperty->ContainerPtrToValuePtr<FGameplayAttributeData>(Src);
		if (ensure(DataPtr))
		{
			return DataPtr->GetCurrentValue();
		}
	}

	check(false);
	return 0.f;
}

const FGameplayAttributeData* FGameplayAttribute::GetGameplayAttributeData(const UAttributeSet* Src) const
{
	if (Src && IsGameplayAttributeDataProperty(Attribute.Get()))
	{
		FStructProperty* StructProperty = CastField<FStructProperty>(Attribute.Get());
		check(StructProperty);
		MARK_PROPERTY_DIRTY(Src, StructProperty);
		return StructProperty->ContainerPtrToValuePtr<FGameplayAttributeData>(Src);
	}

	return nullptr;
}

const FGameplayAttributeData* FGameplayAttribute::GetGameplayAttributeDataChecked(const UAttributeSet* Src) const
{
	if (Src && IsGameplayAttributeDataProperty(Attribute.Get()))
	{
		FStructProperty* StructProperty = CastField<FStructProperty>(Attribute.Get());
		check(StructProperty);
		MARK_PROPERTY_DIRTY(Src, StructProperty);
		return StructProperty->ContainerPtrToValuePtr<FGameplayAttributeData>(Src);
	}

	check(false);
	return nullptr;
}

FGameplayAttributeData* FGameplayAttribute::GetGameplayAttributeData(UAttributeSet* Src) const
{
	if (Src && IsGameplayAttributeDataProperty(Attribute.Get()))
	{
		FStructProperty* StructProperty = CastField<FStructProperty>(Attribute.Get());
		check(StructProperty);
		MARK_PROPERTY_DIRTY(Src, StructProperty);
		return StructProperty->ContainerPtrToValuePtr<FGameplayAttributeData>(Src);
	}

	return nullptr;
}

FGameplayAttributeData* FGameplayAttribute::GetGameplayAttributeDataChecked(UAttributeSet* Src) const
{
	if (Src && IsGameplayAttributeDataProperty(Attribute.Get()))
	{
		FStructProperty* StructProperty = CastField<FStructProperty>(Attribute.Get());
		check(StructProperty);
		MARK_PROPERTY_DIRTY(Src, StructProperty);
		return StructProperty->ContainerPtrToValuePtr<FGameplayAttributeData>(Src);
	}

	check(false);
	return nullptr;
}

bool FGameplayAttribute::IsSystemAttribute() const
{
	return GetAttributeSetClass()->IsChildOf(UAbilitySystemComponent::StaticClass());
}

bool FGameplayAttribute::IsSupportedProperty(const FProperty* Property)
{
	// FGameplayAttributeData is preferred. Floating point properties are supported for legacy reasons.
	// @todo: Deprecate numeric properties and only allow FGameplayAttributeData.
	return Property && (FGameplayAttribute::IsGameplayAttributeDataProperty(Property) || (Property->IsA<FNumericProperty>() && CastField<FNumericProperty>(Property)->IsFloatingPoint()));
}

bool FGameplayAttribute::IsGameplayAttributeDataProperty(const FProperty* Property)
{
	const FStructProperty* StructProp = CastField<FStructProperty>(Property);
	if (StructProp)
	{
		const UStruct* Struct = StructProp->Struct;
		if (Struct && Struct->IsChildOf(FGameplayAttributeData::StaticStruct()))
		{
			return true;
		}
	}

	return false;
}

#if WITH_EDITORONLY_DATA
// Fill in missing attribute information
void FGameplayAttribute::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && Ar.IsPersistent() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE))
	{
		// Once struct is loaded, check if redirectors apply to the imported attribute field path
		const FString PathName = Attribute.ToString();
		const FString RedirectedPathName = FFieldPathProperty::RedirectFieldPathName(PathName);
		if (!RedirectedPathName.Equals(PathName))
		{
			// If the path got redirected, attempt to resolve the new property
			FString NewAttributeOwner;
			FString NewAttributeName;
			if (RedirectedPathName.Split(":", &NewAttributeOwner, &NewAttributeName))
			{
				// Update attribute's field path (may or may not resolve)
				const UStruct* NewClass = FindObject<UStruct>(nullptr, *NewAttributeOwner);
				Attribute = FindFProperty<FProperty>(NewClass, *NewAttributeName);

				// Verbose log any applied redirectors
				FUObjectSerializeContext* LoadContext = FUObjectThreadContext::Get().GetSerializeContext();
				const FString AssetName = (LoadContext && LoadContext->SerializedObject) ? LoadContext->SerializedObject->GetPathName() : TEXT("Unknown Object");
				ABILITY_LOG(Verbose, TEXT("FGameplayAttribute::PostSerialize redirected an attribute '%s' -> '%s'. (Asset: %s)"), *PathName, *RedirectedPathName, *AssetName);
			}
		}

		// The attribute reference is serialized in two ways:
		// - 'Attribute' contains the full path, ex: /Script/GameModule.GameAttributeSet:AttrName
		// - 'AttributeOwner' and 'AttributeName' are cached references to the attribute set UClass and the attribute's name
		// We want the data to stay in sync, with 'Attribute' having priority as source of truth. In both cases, derive one from
		// the other to keep them in sync.
		if (Attribute.Get())
		{
			// Ensure owner and name are in sync with field path
			AttributeOwner = Attribute->GetOwnerStruct();
			Attribute->GetName(AttributeName);
		}
		else if (!AttributeName.IsEmpty())
		{
			if (AttributeOwner != nullptr)
			{
				// Attempt to resolve field path from owner and attribute name
				Attribute = FindFProperty<FProperty>(AttributeOwner, *AttributeName);
			}

			// Log warning if attribute failed to resolve
			if (!Attribute.Get())
			{
				FUObjectSerializeContext* LoadContext = FUObjectThreadContext::Get().GetSerializeContext();
				const FString AssetName = (LoadContext && LoadContext->SerializedObject) ? LoadContext->SerializedObject->GetPathName() : TEXT("Unknown Object");
				const FString OwnerName = AttributeOwner ? AttributeOwner->GetName() : TEXT("NONE");
				ABILITY_LOG(Warning, TEXT("FGameplayAttribute::PostSerialize called on an invalid attribute with owner %s and name %s. (Asset: %s)"), *OwnerName, *AttributeName, *AssetName);
			}
		}
	}
	if (Ar.IsSaving() && IsValid())
	{
		// This marks the attribute "address" for later searching
		Ar.MarkSearchableName(FGameplayAttribute::StaticStruct(), FName(FString::Printf(TEXT("%s.%s"), *GetUProperty()->GetOwnerVariant().GetName(), *GetUProperty()->GetName())));
	}
}
#endif

void FGameplayAttribute::GetAllAttributeProperties(TArray<FProperty*>& OutProperties, FString FilterMetaStr, bool UseEditorOnlyData)
{
	// Gather all UAttribute classes
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass *Class = *ClassIt;
		if (Class->IsChildOf(UAttributeSet::StaticClass()) 
#if WITH_EDITORONLY_DATA
			// ClassGeneratedBy TODO: This is wrong in cooked builds
			&& !Class->ClassGeneratedBy
#endif
			)
		{
			if (UseEditorOnlyData)
			{
				#if WITH_EDITOR
				// Allow entire classes to be filtered globally
				if (Class->HasMetaData(TEXT("HideInDetailsView")))
				{
					continue;
				}
				#endif
			}

			if (Class == UAbilitySystemTestAttributeSet::StaticClass())
			{
				continue;
			}


			for (TFieldIterator<FProperty> PropertyIt(Class, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
			{
				FProperty* Property = *PropertyIt;

				if (UseEditorOnlyData)
				{
					#if WITH_EDITOR
					if (!FilterMetaStr.IsEmpty() && Property->HasMetaData(*FilterMetaStr))
					{
						continue;
					}

					// Allow properties to be filtered globally (never show up)
					if (Property->HasMetaData(TEXT("HideInDetailsView")))
					{
						continue;
					}
					#endif
				}
				
				OutProperties.Add(Property);
			}
		}

#if WITH_EDITOR
		if (UseEditorOnlyData)
		{
			// UAbilitySystemComponent can add 'system' attributes
			if (Class->IsChildOf(UAbilitySystemComponent::StaticClass()) && !Class->ClassGeneratedBy)
			{
				for (TFieldIterator<FProperty> PropertyIt(Class, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
				{
					FProperty* Property = *PropertyIt;


					// SystemAttributes have to be explicitly tagged
					if (Property->HasMetaData(TEXT("SystemGameplayAttribute")) == false)
					{
						continue;
					}
					OutProperties.Add(Property);
				}
			}
		}
#endif
	}
}

UAttributeSet::UAttributeSet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

bool UAttributeSet::IsNameStableForNetworking() const
{
	/** 
	 * IsNameStableForNetworking means an attribute set can be referred to its path name (relative to owning AActor*) over the network
	 *
	 * Attribute sets are net addressable if:
	 *	-They are Default Subobjects (created in C++ constructor)
	 *	-They were loaded directly from a package (placed in map actors)
	 *	-They were explicitly set to bNetAddressable
	 */

	return bNetAddressable || Super::IsNameStableForNetworking();
}

bool UAttributeSet::IsSupportedForNetworking() const
{
	return true;
}

void UAttributeSet::GetAttributesFromSetClass(const TSubclassOf<UAttributeSet>& AttributeSetClass, TArray<FGameplayAttribute>& Attributes)
{
	for (TFieldIterator<FProperty> It(AttributeSetClass); It; ++It)
	{
		if (FGameplayAttribute::IsSupportedProperty(*It))
		{
			Attributes.Add(FGameplayAttribute(*It));
		}
	}
}

void UAttributeSet::SetNetAddressable()
{
	bNetAddressable = true;
}

void UAttributeSet::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	using namespace UE::Net;

	// Build descriptors and allocate PropertyReplicationFragments for this object
	FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UAttributeSet::InitFromMetaDataTable(const UDataTable* DataTable)
{
	static const FString Context = FString(TEXT("UAttribute::BindToMetaDataTable"));

	for( TFieldIterator<FProperty> It(GetClass(), EFieldIteratorFlags::IncludeSuper) ; It ; ++It )
	{
		FProperty* Property = *It;

		// Only process properties that can back gameplay attributes. They will be either FGameplayAttributeData 
		// or a floating-point numeric property (floats, doubles, potentially user custom).
		if (FGameplayAttribute::IsSupportedProperty(Property))
		{
			FString RowNameStr = FString::Printf(TEXT("%s.%s"), *Property->GetOwnerVariant().GetName(), *Property->GetName());
			if (FAttributeMetaData* MetaData = DataTable->FindRow<FAttributeMetaData>(FName(*RowNameStr), Context, false))
			{
				FNumericProperty* NumericProperty = CastField<FNumericProperty>(Property);
				if (NumericProperty)
				{
					// Passing FGameplayAttribute::IsSupportedProperty() as numeric property already implies it's floating point
					check(NumericProperty->IsFloatingPoint());
					void* Data = NumericProperty->ContainerPtrToValuePtr<void>(this);
					NumericProperty->SetFloatingPointPropertyValue(Data, MetaData->BaseValue);
				}
				else if (FGameplayAttribute::IsGameplayAttributeDataProperty(Property))
				{
					FStructProperty* StructProperty = CastField<FStructProperty>(Property);
					check(StructProperty);
					FGameplayAttributeData* DataPtr = StructProperty->ContainerPtrToValuePtr<FGameplayAttributeData>(this);
					check(DataPtr);
					DataPtr->SetBaseValue(MetaData->BaseValue);
					DataPtr->SetCurrentValue(MetaData->BaseValue);
				}
			}
			
		}
	}

	PrintDebug();
}

AActor* UAttributeSet::GetOwningActor() const
{
	return CastChecked<AActor>(GetOuter());
}

UAbilitySystemComponent* UAttributeSet::GetOwningAbilitySystemComponent() const
{
	return UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(GetOwningActor());
}

UAbilitySystemComponent* UAttributeSet::GetOwningAbilitySystemComponentChecked() const
{
	UAbilitySystemComponent* Result = GetOwningAbilitySystemComponent();
	check(Result);
	return Result;
}

FGameplayAbilityActorInfo* UAttributeSet::GetActorInfo() const
{
	UAbilitySystemComponent* ASC = GetOwningAbilitySystemComponent();
	if (ASC)
	{
		return ASC->AbilityActorInfo.Get();
	}

	return nullptr;
}

void UAttributeSet::PrintDebug()
{
	
}

void UAttributeSet::PreNetReceive()
{
	// During the scope of this entire actor's network update, we need to lock our attribute aggregators.
	FScopedAggregatorOnDirtyBatch::BeginNetReceiveLock();
}
	
void UAttributeSet::PostNetReceive()
{
	// Once we are done receiving properties, we can unlock the attribute aggregators and flag them that the 
	// current property values are from the server.
	FScopedAggregatorOnDirtyBatch::EndNetReceiveLock();
}

FAttributeMetaData::FAttributeMetaData()
	: BaseValue(0.0f)
	, MinValue(0.f)
	, MaxValue(1.f)
	, bCanStack(false)
{

}

bool FGameplayAttribute::operator==(const FGameplayAttribute& Other) const
{
	return ((Other.Attribute == Attribute));
}

bool FGameplayAttribute::operator!=(const FGameplayAttribute& Other) const
{
	return ((Other.Attribute != Attribute));
}

// ------------------------------------------------------------------------------------
//
// ------------------------------------------------------------------------------------
TSubclassOf<UAttributeSet> FindBestAttributeClass(TArray<TSubclassOf<UAttributeSet> >& ClassList, FString PartialName)
{
	for (auto Class : ClassList)
	{
		if (Class->GetName().Contains(PartialName))
		{
			return Class;
		}
	}

	return nullptr;
}

/**
 *	Transforms CurveTable data into format more efficient to read at runtime.
 *	UCurveTable requires string parsing to map to GroupName/AttributeSet/Attribute
 *	Each curve in the table represents a *single attribute's values for all levels*.
 *	At runtime, we want *all attribute values at given level*.
 *
 *	This code assumes that your curve data starts with a key of 1.
 */
void FAttributeSetInitterDiscreteLevels::PreloadAttributeSetData(const TArray<UCurveTable*>& CurveData)
{
	if (!ensure(CurveData.Num() > 0))
	{
		return;
	}

	/**
	 *	Get list of AttributeSet classes loaded
	 */

	TArray<TSubclassOf<UAttributeSet> >	ClassList;
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* TestClass = *ClassIt;
		if (TestClass->IsChildOf(UAttributeSet::StaticClass()))
		{
			ClassList.Add(TestClass);
		}
	}

	/**
	 *	Loop through CurveData table and build sets of Defaults that keyed off of Name + Level
	 */
	for (const UCurveTable* CurTable : CurveData)
	{
		for (const TPair<FName, FRealCurve*>& CurveRow : CurTable->GetRowMap())
		{
			FString RowName = CurveRow.Key.ToString();
			FString ClassName;
			FString SetName;
			FString AttributeName;
			FString Temp;

			RowName.Split(TEXT("."), &ClassName, &Temp);
			Temp.Split(TEXT("."), &SetName, &AttributeName);

			if (!ensure(!ClassName.IsEmpty() && !SetName.IsEmpty() && !AttributeName.IsEmpty()))
			{
				ABILITY_LOG(Verbose, TEXT("FAttributeSetInitterDiscreteLevels::PreloadAttributeSetData Unable to parse row %s in %s"), *RowName, *CurTable->GetName());
				continue;
			}

			// Find the AttributeSet

			TSubclassOf<UAttributeSet> Set = FindBestAttributeClass(ClassList, SetName);
			if (!Set)
			{
				// This is ok, we may have rows in here that don't correspond directly to attributes
				ABILITY_LOG(Verbose, TEXT("FAttributeSetInitterDiscreteLevels::PreloadAttributeSetData Unable to match AttributeSet from %s (row: %s)"), *SetName, *RowName);
				continue;
			}

			// Find the FProperty
			FProperty* Property = FindFProperty<FProperty>(*Set, *AttributeName);
			if (!Property)
			{
				ABILITY_LOG(Verbose, TEXT("FAttributeSetInitterDiscreteLevels::PreloadAttributeSetData Unable to match Attribute from %s (row: %s): Property not found"), *AttributeName, *RowName);
				continue;
			}
			else if (!FGameplayAttribute::IsSupportedProperty(Property))
			{
				ABILITY_LOG(Verbose, TEXT("FAttributeSetInitterDiscreteLevels::PreloadAttributeSetData Unable to match Attribute from %s (row: %s): Property type '%s' incompatible. Must be FGameplayAttributeData or a floating point type."), *AttributeName, *RowName, *Property->GetClass()->GetName());
				continue;
			}

			FRealCurve* Curve = CurveRow.Value;
			FName ClassFName = FName(*ClassName);
			FAttributeSetDefaultsCollection& DefaultCollection = Defaults.FindOrAdd(ClassFName);

			float FirstLevelFloat = 0.f;
			float LastLevelFloat = 0.f;
			Curve->GetTimeRange(FirstLevelFloat, LastLevelFloat);

			int32 FirstLevel = FMath::RoundToInt32(FirstLevelFloat);
			int32 LastLevel = FMath::RoundToInt32(LastLevelFloat);

			// Only log these as warnings, as they're not deal breakers.
			if (FirstLevel != 1)
			{
				ABILITY_LOG(Warning, TEXT("FAttributeSetInitterDiscreteLevels::PreloadAttributeSetData First level should be 1"));
				continue;
			}

			DefaultCollection.LevelData.SetNum(FMath::Max(LastLevel, DefaultCollection.LevelData.Num()));

			for (int32 Level = 1; Level <= LastLevel; ++Level)
			{
				float Value = Curve->Eval(float(Level));

				FAttributeSetDefaults& SetDefaults = DefaultCollection.LevelData[Level-1];

				FAttributeDefaultValueList* DefaultDataList = SetDefaults.DataMap.Find(Set);
				if (DefaultDataList == nullptr)
				{
					ABILITY_LOG(Verbose, TEXT("Initializing new default set for %s[%d]. PropertySize: %d.. DefaultSize: %d"), *Set->GetName(), Level, Set->GetPropertiesSize(), UAttributeSet::StaticClass()->GetPropertiesSize());

					DefaultDataList = &SetDefaults.DataMap.Add(Set);
				}

				// Import curve value into default data

				check(DefaultDataList);
				DefaultDataList->AddPair(Property, Value);
			}
		}
	}
}

void FAttributeSetInitterDiscreteLevels::InitAttributeSetDefaults(UAbilitySystemComponent* AbilitySystemComponent, FName GroupName, int32 Level, bool bInitialInit) const
{
	SCOPE_CYCLE_COUNTER(STAT_InitAttributeSetDefaults);
	check(AbilitySystemComponent != nullptr);
	
	const FAttributeSetDefaultsCollection* Collection = Defaults.Find(GroupName);
	if (!Collection)
	{
		ABILITY_LOG(Warning, TEXT("Unable to find DefaultAttributeSet Group %s. Falling back to Defaults"), *GroupName.ToString());
		Collection = Defaults.Find(FName(TEXT("Default")));
		if (!Collection)
		{
			ABILITY_LOG(Error, TEXT("FAttributeSetInitterDiscreteLevels::InitAttributeSetDefaults Default DefaultAttributeSet not found! Skipping Initialization"));
			return;
		}
	}

	if (!Collection->LevelData.IsValidIndex(Level - 1))
	{
		// We could eventually extrapolate values outside of the max defined levels
		ABILITY_LOG(Warning, TEXT("Attribute defaults for Level %d are not defined! Skipping"), Level);
		return;
	}

	const FAttributeSetDefaults& SetDefaults = Collection->LevelData[Level - 1];
	for (const UAttributeSet* Set : AbilitySystemComponent->GetSpawnedAttributes())
	{
		if (!Set)
		{
			continue;
		}
		const FAttributeDefaultValueList* DefaultDataList = SetDefaults.DataMap.Find(Set->GetClass());
		if (DefaultDataList)
		{
			ABILITY_LOG(Log, TEXT("Initializing Set %s"), *Set->GetName());

			for (auto& DataPair : DefaultDataList->List)
			{
				check(DataPair.Property);

				if (Set->ShouldInitProperty(bInitialInit, DataPair.Property))
				{
					FGameplayAttribute AttributeToModify(DataPair.Property);
					AbilitySystemComponent->SetNumericAttributeBase(AttributeToModify, DataPair.Value);
				}
			}
		}		
	}
	
	AbilitySystemComponent->ForceReplication();
}

void FAttributeSetInitterDiscreteLevels::ApplyAttributeDefault(UAbilitySystemComponent* AbilitySystemComponent, FGameplayAttribute& InAttribute, FName GroupName, int32 Level) const
{
	SCOPE_CYCLE_COUNTER(STAT_InitAttributeSetDefaults);

	const FAttributeSetDefaultsCollection* Collection = Defaults.Find(GroupName);
	if (!Collection)
	{
		ABILITY_LOG(Warning, TEXT("Unable to find DefaultAttributeSet Group %s. Falling back to Defaults"), *GroupName.ToString());
		Collection = Defaults.Find(FName(TEXT("Default")));
		if (!Collection)
		{
			ABILITY_LOG(Error, TEXT("FAttributeSetInitterDiscreteLevels::InitAttributeSetDefaults Default DefaultAttributeSet not found! Skipping Initialization"));
			return;
		}
	}

	if (!Collection->LevelData.IsValidIndex(Level - 1))
	{
		// We could eventually extrapolate values outside of the max defined levels
		ABILITY_LOG(Warning, TEXT("Attribute defaults for Level %d are not defined! Skipping"), Level);
		return;
	}

	const FAttributeSetDefaults& SetDefaults = Collection->LevelData[Level - 1];
	for (const UAttributeSet* Set : AbilitySystemComponent->GetSpawnedAttributes())
	{
		if (!Set)
		{
			continue;
		}

		const FAttributeDefaultValueList* DefaultDataList = SetDefaults.DataMap.Find(Set->GetClass());
		if (DefaultDataList)
		{
			ABILITY_LOG(Log, TEXT("Initializing Set %s"), *Set->GetName());

			for (auto& DataPair : DefaultDataList->List)
			{
				check(DataPair.Property);

				if (DataPair.Property == InAttribute.GetUProperty())
				{
					FGameplayAttribute AttributeToModify(DataPair.Property);
					AbilitySystemComponent->SetNumericAttributeBase(AttributeToModify, DataPair.Value);
				}
			}
		}
	}

	AbilitySystemComponent->ForceReplication();
}

TArray<float> FAttributeSetInitterDiscreteLevels::GetAttributeSetValues(UClass* AttributeSetClass, FProperty* AttributeProperty, FName GroupName) const
{
	TArray<float> AttributeSetValues;
	const FAttributeSetDefaultsCollection* Collection = Defaults.Find(GroupName);
	if (!Collection)
	{
		ABILITY_LOG(Error, TEXT("FAttributeSetInitterDiscreteLevels::InitAttributeSetDefaults Default DefaultAttributeSet not found! Skipping Initialization"));
		return TArray<float>();
	}

	for (const FAttributeSetDefaults& SetDefaults : Collection->LevelData)
	{
		const FAttributeDefaultValueList* DefaultDataList = SetDefaults.DataMap.Find(AttributeSetClass);
		if (DefaultDataList)
		{
			for (auto& DataPair : DefaultDataList->List)
			{
				check(DataPair.Property);
				if (DataPair.Property == AttributeProperty)
				{
					AttributeSetValues.Add(DataPair.Value);
				}
			}
		}
	}
	return AttributeSetValues;
}
