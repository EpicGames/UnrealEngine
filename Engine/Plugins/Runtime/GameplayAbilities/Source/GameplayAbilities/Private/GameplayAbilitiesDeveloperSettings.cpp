// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayAbilitiesDeveloperSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilitiesDeveloperSettings)

#if WITH_EDITOR
bool UGameplayAbilitiesDeveloperSettings::CanEditChange(const FProperty* InProperty) const
{
	bool bCanEdit = Super::CanEditChange(InProperty);
	if (InProperty->GetName() == GET_MEMBER_NAME_CHECKED(ThisClass, GlobalGameplayCueManagerClass))
	{
		bCanEdit = GlobalGameplayCueManagerName.IsNull();
	}
	if (InProperty->GetName() == GET_MEMBER_NAME_CHECKED(ThisClass, DefaultGameplayModEvaluationChannel))
	{
		bCanEdit = bAllowGameplayModEvaluationChannels;
	}
	if (InProperty->GetName() == GET_MEMBER_NAME_CHECKED(ThisClass, GameplayModEvaluationChannelAliases))
	{
		bCanEdit = bAllowGameplayModEvaluationChannels;
	}
	return bCanEdit;
}
#endif
