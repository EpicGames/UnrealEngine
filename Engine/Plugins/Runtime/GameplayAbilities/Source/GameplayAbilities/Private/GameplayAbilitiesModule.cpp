// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayAbilitiesModule.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#include "GameFramework/HUD.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"
#include "GameplayAbilitiesDeveloperSettings.h"

#if WITH_GAMEPLAY_DEBUGGER
#include "GameplayDebugger.h"
#include "GameplayDebuggerCategory_Abilities.h"
#endif // WITH_GAMEPLAY_DEBUGGER

class FGameplayAbilitiesModule : public IGameplayAbilitiesModule
{
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface

	virtual UAbilitySystemGlobals* GetAbilitySystemGlobals() override
	{
		// Defer loading of globals to the first time it is requested
		if (!AbilitySystemGlobals)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_IGameplayAbilitiesModule_GetAbilitySystemGlobals_LoadModule);
			const UGameplayAbilitiesDeveloperSettings* Settings = GetDefault<UGameplayAbilitiesDeveloperSettings>();
			FSoftClassPath AbilitySystemClassName = Settings->AbilitySystemGlobalsClassName;

			UClass* SingletonClass = AbilitySystemClassName.TryLoadClass<UObject>();
			checkf(SingletonClass != nullptr, TEXT("Ability config value AbilitySystemGlobalsClassName is not a valid class name."));

			AbilitySystemGlobals = NewObject<UAbilitySystemGlobals>(GetTransientPackage(), SingletonClass, NAME_None);
			AbilitySystemGlobals->AddToRoot();
			AbilitySystemGlobals->InitGlobalData();
			AbilitySystemGlobalsReadyCallback.Broadcast();
		}

		check(AbilitySystemGlobals);
		return AbilitySystemGlobals;
	}

	virtual bool IsAbilitySystemGlobalsAvailable() override
	{
		return AbilitySystemGlobals != nullptr;
	}

	void CallOrRegister_OnAbilitySystemGlobalsReady(FSimpleMulticastDelegate::FDelegate Delegate)
	{
		if (AbilitySystemGlobals)
		{
			Delegate.Execute();
		}
		else
		{
			AbilitySystemGlobalsReadyCallback.Add(Delegate);
		}
	}

	FSimpleMulticastDelegate AbilitySystemGlobalsReadyCallback;
	UAbilitySystemGlobals* AbilitySystemGlobals;
	
};

IMPLEMENT_MODULE(FGameplayAbilitiesModule, GameplayAbilities)

void FGameplayAbilitiesModule::StartupModule()
{	
	// This is loaded upon first request
	AbilitySystemGlobals = nullptr;

#if WITH_GAMEPLAY_DEBUGGER
	IGameplayDebugger& GameplayDebuggerModule = IGameplayDebugger::Get();
	GameplayDebuggerModule.RegisterCategory("Abilities", IGameplayDebugger::FOnGetCategory::CreateStatic(&FGameplayDebuggerCategory_Abilities::MakeInstance));
	GameplayDebuggerModule.NotifyCategoriesChanged();
#endif // WITH_GAMEPLAY_DEBUGGER

	if (!IsRunningDedicatedServer())
	{
		AHUD::OnShowDebugInfo.AddStatic(&UAbilitySystemComponent::OnShowDebugInfo);
	}
}

void FGameplayAbilitiesModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	AbilitySystemGlobals = NULL;

#if WITH_GAMEPLAY_DEBUGGER
	if (IGameplayDebugger::IsAvailable())
	{
		IGameplayDebugger& GameplayDebuggerModule = IGameplayDebugger::Get();
		GameplayDebuggerModule.UnregisterCategory("Abilities");
		GameplayDebuggerModule.NotifyCategoriesChanged();
	}
#endif // WITH_GAMEPLAY_DEBUGGER
}
