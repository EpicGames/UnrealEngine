// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayAbilityBlueprint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityBlueprint)

//////////////////////////////////////////////////////////////////////////
// UGameplayAbilityBlueprint

UGameplayAbilityBlueprint::UGameplayAbilityBlueprint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

/** Returns the most base gameplay ability blueprint for a given blueprint (if it is inherited from another ability blueprint, returning null if only native / non-ability BP classes are it's parent) */
UGameplayAbilityBlueprint* UGameplayAbilityBlueprint::FindRootGameplayAbilityBlueprint(UGameplayAbilityBlueprint* DerivedBlueprint)
{
	UGameplayAbilityBlueprint* ParentBP = NULL;

	// Determine if there is a gameplay ability blueprint in the ancestry of this class
	for (UClass* ParentClass = DerivedBlueprint->ParentClass; ParentClass != UObject::StaticClass(); ParentClass = ParentClass->GetSuperClass())
	{
		if (UGameplayAbilityBlueprint* TestBP = Cast<UGameplayAbilityBlueprint>(ParentClass->ClassGeneratedBy))
		{
			ParentBP = TestBP;
		}
	}

	return ParentBP;
}

#endif

