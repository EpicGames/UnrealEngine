// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbilityRepAnimMontage.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequenceBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityRepAnimMontage)

bool FGameplayAbilityRepAnimMontage::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);

	uint8 bIsMontage = 1;
	if (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::DynamicMontageSerialization)
	{
		if (Ar.IsSaving())
		{
			bIsMontage = Animation && Animation->IsA<UAnimMontage>();
		}
		Ar.SerializeBits(&bIsMontage, 1);
	}

	uint8 RepPosition = bRepPosition;
	Ar.SerializeBits(&RepPosition, 1);
	if (RepPosition)
	{
		bRepPosition = true;

		// when rep'ing position, we don't want to skip correction
		// and we don't need to force the section id to play
		SectionIdToPlay = 0;
		SkipPositionCorrection = false;

		// @note: section frames have such a high amount of precision they use, when
		// removing some of the position precision and packing it into a uint32 caused
		// issues where ability code would pick the end of a previous section instead of
		// the start of a new section. For now serializing the full position again.
		Ar << Position;
	}
	else
	{
		bRepPosition = false;

		// when rep'ing the section to play id, we want to skip
		// correction, and don't want a position
		SkipPositionCorrection = true;
		Position = 0.0f;
		Ar.SerializeBits(&SectionIdToPlay, 7);
	}

	uint8 bIsStopped = IsStopped;
	Ar.SerializeBits(&bIsStopped, 1);
	IsStopped = bIsStopped & 1;

	if (Ar.EngineNetVer() < FEngineNetworkCustomVersion::MontagePlayInstIdSerialization)
	{
		uint8 bForcePlayBit = 0;
		Ar.SerializeBits(&bForcePlayBit, 1);

		if (Ar.IsLoading())
		{
			// Emulate behavior of ForcePlayBit via PlayInstanceId when reading from older-revision data
			PlayInstanceId = (bForcePlayBit ? 1 : 0);
		}
	}

	uint8 bSkipPositionCorrection = SkipPositionCorrection;
	Ar.SerializeBits(&bSkipPositionCorrection, 1);
	SkipPositionCorrection = bSkipPositionCorrection & 1;

	uint8 SkipPlayRate = bSkipPlayRate;
	Ar.SerializeBits(&SkipPlayRate, 1);
	bSkipPlayRate = SkipPlayRate & 1;

	Ar << Animation;
	Ar << PlayRate;
	Ar << BlendTime;
	Ar << NextSectionID;

	if (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::MontagePlayInstIdSerialization)
	{
		Ar << PlayInstanceId;
	}

	PredictionKey.NetSerialize(Ar, Map, bOutSuccess);

	if (!bIsMontage)
	{
		ensure(Ar.EngineNetVer() >= FEngineNetworkCustomVersion::DynamicMontageSerialization);
		Ar << BlendOutTime;
		Ar << SlotName;
	}

	if (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::MontagePlayCountSerialization)
	{
		Ar << PlayCount;
	}

	bOutSuccess = true;
	return true;
}

void FGameplayAbilityRepAnimMontage::SetRepAnimPositionMethod(ERepAnimPositionMethod InMethod)
{
	switch (InMethod)
	{
	case ERepAnimPositionMethod::Position:
		bRepPosition = true;
		break;
	case ERepAnimPositionMethod::CurrentSectionId:
		bRepPosition = false;
		break;
	}
}

UAnimMontage* FGameplayAbilityRepAnimMontage::GetAnimMontage() const
{
	return Cast<UAnimMontage>(Animation);
}
