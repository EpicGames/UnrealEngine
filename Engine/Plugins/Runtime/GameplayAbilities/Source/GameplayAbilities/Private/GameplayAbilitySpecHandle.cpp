// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayAbilitySpecHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilitySpecHandle)

// ----------------------------------------------------

void FGameplayAbilitySpecHandle::GenerateNewHandle()
{
	// Must be in C++ to avoid duplicate statics accross execution units
	static int32 GHandle = 1;
	Handle = GHandle++;
}
