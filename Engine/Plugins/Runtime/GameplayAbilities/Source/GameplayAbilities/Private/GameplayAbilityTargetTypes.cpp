// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbilityTargetTypes.h"
#include "GameplayEffect.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemBlueprintLibrary.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"
#include "Components/MeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityTargetTypes)

namespace UE::AbilitySystem::Private
{
	int32 CVarRecalcuateTargetDataSourceOnApply = 0;
	FAutoConsoleVariableRef CVarRecalcuateTargetDataSourceOnSpecApplication(
		TEXT("AbilitySystem.Fix.RecalcuateTargetDataSourceOnApply"),
		CVarRecalcuateTargetDataSourceOnApply,
		TEXT("Whether to force recalculation of Source Tags and Attributes on application. Prior to 5.7 this recalculated source tags and attributes when applying from target data."),
		ECVF_Default);
}

TArray<FActiveGameplayEffectHandle> FGameplayAbilityTargetData::ApplyGameplayEffect(const UGameplayEffect* GameplayEffect, const FGameplayEffectContextHandle& InEffectContext, float Level, FPredictionKey PredictionKey)
{
	// Make a temp spec and call the spec function. This ends up cloning the spec per target
	FGameplayEffectSpec	TempSpecToApply(GameplayEffect, InEffectContext, Level);

	return ApplyGameplayEffectSpec(TempSpecToApply, PredictionKey);
}

TArray<FActiveGameplayEffectHandle> FGameplayAbilityTargetData::ApplyGameplayEffectSpec(FGameplayEffectSpec& InSpec, FPredictionKey PredictionKey)
{
	TArray<FActiveGameplayEffectHandle>	AppliedHandles;

	if (!ensure(InSpec.GetContext().IsValid() && InSpec.GetContext().GetInstigatorAbilitySystemComponent()))
	{
		return AppliedHandles;
	}

	TArray<TWeakObjectPtr<AActor> > Actors = GetActors();
	
	AppliedHandles.Reserve(Actors.Num());

	for (TWeakObjectPtr<AActor>& TargetActor : Actors)
	{
		UAbilitySystemComponent* TargetComponent = UAbilitySystemBlueprintLibrary::GetAbilitySystemComponent(TargetActor.Get());

		if (TargetComponent)
		{
			// We have to make a new effect spec and context here, because otherwise the targeting info gets accumulated and things take damage multiple times
			FGameplayEffectContextHandle EffectContext = InSpec.GetContext().Duplicate();
			FGameplayEffectSpec	SpecToApply(InSpec, EffectContext);
			if (UE::AbilitySystem::Private::CVarRecalcuateTargetDataSourceOnApply > 0)
			{
				// Force recalculation of the source tags and attributes
				SpecToApply.SetContext(EffectContext);
			}

			AddTargetDataToContext(EffectContext, false);

			AppliedHandles.Add(EffectContext.GetInstigatorAbilitySystemComponent()->ApplyGameplayEffectSpecToTarget(SpecToApply, TargetComponent, PredictionKey));
		}
	}

	return AppliedHandles;
}

void FGameplayAbilityTargetData::AddTargetDataToContext(FGameplayEffectContextHandle& Context, bool bIncludeActorArray) const
{
	if (bIncludeActorArray)
	{
		const TArray<TWeakObjectPtr<AActor>> WeakArray = GetActors();
		if (WeakArray.Num() > 0)
		{
			Context.AddActors(WeakArray);
		}
	}

	if (HasHitResult() && !Context.GetHitResult())
	{
		Context.AddHitResult(*GetHitResult());
	}

	if (HasOrigin())
	{
		Context.AddOrigin(GetOrigin().GetLocation());
	}
}

FString FGameplayAbilityTargetData::ToString() const
{
	return TEXT("BASE CLASS");
}

FTransform FGameplayAbilityTargetingLocationInfo::GetTargetingTransform() const
{
	//Return or calculate based on LocationType.
	switch (LocationType)
	{
	case EGameplayAbilityTargetingLocationType::ActorTransform:
		if (SourceActor)
		{
			return SourceActor->GetTransform();
		}
		break;
	case EGameplayAbilityTargetingLocationType::SocketTransform:
		if (SourceComponent)
		{
			// Bad socket name will just return component transform anyway, so we're safe
			return SourceComponent->GetSocketTransform(SourceSocketName);
		}
		break;
	case EGameplayAbilityTargetingLocationType::LiteralTransform:
		return LiteralTransform;
	default:
		check(false);
		break;
	}

	// It cannot get here
	return FTransform::Identity;
}

FGameplayAbilityTargetDataHandle FGameplayAbilityTargetingLocationInfo::MakeTargetDataHandleFromHitResult(TWeakObjectPtr<UGameplayAbility> Ability, const FHitResult& HitResult) const
{
	TArray<FHitResult> HitResults;
	HitResults.Add(HitResult);
	return MakeTargetDataHandleFromHitResults(Ability, HitResults);
}

FGameplayAbilityTargetDataHandle FGameplayAbilityTargetingLocationInfo::MakeTargetDataHandleFromHitResults(TWeakObjectPtr<UGameplayAbility> Ability, const TArray<FHitResult>& HitResults) const
{
	FGameplayAbilityTargetDataHandle ReturnDataHandle;

	for (int32 i = 0; i < HitResults.Num(); i++)
	{
		/** Note: These are cleaned up by the FGameplayAbilityTargetDataHandle (via an internal TSharedPtr) */
		FGameplayAbilityTargetData_SingleTargetHit* ReturnData = new FGameplayAbilityTargetData_SingleTargetHit();
		ReturnData->HitResult = HitResults[i];
		ReturnDataHandle.Add(ReturnData);
	}
	
	return ReturnDataHandle;
}

FGameplayAbilityTargetDataHandle FGameplayAbilityTargetingLocationInfo::MakeTargetDataHandleFromActors(const TArray<TWeakObjectPtr<AActor> >& TargetActors, bool OneActorPerHandle) const
{
	/** Note: This is cleaned up by the FGameplayAbilityTargetDataHandle (via an internal TSharedPtr) */
	FGameplayAbilityTargetData_ActorArray* ReturnData = new FGameplayAbilityTargetData_ActorArray();
	FGameplayAbilityTargetDataHandle ReturnDataHandle = FGameplayAbilityTargetDataHandle(ReturnData);
	ReturnData->SourceLocation = *this;
	if (OneActorPerHandle)
	{
		if (TargetActors.Num() > 0)
		{
			if (AActor* TargetActor = TargetActors[0].Get())
			{
				ReturnData->TargetActorArray.Add(TargetActor);
			}

			for (int32 i = 1; i < TargetActors.Num(); ++i)
			{
				if (AActor* TargetActor = TargetActors[i].Get())
				{
					FGameplayAbilityTargetData_ActorArray* CurrentData = new FGameplayAbilityTargetData_ActorArray();
					CurrentData->SourceLocation = *this;
					CurrentData->TargetActorArray.Add(TargetActor);
					ReturnDataHandle.Add(CurrentData);
				}
			}
		}
	}
	else
	{
		ReturnData->TargetActorArray = TargetActors;
	}
	return ReturnDataHandle;
}

// If defined, we'll serialize target data in a safer way (untested/unproven still: goal should be to remove old code asap)
#ifndef TARGETDATAHANDLE_SAFE_NET_SERIALIZE
#define TARGETDATAHANDLE_SAFE_NET_SERIALIZE 1
#endif


void FGameplayAbilityTargetDataDeleter::operator()(FGameplayAbilityTargetData* Object) const
{
	check(Object);
	UScriptStruct* ScriptStruct = Object->GetScriptStruct();
	check(ScriptStruct);
	ScriptStruct->DestroyStruct(Object);
	FMemory::Free(Object);
}

bool FGameplayAbilityTargetDataHandle::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	Ar << UniqueId;

	uint8 DataNum;
	if (Ar.IsSaving())
	{
		UE_CLOGF(Data.Num() > MAX_uint8, LogAbilitySystem, Warning, "Too many TargetData sources (%d!) to net serialize. Clamping to %d", Data.Num(), MAX_uint8);
		DataNum = FMath::Min<int32>( Data.Num(), MAX_uint8 );
	}
	Ar << DataNum;
	if (Ar.IsLoading())
	{
		Data.SetNum(DataNum);
		if (DataNum > 32)
		{
			ABILITY_LOG(Warning, TEXT("FGameplayAbilityTargetDataHandle::NetSerialize received with large DataNum: %d"), DataNum);
		}
	}

	for (int32 i = 0; i < DataNum && !Ar.IsError(); ++i)
	{
		TCheckedObjPtr<UScriptStruct> ScriptStruct = Data[i].IsValid() ? Data[i]->GetScriptStruct() : NULL;

#if TARGETDATAHANDLE_SAFE_NET_SERIALIZE
		UAbilitySystemGlobals::Get().TargetDataStructCache.NetSerialize(Ar, ScriptStruct.Get());
#else
		Ar << ScriptStruct;
#endif

		if (ScriptStruct.IsValid())
		{
			if (Ar.IsLoading())
			{
				// For now, just always reset/reallocate the data when loading.
				// Longer term if we want to generalize this and use it for property replication, we should support
				// only reallocating when necessary
				check(!Data[i].IsValid());

				FGameplayAbilityTargetData * NewData = (FGameplayAbilityTargetData*)FMemory::Malloc(ScriptStruct->GetStructureSize());
				ScriptStruct->InitializeStruct(NewData);

				Data[i] = TSharedPtr<FGameplayAbilityTargetData>(NewData, FGameplayAbilityTargetDataDeleter());
			}

			void* ContainerPtr = Data[i].Get();

			if (ScriptStruct->StructFlags & STRUCT_NetSerializeNative)
			{
				ScriptStruct->GetCppStructOps()->NetSerialize(Ar, Map, bOutSuccess, Data[i].Get());
			}
			else
			{
				// This won't work since FStructProperty::NetSerializeItem is deprecrated.
				//	1) we have to manually crawl through the topmost struct's fields since we don't have a FStructProperty for it (just the UScriptProperty)
				//	2) if there are any UStructProperties in the topmost struct's fields, we will assert in FStructProperty::NetSerializeItem.

				ABILITY_LOG(Fatal, TEXT("FGameplayAbilityTargetDataHandle::NetSerialize called on data struct %s without a native NetSerialize"), *ScriptStruct->GetName());

				for (TFieldIterator<FProperty> It(ScriptStruct.Get()); It; ++It)
				{
					if (It->PropertyFlags & CPF_RepSkip)
					{
						continue;
					}

					void* PropertyData = It->ContainerPtrToValuePtr<void*>(ContainerPtr);

					It->NetSerializeItem(Ar, Map, PropertyData);
				}
			}
		}
		else if (ScriptStruct.IsError())
		{
			ABILITY_LOG(Error, TEXT("FGameplayAbilityTargetDataHandle::NetSerialize: Bad ScriptStruct serialized, can't recover."));
			Ar.SetError();
			break;
		}
	}

	if (Ar.IsError())
	{
		// Something bad happened, make sure to not return invalid shared ptrs
		for (int32 i=Data.Num()-1; i >= 0; --i)
		{
			if (Data[i].IsValid() == false)
			{
				Data.RemoveAt(i);
			}
		}
		bOutSuccess = false;
		return false;
	}
	
	bOutSuccess = true;
	return true;
}
bool FGameplayAbilityTargetingLocationInfo::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	Ar << LocationType;

	switch (LocationType)
	{
	case EGameplayAbilityTargetingLocationType::ActorTransform:
		Ar << SourceActor;
		break;
	case EGameplayAbilityTargetingLocationType::SocketTransform:
		Ar << SourceComponent;
		Ar << SourceSocketName;
		break;
	case EGameplayAbilityTargetingLocationType::LiteralTransform:
		Ar << LiteralTransform;
		break;
	default:
		check(false);		//This case should not happen
		break;
	}

	bOutSuccess = true;
	return true;
}

bool FGameplayAbilityTargetData_LocationInfo::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	SourceLocation.NetSerialize(Ar, Map, bOutSuccess);
	TargetLocation.NetSerialize(Ar, Map, bOutSuccess);

	bOutSuccess = true;
	return true;
}

bool FGameplayAbilityTargetData_ActorArray::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	SourceLocation.NetSerialize(Ar, Map, bOutSuccess);
	SafeNetSerializeTArray_Default<31>(Ar, TargetActorArray);

	bOutSuccess = true;
	return true;
}

bool FGameplayAbilityTargetData_SingleTargetHit::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	HitResult.NetSerialize(Ar, Map, bOutSuccess);

	return true;
}

