// Copyright Epic Games, Inc. All Rights Reserved.

#include "Abilities/GameplayAbilityTypes.h"

#include "Abilities/GameplayAbility.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/MovementComponent.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "GameplayPrediction.h"
#include "Misc/NetworkVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityTypes)

//----------------------------------------------------------------------

static TAutoConsoleVariable<bool> CVarRecordAbilityInstancesOnClientSpecUpdates(TEXT("AbilitySystem.RecordAbilityInstancesOnClientSpecUpdates"), true, TEXT("Set this to false if encountering issues with the logic for replicating ability instances in the spec's PreReplicatedRemove, PostReplicatedChange, or PostReplicatedAdd."));

void FGameplayAbilityActorInfo::InitFromActor(AActor *InOwnerActor, AActor *InAvatarActor, UAbilitySystemComponent* InAbilitySystemComponent)
{
	check(InOwnerActor);
	check(InAbilitySystemComponent);

	OwnerActor = InOwnerActor;
	AvatarActor = InAvatarActor;
	AbilitySystemComponent = InAbilitySystemComponent;
	AffectedAnimInstanceTag = InAbilitySystemComponent->AffectedAnimInstanceTag; 
	
	APlayerController* OldPC = PlayerController.Get();

	// Look for a player controller or pawn in the owner chain.
	AActor *TestActor = InOwnerActor;
	while (TestActor)
	{
		if (APlayerController * CastPC = Cast<APlayerController>(TestActor))
		{
			PlayerController = CastPC;
			break;
		}

		if (APawn * Pawn = Cast<APawn>(TestActor))
		{
			PlayerController = Cast<APlayerController>(Pawn->GetController());
			break;
		}

		TestActor = TestActor->GetOwner();
	}

	// Notify ASC if PlayerController was found for first time
	if (OldPC == nullptr && PlayerController.IsValid())
	{
		InAbilitySystemComponent->OnPlayerControllerSet();
	}

	if (AActor* const AvatarActorPtr = AvatarActor.Get())
	{
		// Grab Components that we care about
		SkeletalMeshComponent = AvatarActorPtr->FindComponentByClass<USkeletalMeshComponent>();
		MovementComponent = AvatarActorPtr->FindComponentByClass<UMovementComponent>();
	}
	else
	{
		SkeletalMeshComponent = nullptr;
		MovementComponent = nullptr;
	}
}

void FGameplayAbilityActorInfo::SetAvatarActor(AActor *InAvatarActor)
{
	InitFromActor(OwnerActor.Get(), InAvatarActor, AbilitySystemComponent.Get());
}

void FGameplayAbilityActorInfo::ClearActorInfo()
{
	OwnerActor = nullptr;
	AvatarActor = nullptr;
	PlayerController = nullptr;
	SkeletalMeshComponent = nullptr;
	MovementComponent = nullptr;
}

UAnimInstance* FGameplayAbilityActorInfo::GetAnimInstance() const
{ 
	const USkeletalMeshComponent* SKMC = SkeletalMeshComponent.Get();

	if (SKMC)
	{
		if (AffectedAnimInstanceTag != NAME_None)
		{
			if(UAnimInstance* Instance = SKMC->GetAnimInstance())
			{
				return Instance->GetLinkedAnimGraphInstanceByTag(AffectedAnimInstanceTag);
			}
		}

		return SKMC->GetAnimInstance();
	}

	return nullptr;
}

bool FGameplayAbilityActorInfo::IsLocallyControlled() const
{
	if (const APlayerController* PC = PlayerController.Get())
	{
		return PC->IsLocalController();
	}
	else if (const APawn* OwnerPawn = Cast<APawn>(OwnerActor))
	{
		if (OwnerPawn->IsLocallyControlled())
		{
			return true;
		}
		else if (OwnerPawn->GetController())
		{
			// We're controlled, but we're not locally controlled.
			return false;
		}
	}
	
	return IsNetAuthority();
}

bool FGameplayAbilityActorInfo::IsLocallyControlledPlayer() const
{
	if (const APlayerController* PC = PlayerController.Get())
	{
		return PC->IsLocalController();
	}

	return false;
}

bool FGameplayAbilityActorInfo::IsNetAuthority() const
{
	// Make sure this works on pending kill actors
	AActor* const OwnerActorPtr = OwnerActor.Get(/*bEvenIfPendingKill=*/ true);
	if (OwnerActorPtr)
	{
		return (OwnerActorPtr->GetLocalRole() == ROLE_Authority);
	}

	// This rarely happens during shutdown cases for reasons that aren't quite clear
	ABILITY_LOG(Warning, TEXT("IsNetAuthority called when OwnerActor was invalid. Returning false. AbilitySystemComponent: %s"), *GetNameSafe(AbilitySystemComponent.Get()));
	return false;
}

FGameplayAbilityActivationInfo::FGameplayAbilityActivationInfo(AActor* InActor)
	: bCanBeEndedByOtherInstance(false)	
{
	// On Init, we are either Authority or NonAuthority. We haven't been given a PredictionKey and we haven't been confirmed.
	// NonAuthority essentially means 'I'm not sure what how I'm going to do this yet'.
	ActivationMode = ((InActor && InActor->GetLocalRole() == ROLE_Authority) ? EGameplayAbilityActivationMode::Authority : EGameplayAbilityActivationMode::NonAuthority);
}

void FGameplayAbilityActivationInfo::SetPredicting(FPredictionKey PredictionKey)
{
	ActivationMode = EGameplayAbilityActivationMode::Predicting;
	PredictionKeyWhenActivated = PredictionKey;

	// Abilities can be cancelled by server at any time. There is no reason to have to wait until confirmation.
	// prediction keys keep previous activations of abilities from ending future activations.
	bCanBeEndedByOtherInstance = true;
}

void FGameplayAbilityActivationInfo::ServerSetActivationPredictionKey(FPredictionKey PredictionKey)
{
	PredictionKeyWhenActivated = PredictionKey;
}

void FGameplayAbilityActivationInfo::SetActivationConfirmed()
{
	ActivationMode = EGameplayAbilityActivationMode::Confirmed;
	//Remote (server) commands to end the ability that come in after this point are considered for this instance
	bCanBeEndedByOtherInstance = true;
}

void FGameplayAbilityActivationInfo::SetActivationRejected()
{
	ActivationMode = EGameplayAbilityActivationMode::Rejected;
}

bool FGameplayAbilitySpecDef::operator==(const FGameplayAbilitySpecDef& Other) const
{
	return Ability == Other.Ability &&
		LevelScalableFloat == Other.LevelScalableFloat &&
		InputID == Other.InputID &&
		RemovalPolicy == Other.RemovalPolicy;
}

bool FGameplayAbilitySpecDef::operator!=(const FGameplayAbilitySpecDef& Other) const
{
	return !(*this == Other);
}

bool FGameplayAbilitySpec::IsActive() const
{
	// If ability hasn't replicated yet we're not active
	return Ability != nullptr && ActiveCount > 0;
}

UGameplayAbility* FGameplayAbilitySpec::GetPrimaryInstance() const
{
	if (Ability && Ability->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::InstancedPerActor)
	{
		if (NonReplicatedInstances.Num() > 0)
		{
			return NonReplicatedInstances[0];
		}
		if (ReplicatedInstances.Num() > 0)
		{
			return ReplicatedInstances[0];
		}
	}
	return nullptr;
}

bool FGameplayAbilitySpec::ShouldReplicateAbilitySpec() const
{
	if (Ability && Ability->ShouldReplicateAbilitySpec(*this))
	{
		return true;
	}

	return false;
}

void FGameplayAbilitySpec::PreReplicatedRemove(const struct FGameplayAbilitySpecContainer& InArraySerializer)
{
	if (InArraySerializer.Owner)
	{
		UE_LOGF(LogAbilitySystem, Verbose, "%ls: OnRemoveAbility (Non-Auth): [%ls] %ls. Level: %d", *GetNameSafe(InArraySerializer.Owner->GetOwner()), *Handle.ToString(), *GetNameSafe(Ability), Level)
		UE_VLOG(InArraySerializer.Owner->GetOwner(), VLogAbilitySystem, Verbose, TEXT("OnRemoveAbility (Non-Auth): [%s] %s. Level: %d"), *Handle.ToString(), *GetNameSafe(Ability), Level);

		FScopedAbilityListLock AblityListLock(*InArraySerializer.Owner);

		if (CVarRecordAbilityInstancesOnClientSpecUpdates.GetValueOnGameThread())
		{
			for (UGameplayAbility* AbilityInstance : ReplicatedInstances)
			{
				if (AbilityInstance)
				{
					InArraySerializer.Owner->RemoveReplicatedInstancedAbility(AbilityInstance);
				}
			}
		}
		
		InArraySerializer.Owner->OnRemoveAbility(*this);
	}
}

void FGameplayAbilitySpec::PostReplicatedChange(const struct FGameplayAbilitySpecContainer& InArraySerializer)
{
	if (InArraySerializer.Owner)
	{
		UE_LOGF(LogAbilitySystem, Verbose, "%ls: AbilitySpecChanged (Non-Auth): [%ls] %ls. Level: %d", *GetNameSafe(InArraySerializer.Owner->GetOwner()), *Handle.ToString(), *GetNameSafe(Ability), Level)
		UE_VLOG(InArraySerializer.Owner->GetOwner(), VLogAbilitySystem, Verbose, TEXT("AbilitySpecChanged (Non-Auth): [%s] %s. Level: %d"), *Handle.ToString(), *GetNameSafe(Ability), Level);

		if (CVarRecordAbilityInstancesOnClientSpecUpdates.GetValueOnGameThread())
		{
			for (UGameplayAbility* AbilityInstance : ReplicatedInstances)
			{
				if (AbilityInstance)
				{
					InArraySerializer.Owner->AddReplicatedInstancedAbility(AbilityInstance);
				}
			}
		}
	}
}

void FGameplayAbilitySpec::PostReplicatedAdd(const struct FGameplayAbilitySpecContainer& InArraySerializer)
{
	if (InArraySerializer.Owner)
	{
		UE_LOGF(LogAbilitySystem, Verbose, "%ls: OnGiveAbility (Non-Auth): [%ls] %ls. Level: %d", *GetNameSafe(InArraySerializer.Owner->GetOwner()), *Handle.ToString(), *GetNameSafe(Ability), Level)
		UE_VLOG(InArraySerializer.Owner->GetOwner(), VLogAbilitySystem, Verbose, TEXT("OnGiveAbility (Non-Auth): [%s] %s. Level: %d"), *Handle.ToString(), *GetNameSafe(Ability), Level);

		if (CVarRecordAbilityInstancesOnClientSpecUpdates.GetValueOnGameThread())
		{
			for (UGameplayAbility* AbilityInstance : ReplicatedInstances)
			{
				if (AbilityInstance)
				{
					InArraySerializer.Owner->AddReplicatedInstancedAbility(AbilityInstance);
				}
			}
		}
		
		InArraySerializer.Owner->OnGiveAbility(*this);
	}
}

FString FGameplayAbilitySpec::GetDebugString()
{
	return FString::Printf(TEXT("(%s)"), *GetNameSafe(Ability));
}

void FGameplayAbilitySpecContainer::RegisterWithOwner(UAbilitySystemComponent* InOwner)
{
	Owner = InOwner;
}

bool FGameplayAbilitySpecContainer::NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms)
{
	return FFastArraySerializer::FastArrayDeltaSerialize<FGameplayAbilitySpec, FGameplayAbilitySpecContainer>(Items, DeltaParms, *this);
}

// ----------------------------------------------------

FGameplayAbilitySpec::FGameplayAbilitySpec(UGameplayAbility* InAbility, int32 InLevel, int32 InInputID, UObject* InSourceObject)
	: Ability(InAbility)
	, Level(InLevel)
	, InputID(InInputID)
	, SourceObject(InSourceObject)
	, ActiveCount(0)
	, InputPressed(false)
	, RemoveAfterActivation(false)
	, PendingRemove(false)
	, bActivateOnce(false) 
{
	Handle.GenerateNewHandle();
}

FGameplayAbilitySpec::FGameplayAbilitySpec(TSubclassOf<UGameplayAbility> InAbilityClass, int32 InLevel, int32 InInputID, UObject* InSourceObject)
	: Ability(InAbilityClass ? InAbilityClass.GetDefaultObject() : nullptr)
	, Level(InLevel)
	, InputID(InInputID)
	, SourceObject(InSourceObject)
	, ActiveCount(0)
	, InputPressed(false)
	, RemoveAfterActivation(false)
	, PendingRemove(false)
	, bActivateOnce(false)
{
	Handle.GenerateNewHandle();
}

FGameplayAbilitySpec::FGameplayAbilitySpec(FGameplayAbilitySpecDef& InDef, int32 InGameplayEffectLevel, FActiveGameplayEffectHandle InGameplayEffectHandle)
	: Ability(InDef.Ability ? InDef.Ability->GetDefaultObject<UGameplayAbility>() : nullptr)
	, InputID(InDef.InputID)
	, SourceObject(InDef.SourceObject.Get())
	, ActiveCount(0)
	, InputPressed(false)
	, RemoveAfterActivation(false)
	, PendingRemove(false)
	, bActivateOnce(false)
{
	Handle.GenerateNewHandle();
	InDef.AssignedHandle = Handle;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GameplayEffectHandle = InGameplayEffectHandle;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	SetByCallerTagMagnitudes = InDef.SetByCallerTagMagnitudes;

	FString ContextString = FString::Printf(TEXT("FGameplayAbilitySpec::FGameplayAbilitySpec for %s from %s"), 
		(InDef.Ability ? *InDef.Ability->GetName() : TEXT("INVALID ABILITY")), 
		(InDef.SourceObject.IsValid() ? *InDef.SourceObject->GetName() : TEXT("INVALID ABILITY")));
	Level = InDef.LevelScalableFloat.GetValueAtLevel(InGameplayEffectLevel, &ContextString);
}


// ----------------------------------------------------

FScopedAbilityListLock::FScopedAbilityListLock(UAbilitySystemComponent& InAbilitySystemComponent)
	: AbilitySystemComponent(InAbilitySystemComponent)
{
	AbilitySystemComponent.IncrementAbilityListLock();
}

FScopedAbilityListLock::~FScopedAbilityListLock()
{
	AbilitySystemComponent.DecrementAbilityListLock();
}

// ----------------------------------------------------

FScopedTargetListLock::FScopedTargetListLock(UAbilitySystemComponent& InAbilitySystemComponent, const UGameplayAbility& InAbility)
	: GameplayAbility(InAbility)
	, AbilityLock(InAbilitySystemComponent)
{
	GameplayAbility.IncrementListLock();
}

FScopedTargetListLock::~FScopedTargetListLock()
{
	GameplayAbility.DecrementListLock();
}


// ----------------------------------------------------


TSharedPtr<FAbilityReplicatedDataCache> FGameplayAbilityReplicatedDataContainer::Find(const FGameplayAbilitySpecHandleAndPredictionKey& Key) const
{
	for (const FKeyDataPair& Pair : InUseData)
	{
		if (Pair.Key == Key)
		{
			return Pair.Value;
		}
	}

	return TSharedPtr<FAbilityReplicatedDataCache>();
}

TSharedRef<FAbilityReplicatedDataCache> FGameplayAbilityReplicatedDataContainer::FindOrAdd(const FGameplayAbilitySpecHandleAndPredictionKey& Key)
{
	// Search for existing
	for (const FKeyDataPair& Pair : InUseData)
	{
		if (Pair.Key == Key)
		{
			return Pair.Value;
		}
	}

	// Add New
	TSharedPtr<FAbilityReplicatedDataCache> SharedPtr;
	
	// Look for one to reuse in FreeData.
	for (int32 i=FreeData.Num()-1; i>=0; --i)
	{
		TSharedRef<FAbilityReplicatedDataCache>& FreeRef = FreeData[i];
		if (FreeRef.IsUnique()) // Only reuse if we are the only one hanging on
		{
			SharedPtr = FreeRef;

			// Reset it first (don't do this during remove or you will clear invocation lists of delegates that are being invoked!)
			SharedPtr->ResetAll();

			FreeData.RemoveAtSwap(i, EAllowShrinking::No);
			break;
		}
	}

	// Just allocate one if none available in the free list
	if (SharedPtr.IsValid() == false)
	{
		SharedPtr = TSharedPtr<FAbilityReplicatedDataCache>(new FAbilityReplicatedDataCache());
	}


	TSharedRef<FAbilityReplicatedDataCache> SharedRef = SharedPtr.ToSharedRef();
	InUseData.Emplace(Key, SharedRef);
	return SharedRef;
}

void FGameplayAbilityReplicatedDataContainer::Remove(const FGameplayAbilitySpecHandleAndPredictionKey& Key)
{
	for (int32 i=InUseData.Num()-1; i >= 0; --i)
	{
		if (Key == InUseData[i].Key)
		{
			TSharedRef<FAbilityReplicatedDataCache>& RemovedElement = InUseData[i].Value;

			// Add it to the free list
			FreeData.Add(RemovedElement);

			InUseData.RemoveAtSwap(i, EAllowShrinking::No);
			break;
		}
	}
}

void FGameplayAbilityReplicatedDataContainer::PrintDebug()
{
	ABILITY_LOG(Warning, TEXT("============================="));
	for (auto& Pair : InUseData)
	{
		ABILITY_LOG(Warning, TEXT("  %s. %d"), *Pair.Key.AbilityHandle.ToString(), Pair.Key.PredictionKeyAtCreation);
	}

	ABILITY_LOG(Warning, TEXT("In Use Size: %d Free Size: %d"), InUseData.Num(), FreeData.Num());
	ABILITY_LOG(Warning, TEXT("============================="));
}

