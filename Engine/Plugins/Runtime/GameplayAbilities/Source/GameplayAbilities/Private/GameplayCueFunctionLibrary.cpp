// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueFunctionLibrary.h"
#include "GameplayCueManager.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueFunctionLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogGameplayCueFunctionLibrary, Log, All);

//////////////////////////////////////////////////////////////////////////
// UGameplayCueFunctionLibrary
//////////////////////////////////////////////////////////////////////////
FGameplayCueParameters UGameplayCueFunctionLibrary::MakeGameplayCueParametersFromHitResult(const FHitResult& HitResult)
{
	FGameplayCueParameters CueParameters;

	CueParameters.Location = (HitResult.bBlockingHit ? HitResult.ImpactPoint : HitResult.TraceEnd);
	CueParameters.Normal = HitResult.ImpactNormal;
	CueParameters.PhysicalMaterial = HitResult.PhysMaterial;

	return CueParameters;
}

void UGameplayCueFunctionLibrary::ExecuteGameplayCueOnActor(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters)
{
	if (!Target)
	{
		return;
	}

	UAbilitySystemComponent* ASC = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Target);

	if (ASC)
	{
		// The actor has an ability system so the event will fire on authority only and will be replicated.
		if (Target->GetLocalRole() == ROLE_Authority)
		{
			ASC->ExecuteGameplayCue(GameplayCueTag, Parameters);
		}
	}
	else
	{
		// The actor does not have an ability system so the event will only be fired locally.
		UGameplayCueManager::ExecuteGameplayCue_NonReplicated(Target, GameplayCueTag, Parameters);
	}
}

void UGameplayCueFunctionLibrary::AddGameplayCueOnActor(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters)
{
	if (!Target)
	{
		return;
	}

	UAbilitySystemComponent* ASC = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Target);

	if (ASC)
	{
		// The actor has an ability system so the event will fire on authority only and will be replicated.
		if (Target->GetLocalRole() == ROLE_Authority)
		{
			ASC->AddGameplayCue(GameplayCueTag, Parameters);
		}
	}
	else
	{
		// The actor does not have an ability system so the event will only be fired locally.
		UGameplayCueManager::AddGameplayCue_NonReplicated(Target, GameplayCueTag, Parameters);
	}
}

void UGameplayCueFunctionLibrary::RemoveGameplayCueOnActor(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters)
{
	if (!Target)
	{
		return;
	}

	UAbilitySystemComponent* ASC = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Target);

	if (ASC)
	{
		// The actor has an ability system so the event will fire on authority only and will be replicated.
		if (Target->GetLocalRole() == ROLE_Authority)
		{
			ASC->RemoveGameplayCue(GameplayCueTag);
		}
	}
	else
	{
		// The actor does not have an ability system so the event will only be fired locally.
		UGameplayCueManager::RemoveGameplayCue_NonReplicated(Target, GameplayCueTag, Parameters);
	}
}

