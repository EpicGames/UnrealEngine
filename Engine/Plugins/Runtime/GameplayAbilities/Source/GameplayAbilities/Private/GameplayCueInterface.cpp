// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueInterface.h"
#include "AbilitySystemLog.h"
#include "AbilitySystemStats.h"
#include "GameplayTagsModule.h"
#include "AbilitySystemComponent.h"
#include "GameplayCueSet.h"
#include "Engine/PackageMapClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueInterface)

DEFINE_LOG_CATEGORY_STATIC(LogGameplayCues, Log, All);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#define CANTREPLICATEPARAMS_SEVERITY Warning
#else
	#define CANTREPLICATEPARAMS_SEVERITY Verbose
#endif

namespace GameplayCueInterfacePrivate
{
	struct FCueNameAndUFunction
	{
		FGameplayTag Tag;
		UFunction* Func;
	};
	typedef TMap<FGameplayTag, TArray<FCueNameAndUFunction> > FGameplayCueTagFunctionList;
	static TMap<FObjectKey, FGameplayCueTagFunctionList > PerClassGameplayTagToFunctionMap;

	bool bUseEqualTagCountAndRemovalCallbacks = true;
	static FAutoConsoleVariableRef CVarUseEqualTagCountAndRemovalCallbacks(TEXT("GameplayCue.Fix.UseEqualTagCountAndRemovalCallbacks"), bUseEqualTagCountAndRemovalCallbacks, TEXT("Default: True. When calling RemoveCue, get an equal number of TagCountUpdated callbacks and Removal callbacks rather than 1:many"));

	/**
	 * Given a pointer to a UProperty and the owning UObject, attempt to find the reflected FName of the property.
	 * @Return the property's FName as a string (or a debug string if the parameters are invalid)
	 * Example:
	 *
	 * UCLASS() class UMyClass { UPROPERTY() float MyName; } MyClass;
	 * FindNameOfProperty(&MyClass->MyName, MyClass) returns "MyName".
	 * 
	 * This is different than GET_MEMBER_NAME_CHECKED because it is done at runtime
	 * (e.g. you have a float* but do not know which field it belongs to).
	 * Note: This function isn't exactly fast and should only be used in debug situations.
	 */
	FString FindNameOfProperty(const void* PointerToProperty, const UObject* PropertyOwner)
	{
		if (!PropertyOwner)
		{
			return TEXT("(Null Owner)");
		}

		// Compute the offset and double-check that offset falls within the memory range of PropertyOwner.
		ptrdiff_t ThisOffsetFromOwner = (intptr_t(PointerToProperty) - intptr_t(PropertyOwner));

		const UClass* OwnerClass = PropertyOwner->GetClass();
		if (ThisOffsetFromOwner < 0 || ThisOffsetFromOwner >= OwnerClass->GetStructureSize())
		{
			// This offset is bad
			return TEXT("(Bad Offset)");
		}

		// Go through all of the properties and find one with the correct offset
		for (FProperty* Property = PropertyOwner->GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			int32 PropertyOffset = Property->GetOffset_ForDebug();
			if (PropertyOffset == ThisOffsetFromOwner)
			{
				return Property->GetName();
			}
		}

		return TEXT("(No UProperty)");
	}

	/** Get the owner of the FActiveGameplayCueContainer (which is an FStruct so does not have an Outer) */
	const UObject* GetPropertyOwner(const FActiveGameplayCueContainer& InArray)
	{
#if WITH_PUSH_MODEL
		return (InArray.OwningObject ? InArray.OwningObject : InArray.GetOwner());
#else
		return InArray.GetOwner();
#endif
	}
}


UGameplayCueInterface::UGameplayCueInterface(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

void IGameplayCueInterface::DispatchBlueprintCustomHandler(UObject* Object, UFunction* Func, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters)
{
	GameplayCueInterface_eventBlueprintCustomHandler_Parms Params;
	Params.EventType = EventType;
	Params.Parameters = Parameters;

	Object->ProcessEvent(Func, &Params);
}

void IGameplayCueInterface::ClearTagToFunctionMap()
{
	GameplayCueInterfacePrivate::PerClassGameplayTagToFunctionMap.Empty();
}

void IGameplayCueInterface::HandleGameplayCues(AActor *Self, const FGameplayTagContainer& GameplayCueTags, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters)
{
	HandleGameplayCues((UObject*)Self, GameplayCueTags, EventType, Parameters);
}

void IGameplayCueInterface::HandleGameplayCues(UObject* Self, const FGameplayTagContainer& GameplayCueTags, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters)
{
	for (FGameplayTag CueTag : GameplayCueTags)
	{
		HandleGameplayCue(Self, CueTag, EventType, Parameters);
	}
}

bool IGameplayCueInterface::ShouldAcceptGameplayCue(AActor *Self, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters)
{
	return ShouldAcceptGameplayCue((UObject*)Self, GameplayCueTag, EventType, Parameters);
}

bool IGameplayCueInterface::ShouldAcceptGameplayCue(UObject* Self, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters)
{
	return true;
}

void IGameplayCueInterface::HandleGameplayCue(AActor *Self, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters)
{
	HandleGameplayCue((UObject*)Self, GameplayCueTag, EventType, Parameters);
}

void IGameplayCueInterface::HandleGameplayCue(UObject* Self, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters)
{
	SCOPE_CYCLE_COUNTER(STAT_GameplayCueInterface_HandleGameplayCue);

	if (!Self)
	{
		return;
	}

	if (UE_LOG_ACTIVE(LogGameplayCues, Verbose))
	{
		UE_LOGF(LogGameplayCues, Verbose, "%ls::%s %ls %ls", *GetNameSafe(Self), __func__, *GameplayCueTag.ToString(), *UEnum::GetValueAsString(EventType));
		UE_VLOG(Self, LogGameplayCues, Verbose, TEXT("%hs %s %s"), __func__, *GameplayCueTag.ToString(), *UEnum::GetValueAsString(EventType));
	}

	// Look up a custom function for this gameplay tag. 
	UClass* Class = Self->GetClass();
	FGameplayTagContainer TagAndParentsContainer = GameplayCueTag.GetGameplayTagParents();

	Parameters.OriginalTag = GameplayCueTag;

	//Find entry for the class
	FObjectKey ClassObjectKey(Class);
	GameplayCueInterfacePrivate::FGameplayCueTagFunctionList& GameplayTagFunctionList = GameplayCueInterfacePrivate::PerClassGameplayTagToFunctionMap.FindOrAdd(ClassObjectKey);
	TArray<GameplayCueInterfacePrivate::FCueNameAndUFunction>* FunctionList = GameplayTagFunctionList.Find(GameplayCueTag);
	if (FunctionList == NULL)
	{
		//generate new function list
		FunctionList = &GameplayTagFunctionList.Add(GameplayCueTag);

		for (auto InnerTagIt = TagAndParentsContainer.CreateConstIterator(); InnerTagIt; ++InnerTagIt)
		{
			UFunction* Func = NULL;
			FName CueName = InnerTagIt->GetTagName();

			Func = Class->FindFunctionByName(CueName, EIncludeSuperFlag::IncludeSuper);
			// If the handler calls ForwardGameplayCueToParent, keep calling functions until one consumes the cue and doesn't forward it
			while (Func)
			{
				GameplayCueInterfacePrivate::FCueNameAndUFunction NewCueFunctionPair;
				NewCueFunctionPair.Tag = *InnerTagIt;
				NewCueFunctionPair.Func = Func;
				FunctionList->Add(NewCueFunctionPair);

				Func = Func->GetSuperFunction();
			}

			// Native functions cant be named with ".", so look for them with _. 
			FString NativeCueFuncName = CueName.ToString().Replace(TEXT("."), TEXT("_"));
			FName NativeCueFuncFName(NativeCueFuncName, FNAME_Find);
			
			// The UClass stores all the function names as FNames internally.
			// If the FName isn't found using FNAME_Find then it is not part of the UClass Function set.
			if (!NativeCueFuncFName.IsNone())
			{
				Func = Class->FindFunctionByName(NativeCueFuncFName, EIncludeSuperFlag::IncludeSuper);

				while (Func)
				{
					GameplayCueInterfacePrivate::FCueNameAndUFunction NewCueFunctionPair;
					NewCueFunctionPair.Tag = *InnerTagIt;
					NewCueFunctionPair.Func = Func;
					FunctionList->Add(NewCueFunctionPair);

					Func = Func->GetSuperFunction();
				}
			}
		}
	}

	//Iterate through all functions in the list until we should no longer continue
	check(FunctionList);
		
	bool bShouldContinue = true;
	for (int32 FunctionIndex = 0; bShouldContinue && (FunctionIndex < FunctionList->Num()); ++FunctionIndex)
	{
		const GameplayCueInterfacePrivate::FCueNameAndUFunction& CueFunctionPair = (*FunctionList)[FunctionIndex];
		UFunction* Func = CueFunctionPair.Func;
		Parameters.MatchedTagName = CueFunctionPair.Tag;

		// Reset the forward parameter now, so we can check it after function
		bForwardToParent = false;
		IGameplayCueInterface::DispatchBlueprintCustomHandler(Self, Func, EventType, Parameters);

		bShouldContinue = bForwardToParent;
	}

	if (bShouldContinue)
	{
		if (AActor* SelfActor = Cast<AActor>(Self))
		{
			TArray<UGameplayCueSet*> Sets;
			GetGameplayCueSets(Sets);
			for (UGameplayCueSet* Set : Sets)
			{
				bShouldContinue = Set->HandleGameplayCue(SelfActor, GameplayCueTag, EventType, Parameters);
				if (!bShouldContinue)
				{
					break;
				}
			}
		}
	}

	if (bShouldContinue)
	{
		Parameters.MatchedTagName = GameplayCueTag;
		GameplayCueDefaultHandler(EventType, Parameters);
	}
}

void IGameplayCueInterface::GameplayCueDefaultHandler(EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters)
{
	// No default handler, subclasses can implement
}

void IGameplayCueInterface::ForwardGameplayCueToParent()
{
	// Consumed by HandleGameplayCue
	bForwardToParent = true;
}

void FActiveGameplayCue::PreReplicatedRemove(const struct FActiveGameplayCueContainer &InArray)
{
	if (!InArray.Owner)
	{
		return;
	}

	if (UE_LOG_ACTIVE(LogGameplayCues, VeryVerbose))
	{
		const UObject* PropertyOwner = GameplayCueInterfacePrivate::GetPropertyOwner(InArray);
		const FString PropertyName = GameplayCueInterfacePrivate::FindNameOfProperty(&InArray, PropertyOwner);
		UE_LOGF(LogGameplayCues, VeryVerbose, "%ls.%ls::%s %ls %ls", *GetNameSafe(PropertyOwner), *PropertyName, __func__, *GameplayCueTag.ToString(), *PredictionKey.ToString());
		UE_VLOG(PropertyOwner, LogGameplayCues, VeryVerbose, TEXT("%s::%hs %s %s"), *PropertyName, __func__, *GameplayCueTag.ToString(), *PredictionKey.ToString());
	}

	// We don't check the PredictionKey here like we do in PostReplicatedAdd. PredictionKey tells us
	// if we were predictely created, but this doesn't mean we will predictively remove ourselves.
	if (bPredictivelyRemoved == false)
	{
		// If predicted ignore the add/remove
		InArray.Owner->UpdateTagMap(GameplayCueTag, -1, EGameplayTagReplicationState::None);
		InArray.Owner->InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::Removed, Parameters);
	}
}

void FActiveGameplayCue::PostReplicatedAdd(const struct FActiveGameplayCueContainer &InArray)
{
	if (!InArray.Owner)
	{
		return;
	}

	if (UE_LOG_ACTIVE(LogGameplayCues, VeryVerbose))
	{
		const UObject* PropertyOwner = GameplayCueInterfacePrivate::GetPropertyOwner(InArray);
		const FString PropertyName = GameplayCueInterfacePrivate::FindNameOfProperty(&InArray, PropertyOwner);
		UE_LOGF(LogGameplayCues, VeryVerbose, "%ls.%ls::%s %ls %ls", *GetNameSafe(PropertyOwner), *PropertyName, __func__, *GameplayCueTag.ToString(), *PredictionKey.ToString());
		UE_VLOG(PropertyOwner, LogGameplayCues, VeryVerbose, TEXT("%s::%hs %s %s"), *PropertyName, __func__, *GameplayCueTag.ToString(), *PredictionKey.ToString());
	}

	InArray.Owner->UpdateTagMap(GameplayCueTag, 1, EGameplayTagReplicationState::None);

	if (PredictionKey.IsLocalClientKey() == false)
	{
		// If predicted ignore the add/remove
		InArray.Owner->InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::WhileActive, Parameters);
	}
}

FString FActiveGameplayCue::GetDebugString()
{
	return FString::Printf(TEXT("(%s / %s)"), *GameplayCueTag.ToString(), *PredictionKey.ToString());
}

void FActiveGameplayCueContainer::AddCue(const FGameplayTag& Tag, const FPredictionKey& PredictionKey, const FGameplayCueParameters& Parameters)
{
	if (!Owner)
	{
		return;
	}

	if (UE_LOG_ACTIVE(LogGameplayCues, VeryVerbose))
	{
		const UObject* PropertyOwner = GameplayCueInterfacePrivate::GetPropertyOwner(*this);
		const FString PropertyName = GameplayCueInterfacePrivate::FindNameOfProperty(this, PropertyOwner);
		UE_LOGF(LogGameplayCues, VeryVerbose, "%ls.%ls::%s %ls %ls", *GetNameSafe(PropertyOwner), *PropertyName, __func__, *Tag.ToString(), *PredictionKey.ToString());
		UE_VLOG(PropertyOwner, LogGameplayCues, VeryVerbose, TEXT("%s::%hs %s %s"), *PropertyName, __func__, *Tag.ToString(), *PredictionKey.ToString());
	}

	// Store the prediction key so the client can investigate it
	FActiveGameplayCue&	NewCue = GameplayCues[GameplayCues.AddDefaulted()];
	NewCue.GameplayCueTag = Tag;
	NewCue.PredictionKey = PredictionKey;
	NewCue.Parameters = Parameters;
	MarkItemDirty(NewCue);
	
	Owner->UpdateTagMap(Tag, 1, EGameplayTagReplicationState::None);
}

void FActiveGameplayCueContainer::RemoveCue(const FGameplayTag& Tag)
{
	if (!Owner)
	{
		return;
	}

	if (UE_LOG_ACTIVE(LogGameplayCues, VeryVerbose))
	{
		const UObject* PropertyOwner = GameplayCueInterfacePrivate::GetPropertyOwner(*this);
		const FString PropertyName = GameplayCueInterfacePrivate::FindNameOfProperty(this, PropertyOwner);
		UE_LOGF(LogGameplayCues, VeryVerbose, "%ls.%ls::%s %ls", *GetNameSafe(PropertyOwner), *PropertyName, __func__, *Tag.ToString());
		UE_VLOG(PropertyOwner, LogGameplayCues, VeryVerbose, TEXT("%s::%hs %s"), *PropertyName, __func__, *Tag.ToString());
	}


	int32 CountDelta = 0;

	// This is the old behavior (tentatively removed in UE5.5):  A single callback per multiple removals
	if (!GameplayCueInterfacePrivate::bUseEqualTagCountAndRemovalCallbacks)
	{
		for (const FActiveGameplayCue& GameplayCue : GameplayCues)
		{
			CountDelta += (GameplayCue.GameplayCueTag == Tag);
		}

		if (CountDelta > 0)
		{
			Owner->UpdateTagMap(Tag, -CountDelta, EGameplayTagReplicationState::None);
		}
	}
	
	// Iterate backwards so we can remove during loop
	for (int32 idx=GameplayCues.Num()-1; idx >= 0; --idx)
	{
		FActiveGameplayCue& Cue = GameplayCues[idx];

		if (Cue.GameplayCueTag == Tag)
		{
			if (GameplayCueInterfacePrivate::bUseEqualTagCountAndRemovalCallbacks)
			{
				++CountDelta;
				Owner->UpdateTagMap(Tag, -1, EGameplayTagReplicationState::None);
			}

			Owner->InvokeGameplayCueEvent(Tag, EGameplayCueEvent::Removed, Cue.Parameters);
			GameplayCues.RemoveAt(idx);
		}
	}

	if (CountDelta > 0)
	{
		MarkArrayDirty();
		Owner->ForceReplication();
	}
}

void FActiveGameplayCueContainer::RemoveAllCues()
{
	if (!Owner)
	{
		return;
	}

	for (int32 idx=0; idx < GameplayCues.Num(); ++idx)
	{
		FActiveGameplayCue& Cue = GameplayCues[idx];
		Owner->UpdateTagMap(Cue.GameplayCueTag, -1, EGameplayTagReplicationState::None);
		Owner->InvokeGameplayCueEvent(Cue.GameplayCueTag, EGameplayCueEvent::Removed, Cue.Parameters);
	}
}

void FActiveGameplayCueContainer::PredictiveRemove(const FGameplayTag& Tag)
{
	if (!Owner)
	{
		return;
	}
	

	// Predictive remove: we are predicting the removal of a replicated cue
	// (We are not predicting the removal of a predictive cue. The predictive cue will be implicitly removed when the prediction key catched up)
	for (int32 idx=0; idx < GameplayCues.Num(); ++idx)
	{
		// "Which" cue we predictively remove is only based on the tag and not already being predictively removed.
		// Since there are no handles/identifies for the items in this container, we just go with the first.
		FActiveGameplayCue& Cue = GameplayCues[idx];
		if (Cue.GameplayCueTag == Tag && !Cue.bPredictivelyRemoved)
		{
			Cue.bPredictivelyRemoved = true;
			Owner->UpdateTagMap(Tag, -1, EGameplayTagReplicationState::None);
			Owner->InvokeGameplayCueEvent(Tag, EGameplayCueEvent::Removed, Cue.Parameters);	
			return;
		}
	}
}

void FActiveGameplayCueContainer::PredictiveAdd(const FGameplayTag& Tag, FPredictionKey& PredictionKey)
{
	if (!Owner)
	{
		return;
	}

	Owner->UpdateTagMap(Tag, 1, EGameplayTagReplicationState::None);
	PredictionKey.NewRejectOrCaughtUpDelegate(FPredictionKeyEvent::CreateUObject(ToRawPtr(Owner), &UAbilitySystemComponent::OnPredictiveGameplayCueCatchup, Tag));
}

bool FActiveGameplayCueContainer::HasCue(const FGameplayTag& Tag) const
{
	for (int32 idx=0; idx < GameplayCues.Num(); ++idx)
	{
		const FActiveGameplayCue& Cue = GameplayCues[idx];
		if (Cue.GameplayCueTag == Tag)
		{
			return true;
		}
	}

	return false;
}

bool FActiveGameplayCueContainer::ShouldReplicate() const
{
	if (bMinimalReplication && (Owner && Owner->ReplicationMode == EGameplayEffectReplicationMode::Full))
	{
		return false;
	}

	return true;
}

bool FActiveGameplayCueContainer::NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms)
{
	if (!ShouldReplicate())
	{
		return false;
	}

	return FastArrayDeltaSerialize<FActiveGameplayCue>(GameplayCues, DeltaParms, *this);
}

void FActiveGameplayCueContainer::SetOwner(UAbilitySystemComponent* InOwner)
{
	Owner = InOwner;
	
	// If we already have cues, pretend they were just added
	for (FActiveGameplayCue& Cue : GameplayCues)
	{
		Cue.PostReplicatedAdd(*this);
	}
}

UAbilitySystemComponent* FActiveGameplayCueContainer::GetOwner() const
{
	return Owner;
}


// ----------------------------------------------------------------------------------------

FMinimalGameplayCueReplicationProxy::FMinimalGameplayCueReplicationProxy()
{
	InitGameplayCueParametersFunc = [](FGameplayCueParameters& GameplayCueParameters, UAbilitySystemComponent* InOwner)
	{
		if (InOwner)
		{
			InOwner->InitDefaultGameplayCueParameters(GameplayCueParameters);
		}
	};
}

void FMinimalGameplayCueReplicationProxy::SetOwner(UAbilitySystemComponent* ASC)
{
	Owner = ASC;
	if (Owner && ReplicatedTags.Num() > 0)
	{
		const bool bLocationsValid = ensure(ReplicatedLocations.Num() >= ReplicatedTags.Num());

		// Invoke events in case we skipped them during ::NetSerialize
		FGameplayCueParameters Parameters;
		InitGameplayCueParametersFunc(Parameters, Owner);

		const FVector OriginalLocationParameter = Parameters.Location;

		for (int32 idx=0; idx < ReplicatedTags.Num(); ++idx)
		{
			if (bLocationsValid)
			{
				Parameters.Location = ReplicatedLocations[idx].IsZero() ? OriginalLocationParameter : ReplicatedLocations[idx];
			}
			Owner->SetTagMapCount(ReplicatedTags[idx], 1);
			Owner->InvokeGameplayCueEvent(ReplicatedTags[idx], EGameplayCueEvent::WhileActive, Parameters);
		}
	}
}

void FMinimalGameplayCueReplicationProxy::PreReplication(const FActiveGameplayCueContainer& SourceContainer)
{
	if (LastSourceArrayReplicationKey != SourceContainer.ArrayReplicationKey)
	{
		LastSourceArrayReplicationKey = SourceContainer.ArrayReplicationKey;

		if (UE_LOG_ACTIVE(LogGameplayCues, CANTREPLICATEPARAMS_SEVERITY))
		{
			// Check if any new cue has parameters that we won't be able to replicate and warn the user
			FGameplayCueParameters ParamsThatCanReplicate;
			for (const FActiveGameplayCue& Cue : SourceContainer.GameplayCues)
			{
				// Look for new tags
				if (!ReplicatedTags.Contains(Cue.GameplayCueTag))
				{
					ParamsThatCanReplicate.bReplicateLocationWhenUsingMinimalRepProxy = Cue.Parameters.bReplicateLocationWhenUsingMinimalRepProxy;
					ParamsThatCanReplicate.Location = ParamsThatCanReplicate.bReplicateLocationWhenUsingMinimalRepProxy ? Cue.Parameters.Location : FVector::ZeroVector;

					// Those don't replicate
					ParamsThatCanReplicate.MatchedTagName = Cue.Parameters.MatchedTagName;
					ParamsThatCanReplicate.OriginalTag = Cue.Parameters.OriginalTag;

					// Effect context is default constructed at some places so do not warn about it
					ParamsThatCanReplicate.EffectContext = Cue.Parameters.EffectContext;

					if (Cue.Parameters != ParamsThatCanReplicate)
					{
						TPropertyValueIterator<const FProperty> CueParams(FGameplayCueParameters::StaticStruct(), &Cue.Parameters);
						TPropertyValueIterator<const FProperty> ReplicatedParams(FGameplayCueParameters::StaticStruct(), &ParamsThatCanReplicate);

						for (; CueParams && ReplicatedParams; ++CueParams, ++ReplicatedParams)
						{
							if (!ensure(CueParams->Key != nullptr))
							{
								continue;
							}

							if (!CueParams->Key->Identical(CueParams->Value, ReplicatedParams->Value))
							{
								UE_LOGF(LogGameplayCues, CANTREPLICATEPARAMS_SEVERITY, "%s: Parameter %ls on gameplay cue %ls is set but cannot be replicated", __FUNCTION__, *CueParams->Key->GetName(), *Cue.GameplayCueTag.ToString());
							}
						}
					}
				}
			}
		}

		ReplicatedTags.SetNum(SourceContainer.GameplayCues.Num(), EAllowShrinking::No);
		ReplicatedLocations.SetNum(SourceContainer.GameplayCues.Num(), EAllowShrinking::No);
		for (int32 idx=0; idx < SourceContainer.GameplayCues.Num(); ++idx)
		{
			ReplicatedTags[idx] = SourceContainer.GameplayCues[idx].GameplayCueTag;
			if (SourceContainer.GameplayCues[idx].Parameters.bReplicateLocationWhenUsingMinimalRepProxy)
			{
				ReplicatedLocations[idx] = SourceContainer.GameplayCues[idx].Parameters.Location;
			}
			else
			{
				ReplicatedLocations[idx] = FVector::ZeroVector;
			}
		}
	}
}

bool FMinimalGameplayCueReplicationProxy::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	enum { NumBits = 5 }; // Number of bits to use for number of array
	enum { MaxNum = (1 << NumBits) -1 }; // Number of bits to use for number of array

	uint8 NumElements;

	if (Ar.IsSaving())
	{
		NumElements = ReplicatedTags.Num();
		if (NumElements > MaxNum)
		{
			FString Str;
			for (const FGameplayTag& Tag : ReplicatedTags)
			{
				Str += Tag.ToString() + TEXT(" ");
			}
			ABILITY_LOG(Warning, TEXT("Too many tags in ReplicatedTags on %s. %d total: %s. Dropping"), *GetPathNameSafe(Owner), NumElements, *Str);
			NumElements = MaxNum;
			ReplicatedTags.SetNum(NumElements);
		}

		Ar.SerializeBits(&NumElements, NumBits);

		for (uint8 i=0; i < NumElements; ++i)
		{
			ReplicatedTags[i].NetSerialize(Ar, Map, bOutSuccess);
			if (ReplicatedLocations[i].IsZero())
			{
				bool bHasLocation = false;
				Ar << bHasLocation;
			}
			else
			{
				bool bHasLocation = true;
				Ar << bHasLocation;
				ReplicatedLocations[i].NetSerialize(Ar, Map, bOutSuccess);
			}
		}
	}
	else
	{
		// Only actually update the owner's tag map if we not the 
		bool UpdateOwnerTagMap = Owner != nullptr;
		if (bRequireNonOwningNetConnection && Owner)
		{
			if (AActor* OwningActor = Owner->GetOwner())
			{
				// Note we deliberately only want to do this if the NetConnection is not null
				if (UNetConnection* OwnerNetConnection = OwningActor->GetNetConnection()) 
				{
					if (OwnerNetConnection == CastChecked<UPackageMapClient>(Map)->GetConnection())
					{
						UpdateOwnerTagMap = false;
					}
				}
			}
		}


		NumElements = 0;
		Ar.SerializeBits(&NumElements, NumBits);

		LocalTags = MoveTemp(ReplicatedTags);
		LocalBitMask.Init(true, LocalTags.Num());
		
		ReplicatedTags.SetNumUninitialized(NumElements, EAllowShrinking::No);
		ReplicatedLocations.SetNum(NumElements, EAllowShrinking::No);

		// This struct does not serialize GC parameters but will synthesize them on the receiving side.
		FGameplayCueParameters Parameters;
		InitGameplayCueParametersFunc(Parameters, Owner);
		FVector OriginalLocationParameter = Parameters.Location;

		for (uint8 i=0; i < NumElements; ++i)
		{
			FGameplayTag& ReplicatedTag = ReplicatedTags[i];

			ReplicatedTag.NetSerialize(Ar, Map, bOutSuccess);

			bool bHasReplicatedLocation = false;
			Ar << bHasReplicatedLocation;
			if (bHasReplicatedLocation)
			{
				FVector_NetQuantize& ReplicatedLocation = ReplicatedLocations[i];
				ReplicatedLocation.NetSerialize(Ar, Map, bOutSuccess);
				Parameters.Location = ReplicatedLocation;
			}
			else
			{
				Parameters.Location = OriginalLocationParameter;
			}

			int32 LocalIdx = LocalTags.IndexOfByKey(ReplicatedTag);
			if (LocalIdx != INDEX_NONE)
			{
				// This tag already existed and is accounted for
				LocalBitMask[LocalIdx] = false;
			}
			else if (UpdateOwnerTagMap)
			{
				bCachedModifiedOwnerTags = true;
				// This is a new tag, we need to invoke the WhileActive gameplaycue event
				Owner->SetTagMapCount(ReplicatedTag, 1);
				Owner->InvokeGameplayCueEvent(ReplicatedTag, EGameplayCueEvent::WhileActive, Parameters);

				// The demo recorder needs to believe that this structure is dirty so it will get saved into the demo stream
				LastSourceArrayReplicationKey++;
			}
		}

		// Restore the location in case we touched it
		Parameters.Location = OriginalLocationParameter;

		if (UpdateOwnerTagMap)
		{
			bCachedModifiedOwnerTags = true;
			for (TConstSetBitIterator<TInlineAllocator<NumInlineTags>> It(LocalBitMask); It; ++It)
			{
				FGameplayTag& RemovedTag = LocalTags[It.GetIndex()];
				Owner->SetTagMapCount(RemovedTag, 0);
				Owner->InvokeGameplayCueEvent(RemovedTag, EGameplayCueEvent::Removed, Parameters);

				// The demo recorder needs to believe that this structure is dirty so it will get saved into the demo stream
				LastSourceArrayReplicationKey++;
			}
		}
	}


	bOutSuccess = true;
	return true;
}

void FMinimalGameplayCueReplicationProxy::RemoveAllCues()
{
	if (!Owner || !bCachedModifiedOwnerTags)
	{
		return;
	}

	FGameplayCueParameters Parameters;
	InitGameplayCueParametersFunc(Parameters, Owner);

	for (int32 idx=0; idx < ReplicatedTags.Num(); ++idx)
	{
		const FGameplayTag& GameplayCueTag = ReplicatedTags[idx];
		Owner->SetTagMapCount(GameplayCueTag, 0);
		Owner->InvokeGameplayCueEvent(GameplayCueTag, EGameplayCueEvent::Removed, Parameters);
	}
}

#undef CANTREPLICATEPARAMS_SEVERITY