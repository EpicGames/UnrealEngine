// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueManager.h"
#include "Engine/Blueprint.h"
#include "Engine/ObjectLibrary.h"
#include "GameplayCueNotify_Actor.h"
#include "Misc/MessageDialog.h"
#include "Stats/StatsMisc.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "DrawDebugHelpers.h"
#include "GameplayTagsManager.h"
#include "GameplayTagsModule.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "GameplayCueInterface.h"
#include "GameplayCueSet.h"
#include "GameplayCueNotify_Static.h"
#include "AbilitySystemComponent.h"
#include "Net/DataReplication.h"
#include "Engine/ActorChannel.h"
#include "Engine/NetConnection.h"
#include "Net/UnrealNetwork.h"
#include "Misc/CoreDelegates.h"
#include "AbilitySystemReplicationProxyInterface.h"
#include "UObject/LinkerLoad.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueManager)

#if WITH_EDITOR
#include "Editor.h"
#include "Engine/Engine.h"
#include "ISequenceRecorder.h"
#define LOCTEXT_NAMESPACE "GameplayCueManager"
#endif

int32 LogGameplayCueActorSpawning = 0;
static FAutoConsoleVariableRef CVarLogGameplayCueActorSpawning(TEXT("AbilitySystem.LogGameplayCueActorSpawning"),	LogGameplayCueActorSpawning, TEXT("Log when we create GameplayCueNotify_Actors"), ECVF_Default	);

int32 DisplayGameplayCues = 0;
static FAutoConsoleVariableRef CVarDisplayGameplayCues(TEXT("AbilitySystem.DisplayGameplayCues"),	DisplayGameplayCues, TEXT("Display GameplayCue events in world as text."), ECVF_Default	);

int32 DisableGameplayCues = 0;
static FAutoConsoleVariableRef CVarDisableGameplayCues(TEXT("AbilitySystem.DisableGameplayCues"),	DisableGameplayCues, TEXT("Disables all GameplayCue events in the world."), ECVF_Default );

float DisplayGameplayCueDuration = 5.f;
static FAutoConsoleVariableRef CVarDurationeGameplayCues(TEXT("AbilitySystem.GameplayCue.DisplayDuration"),	DisplayGameplayCueDuration, TEXT("Disables all GameplayCue events in the world."), ECVF_Default );

int32 GameplayCueRunOnDedicatedServer = 0;
static FAutoConsoleVariableRef CVarDedicatedServerGameplayCues(TEXT("AbilitySystem.GameplayCue.RunOnDedicatedServer"), GameplayCueRunOnDedicatedServer, TEXT("Run gameplay cue events on dedicated server"), ECVF_Default );

bool EnableSuppressCuesOnGameplayCueManager = true;
static FAutoConsoleVariableRef CVarEnableSuppressCuesOnGameplayCueManager(TEXT("AbilitySystem.GameplayCue.EnableSuppressCuesOnGameplayCueManager"), EnableSuppressCuesOnGameplayCueManager, TEXT("Allows the GameplayCueManager to suppress cues when the bSuppressGameplayCues is set on the target AbilitySystemComponent"), ECVF_Default );

#if WITH_EDITOR
USceneComponent* UGameplayCueManager::PreviewComponent = nullptr;
UWorld* UGameplayCueManager::PreviewWorld = nullptr;
FGameplayCueProxyTick UGameplayCueManager::PreviewProxyTick;
#endif

UWorld* UGameplayCueManager::CurrentWorld = nullptr;

UGameplayCueManager::UGameplayCueManager(const FObjectInitializer& PCIP)
: Super(PCIP)
{
#if WITH_EDITOR
	bAccelerationMapOutdated = true;
	EditorObjectLibraryFullyInitialized = false;
#endif
}

void UGameplayCueManager::OnCreated()
{
	FWorldDelegates::OnPostWorldCleanup.AddUObject(this, &UGameplayCueManager::OnPostWorldCleanup);
	FNetworkReplayDelegates::OnPreScrub.AddUObject(this, &UGameplayCueManager::OnPreReplayScrub);
		
#if WITH_EDITOR
	if (GIsRunning)
	{
		// Engine init already completed
		OnEngineInitComplete();
	}
	else
	{
		FCoreDelegates::OnFEngineLoopInitComplete.AddUObject(this, &UGameplayCueManager::OnEngineInitComplete);
	}
#endif
}

void UGameplayCueManager::OnEngineInitComplete()
{
#if WITH_EDITOR
	FCoreDelegates::OnFEngineLoopInitComplete.AddUObject(this, &UGameplayCueManager::OnEngineInitComplete);
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().OnInMemoryAssetCreated().AddUObject(this, &UGameplayCueManager::HandleAssetAdded);
	AssetRegistryModule.Get().OnInMemoryAssetDeleted().AddUObject(this, &UGameplayCueManager::HandleAssetDeleted);
	AssetRegistryModule.Get().OnAssetRenamed().AddUObject(this, &UGameplayCueManager::HandleAssetRenamed);
	FWorldDelegates::OnPreWorldInitialization.AddUObject(this, &UGameplayCueManager::ReloadObjectLibrary);

	if (GIsEditor)
	{
		InitializeEditorObjectLibrary();
	}
#endif
}

bool IsDedicatedServerForGameplayCue()
{
#if WITH_EDITOR
	// This will handle dedicated server PIE case properly
	return GEngine->ShouldAbsorbCosmeticOnlyEvent();
#else
	// When in standalone non editor, this is the fastest way to check
	return IsRunningDedicatedServer();
#endif
}


void UGameplayCueManager::HandleGameplayCues(AActor* TargetActor, const FGameplayTagContainer& GameplayCueTags, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters, EGameplayCueExecutionOptions Options)
{
#if WITH_EDITOR
	if (GIsEditor && TargetActor == nullptr && UGameplayCueManager::PreviewComponent)
	{
		TargetActor = GetMutableDefault<AActor>();
	}
#endif

	if (!(Options & EGameplayCueExecutionOptions::IgnoreSuppression) && ShouldSuppressGameplayCues(TargetActor))
	{
		return;
	}

	for (auto It = GameplayCueTags.CreateConstIterator(); It; ++It)
	{
		Parameters.OriginalTag = *It;
		HandleGameplayCue(TargetActor, *It, EventType, Parameters, Options);
	}
}

void UGameplayCueManager::HandleGameplayCue(AActor* TargetActor, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters, EGameplayCueExecutionOptions Options)
{
#if WITH_SERVER_CODE
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GameplayCueManager_HandleGameplayCue);
#endif

#if WITH_EDITOR
	if (RuntimeGameplayCueObjectLibrary.bHasBeenInitialized == false && ShouldDeferScanningRuntimeLibraries())
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		if (AssetRegistryModule.Get().IsGathering())
		{
			ABILITY_LOG(Warning, TEXT("Waiting for AR scan in HandleGameplayCue..."));
			// This will trigger the callback to InitializeRuntimeObjectLibrary via OnKnownGathersComplete
			AssetRegistryModule.Get().WaitForCompletion();
		}
	}

	if (GIsEditor && TargetActor == nullptr && UGameplayCueManager::PreviewComponent)
	{
		TargetActor = Cast<AActor>(AActor::StaticClass()->GetDefaultObject());
	}
#endif

	if (!(Options & EGameplayCueExecutionOptions::IgnoreSuppression) && ShouldSuppressGameplayCues(TargetActor))
	{
		return;
	}

	if (!(Options & EGameplayCueExecutionOptions::IgnoreTranslation))
	{
		TranslateGameplayCue(GameplayCueTag, TargetActor, Parameters);
	}
	
	RouteGameplayCue(TargetActor, GameplayCueTag, EventType, Parameters, Options);
}

bool UGameplayCueManager::ShouldSuppressGameplayCues(AActor* TargetActor)
{
	if (DisableGameplayCues ||
		!TargetActor ||
		(GameplayCueRunOnDedicatedServer == 0 && IsDedicatedServerForGameplayCue()))
	{
		return true;
	}

	return false;

}

void UGameplayCueManager::RouteGameplayCue(AActor* TargetActor, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters, EGameplayCueExecutionOptions Options)
{
#if WITH_SERVER_CODE
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GameplayCueManager_RouteGameplayCue);
#endif

	if (!Parameters.OriginalTag.IsValid())
	{
		Parameters.OriginalTag = GameplayCueTag;
	}

	// If we want to ignore interfaces, set the pointer to null
	IGameplayCueInterface* GameplayCueInterface = !(Options & EGameplayCueExecutionOptions::IgnoreInterfaces) ? Cast<IGameplayCueInterface>(TargetActor) : nullptr;
	bool bAcceptsCue = true;
	if (GameplayCueInterface)
	{
		bAcceptsCue = GameplayCueInterface->ShouldAcceptGameplayCue(TargetActor, GameplayCueTag, EventType, Parameters);
	}

#if !UE_BUILD_SHIPPING
	if (OnRouteGameplayCue.IsBound())
	{
		OnRouteGameplayCue.Broadcast(TargetActor, GameplayCueTag, EventType, Parameters, Options);
	}
#endif // !UE_BUILD_SHIPPING

#if ENABLE_DRAW_DEBUG
	if (DisplayGameplayCues && !(Options & EGameplayCueExecutionOptions::IgnoreDebug))
	{
		FString DebugStr = FString::Printf(TEXT("[%s] %s - %s"), *GetNameSafe(TargetActor), *GameplayCueTag.ToString(), *EGameplayCueEventToString(EventType) );
		FColor DebugColor = EventType == EGameplayCueEvent::Removed ? FColor::Red : FColor::Green;
		DrawDebugString(TargetActor->GetWorld(), FVector(0.f, 0.f, -10.0f * static_cast<int>(EventType)), DebugStr, TargetActor, DebugColor, DisplayGameplayCueDuration);
		ABILITY_LOG(Display, TEXT("%s"), *DebugStr);
	}
#endif // ENABLE_DRAW_DEBUG

	CurrentWorld = TargetActor->GetWorld();

	// Don't handle gameplay cues when world is tearing down
	if (!GetWorld() || GetWorld()->bIsTearingDown)
	{
		return;
	}

	// Give the global set a chance
	if (bAcceptsCue && !(Options & EGameplayCueExecutionOptions::IgnoreNotifies))
	{
		RuntimeGameplayCueObjectLibrary.CueSet->HandleGameplayCue(TargetActor, GameplayCueTag, EventType, Parameters);
	}

	// Use the interface even if it's not in the map
	if (GameplayCueInterface && bAcceptsCue)
	{
		GameplayCueInterface->HandleGameplayCue(TargetActor, GameplayCueTag, EventType, Parameters);
	}

	// This is to force client side replays to record the target of the GC on the next frame. (ForceNetUpdates from the server will not translate into ForceNetUpdates on the client/replays)
	TargetActor->ForceNetUpdate();

	CurrentWorld = nullptr;
}

void UGameplayCueManager::TranslateGameplayCue(FGameplayTag& Tag, AActor* TargetActor, const FGameplayCueParameters& Parameters)
{
#if WITH_SERVER_CODE
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GameplayCueManager_TranslateGameplayCue);
#endif

	TranslationManager.TranslateTag(Tag, TargetActor, Parameters);
}

void UGameplayCueManager::AddGameplayCue_NonReplicated(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters)
{
	if (UAbilitySystemComponent* ASC = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Target))
	{
		ASC->AddLooseGameplayTag(GameplayCueTag);
	}

	if (UGameplayCueManager* GCM = UAbilitySystemGlobals::Get().GetGameplayCueManager())
	{
		GCM->HandleGameplayCue(Target, GameplayCueTag, EGameplayCueEvent::OnActive, Parameters);
		GCM->HandleGameplayCue(Target, GameplayCueTag, EGameplayCueEvent::WhileActive, Parameters);
	}
}

void UGameplayCueManager::RemoveGameplayCue_NonReplicated(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters)
{
	if (UAbilitySystemComponent* ASC = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Target))
	{
		ASC->RemoveLooseGameplayTag(GameplayCueTag);
	}

	if (UGameplayCueManager* GCM = UAbilitySystemGlobals::Get().GetGameplayCueManager())
	{
		GCM->HandleGameplayCue(Target, GameplayCueTag, EGameplayCueEvent::Removed, Parameters);
	}
}

void UGameplayCueManager::ExecuteGameplayCue_NonReplicated(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters)
{
	if (UGameplayCueManager* GCM = UAbilitySystemGlobals::Get().GetGameplayCueManager())
	{
		GCM->HandleGameplayCue(Target, GameplayCueTag, EGameplayCueEvent::Executed, Parameters);
	}
}

void UGameplayCueManager::EndGameplayCuesFor(AActor* TargetActor)
{
	// Make a copy so that OnOwnerDestroyed can remove itself (if it so chooses)
	TArray<TObjectPtr<AActor>> Children = TargetActor->Children;
	for (AActor* Child : Children)
	{
		AGameplayCueNotify_Actor* NotifyActor = Cast<AGameplayCueNotify_Actor>(Child);
		if (NotifyActor)
		{
			NotifyActor->OnOwnerDestroyed(TargetActor);
		}
	}
}

int32 GameplayCueActorRecycle = 1;
static FAutoConsoleVariableRef CVarGameplayCueActorRecycle(TEXT("AbilitySystem.GameplayCueActorRecycle"), GameplayCueActorRecycle, TEXT("Allow recycling of GameplayCue Actors"), ECVF_Default );

int32 GameplayCueActorRecycleDebug = 0;
static FAutoConsoleVariableRef CVarGameplayCueActorRecycleDebug(TEXT("AbilitySystem.GameplayCueActorRecycleDebug"), GameplayCueActorRecycleDebug, TEXT("Prints logs for GC actor recycling debugging"), ECVF_Default );

bool UGameplayCueManager::IsGameplayCueRecylingEnabled()
{
	return GameplayCueActorRecycle > 0;
}

bool UGameplayCueManager::ShouldSyncLoadMissingGameplayCues() const
{
	return false;
}

bool UGameplayCueManager::ShouldAsyncLoadMissingGameplayCues() const
{
	return true;
}

bool UGameplayCueManager::IsAsyncLoadingGameplayCueNotifyClass(const FSoftObjectPath& CueNotifyClass) const
{
	return AsyncLoadPendingGameplayCues.Contains(CueNotifyClass);
}

bool UGameplayCueManager::HandleMissingGameplayCue(UGameplayCueSet* OwningSet, struct FGameplayCueNotifyData& CueData, AActor* TargetActor, EGameplayCueEvent::Type EventType, FGameplayCueParameters& Parameters)
{
	if (ShouldSyncLoadMissingGameplayCues())
	{
		CueData.LoadedGameplayCueClass = Cast<UClass>(StreamableManager.LoadSynchronous(CueData.GameplayCueNotifyObj, false));

		if (CueData.LoadedGameplayCueClass)
		{
			ABILITY_LOG(Display, TEXT("GameplayCueNotify %s was not loaded when GameplayCue was invoked, did synchronous load."), *CueData.GameplayCueNotifyObj.ToString());
			return true;
		}
		else
		{
			ABILITY_LOG(Warning, TEXT("Late load of GameplayCueNotify %s failed!"), *CueData.GameplayCueNotifyObj.ToString());
		}
	}
	else if (ShouldAsyncLoadMissingGameplayCues())
	{
		const bool bStartAsyncLoad = !IsAsyncLoadingGameplayCueNotifyClass(CueData.GameplayCueNotifyObj);
		const FAsyncLoadPendingGameplayCue PendingCue(TWeakObjectPtr<UGameplayCueSet>(OwningSet), CueData.GameplayCueTag, MakeWeakObjectPtr(TargetActor), EventType, Parameters);
		if (bStartAsyncLoad)
		{
			ABILITY_LOG(Display, TEXT("Deferred gameplay cue with tag '%s' and event type '%d' until GCN class '%s' is loaded (started async loading)."), *CueData.GameplayCueTag.ToString(), int32(EventType), *CueData.GameplayCueNotifyObj.ToString());

			// Cache the cue parameters to call it when the GCN class finishes async loading
			TArray<FAsyncLoadPendingGameplayCue>& PendingEvents = AsyncLoadPendingGameplayCues.Add(CueData.GameplayCueNotifyObj);
			PendingEvents.Add(PendingCue);

			// Start async loading the GCN class
			StreamableManager.RequestAsyncLoad(CueData.GameplayCueNotifyObj, FStreamableDelegate::CreateUObject(this, &UGameplayCueManager::OnMissingCueAsyncLoadComplete, CueData.GameplayCueNotifyObj));
		}
		else
		{
			ABILITY_LOG(Display, TEXT("Deferred gameplay cue with tag '%s' and event type '%d' until GCN class '%s' finishes async loading (already loading)."), *CueData.GameplayCueTag.ToString(), int32(EventType), *CueData.GameplayCueNotifyObj.ToString());

			// Cache the cue parameters to call it when the GCN class finishes async loading
			AsyncLoadPendingGameplayCues[CueData.GameplayCueNotifyObj].Add(PendingCue);
		}
	}
	return false;
}

void UGameplayCueManager::OnMissingCueAsyncLoadComplete(FSoftObjectPath LoadedNotifyClass)
{
	if (!LoadedNotifyClass.ResolveObject())
	{
		// Load failed
		ABILITY_LOG(Warning, TEXT("Late load of GameplayCueNotify %s failed!"), *LoadedNotifyClass.ToString());
		return;
	}

	if (ensureMsgf(AsyncLoadPendingGameplayCues.Contains(LoadedNotifyClass), TEXT("Expected an entry for '%s' that was created when async loading for it started"), *LoadedNotifyClass.ToString()))
	{
		// Keep the array of the pending cue events and their parameters. Remove the GCN class from the map of pending events 
		// because we're no longer async loading the class: the calls to HandleGameplayCue that we're doing now should execute directly.
		TArray<FAsyncLoadPendingGameplayCue> PendingEventsCopy = MoveTemp(AsyncLoadPendingGameplayCues[LoadedNotifyClass]);
		AsyncLoadPendingGameplayCues.Remove(LoadedNotifyClass);

		ABILITY_LOG(Display, TEXT("Finished async loading GameplayCueNotify class '%s'. Executing %d deferred gameplay cues."), *LoadedNotifyClass.ToString(), PendingEventsCopy.Num());

		// Call all pending cues for the GCN class that finished loading
		for (const FAsyncLoadPendingGameplayCue& PendingEvent : PendingEventsCopy)
		{
			// Owning set may have been GCed in the meantime
			if (PendingEvent.OwningSet.IsValid())
			{
				CurrentWorld = PendingEvent.TargetActor.IsValid() ? PendingEvent.TargetActor->GetWorld() : nullptr;
				if (!CurrentWorld)
				{
					// TargetActor has since been destroyed.  Attempt to get the world from the other actors.
					const AActor* CueInstigator = PendingEvent.Parameters.GetInstigator();
					CurrentWorld = CueInstigator ? CueInstigator->GetWorld() : nullptr;
					if (!CurrentWorld)
					{
						const AActor* EffectCauser = PendingEvent.Parameters.GetEffectCauser();
						CurrentWorld = EffectCauser ? EffectCauser->GetWorld() : nullptr;
					}
				}

				// Don't handle gameplay cues for worlds that are tearing down
				if (!GetWorld() || GetWorld()->bIsTearingDown)
				{
					ABILITY_LOG(Verbose, TEXT("Skipping deferred gameplay cue due to invalid world. GCN class: %s | GameplayCueTag: %s | EventType: %d"), *LoadedNotifyClass.ToString(), *PendingEvent.GameplayCueTag.ToString(), int32(PendingEvent.EventType));
					continue;
				}

				// Objects are still valid, re-execute cue
				ABILITY_LOG(Verbose, TEXT("Executing deferred gameplay cue. GCN class: %s | GameplayCueTag: %s | EventType: %d"), *LoadedNotifyClass.ToString(), *PendingEvent.GameplayCueTag.ToString(), int32(PendingEvent.EventType));
				PendingEvent.OwningSet->HandleGameplayCue(PendingEvent.TargetActor.Get(), PendingEvent.GameplayCueTag, PendingEvent.EventType, PendingEvent.Parameters);

				CurrentWorld = nullptr;
			}
		}
	}
}

AGameplayCueNotify_Actor* UGameplayCueManager::FindExistingCueOnActor(const AActor& TargetActor, const TSubclassOf<AGameplayCueNotify_Actor>& CueClass, const FGameplayCueParameters& Parameters) const
{
	for (AActor* Child : TargetActor.Children)
	{
		if (IsValid(Child) && Child->IsA(CueClass))
		{
			AGameplayCueNotify_Actor* ChildNotify = CastChecked<AGameplayCueNotify_Actor>(Child);

			// Somehow the LifeSpan can end up being zero, meaning we're about to be destroyed (so don't reuse)
			if (ChildNotify->GameplayCuePendingRemove())
			{
				UE_LOGF(LogAbilitySystem, Verbose, "FindExistingCueActor considered %ls, but it was pending remove", *GetNameSafe(ChildNotify));
				continue;
			}

			const bool bInstigatorMatches = !ChildNotify->bUniqueInstancePerInstigator || ChildNotify->CueInstigator == Parameters.GetInstigator();
			const bool bSourceMatches = !ChildNotify->bUniqueInstancePerSourceObject || ChildNotify->CueSourceObject == Parameters.GetSourceObject();
			if (bInstigatorMatches && bSourceMatches)
			{
				return ChildNotify;
			}
		}
	}

	return nullptr;
}

AGameplayCueNotify_Actor* UGameplayCueManager::GetInstancedCueActor(AActor* TargetActor, UClass* GameplayCueNotifyActorClass, const FGameplayCueParameters& Parameters)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GameplayCueManager_GetInstancedCueActor);

	const TSubclassOf<AGameplayCueNotify_Actor> CueClass = GameplayCueNotifyActorClass;
	if (!ensure(TargetActor) || !ensure(CueClass))
	{
		UE_LOGF(LogAbilitySystem, Error, "GetInstancedCueActor called with invalid parameters (TargetActor = %ls, CueClass = %ls)", *GetNameSafe(TargetActor), *GetNameSafe(GameplayCueNotifyActorClass));
		return nullptr;
	}

	if (TargetActor && TargetActor->GetActorTransform().ContainsNaN())
	{
		UE_LOGF(LogAbilitySystem, Error, "GetInstancedCueActor called with invalid target actor transform (TargetActor = %ls)", *GetNameSafe(TargetActor));
		return nullptr;
	}

	// There used to be special code here to handle the case where the TargetActor was a CDO.  I'm not sure why that would be (or how that's even possible -- perhaps in the default Blueprint Viewport? But I can't trigger it.)
	// Let's log it in case a user comes across this issue.
	//	Animtion preview hack. If we are trying to play the GC on a CDO, then don't use actor recycling and don't set the owner (to the CDO, which would cause problems)
	//	And we'll try to manually find (and reuse) the existing instance on the TargetActor.
	UE_CLOGF(WITH_EDITOR && TargetActor->HasAnyFlags(RF_ClassDefaultObject), LogAbilitySystem, Warning, "Adding %ls to CDO %ls. This used to be explicitly disallowed.", *GetNameSafe(CueClass), *GetNameSafe(TargetActor));

	UWorld* World = TargetActor->GetWorld();
	if (!World)
	{
		UE_LOGF(LogAbilitySystem, Warning, "GetInstancedCueActor called on TargetActor %ls which did not belong to a world (is it a CDO or being destroyed?)", *GetNameSafe(TargetActor));
		return nullptr;
	}
	UE_CLOGF(CurrentWorld != World, LogAbilitySystem, Error, "GetInstancedCueActor had CurrentWorld set to %ls but TargetActor is in World %ls", *GetNameSafe(CurrentWorld), *GetNameSafe(World));

	// We found the exact Cue we're looking for already on the TargetActor.  Use that.
	AGameplayCueNotify_Actor* ExistingCueOnActor = FindExistingCueOnActor(*TargetActor, CueClass, Parameters);
	if (ExistingCueOnActor)
	{
		ExistingCueOnActor->CueInstigator = Parameters.GetInstigator();
		ExistingCueOnActor->CueSourceObject = Parameters.GetSourceObject();
		return ExistingCueOnActor;
	}

	const bool bUseActorRecycling = (GameplayCueActorRecycle > 0);
	if (bUseActorRecycling)
	{
		if (AGameplayCueNotify_Actor* RecycledCue = FindRecycledCue(CueClass, *World))
		{
			RecycledCue->bInRecycleQueue = false;
			RecycledCue->SetOwner(TargetActor);
			RecycledCue->SetActorLocationAndRotation(TargetActor->GetActorLocation(), TargetActor->GetActorRotation());
			RecycledCue->ReuseAfterRecycle();
			RecycledCue->CueInstigator = Parameters.GetInstigator();
			RecycledCue->CueSourceObject = Parameters.GetSourceObject();

			UE_CLOGF((GameplayCueActorRecycleDebug > 0), LogAbilitySystem, Display, "GetInstancedCueActor reusing Recycled CueActor: %ls", *GetNameSafe(RecycledCue));

#if WITH_EDITOR
			// let things know that we 'spawned'
			ISequenceRecorder& SequenceRecorder = FModuleManager::LoadModuleChecked<ISequenceRecorder>("SequenceRecorder");
			SequenceRecorder.NotifyActorStartRecording(RecycledCue);
#endif

			return RecycledCue;
		}
	}

	// If we can't reuse, then spawn a new one. Since TargetActor is the Owner, a reference to this CueNotify Actor will live in TargetActor::Children.
	FActorSpawnParameters SpawnParams;
	SpawnParams.Owner = TargetActor;
	SpawnParams.OverrideLevel = World->PersistentLevel;
	AGameplayCueNotify_Actor* SpawnedCue = World->SpawnActor<AGameplayCueNotify_Actor>(CueClass, TargetActor->GetActorLocation(), TargetActor->GetActorRotation(), SpawnParams);
	if (ensureMsgf(SpawnedCue != nullptr, TEXT("[%s] - Failed to spawn the cue actor! Check the log for a potential reason spawn actor failed!"), ANSI_TO_TCHAR(__FUNCTION__)))
	{
		SpawnedCue->CueInstigator = Parameters.GetInstigator();
		SpawnedCue->CueSourceObject = Parameters.GetSourceObject();
	}

	UE_CLOGF(LogGameplayCueActorSpawning > 0, LogAbilitySystem, Warning, "Spawned Gameplay Cue Notify Actor: %ls (instance %ls)", *CueClass->GetName(), *GetNameSafe(SpawnedCue));
	return SpawnedCue;
}

AGameplayCueNotify_Actor* UGameplayCueManager::FindRecycledCue(const TSubclassOf<AGameplayCueNotify_Actor>& CueClass, const UWorld& FindInWorld)
{
	FPreallocationInfo& Info = GetPreallocationInfo(&FindInWorld);
	FGameplayCueNotifyActorArray* PreallocatedList = Info.PreallocatedInstances.Find(CueClass);
	if (!PreallocatedList)
	{
		// No preallocated instances yet
		return nullptr;
	}

	while (PreallocatedList->Actors.Num() > 0)
	{
		AGameplayCueNotify_Actor* RecycledCue = PreallocatedList->Actors.Pop(EAllowShrinking::No);

		// Normal check: if cue was destroyed or is pending kill, then don't use it.
		if (IsValid(RecycledCue))
		{
			return RecycledCue;
		}
					
		// outside of replays, this should not happen. GC Notifies should not be actually destroyed.
		ensureMsgf(FindInWorld.IsPlayingReplay(), TEXT("RecycledCue is pending kill, garbage or null: %s."), *GetNameSafe(RecycledCue));
	}

	return nullptr;
}

void UGameplayCueManager::NotifyGameplayCueActorFinished(AGameplayCueNotify_Actor* Actor)
{
	if (!IsValid(Actor))
	{
		ensureMsgf(GetWorld() && GetWorld()->IsPlayingReplay(), TEXT("GameplayCueNotify %s is pending kill or garbage in ::NotifyGameplayCueActorFinished (and not in network demo)"), *GetNameSafe(Actor));
		return;
	}

	bool bUseActorRecycling = (GameplayCueActorRecycle > 0);

#if WITH_EDITOR	
	// Don't recycle in preview worlds
	if (Actor->GetWorld()->IsPreviewWorld())
	{
		bUseActorRecycling = false;
	}
#endif

	if (bUseActorRecycling)
	{
		if (Actor->bInRecycleQueue)
		{
			// We are already in the recycle queue. This can happen normally
			// (For example the GC is removed and the owner is destroyed in the same frame)
			return;
		}

		UE_CLOGF((GameplayCueActorRecycleDebug > 0), LogAbilitySystem, Warning, "Recycling CueActor %ls", *GetNameSafe(Actor));
		
		if (Actor->Recycle())
		{

			Actor->bInRecycleQueue = true;

			FPreallocationInfo& Info = GetPreallocationInfo(Actor->GetWorld());
			FGameplayCueNotifyActorArray& PreAllocatedList = Info.PreallocatedInstances.FindOrAdd(Actor->GetClass());

			// Put the actor back in the list
			if (ensureMsgf(PreAllocatedList.Actors.Contains(Actor)==false, TEXT("GC Actor PreallocationList already contains Actor %s"), *GetNameSafe(Actor)))
			{
				PreAllocatedList.Actors.Push(Actor);
			}
			
#if WITH_EDITOR
			// let things know that we 'de-spawned'
			ISequenceRecorder& SequenceRecorder	= FModuleManager::LoadModuleChecked<ISequenceRecorder>("SequenceRecorder");
			SequenceRecorder.NotifyActorStopRecording(Actor);
#endif
			return;
		}

		UE_CLOGF((GameplayCueActorRecycleDebug > 0), LogAbilitySystem, Error, "Could not Recycle CueActor %ls, Destroying", *GetNameSafe(Actor));
	}	

	// We didn't recycle, so just destroy
	Actor->Destroy();
}

void UGameplayCueManager::NotifyGameplayCueActorEndPlay(AGameplayCueNotify_Actor* Actor)
{
	if (Actor && Actor->bInRecycleQueue)
	{
		FPreallocationInfo& Info = GetPreallocationInfo(Actor->GetWorld());
		FGameplayCueNotifyActorArray& PreAllocatedList = Info.PreallocatedInstances.FindOrAdd(Actor->GetClass());
		PreAllocatedList.Actors.Remove(Actor);
	}
}

// ------------------------------------------------------------------------

bool UGameplayCueManager::ShouldSyncScanRuntimeObjectLibraries() const
{
	// Always sync scan the runtime object library
	return true;
}
bool UGameplayCueManager::ShouldSyncLoadRuntimeObjectLibraries() const
{
	// No real need to sync load it anymore
	return false;
}
bool UGameplayCueManager::ShouldAsyncLoadRuntimeObjectLibraries() const
{
	// Async load the run time library at startup
	return true;
}

bool UGameplayCueManager::ShouldDeferScanningRuntimeLibraries() const
{
	return false;
}

void UGameplayCueManager::InitializeRuntimeObjectLibrary()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	if (AssetRegistryModule.Get().IsGathering() && ShouldDeferScanningRuntimeLibraries())
	{
		ABILITY_LOG(Display, TEXT("Asset registry still gathering. Deferring InitializeRuntimeObjectLibrary."));
		if (!AssetRegistryModule.Get().OnKnownGathersComplete().IsBoundToObject(this))
		{
			AssetRegistryModule.Get().OnKnownGathersComplete().AddUObject(this, &UGameplayCueManager::InitializeRuntimeObjectLibrary);
		}
		return;
	}

	ABILITY_LOG(Display, TEXT("Processing InitializeRuntimeObjectLibrary."))

	AssetRegistryModule.Get().OnKnownGathersComplete().RemoveAll(this);

	UE_SCOPED_ENGINE_ACTIVITY(TEXT("Initializing GameplayCueManager Runtime Object Library"));

	RuntimeGameplayCueObjectLibrary.Paths = GetAlwaysLoadedGameplayCuePaths();
	if (RuntimeGameplayCueObjectLibrary.CueSet == nullptr)
	{
		RuntimeGameplayCueObjectLibrary.CueSet = NewObject<UGameplayCueSet>(this, TEXT("GlobalGameplayCueSet"));
	}

	RuntimeGameplayCueObjectLibrary.CueSet->Empty();
	RuntimeGameplayCueObjectLibrary.bHasBeenInitialized = true;
	
	RuntimeGameplayCueObjectLibrary.bShouldSyncScan = ShouldSyncScanRuntimeObjectLibraries();
	RuntimeGameplayCueObjectLibrary.bShouldSyncLoad = ShouldSyncLoadRuntimeObjectLibraries();
	RuntimeGameplayCueObjectLibrary.bShouldAsyncLoad = ShouldAsyncLoadRuntimeObjectLibraries();

	InitObjectLibrary(RuntimeGameplayCueObjectLibrary);
}

#if WITH_EDITOR
void UGameplayCueManager::InitializeEditorObjectLibrary()
{
	SCOPE_LOG_TIME_IN_SECONDS(*FString::Printf(TEXT("UGameplayCueManager::InitializeEditorObjectLibrary")), nullptr)

	EditorGameplayCueObjectLibrary.Paths = GetValidGameplayCuePaths();
	if (EditorGameplayCueObjectLibrary.CueSet == nullptr)
	{
		EditorGameplayCueObjectLibrary.CueSet = NewObject<UGameplayCueSet>(this, TEXT("EditorGameplayCueSet"));
	}

	EditorGameplayCueObjectLibrary.CueSet->Empty();
	EditorGameplayCueObjectLibrary.bHasBeenInitialized = true;

	// Don't load anything for the editor. Just read whatever the asset registry has.
	EditorGameplayCueObjectLibrary.bShouldSyncScan = IsRunningCommandlet();				// If we are cooking, then sync scan it right away so that we don't miss anything
	EditorGameplayCueObjectLibrary.bShouldAsyncLoad = false;
	EditorGameplayCueObjectLibrary.bShouldSyncLoad = false;

	InitObjectLibrary(EditorGameplayCueObjectLibrary);
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	if ( AssetRegistryModule.Get().IsLoadingAssets() )
	{
		// Let us know when we are done
		static FDelegateHandle DoOnce =
		AssetRegistryModule.Get().OnFilesLoaded().AddUObject(this, &UGameplayCueManager::InitializeEditorObjectLibrary);
	}
	else
	{
		EditorObjectLibraryFullyInitialized = true;
		if (EditorPeriodicUpdateHandle.IsValid())
		{
			GEditor->GetTimerManager()->ClearTimer(EditorPeriodicUpdateHandle);
			EditorPeriodicUpdateHandle.Invalidate();
		}
	}

	OnEditorObjectLibraryUpdated.Broadcast();
}

void UGameplayCueManager::RequestPeriodicUpdateOfEditorObjectLibraryWhileWaitingOnAssetRegistry()
{
	// Asset registry is still loading, so update every 15 seconds until its finished
	if (!EditorObjectLibraryFullyInitialized && !EditorPeriodicUpdateHandle.IsValid())
	{
		GEditor->GetTimerManager()->SetTimer( EditorPeriodicUpdateHandle, FTimerDelegate::CreateUObject(this, &UGameplayCueManager::InitializeEditorObjectLibrary), 15.f, true);
	}
}

void UGameplayCueManager::ReloadObjectLibrary(UWorld* World, const UWorld::InitializationValues IVS)
{
	if (bAccelerationMapOutdated)
	{
		RefreshObjectLibraries();
	}
}

void UGameplayCueManager::GetEditorObjectLibraryGameplayCueNotifyFilenames(TArray<FString>& Filenames) const
{
	if (ensure(EditorGameplayCueObjectLibrary.CueSet))
	{
		EditorGameplayCueObjectLibrary.CueSet->GetFilenames(Filenames);
	}
}

void UGameplayCueManager::LoadNotifyForEditorPreview(FGameplayTag GameplayCueTag)
{
	if (ensure(EditorGameplayCueObjectLibrary.CueSet) && ensure(RuntimeGameplayCueObjectLibrary.CueSet))
	{
		EditorGameplayCueObjectLibrary.CueSet->CopyCueDataToSetForEditorPreview(GameplayCueTag, RuntimeGameplayCueObjectLibrary.CueSet);
	}
}

#endif // WITH_EDITOR

TArray<FString> UGameplayCueManager::GetAlwaysLoadedGameplayCuePaths()
{
	return UAbilitySystemGlobals::Get().GetGameplayCueNotifyPaths();
}

void UGameplayCueManager::AddGameplayCueNotifyPath(const FString& InPath, const bool bShouldRescanCueAssets /* = true */)
{
	UAbilitySystemGlobals::Get().AddGameplayCueNotifyPath(InPath);
	const bool bAlreadyExists = RuntimeGameplayCueObjectLibrary.Paths.Find(InPath) != INDEX_NONE;
	if (!bAlreadyExists)
	{
		RuntimeGameplayCueObjectLibrary.Paths.Add(InPath);

		if(bShouldRescanCueAssets)
		{
			InitializeRuntimeObjectLibrary();
		}
	}
}

int32 UGameplayCueManager::RemoveGameplayCueNotifyPath(const FString& InPath, const bool bShouldRescanCueAssets /* = true */)
{
	int32 NumRemovedGlobal = UAbilitySystemGlobals::Get().RemoveGameplayCueNotifyPath(InPath);
	int32 NumRemovedRuntime = RuntimeGameplayCueObjectLibrary.Paths.Remove(InPath);

	ensureMsgf(NumRemovedGlobal == NumRemovedRuntime, TEXT("Unexpected number of cue paths removed for '%s'"), *InPath);
	
	if(bShouldRescanCueAssets && NumRemovedGlobal > 0)
	{
		InitializeRuntimeObjectLibrary();		
	}
	
	return NumRemovedRuntime;
}

void UGameplayCueManager::RefreshObjectLibraries()
{
	if (RuntimeGameplayCueObjectLibrary.bHasBeenInitialized)
	{
		check(RuntimeGameplayCueObjectLibrary.CueSet);
		RuntimeGameplayCueObjectLibrary.CueSet->Empty();
		InitObjectLibrary(RuntimeGameplayCueObjectLibrary);
	}

	if (EditorGameplayCueObjectLibrary.bHasBeenInitialized)
	{
		check(EditorGameplayCueObjectLibrary.CueSet);
		EditorGameplayCueObjectLibrary.CueSet->Empty();
		InitObjectLibrary(EditorGameplayCueObjectLibrary);
	}
}

TSharedPtr<FStreamableHandle> UGameplayCueManager::InitObjectLibrary(FGameplayCueObjectLibrary& Lib)
{
	TSharedPtr<FStreamableHandle> RetVal;

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Loading Library"), STAT_ObjectLibrary, STATGROUP_LoadTime);

	// Instantiate the UObjectLibraries if they aren't there already
	// When running in editor, query in-memory assets so any unsaved assets are discovered. When cooking, only query on-disk assets to avoid indeterministic results
	if (!Lib.StaticObjectLibrary)
	{
		Lib.StaticObjectLibrary = UObjectLibrary::CreateLibrary(UGameplayCueNotify_Static::StaticClass(), true, GIsEditor && !IsRunningCommandlet());
		if (GIsEditor)
		{
			Lib.StaticObjectLibrary->bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();
		}
	}
	if (!Lib.ActorObjectLibrary)
	{
		Lib.ActorObjectLibrary = UObjectLibrary::CreateLibrary(AGameplayCueNotify_Actor::StaticClass(), true, GIsEditor && !IsRunningCommandlet());
		if (GIsEditor)
		{
			Lib.ActorObjectLibrary->bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();
		}
	}	

	Lib.bHasBeenInitialized = true;

#if WITH_EDITOR
	bAccelerationMapOutdated = false;
#endif

	FScopeCycleCounterUObject PreloadScopeActor(Lib.ActorObjectLibrary);

	// ------------------------------------------------------------------------------------------------------------------
	//	Scan asset data. If bShouldSyncScan is false, whatever state the asset registry is in will be what is returned.
	// ------------------------------------------------------------------------------------------------------------------

	{
		//SCOPE_LOG_TIME_IN_SECONDS(*FString::Printf(TEXT("UGameplayCueManager::InitObjectLibraries    Actors. Paths: %s"), *FString::Join(Lib.Paths, TEXT(", "))), nullptr)
		Lib.ActorObjectLibrary->LoadBlueprintAssetDataFromPaths(Lib.Paths, Lib.bShouldSyncScan);
	}
	{
		//SCOPE_LOG_TIME_IN_SECONDS(*FString::Printf(TEXT("UGameplayCueManager::InitObjectLibraries    Objects")), nullptr)
		Lib.StaticObjectLibrary->LoadBlueprintAssetDataFromPaths(Lib.Paths, Lib.bShouldSyncScan);
	}

	// ---------------------------------------------------------
	// Sync load if told to do so	
	// ---------------------------------------------------------
	if (Lib.bShouldSyncLoad)
	{
#if STATS
		FString PerfMessage = FString::Printf(TEXT("Fully Loaded GameplayCueNotify object library"));
		SCOPE_LOG_TIME_IN_SECONDS(*PerfMessage, nullptr)
#endif
		Lib.ActorObjectLibrary->LoadAssetsFromAssetData();
		Lib.StaticObjectLibrary->LoadAssetsFromAssetData();
	}

	// ---------------------------------------------------------
	// Look for GameplayCueNotifies that handle events
	// ---------------------------------------------------------
	
	TArray<FAssetData> ActorAssetDatas;
	Lib.ActorObjectLibrary->GetAssetDataList(ActorAssetDatas);

	TArray<FAssetData> StaticAssetDatas;
	Lib.StaticObjectLibrary->GetAssetDataList(StaticAssetDatas);

	TArray<FGameplayCueReferencePair> CuesToAdd;
	TArray<FSoftObjectPath> AssetsToLoad;

	// ------------------------------------------------------------------------------------------------------------------
	// Build Cue lists for loading. Determines what from the obj library needs to be loaded
	// ------------------------------------------------------------------------------------------------------------------
	BuildCuesToAddToGlobalSet(ActorAssetDatas, GET_MEMBER_NAME_CHECKED(AGameplayCueNotify_Actor, GameplayCueName), CuesToAdd, AssetsToLoad, Lib.ShouldLoad);
	BuildCuesToAddToGlobalSet(StaticAssetDatas, GET_MEMBER_NAME_CHECKED(UGameplayCueNotify_Static, GameplayCueName), CuesToAdd, AssetsToLoad, Lib.ShouldLoad);

	const FName PropertyName = GET_MEMBER_NAME_CHECKED(AGameplayCueNotify_Actor, GameplayCueName);
	check(PropertyName == GET_MEMBER_NAME_CHECKED(UGameplayCueNotify_Static, GameplayCueName));

	// ------------------------------------------------------------------------------------------------------------------------------------
	// Add these cues to the set. The UGameplayCueSet is the data structure used in routing the gameplay cue events at runtime.
	// ------------------------------------------------------------------------------------------------------------------------------------
	UGameplayCueSet* SetToAddTo = Lib.CueSet;
	if (!SetToAddTo)
	{
		SetToAddTo = RuntimeGameplayCueObjectLibrary.CueSet;
	}
	check(SetToAddTo);
	SetToAddTo->AddCues(CuesToAdd);

	// --------------------------------------------
	// Start loading them if necessary
	// --------------------------------------------
	if (Lib.bShouldAsyncLoad)
	{
		auto ForwardLambda = [](TArray<FSoftObjectPath> AssetList, FOnGameplayCueNotifySetLoaded OnLoadedDelegate)
		{
			OnLoadedDelegate.ExecuteIfBound(AssetList);
		};

		if (AssetsToLoad.Num() > 0)
		{
			FStreamableDelegate Del = FStreamableDelegate::CreateStatic(ForwardLambda, AssetsToLoad, Lib.OnLoaded);
			GameplayCueAssetHandle = StreamableManager.RequestAsyncLoad(MoveTemp(AssetsToLoad), MoveTemp(Del), Lib.AsyncPriority);
			RetVal = GameplayCueAssetHandle;
		}
		else
		{
			// Still fire the delegate even if nothing was found to load
			Lib.OnLoaded.ExecuteIfBound(MoveTemp(AssetsToLoad));
		}
	}

	// Build Tag Translation table
	TranslationManager.BuildTagTranslationTable();
	return RetVal;
}

static FAutoConsoleVariable CVarGameplyCueAddToGlobalSetDebug(TEXT("GameplayCue.AddToGlobalSet.DebugTag"), TEXT(""), TEXT("Debug Tag adding to global set"), ECVF_Default	);

void UGameplayCueManager::BuildCuesToAddToGlobalSet(TConstArrayView<FAssetData> AssetDataList, FName TagPropertyName, TArray<FGameplayCueReferencePair>& OutCuesToAdd, TArray<FSoftObjectPath>& OutAssetsToLoad, FShouldLoadGCNotifyDelegate ShouldLoad)
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	OutAssetsToLoad.Reserve(OutAssetsToLoad.Num() + AssetDataList.Num());

	for (const FAssetData& Data: AssetDataList)
	{
		const FName FoundGameplayTag = Data.GetTagValueRef<FName>(TagPropertyName);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (CVarGameplyCueAddToGlobalSetDebug->GetString().IsEmpty() == false && FoundGameplayTag.ToString().Contains(CVarGameplyCueAddToGlobalSetDebug->GetString()))
		{
			ABILITY_LOG(Display, TEXT("Adding Tag %s to GlobalSet"), *FoundGameplayTag.ToString());
		}
#endif

		// If ShouldLoad delegate is bound and it returns false, don't load this one
		if (ShouldLoad.IsBound() && (ShouldLoad.Execute(Data, FoundGameplayTag) == false))
		{
			continue;
		}
		
		if (ShouldLoadGameplayCueAssetData(Data) == false)
		{
			continue;
		}
		
		if (!FoundGameplayTag.IsNone())
		{
			const FString GeneratedClassTag = Data.GetTagValueRef<FString>(FBlueprintTags::GeneratedClassPath);
			if (GeneratedClassTag.IsEmpty())
			{
				ABILITY_LOG(Warning, TEXT("Unable to find GeneratedClass value for AssetData %s"), *Data.GetObjectPathString());
				continue;
			}

			ABILITY_LOG(Log, TEXT("GameplayCueManager Found: %s / %s"), *FoundGameplayTag.ToString(), *GeneratedClassTag);

			FGameplayTag  GameplayCueTag = Manager.RequestGameplayTag(FoundGameplayTag, false);
			if (GameplayCueTag.IsValid())
			{
				// Add a new NotifyData entry to our flat list for this one
				FSoftObjectPath StringRef;
				StringRef.SetPath(FPackageName::ExportTextPathToObjectPath(GeneratedClassTag));
				bool bFixedUpRef = StringRef.FixupCoreRedirects();
				if (bFixedUpRef)
				{
					ABILITY_LOG(Log, TEXT("GameplayCueManager Redirected: %s -> %s"), *GeneratedClassTag, *StringRef.ToString());
				}

				OutCuesToAdd.Add(FGameplayCueReferencePair(GameplayCueTag, StringRef));

				OutAssetsToLoad.Add(StringRef);

				// Make sure core knows about this ref so it can be properly detected during cook.
				StringRef.PostLoadPath(GetLinker());
			}
			else
			{
				// Warn about this tag but only once to cut down on spam (we may build cue sets multiple times in the editor)
				static TSet<FName> WarnedTags;
				if (WarnedTags.Contains(FoundGameplayTag) == false)
				{
					ABILITY_LOG(Warning, TEXT("Found GameplayCue tag %s in asset %s but there is no corresponding tag in the GameplayTagManager."), *FoundGameplayTag.ToString(), *Data.PackageName.ToString());
					WarnedTags.Add(FoundGameplayTag);
				}
			}
		}
	}
}

int32 GameplayCueCheckForTooManyRPCs = 1;
static FAutoConsoleVariableRef CVarGameplayCueCheckForTooManyRPCs(TEXT("AbilitySystem.GameplayCueCheckForTooManyRPCs"), GameplayCueCheckForTooManyRPCs, TEXT("Warns if gameplay cues are being throttled by network code"), ECVF_Default );

void UGameplayCueManager::CheckForTooManyRPCs(FName FuncName, const FGameplayCuePendingExecute& PendingCue, const FString& CueID, const FGameplayEffectContext* EffectContext)
{
	if (GameplayCueCheckForTooManyRPCs)
	{
		static IConsoleVariable* MaxRPCPerNetUpdateCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("net.MaxRPCPerNetUpdate"));
		if (MaxRPCPerNetUpdateCVar)
		{
			AActor* Owner = PendingCue.OwningComponent ? PendingCue.OwningComponent->GetOwner() : nullptr;
			UWorld* World = Owner ? Owner->GetWorld() : nullptr;
			UNetDriver* NetDriver = World ? World->GetNetDriver() : nullptr;
			if (NetDriver)
			{
				const int32 MaxRPCs = MaxRPCPerNetUpdateCVar->GetInt();
				for (UNetConnection* ClientConnection : NetDriver->ClientConnections)
				{
					if (ClientConnection)
					{
						UActorChannel** OwningActorChannelPtr = ClientConnection->FindActorChannel(Owner);
						TSharedRef<FObjectReplicator>* ComponentReplicatorPtr = (OwningActorChannelPtr && *OwningActorChannelPtr) ? (*OwningActorChannelPtr)->ReplicationMap.Find(PendingCue.OwningComponent) : nullptr;
						if (ComponentReplicatorPtr)
						{
							const TArray<FObjectReplicator::FRPCCallInfo>& RemoteFuncInfo = (*ComponentReplicatorPtr)->RemoteFuncInfo;
							for (const FObjectReplicator::FRPCCallInfo& CallInfo : RemoteFuncInfo)
							{
								if (CallInfo.FuncName == FuncName)
								{
									if (CallInfo.Calls > MaxRPCs)
									{
										const FString Instigator = EffectContext ? EffectContext->ToString() : TEXT("None");
										ABILITY_LOG(Warning, TEXT("Attempted to fire %s when no more RPCs are allowed this net update. Max:%d Cue:%s Instigator:%s Component:%s"), *FuncName.ToString(), MaxRPCs, *CueID, *Instigator, *GetPathNameSafe(PendingCue.OwningComponent));
									
										// Returning here to only log once per offending RPC.
										return;
									}

									break;
								}
							}
						}
					}
				}
			}
		}
	}
}

void UGameplayCueManager::OnGameplayCueNotifyAsyncLoadComplete(TArray<FSoftObjectPath> AssetList)
{
	for (FSoftObjectPath StringRef : AssetList)
	{
		UClass* GCClass = FindObject<UClass>(nullptr, *StringRef.ToString());
		if (ensure(GCClass))
		{
			LoadedGameplayCueNotifyClasses.Add(GCClass);
			CheckForPreallocation(GCClass);
		}
	}
}

int32 UGameplayCueManager::FinishLoadingGameplayCueNotifies()
{
	int32 NumLoadeded = 0;
	return NumLoadeded;
}

UGameplayCueSet* UGameplayCueManager::GetRuntimeCueSet()
{
	return RuntimeGameplayCueObjectLibrary.CueSet;
}

TArray<UGameplayCueSet*> UGameplayCueManager::GetGlobalCueSets()
{
	TArray<UGameplayCueSet*> Set;
	if (RuntimeGameplayCueObjectLibrary.CueSet)
	{
		Set.Add(RuntimeGameplayCueObjectLibrary.CueSet);
	}
	if (EditorGameplayCueObjectLibrary.CueSet)
	{
		Set.Add(EditorGameplayCueObjectLibrary.CueSet);
	}
	return Set;
}

#if WITH_EDITOR

UGameplayCueSet* UGameplayCueManager::GetEditorCueSet()
{
	return EditorGameplayCueObjectLibrary.CueSet;
}

void UGameplayCueManager::HandleAssetAdded(UObject *Object)
{
	UBlueprint* Blueprint = Cast<UBlueprint>(Object);
	if (Blueprint && Blueprint->GeneratedClass)
	{
		UGameplayCueNotify_Static* StaticCDO = Cast<UGameplayCueNotify_Static>(Blueprint->GeneratedClass->GetDefaultObject(false));
		AGameplayCueNotify_Actor* ActorCDO = Cast<AGameplayCueNotify_Actor>(Blueprint->GeneratedClass->GetDefaultObject(false));
		
		if (StaticCDO || ActorCDO)
		{
			if (!Blueprint->GetOutermost()->HasAnyPackageFlags(PKG_ForDiffing) && VerifyNotifyAssetIsInValidPath(Blueprint->GetOuter()->GetPathName()))
			{
				FSoftObjectPath StringRef;
				StringRef.SetPath(Blueprint->GeneratedClass->GetPathName());

				TArray<FGameplayCueReferencePair> CuesToAdd;
				if (StaticCDO)
				{
					CuesToAdd.Add(FGameplayCueReferencePair(StaticCDO->GameplayCueTag, StringRef));
				}
				else if (ActorCDO)
				{
					CuesToAdd.Add(FGameplayCueReferencePair(ActorCDO->GameplayCueTag, StringRef));
				}

				// Make sure core knows about this ref so it can be properly detected during cook.
				StringRef.PostLoadPath(Object->GetLinker());

				for (UGameplayCueSet* Set : GetGlobalCueSets())
				{
					Set->AddCues(CuesToAdd);
				}

				OnGameplayCueNotifyAddOrRemove.Broadcast();
			}
		}
	}
}

/** Handles cleaning up an object library if it matches the passed in object */
void UGameplayCueManager::HandleAssetDeleted(UObject *Object)
{
	FSoftObjectPath StringRefToRemove;
	UBlueprint* Blueprint = Cast<UBlueprint>(Object);
	if (Blueprint && Blueprint->GeneratedClass)
	{
		UGameplayCueNotify_Static* StaticCDO = Cast<UGameplayCueNotify_Static>(Blueprint->GeneratedClass->GetDefaultObject(false));
		AGameplayCueNotify_Actor* ActorCDO = Cast<AGameplayCueNotify_Actor>(Blueprint->GeneratedClass->GetDefaultObject(false));
		
		if (StaticCDO || ActorCDO)
		{
			StringRefToRemove.SetPath(Blueprint->GeneratedClass->GetPathName());
		}
	}

	if (StringRefToRemove.IsValid())
	{
		TArray<FSoftObjectPath> StringRefs;
		StringRefs.Add(StringRefToRemove);
		
		
		for (UGameplayCueSet* Set : GetGlobalCueSets())
		{
			Set->RemoveCuesByStringRefs(StringRefs);
		}

		OnGameplayCueNotifyAddOrRemove.Broadcast();
	}
}

/** Handles cleaning up an object library if it matches the passed in object */
void UGameplayCueManager::HandleAssetRenamed(const FAssetData& Data, const FString& String)
{
	const FString ParentClassName = Data.GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);
	if (!ParentClassName.IsEmpty())
	{
		UClass* DataClass = FindObject<UClass>(nullptr, *ParentClassName);
		if (DataClass)
		{
			UGameplayCueNotify_Static* StaticCDO = Cast<UGameplayCueNotify_Static>(DataClass->GetDefaultObject(false));
			AGameplayCueNotify_Actor* ActorCDO = Cast<AGameplayCueNotify_Actor>(DataClass->GetDefaultObject(false));
			if (StaticCDO || ActorCDO)
			{
				VerifyNotifyAssetIsInValidPath(Data.PackagePath.ToString());

				for (UGameplayCueSet* Set : GetGlobalCueSets())
				{
					Set->UpdateCueByStringRefs(String + TEXT("_C"), Data.GetObjectPathString() + TEXT("_C"));
				}
				OnGameplayCueNotifyAddOrRemove.Broadcast();
			}
		}
	}
}

bool UGameplayCueManager::VerifyNotifyAssetIsInValidPath(FString Path)
{
	bool ValidPath = false;
	for (FString& str: GetValidGameplayCuePaths())
	{
		if (Path.Contains(str))
		{
			ValidPath = true;
		}
	}

	if (!ValidPath)
	{
		FString MessageTry = FString::Printf(TEXT("Warning: Invalid GameplayCue Path %s"), *Path);
		MessageTry += TEXT("\n\nGameplayCue Notifies should only be saved in the following folders:");

		ABILITY_LOG(Warning, TEXT("Warning: Invalid GameplayCuePath: %s"), *Path);
		ABILITY_LOG(Warning, TEXT("Valid Paths: "));
		for (FString& str: GetValidGameplayCuePaths())
		{
			ABILITY_LOG(Warning, TEXT("  %s"), *str);
			MessageTry += FString::Printf(TEXT("\n  %s"), *str);
		}

		MessageTry += FString::Printf(TEXT("\n\nThis asset must be moved to a valid location to work in game."));

		const FText MessageText = FText::FromString(MessageTry);
		const FText TitleText = NSLOCTEXT("GameplayCuePathWarning", "GameplayCuePathWarningTitle", "Invalid GameplayCue Path");
		FMessageDialog::Open(EAppMsgType::Ok, MessageText, TitleText);
	}

	return ValidPath;
}

#endif


UWorld* UGameplayCueManager::GetWorld() const
{
	return GetCachedWorldForGameplayCueNotifies();
}

/* static */ UWorld* UGameplayCueManager::GetCachedWorldForGameplayCueNotifies()
{
#if WITH_EDITOR
	if (PreviewWorld)
		return PreviewWorld;
#endif

	return CurrentWorld;
}

void UGameplayCueManager::PrintGameplayCueNotifyMap()
{
	if (ensure(RuntimeGameplayCueObjectLibrary.CueSet))
	{
		RuntimeGameplayCueObjectLibrary.CueSet->PrintCues();
	}
}

void UGameplayCueManager::PrintLoadedGameplayCueNotifyClasses()
{
	for (UClass* NotifyClass : LoadedGameplayCueNotifyClasses)
	{
		ABILITY_LOG(Display, TEXT("%s"), *GetNameSafe(NotifyClass));
	}
	ABILITY_LOG(Display, TEXT("%d total classes"), LoadedGameplayCueNotifyClasses.Num());
}

static void	PrintGameplayCueNotifyMapConsoleCommandFunc(UWorld* InWorld)
{
	UAbilitySystemGlobals::Get().GetGameplayCueManager()->PrintGameplayCueNotifyMap();
}

FAutoConsoleCommandWithWorld PrintGameplayCueNotifyMapConsoleCommand(
	TEXT("GameplayCue.PrintGameplayCueNotifyMap"),
	TEXT("Displays GameplayCue notify map"),
	FConsoleCommandWithWorldDelegate::CreateStatic(PrintGameplayCueNotifyMapConsoleCommandFunc)
	);

static void	PrintLoadedGameplayCueNotifyClasses(UWorld* InWorld)
{
	UAbilitySystemGlobals::Get().GetGameplayCueManager()->PrintLoadedGameplayCueNotifyClasses();
}

FAutoConsoleCommandWithWorld PrintLoadedGameplayCueNotifyClassesCommand(
	TEXT("GameplayCue.PrintLoadedGameplayCueNotifyClasses"),
	TEXT("Displays GameplayCue Notify classes that are loaded"),
	FConsoleCommandWithWorldDelegate::CreateStatic(PrintLoadedGameplayCueNotifyClasses)
	);

FScopedGameplayCueSendContext::FScopedGameplayCueSendContext()
{
	UAbilitySystemGlobals::Get().GetGameplayCueManager()->StartGameplayCueSendContext();
}
FScopedGameplayCueSendContext::~FScopedGameplayCueSendContext()
{
	UAbilitySystemGlobals::Get().GetGameplayCueManager()->EndGameplayCueSendContext();
}

template<class AllocatorType>
void PullGameplayCueTagsFromSpec(const FGameplayEffectSpec& Spec, TArray<FGameplayTag, AllocatorType>& OutArray)
{
	// Add all GameplayCue Tags from the GE into the GameplayCueTags PendingCue.list
	for (const FGameplayEffectCue& EffectCue : Spec.Def->GameplayCues)
	{
		for (const FGameplayTag& Tag: EffectCue.GameplayCueTags)
		{
			if (Tag.IsValid())
			{
				OutArray.Add(Tag);
			}
		}
	}
}

/**
 *	Enabling AbilitySystemAlwaysConvertGESpecToGCParams will mean that all calls to gameplay cues with GameplayEffectSpecs will be converted into GameplayCue Parameters server side and then replicated.
 *	This potentially saved bandwidth but also has less information, depending on how the GESpec is converted to GC Parameters and what your GC's need to know.
 */

int32 AbilitySystemAlwaysConvertGESpecToGCParams = 0;
static FAutoConsoleVariableRef CVarAbilitySystemAlwaysConvertGESpecToGCParams(TEXT("AbilitySystem.AlwaysConvertGESpecToGCParams"), AbilitySystemAlwaysConvertGESpecToGCParams, TEXT("Always convert a GameplayCue from GE Spec to GC from GC Parameters on the server"), ECVF_Default );

void UGameplayCueManager::InvokeGameplayCueAddedAndWhileActive_FromSpec(UAbilitySystemComponent* OwningComponent, const FGameplayEffectSpec& Spec, FPredictionKey PredictionKey)
{
	if (Spec.Def->GameplayCues.Num() == 0)
	{
		return;
	}

	if (EnableSuppressCuesOnGameplayCueManager && OwningComponent && OwningComponent->bSuppressGameplayCues)
	{
		return;
	}

	IAbilitySystemReplicationProxyInterface* ReplicationInterface = OwningComponent->GetReplicationInterface();
	if (ReplicationInterface == nullptr)
	{
		// No available Replication Interface, we are going to drop these calls. (By design: someone who wants proxy replication should be ok with GC rpcs being dropped when the proxy is null)
		return;
	}

	if (AbilitySystemAlwaysConvertGESpecToGCParams)
	{
		// Transform the GE Spec into GameplayCue parmameters here (on the server)

		FGameplayCueParameters Parameters;
		UAbilitySystemGlobals::Get().InitGameplayCueParameters_GESpec(Parameters, Spec);

		static TArray<FGameplayTag, TInlineAllocator<4> > Tags;
		Tags.Reset();

		PullGameplayCueTagsFromSpec(Spec, Tags);

		if (Tags.Num() == 1)
		{
			ReplicationInterface->Call_InvokeGameplayCueAddedAndWhileActive_WithParams(Tags[0], PredictionKey, Parameters);
			
		}
		else if (Tags.Num() > 1)
		{
			ReplicationInterface->Call_InvokeGameplayCuesAddedAndWhileActive_WithParams(FGameplayTagContainer::CreateFromArray(Tags), PredictionKey, Parameters);
		}
		else
		{
			ABILITY_LOG(Warning, TEXT("No actual gameplay cue tags found in GameplayEffect %s (despite it having entries in its gameplay cue list!"), *Spec.Def->GetName());

		}
	}
	else
	{
		ReplicationInterface->Call_InvokeGameplayCueAddedAndWhileActive_FromSpec(Spec, PredictionKey);

	}
}

void UGameplayCueManager::InvokeGameplayCueExecuted_FromSpec(UAbilitySystemComponent* OwningComponent, const FGameplayEffectSpec& Spec, FPredictionKey PredictionKey)
{	
	if (Spec.Def->GameplayCues.Num() == 0)
	{
		// This spec doesn't have any GCs, so early out
		ABILITY_LOG(Verbose, TEXT("No GCs in this Spec, so early out: %s"), *Spec.Def->GetName());
		return;
	}

	if (EnableSuppressCuesOnGameplayCueManager && OwningComponent && OwningComponent->bSuppressGameplayCues)
	{
		return;
	}

	FGameplayCuePendingExecute PendingCue;

	if (AbilitySystemAlwaysConvertGESpecToGCParams)
	{
		// Transform the GE Spec into GameplayCue parameters here (on the server)
		PendingCue.PayloadType = EGameplayCuePayloadType::CueParameters;
		PendingCue.OwningComponent = OwningComponent;
		PendingCue.PredictionKey = PredictionKey;

		PullGameplayCueTagsFromSpec(Spec, PendingCue.GameplayCueTags);
		if (PendingCue.GameplayCueTags.Num() == 0)
		{
			ABILITY_LOG(Warning, TEXT("GE %s has GameplayCues but not valid GameplayCue tag."), *Spec.Def->GetName());			
			return;
		}
		
		UAbilitySystemGlobals::Get().InitGameplayCueParameters_GESpec(PendingCue.CueParameters, Spec);
	}
	else
	{
		// Transform the GE Spec into a FGameplayEffectSpecForRPC (holds less information than the GE Spec itself, but more information than the FGameplayCueParameter)
		PendingCue.PayloadType = EGameplayCuePayloadType::FromSpec;
		PendingCue.OwningComponent = OwningComponent;
		PendingCue.FromSpec = FGameplayEffectSpecForRPC(Spec);
		PendingCue.PredictionKey = PredictionKey;
	}

	AddPendingCueExecuteInternal(PendingCue);
}

void UGameplayCueManager::InvokeGameplayCueExecuted(UAbilitySystemComponent* OwningComponent, const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext)
{
	if (!GameplayCueTag.IsValid())
	{
		return;
	}

	if (EnableSuppressCuesOnGameplayCueManager && OwningComponent && OwningComponent->bSuppressGameplayCues)
	{
		return;
	}

	if (OwningComponent)
	{
		FGameplayCuePendingExecute PendingCue;
		PendingCue.PayloadType = EGameplayCuePayloadType::CueParameters;
		PendingCue.GameplayCueTags.Add(GameplayCueTag);
		PendingCue.OwningComponent = OwningComponent;
		UAbilitySystemGlobals::Get().InitGameplayCueParameters(PendingCue.CueParameters, EffectContext);
		PendingCue.PredictionKey = PredictionKey;

		AddPendingCueExecuteInternal(PendingCue);
	}
}

void UGameplayCueManager::InvokeGameplayCueExecuted_WithParams(UAbilitySystemComponent* OwningComponent, const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters)
{
	if (!GameplayCueTag.IsValid())
	{
		return;
	}

	if (EnableSuppressCuesOnGameplayCueManager && OwningComponent && OwningComponent->bSuppressGameplayCues)
	{
		return;
	}

	if (OwningComponent)
	{
		FGameplayCuePendingExecute PendingCue;
		PendingCue.PayloadType = EGameplayCuePayloadType::CueParameters;
		PendingCue.GameplayCueTags.Add(GameplayCueTag);
		PendingCue.OwningComponent = OwningComponent;
		PendingCue.CueParameters = GameplayCueParameters;
		PendingCue.PredictionKey = PredictionKey;

		AddPendingCueExecuteInternal(PendingCue);
	}
}

void UGameplayCueManager::AddPendingCueExecuteInternal(FGameplayCuePendingExecute& PendingCue)
{
	if (ProcessPendingCueExecute(PendingCue))
	{
		PendingExecuteCues.Add(PendingCue);
	}

	if (GameplayCueSendContextCount == 0)
	{
		// Not in a context, flush now
		FlushPendingCues();
	}
}

void UGameplayCueManager::StartGameplayCueSendContext()
{
	GameplayCueSendContextCount++;
}

void UGameplayCueManager::EndGameplayCueSendContext()
{
	GameplayCueSendContextCount--;

	if (GameplayCueSendContextCount == 0)
	{
		FlushPendingCues();
	}
	else if (GameplayCueSendContextCount < 0)
	{
		ABILITY_LOG(Warning, TEXT("UGameplayCueManager::EndGameplayCueSendContext called too many times! Negative context count"));
	}
}

void UGameplayCueManager::FlushPendingCues()
{
	OnFlushPendingCues.Broadcast();

	TArray<FGameplayCuePendingExecute> LocalPendingExecuteCues = PendingExecuteCues;
	PendingExecuteCues.Empty();
	for (int32 i = 0; i < LocalPendingExecuteCues.Num(); i++)
	{
		FGameplayCuePendingExecute& PendingCue = LocalPendingExecuteCues[i];

		// Our component may have gone away
		if (PendingCue.OwningComponent)
		{
			bool bHasAuthority = PendingCue.OwningComponent->IsOwnerActorAuthoritative();
			bool bLocalPredictionKey = PendingCue.PredictionKey.IsLocalClientKey();

			IAbilitySystemReplicationProxyInterface* RepInterface = PendingCue.OwningComponent->GetReplicationInterface();
			if (RepInterface == nullptr)
			{
				// If this returns null, it means "we are replicating througha proxy and have no avatar". Which in this case, we should skip
				continue;
			}

			// TODO: Could implement non-rpc method for replicating if desired
			if (PendingCue.PayloadType == EGameplayCuePayloadType::CueParameters)
			{
				if (ensure(PendingCue.GameplayCueTags.Num() >= 1))
				{
					if (bHasAuthority)
					{
						RepInterface->ForceReplication();
						if (PendingCue.GameplayCueTags.Num() > 1)
						{
							RepInterface->Call_InvokeGameplayCuesExecuted_WithParams(FGameplayTagContainer::CreateFromArray(PendingCue.GameplayCueTags), PendingCue.PredictionKey, PendingCue.CueParameters);
						}
						else
						{
							RepInterface->Call_InvokeGameplayCueExecuted_WithParams(PendingCue.GameplayCueTags[0], PendingCue.PredictionKey, PendingCue.CueParameters);

							static FName NetMulticast_InvokeGameplayCueExecuted_WithParamsName = TEXT("NetMulticast_InvokeGameplayCueExecuted_WithParams");
							CheckForTooManyRPCs(NetMulticast_InvokeGameplayCueExecuted_WithParamsName, PendingCue, PendingCue.GameplayCueTags[0].ToString(), nullptr);
						}
					}
					else if (bLocalPredictionKey)
					{
						for (const FGameplayTag& Tag : PendingCue.GameplayCueTags)
						{
							PendingCue.OwningComponent->InvokeGameplayCueEvent(Tag, EGameplayCueEvent::Executed, PendingCue.CueParameters);
						}
					}
				}
			}
			else if (PendingCue.PayloadType == EGameplayCuePayloadType::FromSpec)
			{
				if (bHasAuthority)
				{
					RepInterface->ForceReplication();
					RepInterface->Call_InvokeGameplayCueExecuted_FromSpec(PendingCue.FromSpec, PendingCue.PredictionKey);

					static FName NetMulticast_InvokeGameplayCueExecuted_FromSpecName = TEXT("NetMulticast_InvokeGameplayCueExecuted_FromSpec");
					CheckForTooManyRPCs(NetMulticast_InvokeGameplayCueExecuted_FromSpecName, PendingCue, PendingCue.FromSpec.Def ? PendingCue.FromSpec.ToSimpleString() : TEXT("FromSpecWithNoDef"), PendingCue.FromSpec.EffectContext.Get());
				}
				else if (bLocalPredictionKey)
				{
					PendingCue.OwningComponent->InvokeGameplayCueEvent(PendingCue.FromSpec, EGameplayCueEvent::Executed);
				}
			}
		}
	}
}

bool UGameplayCueManager::ProcessPendingCueExecute(FGameplayCuePendingExecute& PendingCue)
{
	// Subclasses can do something here
	return true;
}

bool UGameplayCueManager::DoesPendingCueExecuteMatch(FGameplayCuePendingExecute& PendingCue, FGameplayCuePendingExecute& ExistingCue)
{
	const FHitResult* PendingHitResult = NULL;
	const FHitResult* ExistingHitResult = NULL;

	if (PendingCue.PayloadType != ExistingCue.PayloadType)
	{
		return false;
	}

	if (PendingCue.OwningComponent != ExistingCue.OwningComponent)
	{
		return false;
	}

	if (PendingCue.PredictionKey.GetPredictiveConnectionKey() != ExistingCue.PredictionKey.GetPredictiveConnectionKey())
	{
		// They can both by null, but if they were predicted by different people exclude it
		return false;
	}

	if (PendingCue.PayloadType == EGameplayCuePayloadType::FromSpec)
	{
		if (PendingCue.FromSpec.Def != ExistingCue.FromSpec.Def)
		{
			return false;
		}

		if (PendingCue.FromSpec.Level != ExistingCue.FromSpec.Level)
		{
			return false;
		}
	}
	else
	{
		if (PendingCue.GameplayCueTags != ExistingCue.GameplayCueTags)
		{
			return false;
		}
	}

	return true;
}

void UGameplayCueManager::CheckForPreallocation(UClass* GCClass)
{
	if (AGameplayCueNotify_Actor* InstancedCue = Cast<AGameplayCueNotify_Actor>(GCClass->GetDefaultObject(false)))
	{
		if (InstancedCue->NumPreallocatedInstances > 0 && GameplayCueClassesForPreallocation.Contains(GCClass) == false)
		{
			// Add this to the global list
			GameplayCueClassesForPreallocation.Add(GCClass);

			// Add it to any world specific lists
			for (FPreallocationInfo& Info : PreallocationInfoList_Internal)
			{
				ensure(Info.ClassesNeedingPreallocation.Contains(GCClass)==false);
				Info.ClassesNeedingPreallocation.Push(GCClass);
			}
		}
	}
}

// -------------------------------------------------------------

void UGameplayCueManager::ResetPreallocation(UWorld* World)
{
	FPreallocationInfo& Info = GetPreallocationInfo(World);

	Info.PreallocatedInstances.Reset();
	Info.ClassesNeedingPreallocation = GameplayCueClassesForPreallocation;
}

void UGameplayCueManager::UpdatePreallocation(UWorld* World)
{
#if WITH_EDITOR	
	// Don't preallocate
	if (World->IsPreviewWorld())
	{
		return;
	}
#endif

	FPreallocationInfo& Info = GetPreallocationInfo(World);

	if (Info.ClassesNeedingPreallocation.Num() > 0)
	{
		TSubclassOf<AGameplayCueNotify_Actor> GCClass = Info.ClassesNeedingPreallocation.Last();
		AGameplayCueNotify_Actor* CDO = GCClass->GetDefaultObject<AGameplayCueNotify_Actor>();
		FGameplayCueNotifyActorArray& PreallocatedList = Info.PreallocatedInstances.FindOrAdd(CDO->GetClass());

		AGameplayCueNotify_Actor* PrespawnedInstance = Cast<AGameplayCueNotify_Actor>(World->SpawnActor(CDO->GetClass()));
		if (ensureMsgf(PrespawnedInstance, TEXT("Failed to prespawn GC notify for: %s"), *GetNameSafe(CDO)))
		{
			ensureMsgf(IsValid(PrespawnedInstance) == true, TEXT("Newly spawned GC is invalid: %s"), *GetNameSafe(CDO));

			if (LogGameplayCueActorSpawning)
			{
				ABILITY_LOG(Warning, TEXT("Prespawning GC %s"), *GetNameSafe(CDO));
			}

			PrespawnedInstance->bInRecycleQueue = true;
			PreallocatedList.Actors.Push(PrespawnedInstance);
			PrespawnedInstance->SetActorHiddenInGame(true);

			if (PreallocatedList.Actors.Num() >= CDO->NumPreallocatedInstances)
			{
				Info.ClassesNeedingPreallocation.Pop(EAllowShrinking::No);
			}
		}
	}
}

FPreallocationInfo& UGameplayCueManager::GetPreallocationInfo(const UWorld* World)
{
	FObjectKey ObjKey(World);

	for (FPreallocationInfo& Info : PreallocationInfoList_Internal)
	{
		if (ObjKey == Info.OwningWorldKey)
		{
			return Info;
		}
	}

	FPreallocationInfo NewInfo;
	NewInfo.OwningWorldKey = ObjKey;

	PreallocationInfoList_Internal.Add(NewInfo);
	return PreallocationInfoList_Internal.Last();
}

void UGameplayCueManager::OnPostWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	const FObjectKey WorldObjectKey(World);

	for (int32 idx=0; idx < PreallocationInfoList_Internal.Num(); ++idx)
	{
		FPreallocationInfo& PreallocationInfo = PreallocationInfoList_Internal[idx];
		if (PreallocationInfo.OwningWorldKey != WorldObjectKey)
		{
			continue;
		}
		
		ABILITY_LOG(Verbose, TEXT("UGameplayCueManager::OnPostWorldCleanup %s Removing PreallocationInfoList_Internal element %d"), *GetNameSafe(World), idx);

		// Spit out some debug information to help us track down memory issues
		constexpr bool bWarnOnActiveActors = true;
		DumpPreallocationStats(PreallocationInfo, bWarnOnActiveActors);

		// Actually remove the entry which can contain hard references
		PreallocationInfoList_Internal.RemoveAtSwap(idx, EAllowShrinking::No);
		idx--;
	}

	IGameplayCueInterface::ClearTagToFunctionMap();
}

void UGameplayCueManager::DumpPreallocationStats(const FPreallocationInfo& PreallocationInfo, bool bWarnOnActiveActors)
{
	constexpr bool bAllowLoggingInBuild = !(UE_BUILD_SHIPPING || UE_BUILD_TEST);
	if (!bAllowLoggingInBuild || !UE_LOG_ACTIVE(LogAbilitySystem, Display))
	{
		return;
	}

	for (auto& It : PreallocationInfo.PreallocatedInstances)
	{
		if (const UClass* ThisClass = It.Key)
		{
			if (const AGameplayCueNotify_Actor* CDO = ThisClass->GetDefaultObject<AGameplayCueNotify_Actor>())
			{
				const TArray<AGameplayCueNotify_Actor*>& List = It.Value.Actors;
				if (List.Num() > CDO->NumPreallocatedInstances)
				{
					ABILITY_LOG(Display, TEXT("  GameplayCueNotify Class '%s' was used simultaneously %d times. The CDO default is only %d preallocated instances."), *ThisClass->GetName(), List.Num(), CDO->NumPreallocatedInstances);
				}

				if (bWarnOnActiveActors)
				{
					int StillActive = 0;
					for (const AGameplayCueNotify_Actor* Actor : List)
					{
						StillActive += (!Actor->bInRecycleQueue);
					}

					// NotifyGameplayCueActorFinished should have been called on all of these Actors by the time we're done tearing down the world.
					UE_CLOGF(StillActive > 0, LogAbilitySystem, Error, "  GameplayCueNotify Class '%ls' had %d instances still active.  This shouldn't happen.", *ThisClass->GetName(), StillActive);
				}
			}
		}
	}
}

void UGameplayCueManager::OnPreReplayScrub(UWorld* World)
{
	// See if the World's demo net driver is the duplicated collection's driver,
	// and if so, don't reset preallocated instances. Since the preallocations are global
	// among all level collections, this would clear all current preallocated instances from the list,
	// but there's no need to, and the actor instances would still be around, causing a leak.
	const FLevelCollection* const DuplicateLevelCollection = World ? World->FindCollectionByType(ELevelCollectionType::DynamicDuplicatedLevels) : nullptr;
	if (DuplicateLevelCollection && DuplicateLevelCollection->GetDemoNetDriver() == World->GetDemoNetDriver())
	{
		return;
	}

	FPreallocationInfo& Info = GetPreallocationInfo(World);
	Info.PreallocatedInstances.Reset();
}

#if GAMEPLAYCUE_DEBUG
FGameplayCueDebugInfo* UGameplayCueManager::GetDebugInfo(int32 Handle, bool Reset)
{
	static const int32 MaxDebugEntries = 256;
	int32 Index = Handle % MaxDebugEntries;

	static TArray<FGameplayCueDebugInfo> DebugArray;
	if (DebugArray.Num() == 0)
	{
		DebugArray.AddDefaulted(MaxDebugEntries);
	}
	if (Reset)
	{
		DebugArray[Index] = FGameplayCueDebugInfo();
	}

	return &DebugArray[Index];
}
#endif

// ----------------------------------------------------------------

static void	RunGameplayCueTranslator(UWorld* InWorld)
{
	UAbilitySystemGlobals::Get().GetGameplayCueManager()->TranslationManager.BuildTagTranslationTable();
}

FAutoConsoleCommandWithWorld RunGameplayCueTranslatorCmd(
	TEXT("GameplayCue.BuildGameplayCueTranslator"),
	TEXT("Displays GameplayCue notify map"),
	FConsoleCommandWithWorldDelegate::CreateStatic(RunGameplayCueTranslator)
	);

// -----------------------------------------------------

static void	PrintGameplayCueTranslator(UWorld* InWorld)
{
	UAbilitySystemGlobals::Get().GetGameplayCueManager()->TranslationManager.PrintTranslationTable();
}

FAutoConsoleCommandWithWorld PrintGameplayCueTranslatorCmd(
	TEXT("GameplayCue.PrintGameplayCueTranslator"),
	TEXT("Displays GameplayCue notify map"),
	FConsoleCommandWithWorldDelegate::CreateStatic(PrintGameplayCueTranslator)
	);


#if WITH_EDITOR
#undef LOCTEXT_NAMESPACE
#endif
