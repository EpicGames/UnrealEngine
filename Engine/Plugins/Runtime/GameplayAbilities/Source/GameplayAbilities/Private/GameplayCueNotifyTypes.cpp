// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueNotifyTypes.h"
#include "Components/SkeletalMeshComponent.h"
#include "Camera/CameraLensEffectInterface.h"
#include "Engine/World.h"
#include "GameFramework/ForceFeedbackEffect.h"
#include "NiagaraSystem.h"
#include "NiagaraComponent.h"
#include "NiagaraFunctionLibrary.h"
#include "Components/AudioComponent.h"
#include "Sound/SoundBase.h"
#include "Camera/CameraShakeBase.h"
#include "Components/ForceFeedbackComponent.h"
#include "Materials/MaterialInterface.h"
#include "Components/DecalComponent.h"
#include "Kismet/GameplayStatics.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerState.h"
#include "GameFramework/Character.h"
#include "GameFramework/InputDeviceSubsystem.h"
#include "GameFramework/InputDeviceProperties.h"
#include "Sound/SoundWaveProcedural.h"
#include "Misc/DataValidation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueNotifyTypes)


DEFINE_LOG_CATEGORY(LogGameplayCueNotify);


#define LOCTEXT_NAMESPACE "GameplayCueNotify"


static APlayerController* FindPlayerControllerFromActor(AActor* Actor)
{
	if (!Actor)
	{
		return nullptr;
	}

	// Check the actual actor first.
	APlayerController* PC = Cast<APlayerController>(Actor);
	if (PC)
	{
		return PC;
	}

	// Check using pawn next.
	APawn* Pawn = Cast<APawn>(Actor);
	if (!Pawn)
	{
		Pawn = Actor->GetInstigator<APawn>();
	}

	PC = (Pawn ? Cast<APlayerController>(Pawn->GetController()) : nullptr);
	if (PC)
	{
		return PC;
	}

	// Check using player state.
	APlayerState* PS = Cast<APlayerState>(Actor);

	PC = (PS ? Cast<APlayerController>(PS->GetOwner()) : nullptr);
	if (PC)
	{
		return PC;
	}

	return nullptr;
}

static float CalculateFalloffIntensity(const FVector& SourceLocation, const FVector& TargetLocation, float InnerRadius, float OuterRadius, float FalloffExponent)
{
	if ((InnerRadius <= 0.0f) && (OuterRadius <= 0.0f))
	{
		return 1.0f;
	}

	const float DistanceSqr = FVector::DistSquared(SourceLocation, TargetLocation);

	if (InnerRadius < OuterRadius)
	{
		const float Distance = FMath::Sqrt(DistanceSqr);
		const float Percent = 1.0f - FMath::Clamp(((Distance - InnerRadius) / (OuterRadius - InnerRadius)), 0.0f, 1.0f);

		return FMath::Pow(Percent, FalloffExponent);
	}

	// Ignore the outer radius and do a cliff falloff at the inner radius.
	return (DistanceSqr < FMath::Square(InnerRadius)) ? 1.0f : 0.0f;
}

static EAttachLocation::Type GetAttachLocationTypeFromRule(EAttachmentRule AttachmentRule)
{
	switch (AttachmentRule)
	{
	case EAttachmentRule::KeepRelative:
		return EAttachLocation::KeepRelativeOffset;

	case EAttachmentRule::KeepWorld:
		return EAttachLocation::KeepWorldPosition;

	case EAttachmentRule::SnapToTarget:
		return EAttachLocation::SnapToTarget;

	default:
		checkf(false, TEXT("GameplayCueNotify: Invalid attachment rule [%d]\n"), (uint8)AttachmentRule);
	}

	return EAttachLocation::KeepWorldPosition;
}

static ECameraShakePlaySpace GetCameraShakePlaySpace(EGameplayCueNotify_EffectPlaySpace EffectPlaySpace)
{
	switch (EffectPlaySpace)
	{
	case EGameplayCueNotify_EffectPlaySpace::WorldSpace:
		return ECameraShakePlaySpace::World;

	case EGameplayCueNotify_EffectPlaySpace::CameraSpace:
		return ECameraShakePlaySpace::CameraLocal;

	default:
		checkf(false, TEXT("GameplayCueNotify: Invalid effect play space [%d]\n"), (uint8)EffectPlaySpace);
	}

	return ECameraShakePlaySpace::World;
}


//////////////////////////////////////////////////////////////////////////
// FGameplayCueNotify_SpawnCondition
//////////////////////////////////////////////////////////////////////////
FGameplayCueNotify_SpawnCondition::FGameplayCueNotify_SpawnCondition()
{
	LocallyControlledSource = EGameplayCueNotify_LocallyControlledSource::InstigatorActor;
	LocallyControlledPolicy = EGameplayCueNotify_LocallyControlledPolicy::Always;

	ChanceToPlay = 1.0f;

	AllowedSurfaceTypes.Reset();
	AllowedSurfaceMask = 0x0;

	RejectedSurfaceTypes.Reset();
	RejectedSurfaceMask = 0x0;
}

bool FGameplayCueNotify_SpawnCondition::ShouldSpawn(const FGameplayCueNotify_SpawnContext& SpawnContext) const
{
	const APlayerController* PC = SpawnContext.FindLocalPlayerController(LocallyControlledSource);

	const bool bIsLocallyControlledSource = (PC != nullptr);

	if ((LocallyControlledPolicy == EGameplayCueNotify_LocallyControlledPolicy::LocalOnly) && !bIsLocallyControlledSource)
	{
		return false;
	}
	else if ((LocallyControlledPolicy == EGameplayCueNotify_LocallyControlledPolicy::NotLocal) && bIsLocallyControlledSource)
	{
		return false;
	}

	if ((ChanceToPlay < 1.0f) && (ChanceToPlay < FMath::FRand()))
	{
		return false;
	}

	const FGameplayCueNotify_SurfaceMask SpawnContextSurfaceMask = (1ULL << SpawnContext.SurfaceType);

	if (AllowedSurfaceTypes.Num() > 0)
	{
		// Build the surface type mask if needed.
		if (AllowedSurfaceMask == 0x0)
		{
			for (EPhysicalSurface SurfaceType : AllowedSurfaceTypes)
			{
				AllowedSurfaceMask |= (1ULL << SurfaceType);
			}
		}

		if ((AllowedSurfaceMask & SpawnContextSurfaceMask) == 0x0)
		{
			return false;
		}
	}

	if (RejectedSurfaceTypes.Num() > 0)
	{
		// Build the surface type mask if needed.
		if (RejectedSurfaceMask == 0x0)
		{
			for (EPhysicalSurface SurfaceType : RejectedSurfaceTypes)
			{
				RejectedSurfaceMask |= (1ULL << SurfaceType);
			}
		}

		if ((RejectedSurfaceMask & SpawnContextSurfaceMask) != 0x0)
		{
			return false;
		}
	}

	return true;
}


//////////////////////////////////////////////////////////////////////////
// FGameplayCueNotify_PlacementInfo
//////////////////////////////////////////////////////////////////////////
FGameplayCueNotify_PlacementInfo::FGameplayCueNotify_PlacementInfo()
{
	SocketName = NAME_None;
	AttachPolicy = EGameplayCueNotify_AttachPolicy::DoNotAttach;
	AttachmentRule = EAttachmentRule::KeepWorld;

	bOverrideRotation = false;
	bOverrideScale = true;
	RotationOverride = FRotator::ZeroRotator;
	ScaleOverride = FVector::OneVector;
}

bool FGameplayCueNotify_PlacementInfo::FindSpawnTransform(const FGameplayCueNotify_SpawnContext& SpawnContext, FTransform& OutTransform) const
{
	const FGameplayCueParameters& CueParameters = SpawnContext.CueParameters;

	bool bSetTransform = false;

	// First attempt to get the transform from the hit result.
	// If that fails, use the gameplay cue parameters.
	// If that fails, fall back to the target actor.
	if (SpawnContext.HitResult && SpawnContext.HitResult->bBlockingHit)
	{
		OutTransform = FTransform(SpawnContext.HitResult->ImpactNormal.Rotation(), SpawnContext.HitResult->ImpactPoint);
		bSetTransform = true;
	}
	else if (!CueParameters.Location.IsZero())
	{
		OutTransform = FTransform(CueParameters.Normal.Rotation(), CueParameters.Location);
		bSetTransform = true;
	}
	else if (SpawnContext.TargetComponent)
	{
		OutTransform = SpawnContext.TargetComponent->GetSocketTransform(SocketName);
		bSetTransform = true;
	}

	if (bSetTransform)
	{
		if (bOverrideRotation)
		{
			OutTransform.SetRotation(RotationOverride.Quaternion());
		}

		if (bOverrideScale)
		{
			OutTransform.SetScale3D(ScaleOverride);
		}
	}
	else
	{
		// This can happen if the target actor if destroyed out from under us.  We could alternatively pass the actor down, from ::GameplayCueFinishedCallback.
		// Right now it seems to make more sense to skip spawning more stuff in this case.  As the OnRemove effects will spawn if the GC OnRemove is actually broadcasted "naturally".
		// The actor dieing and forcing the OnRemove event is more of a "please cleanup everything" event. This could be changed in the future.
		UE_LOGF(LogGameplayCueNotify, Log, "GameplayCueNotify: Failed to find spawn transform for gameplay cue notify [%ls].", *CueParameters.MatchedTagName.ToString());
	}

	return bSetTransform;
}

void FGameplayCueNotify_PlacementInfo::SetComponentTransform(USceneComponent* Component, const FTransform& Transform) const
{
	check(Component);

	Component->SetAbsolute(Component->IsUsingAbsoluteLocation(), bOverrideRotation, bOverrideScale);
	Component->SetWorldTransform(Transform);
}

void FGameplayCueNotify_PlacementInfo::TryComponentAttachment(USceneComponent* Component, USceneComponent* AttachComponent) const
{
	check(Component);

	if (AttachComponent && (AttachPolicy == EGameplayCueNotify_AttachPolicy::AttachToTarget))
	{
		FAttachmentTransformRules AttachmentRules(AttachmentRule, true);

		Component->AttachToComponent(AttachComponent, AttachmentRules, SocketName);
	}
}


//////////////////////////////////////////////////////////////////////////
// FGameplayCueNotify_SpawnContext
//////////////////////////////////////////////////////////////////////////
FGameplayCueNotify_SpawnContext::FGameplayCueNotify_SpawnContext(UWorld* InWorld, AActor* InTargetActor, const FGameplayCueParameters& InCueParameters)
	: World(InWorld)
	, TargetActor(InTargetActor)
	, CueParameters(InCueParameters)
{
	HitResult = nullptr;
	TargetComponent = nullptr;
	SurfaceType = EPhysicalSurface::SurfaceType_Default;

	DefaultSpawnCondition = nullptr;
	DefaultPlacementInfo = nullptr;

	InitializeContext();
}

void FGameplayCueNotify_SpawnContext::InitializeContext()
{
	/*
		Note that FGameplayCueParameters::EffectContext may not be valid in some projects.
		Projects can override UAbilitySystemGlobals::InitGameplayCueParameters() to extract only necessary data from FGameplayEffectContext directly into FGameplayCueParameters.
		See UAbilitySystemGlobals::InitGameplayCueParameters() and AbilitySystem.AlwaysConvertGESpecToGCParams for more info.
	*/

	if (!ensure(TargetActor))
	{
		return;
	}

	if (CueParameters.EffectContext.IsValid())
	{
		HitResult = CueParameters.EffectContext.GetHitResult();
	}

	// Find the target component.
	TargetComponent = CueParameters.TargetAttachComponent.Get();
	if (!TargetComponent)
	{
		const ACharacter* TargetCharacter = Cast<ACharacter>(TargetActor);

		// Always use the mesh for characters since the mesh position will be smoothed out.
		TargetComponent = (TargetCharacter ? TargetCharacter->GetMesh() : TargetActor->GetRootComponent());
	}

	// Find the surface type.
	const UPhysicalMaterial* PhysicalMaterial = nullptr;

	if (HitResult && HitResult->PhysMaterial.IsValid())
	{
		PhysicalMaterial = HitResult->PhysMaterial.Get();
	}
	else
	{
		PhysicalMaterial = CueParameters.PhysicalMaterial.Get();
	}

	if (PhysicalMaterial)
	{
		SurfaceType = PhysicalMaterial->SurfaceType;
	}
}

APlayerController* FGameplayCueNotify_SpawnContext::FindLocalPlayerController(EGameplayCueNotify_LocallyControlledSource Source) const
{
	AActor* ActorToSearch = nullptr;

	switch (Source)
	{
	case EGameplayCueNotify_LocallyControlledSource::InstigatorActor:
		ActorToSearch = CueParameters.Instigator.Get();
		break;

	case EGameplayCueNotify_LocallyControlledSource::TargetActor:
		ActorToSearch = TargetActor;
		break;

	default:
		break;
	}

	APlayerController* PC = FindPlayerControllerFromActor(ActorToSearch);

	return (PC && PC->IsLocalController()) ? PC : nullptr;
}


//////////////////////////////////////////////////////////////////////////
// FGameplayCueNotify_SpawnResult
//////////////////////////////////////////////////////////////////////////
FGameplayCueNotify_SpawnResult::FGameplayCueNotify_SpawnResult()
{
	Reset();
}

void FGameplayCueNotify_SpawnResult::Reset()
{
	FxSystemComponents.Reset();
	AudioComponents.Reset();
	CameraShakes.Reset();
	CameraLensEffects.Reset();
	ForceFeedbackComponent = nullptr;
	ForceFeedbackTargetPC = nullptr;
	DecalComponent = nullptr;
}


//////////////////////////////////////////////////////////////////////////
// FGameplayCueNotify_ParticleInfo
//////////////////////////////////////////////////////////////////////////
FGameplayCueNotify_ParticleInfo::FGameplayCueNotify_ParticleInfo()
{
	NiagaraSystem = nullptr;
	bOverrideSpawnCondition = false;
	bOverridePlacementInfo = false;
	bCastShadow = false;
}

bool FGameplayCueNotify_ParticleInfo::PlayParticleEffect(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& OutSpawnResult) const
{
	UFXSystemComponent* SpawnedFXSC = nullptr;

	if (NiagaraSystem != nullptr)
	{
		const FGameplayCueNotify_SpawnCondition& SpawnCondition = SpawnContext.GetSpawnCondition(bOverrideSpawnCondition, SpawnConditionOverride);
		const FGameplayCueNotify_PlacementInfo& PlacementInfo = SpawnContext.GetPlacementInfo(bOverridePlacementInfo, PlacementInfoOverride);

		if (SpawnCondition.ShouldSpawn(SpawnContext))
		{
			FTransform SpawnTransform;

			if (PlacementInfo.FindSpawnTransform(SpawnContext, SpawnTransform))
			{
				check(SpawnContext.World);

				const FVector SpawnLocation = SpawnTransform.GetLocation();
				const FRotator SpawnRotation = SpawnTransform.Rotator();
				const FVector SpawnScale = SpawnTransform.GetScale3D();
				const bool bAutoDestroy = false;
				const bool bAutoActivate = true;

				if (SpawnContext.TargetComponent && (PlacementInfo.AttachPolicy == EGameplayCueNotify_AttachPolicy::AttachToTarget))
				{
					const EAttachLocation::Type AttachLocationType = GetAttachLocationTypeFromRule(PlacementInfo.AttachmentRule);

					SpawnedFXSC = UNiagaraFunctionLibrary::SpawnSystemAttached(NiagaraSystem, SpawnContext.TargetComponent, PlacementInfo.SocketName,
						SpawnLocation, SpawnRotation, SpawnScale, AttachLocationType, bAutoDestroy, ENCPoolMethod::AutoRelease, bAutoActivate);
				}
				else
				{
					SpawnedFXSC = UNiagaraFunctionLibrary::SpawnSystemAtLocation(SpawnContext.World, NiagaraSystem,
						SpawnLocation, SpawnRotation, SpawnScale, bAutoDestroy, bAutoActivate, ENCPoolMethod::AutoRelease);
				}

				if (SpawnedFXSC)
				{
					SpawnedFXSC->SetCastShadow(bCastShadow);
				}
			}
		}
	}

	// Always add to the list, even if null, so that the list is stable and in order for blueprint users.
	OutSpawnResult.FxSystemComponents.Add(SpawnedFXSC);

	return (SpawnedFXSC != nullptr);
}

void FGameplayCueNotify_ParticleInfo::ValidateBurstAssets(const UObject* ContainingAsset, const FString& Context, FDataValidationContext& ValidationContext) const
{
#if WITH_EDITORONLY_DATA
	if (NiagaraSystem != nullptr)
	{
		if (NiagaraSystem->IsLooping())
		{
			ValidationContext.AddError(FText::Format(
				LOCTEXT("NiagaraSystem_ShouldNotLoop", "Niagara system [{0}] used in slot [{1}] for asset [{2}] is set to looping, but the slot is a one-shot (the instance will leak)."),
				FText::AsCultureInvariant(NiagaraSystem->GetPathName()),
				FText::AsCultureInvariant(Context),
				FText::AsCultureInvariant(ContainingAsset->GetPathName())));
		}
	}
#endif // #if WITH_EDITORONLY_DATA
}


//////////////////////////////////////////////////////////////////////////
// FGameplayCueNotify_SoundInfo
//////////////////////////////////////////////////////////////////////////

FGameplayCueNotify_SoundParameterInterfaceInfo::FGameplayCueNotify_SoundParameterInterfaceInfo()
{
	StopTriggerName = TEXT("OnStop");
}

FGameplayCueNotify_SoundInfo::FGameplayCueNotify_SoundInfo()
{
	Sound = nullptr;
	SoundCue = nullptr;
	LoopingFadeOutDuration = 0.5f;
	LoopingFadeVolumeLevel = 0.0f;
	bOverrideSpawnCondition = false;
	bOverridePlacementInfo = false;
	bUseSoundParameterInterface = false;
}

bool FGameplayCueNotify_SoundInfo::PlaySound(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& OutSpawnResult) const
{
	UAudioComponent* AudioComponent = nullptr;
	bool bSoundPlayed = false;

	if (Sound != nullptr)
	{
		const FGameplayCueNotify_SpawnCondition& SpawnCondition = SpawnContext.GetSpawnCondition(bOverrideSpawnCondition, SpawnConditionOverride);
		const FGameplayCueNotify_PlacementInfo& PlacementInfo = SpawnContext.GetPlacementInfo(bOverridePlacementInfo, PlacementInfoOverride);

		if (SpawnCondition.ShouldSpawn(SpawnContext))
		{
			FTransform SpawnTransform;

			if (PlacementInfo.FindSpawnTransform(SpawnContext, SpawnTransform))
			{
				const bool bStopWhenAttachedToDestroyed = false;
				const bool bAutoDestroy = true;
				const float VolumeMultiplier = 1.0f;
				const float PitchMultiplier = 1.0f;
				const float StartTime = 0.0f;
				USoundAttenuation* AttenuationSettings = nullptr;
				USoundConcurrency* ConcurrencySettings = nullptr;

				const FVector SpawnLocation = SpawnTransform.GetLocation();
				const FRotator SpawnRotation = SpawnTransform.Rotator();

				if (SpawnContext.TargetComponent && (PlacementInfo.AttachPolicy == EGameplayCueNotify_AttachPolicy::AttachToTarget))
				{
					const EAttachLocation::Type AttachLocationType = GetAttachLocationTypeFromRule(PlacementInfo.AttachmentRule);

					AudioComponent = UGameplayStatics::SpawnSoundAttached(Sound, SpawnContext.TargetComponent, PlacementInfo.SocketName,
						SpawnLocation, SpawnRotation, AttachLocationType, bStopWhenAttachedToDestroyed,
						VolumeMultiplier, PitchMultiplier, StartTime, AttenuationSettings, ConcurrencySettings, bAutoDestroy);
				}
				else if (Sound->IsLooping())
				{
					AudioComponent = UGameplayStatics::SpawnSoundAtLocation(SpawnContext.World, Sound, SpawnLocation, SpawnRotation,
						VolumeMultiplier, PitchMultiplier, StartTime, AttenuationSettings, ConcurrencySettings, bAutoDestroy);
				}
				else
				{
					UGameplayStatics::PlaySoundAtLocation(SpawnContext.World, Sound, SpawnLocation, SpawnRotation,
						VolumeMultiplier, PitchMultiplier, StartTime, AttenuationSettings, ConcurrencySettings, SpawnContext.TargetActor);
				}

				bSoundPlayed = true;
			}
		}
	}

	// Always add to the list, even if null, so that the list is stable and in order for blueprint users.
	OutSpawnResult.AudioComponents.Add(AudioComponent);

	return bSoundPlayed;
}

void FGameplayCueNotify_SoundInfo::ValidateBurstAssets(const UObject* ContainingAsset, const FString& Context, FDataValidationContext& ValidationContext) const
{
#if WITH_EDITORONLY_DATA
	if (Sound != nullptr)
	{
		if (!Sound->IsOneShot())
		{
			ValidationContext.AddError(FText::Format(
				LOCTEXT("SoundCue_ShouldNotLoop", "Sound [{0}] used in slot [{1}] for asset [{2}] is not a one-shot, but the slot is a one-shot (the instance will be orphaned)."),
				FText::AsCultureInvariant(Sound->GetPathName()),
				FText::AsCultureInvariant(Context),
				FText::AsCultureInvariant(ContainingAsset->GetPathName())));
		}
	}
#endif //#if WITH_EDITORONLY_DATA
}

//////////////////////////////////////////////////////////////////////////
// FGameplayCueNotify_CameraShakeInfo
//////////////////////////////////////////////////////////////////////////
FGameplayCueNotify_CameraShakeInfo::FGameplayCueNotify_CameraShakeInfo()
{
	CameraShake = nullptr;
	ShakeScale = 1.0f;

	PlaySpace = EGameplayCueNotify_EffectPlaySpace::CameraSpace;

	bOverrideSpawnCondition = false;
	bOverridePlacementInfo = false;

	bPlayInWorld = false;
	WorldInnerRadius = 0.0f;
	WorldOuterRadius = 0.0f;
	WorldFalloffExponent = 1.0f;
}

bool FGameplayCueNotify_CameraShakeInfo::PlayCameraShake(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& OutSpawnResult) const
{
	if (!CameraShake || (ShakeScale <= 0.0f))
	{
		return false;
	}

	const FGameplayCueNotify_SpawnCondition& SpawnCondition = SpawnContext.GetSpawnCondition(bOverrideSpawnCondition, SpawnConditionOverride);
	const FGameplayCueNotify_PlacementInfo& PlacementInfo = SpawnContext.GetPlacementInfo(bOverridePlacementInfo, PlacementInfoOverride);

	if (!SpawnCondition.ShouldSpawn(SpawnContext))
	{
		return false;
	}

	const ECameraShakePlaySpace CameraShakePlaySpace = GetCameraShakePlaySpace(PlaySpace);

	// Check if the camera shake should be played in world for all players.
	if (bPlayInWorld)
	{
		FTransform SpawnTransform;
		if (!PlacementInfo.FindSpawnTransform(SpawnContext, SpawnTransform))
		{
			return false;
		}

		const FVector SpawnLocation = SpawnTransform.GetLocation();

		check(SpawnContext.World);
		for (FConstPlayerControllerIterator Iterator = SpawnContext.World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PC = Iterator->Get();
			if (!PC || !PC->PlayerCameraManager || !PC->IsLocalController())
			{
				continue;
			}

			const APawn* Pawn = PC->GetPawn();
			if (!Pawn)
			{
				continue;
			}

			const FVector PawnLocation = Pawn->GetActorLocation();
			const float FalloffIntensity = CalculateFalloffIntensity(SpawnLocation, PawnLocation, WorldInnerRadius, WorldOuterRadius, WorldFalloffExponent);

			if (FalloffIntensity <= 0.0f)
			{
				continue;
			}

			UCameraShakeBase* CameraShakeInstance = PC->PlayerCameraManager->StartCameraShake(CameraShake, (ShakeScale * FalloffIntensity), CameraShakePlaySpace, FRotator::ZeroRotator);
			OutSpawnResult.CameraShakes.Add(CameraShakeInstance);
		}
	}
	else
	{
		// Play the camera shake directly on the target player controller.  No intensity falloff is applied when playing directly on the target.
		APlayerController* TargetPC = SpawnContext.FindLocalPlayerController(EGameplayCueNotify_LocallyControlledSource::TargetActor);

		if (TargetPC && TargetPC->PlayerCameraManager)
		{
			UCameraShakeBase* CameraShakeInstance = TargetPC->PlayerCameraManager->StartCameraShake(CameraShake, ShakeScale, CameraShakePlaySpace, FRotator::ZeroRotator);
			OutSpawnResult.CameraShakes.Add(CameraShakeInstance);
		}
	}

	return (OutSpawnResult.CameraShakes.Num() > 0);
}

void FGameplayCueNotify_CameraShakeInfo::ValidateBurstAssets(const UObject* ContainingAsset, const FString& Context, FDataValidationContext& ValidationContext) const
{
#if WITH_EDITORONLY_DATA
	if (CameraShake != nullptr)
	{
		UCameraShakeBase* CameraShakeCDO = CameraShake->GetDefaultObject<UCameraShakeBase>();
		FCameraShakeInfo CameraShakeInfo;
		CameraShakeCDO->GetShakeInfo(CameraShakeInfo);
		if (CameraShakeInfo.Duration.IsInfinite())
		{
			ValidationContext.AddError(FText::Format(
				LOCTEXT("CameraShake_ShouldNotLoop", "Camera shake [{0}] used in slot [{1}] for asset [{2}] will oscillate forever, but the slot is a one-shot (the instance will leak)."),
				FText::AsCultureInvariant(CameraShake->GetPathName()),
				FText::AsCultureInvariant(Context),
				FText::AsCultureInvariant(ContainingAsset->GetPathName())));
		}
	}
#endif // #if WITH_EDITORONLY_DATA
}


//////////////////////////////////////////////////////////////////////////
// FGameplayCueNotify_CameraLensEffectInfo
//////////////////////////////////////////////////////////////////////////
FGameplayCueNotify_CameraLensEffectInfo::FGameplayCueNotify_CameraLensEffectInfo()
{
	CameraLensEffect = nullptr;

	bOverrideSpawnCondition = false;
	bOverridePlacementInfo = false;

	bPlayInWorld = false;
	WorldInnerRadius = 0.0f;
	WorldOuterRadius = 0.0f;
}

bool FGameplayCueNotify_CameraLensEffectInfo::PlayCameraLensEffect(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& OutSpawnResult) const
{
	if (!CameraLensEffect)
	{
		return false;
	}

	const FGameplayCueNotify_SpawnCondition& SpawnCondition = SpawnContext.GetSpawnCondition(bOverrideSpawnCondition, SpawnConditionOverride);
	const FGameplayCueNotify_PlacementInfo& PlacementInfo = SpawnContext.GetPlacementInfo(bOverridePlacementInfo, PlacementInfoOverride);

	if (!SpawnCondition.ShouldSpawn(SpawnContext))
	{
		return false;
	}

	// Check if the camera lens effect should be played in world for all players.
	if (bPlayInWorld)
	{
		FTransform SpawnTransform;
		if (!PlacementInfo.FindSpawnTransform(SpawnContext, SpawnTransform))
		{
			return false;
		}

		const FVector SpawnLocation = SpawnTransform.GetLocation();

		check(SpawnContext.World);
		for (FConstPlayerControllerIterator Iterator = SpawnContext.World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PC = Iterator->Get();
			if (!PC || !PC->PlayerCameraManager || !PC->IsLocalController())
			{
				continue;
			}

			const APawn* Pawn = PC->GetPawn();
			if (!Pawn)
			{
				continue;
			}

			const FVector PawnLocation = Pawn->GetActorLocation();
			const float FalloffIntensity = CalculateFalloffIntensity(SpawnLocation, PawnLocation, WorldInnerRadius, WorldOuterRadius, 1.0f);

			if (FalloffIntensity <= 0.0f)
			{
				continue;
			}

			TScriptInterface<ICameraLensEffectInterface> CameraLensEffectInstance = PC->PlayerCameraManager->AddGenericCameraLensEffect(CameraLensEffect);
			OutSpawnResult.CameraLensEffects.Add(CameraLensEffectInstance);
		}
	}
	else
	{
		// Play the camera lens effect directly on the target player controller.  No radius check when playing directly on the target.
		APlayerController* TargetPC = SpawnContext.FindLocalPlayerController(EGameplayCueNotify_LocallyControlledSource::TargetActor);

		if (TargetPC && TargetPC->PlayerCameraManager)
		{
			TScriptInterface<ICameraLensEffectInterface> CameraLensEffectInstance = TargetPC->PlayerCameraManager->AddGenericCameraLensEffect(CameraLensEffect);
			OutSpawnResult.CameraLensEffects.Add(CameraLensEffectInstance);
		}
	}

	return (OutSpawnResult.CameraLensEffects.Num() > 0);
}

void FGameplayCueNotify_CameraLensEffectInfo::ValidateBurstAssets(const UObject* ContainingAsset, const FString& Context, FDataValidationContext& ValidationContext) const
{
#if WITH_EDITORONLY_DATA
	if (CameraLensEffect != nullptr)
	{
		if (ICameraLensEffectInterface* CameraLensEffectCDO = Cast<ICameraLensEffectInterface>(CameraLensEffect->GetDefaultObject<AActor>()))
		{
			if (CameraLensEffectCDO->IsLooping())
			{
				ValidationContext.AddError(FText::Format(
					LOCTEXT("CameraLensEffect_ShouldNotLoop", "Camera lens effect [{0}] used in slot [{1}] for asset [{2}] is set to looping, but the slot is a one-shot (the instance will leak)."),
					FText::AsCultureInvariant(CameraLensEffect->GetPathName()),
					FText::AsCultureInvariant(Context),
					FText::AsCultureInvariant(ContainingAsset->GetPathName())));
			}
		}
	}
#endif // #if WITH_EDITORONLY_DATA
}


//////////////////////////////////////////////////////////////////////////
// FGameplayCueNotify_ForceFeedbackInfo
//////////////////////////////////////////////////////////////////////////
FGameplayCueNotify_ForceFeedbackInfo::FGameplayCueNotify_ForceFeedbackInfo()
{
	ForceFeedbackEffect = nullptr;
	ForceFeedbackTag = NAME_None;

	bIsLooping = false;
	bOverrideSpawnCondition = false;
	bOverridePlacementInfo = false;

	bPlayInWorld = false;
	WorldIntensity = 1.0f;
	WorldAttenuation = nullptr;
}

bool FGameplayCueNotify_ForceFeedbackInfo::PlayForceFeedback(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& OutSpawnResult) const
{
	if (!ForceFeedbackEffect)
	{
		return false;
	}

	const FGameplayCueNotify_SpawnCondition& SpawnCondition = SpawnContext.GetSpawnCondition(bOverrideSpawnCondition, SpawnConditionOverride);
	const FGameplayCueNotify_PlacementInfo& PlacementInfo = SpawnContext.GetPlacementInfo(bOverridePlacementInfo, PlacementInfoOverride);

	if (!SpawnCondition.ShouldSpawn(SpawnContext))
	{
		return false;
	}

	// Spawn a component only if the force feedback is playing in world for all players.
	if (bPlayInWorld)
	{
		FTransform SpawnTransform;
		if (!PlacementInfo.FindSpawnTransform(SpawnContext, SpawnTransform))
		{
			return false;
		}

		const FVector SpawnLocation = SpawnTransform.GetLocation();
		const FRotator SpawnRotation = SpawnTransform.Rotator();
		const bool bStopWhenAttachedToDestroyed = false;
		const bool bAutoDestroy = true;
		const float StartTime = 0.0f;

		if (SpawnContext.TargetComponent && (PlacementInfo.AttachPolicy == EGameplayCueNotify_AttachPolicy::AttachToTarget))
		{
			const EAttachLocation::Type AttachLocationType = GetAttachLocationTypeFromRule(PlacementInfo.AttachmentRule);

			OutSpawnResult.ForceFeedbackComponent = UGameplayStatics::SpawnForceFeedbackAttached(ForceFeedbackEffect, SpawnContext.TargetComponent, PlacementInfo.SocketName,
				SpawnLocation, SpawnRotation, AttachLocationType, bStopWhenAttachedToDestroyed,
				bIsLooping, WorldIntensity, StartTime, WorldAttenuation, bAutoDestroy);
		}
		else
		{
			OutSpawnResult.ForceFeedbackComponent = UGameplayStatics::SpawnForceFeedbackAtLocation(SpawnContext.World, ForceFeedbackEffect, SpawnLocation, SpawnRotation,
				bIsLooping, WorldIntensity, StartTime, WorldAttenuation, bAutoDestroy);
		}

		return (OutSpawnResult.ForceFeedbackComponent != nullptr);
	}

	// Play the force feedback directly on the target player controller.  No attenuation is used when playing directly on the player controller.
	APlayerController* TargetPC = SpawnContext.FindLocalPlayerController(EGameplayCueNotify_LocallyControlledSource::TargetActor);
	if (TargetPC)
	{
		FForceFeedbackParameters ForceFeedbackParameters;
		ForceFeedbackParameters.Tag = ForceFeedbackTag;
		ForceFeedbackParameters.bLooping = bIsLooping;
		ForceFeedbackParameters.bIgnoreTimeDilation = false;
		ForceFeedbackParameters.bPlayWhilePaused = false;

		TargetPC->ClientPlayForceFeedback(ForceFeedbackEffect, ForceFeedbackParameters);

		OutSpawnResult.ForceFeedbackTargetPC = TargetPC;
		return true;
	}

	return false;
}

bool FGameplayCueNotify_InputDevicePropertyInfo::SetDeviceProperties(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& OutSpawnResult) const
{
	if (DeviceProperties.IsEmpty())
	{
		return false;
	}
	
	if (APlayerController* TargetPC = SpawnContext.FindLocalPlayerController(EGameplayCueNotify_LocallyControlledSource::TargetActor))
	{
		// Apply any device properties from this gameplay cue
		if (UInputDeviceSubsystem* System = UInputDeviceSubsystem::Get())
		{
			FActivateDevicePropertyParams Params = {};
			Params.UserId = TargetPC->GetPlatformUserId();
			
			if (ensure(Params.UserId.IsValid()))
			{
				for (TSubclassOf<UInputDeviceProperty> PropClass : DeviceProperties)
				{
					FInputDevicePropertyHandle Handle = System->ActivateDevicePropertyOfClass(PropClass, Params);
					ensure(Handle.IsValid());
				}
				return true;
			}
		}
	}

	return false;
}

void FGameplayCueNotify_InputDevicePropertyInfo::ValidateBurstAssets(const UObject* ContainingAsset, const FString& Context, FDataValidationContext& ValidationContext) const
{
#if WITH_EDITORONLY_DATA
	for (const TSubclassOf<UInputDeviceProperty>& PropClass : DeviceProperties)
	{
		if (!PropClass)
		{
			ValidationContext.AddError(FText::Format(
				LOCTEXT("InputDeviceProperty_ShouldNotBeNull", "There is a null device property class used in slot [{0}] for asset [{1}]."),
				FText::AsCultureInvariant(Context),
				FText::AsCultureInvariant(ContainingAsset->GetPathName())));
		}
	}
#endif
}

void FGameplayCueNotify_ForceFeedbackInfo::ValidateBurstAssets(const UObject* ContainingAsset, const FString& Context, FDataValidationContext& ValidationContext) const
{
#if WITH_EDITORONLY_DATA
	if (ForceFeedbackEffect)
	{
		if (bIsLooping)
		{
			ValidationContext.AddError(FText::Format(
				LOCTEXT("ForceFeedback_ShouldNotLoop", "Force feedback effect [{0}] used in slot [{1}] for asset [{2}] is set to looping, but the slot is a one-shot (the instance will leak)."),
				FText::AsCultureInvariant(ForceFeedbackEffect->GetPathName()),
				FText::AsCultureInvariant(Context),
				FText::AsCultureInvariant(ContainingAsset->GetPathName())));
		}
	}
#endif // #if WITH_EDITORONLY_DATA
}


//////////////////////////////////////////////////////////////////////////
// FGameplayCueNotify_DecalInfo
//////////////////////////////////////////////////////////////////////////
FGameplayCueNotify_DecalInfo::FGameplayCueNotify_DecalInfo()
{
	DecalMaterial = nullptr;
	DecalSize = FVector(128.0f, 256.0f, 256.0f);

	bOverrideSpawnCondition = false;
	bOverridePlacementInfo = false;

	bOverrideFadeOut = false;
	FadeOutStartDelay = 0.0f;
	FadeOutDuration = 0.0f;
}

bool FGameplayCueNotify_DecalInfo::SpawnDecal(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& OutSpawnResult) const
{
	if (!DecalMaterial)
	{
		return false;
	}

	const FGameplayCueNotify_SpawnCondition& SpawnCondition = SpawnContext.GetSpawnCondition(bOverrideSpawnCondition, SpawnConditionOverride);
	const FGameplayCueNotify_PlacementInfo& PlacementInfo = SpawnContext.GetPlacementInfo(bOverridePlacementInfo, PlacementInfoOverride);

	if (!SpawnCondition.ShouldSpawn(SpawnContext))
	{
		return false;
	}

	FTransform SpawnTransform;
	if (!PlacementInfo.FindSpawnTransform(SpawnContext, SpawnTransform))
	{
		return false;
	}

	// Invert inherited rotation as decals are reversed generally.
	if (!PlacementInfo.bOverrideRotation)
	{
		const FQuat InverseRotation = SpawnTransform.GetRotation().Inverse();
		SpawnTransform.SetRotation(InverseRotation);
	}

	UDecalComponent* SpawnedDecalComponent = nullptr;

	const FVector SpawnLocation = SpawnTransform.GetLocation();
	const FRotator SpawnRotation = SpawnTransform.Rotator();
	const float LifeSpan = 0.0f;

	if (SpawnContext.TargetComponent && (PlacementInfo.AttachPolicy == EGameplayCueNotify_AttachPolicy::AttachToTarget))
	{
		const EAttachLocation::Type AttachLocationType = GetAttachLocationTypeFromRule(PlacementInfo.AttachmentRule);

		SpawnedDecalComponent = UGameplayStatics::SpawnDecalAttached(DecalMaterial, DecalSize, SpawnContext.TargetComponent, PlacementInfo.SocketName,
			SpawnLocation, SpawnRotation, AttachLocationType, LifeSpan);
	}
	else
	{
		SpawnedDecalComponent = UGameplayStatics::SpawnDecalAtLocation(SpawnContext.World, DecalMaterial, DecalSize, SpawnLocation, SpawnRotation, LifeSpan);
	}

	if (ensure(SpawnedDecalComponent))
	{
		if (bOverrideFadeOut)
		{
			constexpr bool bDestroyOwnerAfterFade = false;
			SpawnedDecalComponent->SetFadeOut(FadeOutStartDelay, FadeOutDuration, bDestroyOwnerAfterFade);
		}
	}

	OutSpawnResult.DecalComponent = SpawnedDecalComponent;

	return (OutSpawnResult.DecalComponent != nullptr);
}


//////////////////////////////////////////////////////////////////////////
// FGameplayCueNotify_BurstEffects
//////////////////////////////////////////////////////////////////////////
FGameplayCueNotify_BurstEffects::FGameplayCueNotify_BurstEffects()
{
}

void FGameplayCueNotify_BurstEffects::ExecuteEffects(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& OutSpawnResult) const
{
	if (!SpawnContext.World)
	{
		UE_LOGF(LogGameplayCueNotify, Error, "GameplayCueNotify: Trying to execute Burst effects with a NULL world.");
		return;
	}

	for (const FGameplayCueNotify_ParticleInfo& ParticleInfo : BurstParticles)
	{
		ParticleInfo.PlayParticleEffect(SpawnContext, OutSpawnResult);
	}

	for (const FGameplayCueNotify_SoundInfo& SoundInfo : BurstSounds)
	{
		SoundInfo.PlaySound(SpawnContext, OutSpawnResult);
	}

	BurstCameraShake.PlayCameraShake(SpawnContext, OutSpawnResult);
	BurstCameraLensEffect.PlayCameraLensEffect(SpawnContext, OutSpawnResult);
	BurstForceFeedback.PlayForceFeedback(SpawnContext, OutSpawnResult);
	BurstDevicePropertyEffect.SetDeviceProperties(SpawnContext, OutSpawnResult);
	BurstDecal.SpawnDecal(SpawnContext, OutSpawnResult);
}

void FGameplayCueNotify_BurstEffects::ValidateAssociatedAssets(const UObject* ContainingAsset, const FString& Context, FDataValidationContext& ValidationContext) const
{
	for (const FGameplayCueNotify_ParticleInfo& ParticleInfo : BurstParticles)
	{
		ParticleInfo.ValidateBurstAssets(ContainingAsset, Context + TEXT(".BurstParticles"), ValidationContext);
	}

	for (const FGameplayCueNotify_SoundInfo& SoundInfo : BurstSounds)
	{
		SoundInfo.ValidateBurstAssets(ContainingAsset, Context + TEXT(".BurstSounds"), ValidationContext);
	}

	BurstCameraShake.ValidateBurstAssets(ContainingAsset, Context + TEXT(".BurstCameraShake"), ValidationContext);
	BurstCameraLensEffect.ValidateBurstAssets(ContainingAsset, Context + TEXT(".BurstCameraLensEffect"), ValidationContext);
	BurstForceFeedback.ValidateBurstAssets(ContainingAsset, Context + TEXT(".BurstForceFeedback"), ValidationContext);
	BurstDevicePropertyEffect.ValidateBurstAssets(ContainingAsset, Context + TEXT(".BurstDevicePropertyEffect"), ValidationContext);
}


//////////////////////////////////////////////////////////////////////////
// FGameplayCueNotify_LoopingEffects
//////////////////////////////////////////////////////////////////////////
FGameplayCueNotify_LoopingEffects::FGameplayCueNotify_LoopingEffects()
{
}

void FGameplayCueNotify_LoopingEffects::StartEffects(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& OutSpawnResult) const
{
	if (!SpawnContext.World)
	{
		UE_LOGF(LogGameplayCueNotify, Error, "GameplayCueNotify: Trying to start Looping effects with a NULL world.");
		return;
	}

	for (const FGameplayCueNotify_ParticleInfo& ParticleInfo : LoopingParticles)
	{
		ParticleInfo.PlayParticleEffect(SpawnContext, OutSpawnResult);
	}

	for (const FGameplayCueNotify_SoundInfo& SoundInfo : LoopingSounds)
	{
		SoundInfo.PlaySound(SpawnContext, OutSpawnResult);
	}

	LoopingCameraShake.PlayCameraShake(SpawnContext, OutSpawnResult);
	LoopingCameraLensEffect.PlayCameraLensEffect(SpawnContext, OutSpawnResult);
	LoopingForceFeedback.PlayForceFeedback(SpawnContext, OutSpawnResult);
	LoopingInputDevicePropertyEffect.SetDeviceProperties(SpawnContext, OutSpawnResult);
}

void FGameplayCueNotify_LoopingEffects::StopEffects(FGameplayCueNotify_SpawnResult& SpawnResult) const
{
	// Stop all particle effects.
	for (UFXSystemComponent* FXSC : SpawnResult.FxSystemComponents)
	{
		if (FXSC)
		{
			FXSC->Deactivate();
		}
	}

	// Stop all sound effects.  This assumes there is one AudioComponent entry for each FGameplayCueNotify_SoundInfo.
	ensure(LoopingSounds.Num() == SpawnResult.AudioComponents.Num());

	for (int32 SoundIndex = 0; SoundIndex < SpawnResult.AudioComponents.Num(); ++SoundIndex)
	{
		UAudioComponent* AudioComponent = SpawnResult.AudioComponents[SoundIndex];
		if (AudioComponent)
		{
			const FGameplayCueNotify_SoundInfo* SoundInfo = &LoopingSounds[SoundIndex];

			if(SoundInfo->bUseSoundParameterInterface)
			{
				// Call the Stop Trigger by Name
				AudioComponent->SetTriggerParameter(SoundInfo->SoundParameterInterfaceInfo.StopTriggerName);
			}

			if (LoopingSounds.IsValidIndex(SoundIndex))
			{
				if (SoundInfo->LoopingFadeOutDuration > 0.0f)
				{
					AudioComponent->FadeOut(SoundInfo->LoopingFadeOutDuration, SoundInfo->LoopingFadeVolumeLevel);
					continue;
				}
			}

			AudioComponent->Stop();
		}
	}

	// Stop all camera shakes.
	for (UCameraShakeBase* CameraShake : SpawnResult.CameraShakes)
	{
		if (CameraShake)
		{
			const bool bStopImmediately = false;
			CameraShake->StopShake(bStopImmediately);
		}
	}

	// Stop the camera lens effect.
	for (TScriptInterface<ICameraLensEffectInterface> CameraLensEffect : SpawnResult.CameraLensEffects)
	{
		if (CameraLensEffect)
		{
			CameraLensEffect->DeactivateLensEffect();
		}
	}

	// Stop the force feedback.  The component is only created when the effect is played in world.
	// If it's not in world, it needs to be stopped on the player controller.
	if (SpawnResult.ForceFeedbackComponent)
	{
		SpawnResult.ForceFeedbackComponent->Stop();
	}

	if (SpawnResult.ForceFeedbackTargetPC)
	{
		SpawnResult.ForceFeedbackTargetPC->ClientStopForceFeedback(LoopingForceFeedback.ForceFeedbackEffect, LoopingForceFeedback.ForceFeedbackTag);
	}

	// There should be no decal on looping gameplay cues.
	ensure(SpawnResult.DecalComponent == nullptr);

	// Clear the spawn results.
	SpawnResult.Reset();
}

void FGameplayCueNotify_LoopingEffects::ValidateAssociatedAssets(const UObject* ContainingAsset, const FString& Context, class FDataValidationContext& ValidationErrors) const
{
}


#undef LOCTEXT_NAMESPACE

