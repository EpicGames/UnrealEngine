// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueNotify_Actor.h"
#include "TimerManager.h"
#include "Engine/Blueprint.h"
#include "Components/TimelineComponent.h"
#include "AbilitySystemStats.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"
#include "GameplayCueManager.h"
#include "GameplayCueNotifyTypes.h"

#if WITH_EDITOR
#include "Engine/BlueprintGeneratedClass.h"
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueNotify_Actor)

#define LOCTEXT_NAMESPACE "GameplayCueNotify_Actor"

static TAutoConsoleVariable<bool> CVarGameplayCueNotifyMultipleRemoveEvents(
	TEXT("AbilitySystem.GameplayCueNotifyMultipleRemoveEvents"),
	true,
	TEXT("Controls how many times a GameplayCueNotify_Actor can execute its OnRemove/OnCeaseRelevant event per instance.")
	TEXT(" true: each WhileActive/OnBecameRelevant event will permit a following OnRemove/OnCeaseRelevant event.\n")
	TEXT(" false: only one OnRemove/OnCeaseRelevant event will execute per instance."));

namespace FAbilitySystemTweaks
{
	int ClearCueNotifyTimers = 1;
	FAutoConsoleVariableRef CVarClearCueNotifyTimers(TEXT("AbilitySystem.ClearCueNotifyTimers"), FAbilitySystemTweaks::ClearCueNotifyTimers, TEXT("Whether to call ClearAllTimersForObject when cue is getting recycled"), ECVF_Default);
}


AGameplayCueNotify_Actor::AGameplayCueNotify_Actor(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	IsOverride = true;
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = false;
	bAutoDestroyOnRemove = false;
	AutoDestroyDelay = 0.f;
	bUniqueInstancePerSourceObject = false;
	bUniqueInstancePerInstigator = false;
	bAllowMultipleOnActiveEvents = true;
	bAllowMultipleWhileActiveEvents = true;
	bHasHandledOnRemoveEvent = false;

	NumPreallocatedInstances = 0;

	bHasHandledOnActiveEvent = false;
	bHasHandledWhileActiveEvent = false;
	bInRecycleQueue = false;
	bAutoAttachToOwner = false;

	WarnIfLatentActionIsStillRunning = true;
	WarnIfTimelineIsStillRunning = true;
}

void AGameplayCueNotify_Actor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (UGameplayCueManager::IsGameplayCueRecylingEnabled() && (EndPlayReason == EEndPlayReason::Destroyed))
	{
		UWorld* World = GetWorld();
		if (World && !World->IsPlayingReplay() && (World->WorldType == EWorldType::Game))
		{
			UE_LOGF(LogGameplayCueNotify, Error, "GameplayCueNotify [%ls] is calling EndPlay(). This should not happen since they are recycled through the GameplayCueManager.", *GetName());
		}
	}

	if (EndPlayReason == EEndPlayReason::Destroyed)
	{
		UAbilitySystemGlobals::Get().GetGameplayCueManager()->NotifyGameplayCueActorEndPlay(this);
	}

	Super::EndPlay(EndPlayReason);
}

void AGameplayCueNotify_Actor::K2_DestroyActor()
{
	if (UGameplayCueManager::IsGameplayCueRecylingEnabled())
	{
		UE_LOGF(LogGameplayCueNotify, Warning, "GameplayCueNotify [%ls] is calling DesotryActor(). This is not necessary as GCs will be cleaned up and recycled automatically.", *GetName());
	}
	else
	{
		Super::K2_DestroyActor();
	}
}

void AGameplayCueNotify_Actor::Destroyed()
{
	if (UGameplayCueManager::IsGameplayCueRecylingEnabled())
	{
		UWorld* World = GetWorld();
		if (World && !World->IsPlayingReplay() && (World->WorldType == EWorldType::Game))
		{
			UE_LOGF(LogGameplayCueNotify, Error, "GameplayCueNotify [%ls] is calling Destroyed(). This should not happen since they are recycled through the GameplayCueManager.", *GetName());
		}
	}

	Super::Destroyed();
}

#if WITH_EDITOR
EDataValidationResult AGameplayCueNotify_Actor::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult ReturnValue = Super::IsDataValid(Context);

	auto IsBlueprintOverride = [&](const FName& FuncName) -> bool
	{
		const UFunction* Func = GetClass()->FindFunctionByName(FuncName);
		return Func && ensure(Func->GetOuter())
			&& Func->GetOuter()->IsA(UBlueprintGeneratedClass::StaticClass());
	};

	bool bHasOnActiveInBP = IsBlueprintOverride(GET_FUNCTION_NAME_CHECKED(AGameplayCueNotify_Actor, OnActive));
	bool bHasWhileActiveInBP = IsBlueprintOverride(GET_FUNCTION_NAME_CHECKED(AGameplayCueNotify_Actor, WhileActive));
	bool bHasOnRemoveInBP = IsBlueprintOverride(GET_FUNCTION_NAME_CHECKED(AGameplayCueNotify_Actor, OnRemove));

	if (bHasOnActiveInBP && bHasOnRemoveInBP && !bHasWhileActiveInBP)
	{
		Context.AddWarning(LOCTEXT("OnActiveAndOnRemove", "OnCeaseRelevant implemented without OnBecomeRelevant. OnBurst is likely a mistake (if not, implement a blank OnBecomeRelevant)."));
	}

	return ReturnValue;
}

void AGameplayCueNotify_Actor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(GetClass());

	if (PropertyThatChanged && PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(AGameplayCueNotify_Actor, GameplayCueTag))
	{
		DeriveGameplayCueTagFromAssetName();
		UAbilitySystemGlobals::Get().GetGameplayCueManager()->HandleAssetDeleted(Blueprint);
		UAbilitySystemGlobals::Get().GetGameplayCueManager()->HandleAssetAdded(Blueprint);
	}
}
#endif

void AGameplayCueNotify_Actor::DeriveGameplayCueTagFromAssetName()
{
	UAbilitySystemGlobals::DeriveGameplayCueTagFromClass<AGameplayCueNotify_Actor>(this);
}

void AGameplayCueNotify_Actor::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		DeriveGameplayCueTagFromAssetName();
	}

	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		DeriveGameplayCueTagFromAssetName();
	}
}

void AGameplayCueNotify_Actor::BeginPlay()
{
	Super::BeginPlay();
	AttachToOwnerIfNecessary();
}

void AGameplayCueNotify_Actor::SetOwner( AActor* InNewOwner )
{
	// Remove our old delegate
	ClearOwnerDestroyedDelegate();

	Super::SetOwner(InNewOwner);
	if (AActor* NewOwner = GetOwner())
	{
		NewOwner->OnDestroyed.AddDynamic(this, &AGameplayCueNotify_Actor::OnOwnerDestroyed);
		AttachToOwnerIfNecessary();
	}
}

void AGameplayCueNotify_Actor::AttachToOwnerIfNecessary()
{
	if (AActor* MyOwner = GetOwner())
	{
		if (bAutoAttachToOwner)
		{
			AttachToActor(MyOwner, FAttachmentTransformRules::SnapToTargetNotIncludingScale);
		}
	}
}

void AGameplayCueNotify_Actor::ClearOwnerDestroyedDelegate()
{
	AActor* OldOwner = GetOwner();
	if (OldOwner)
	{
		OldOwner->OnDestroyed.RemoveDynamic(this, &AGameplayCueNotify_Actor::OnOwnerDestroyed);
	}
}

void AGameplayCueNotify_Actor::PostInitProperties()
{
	Super::PostInitProperties();
	DeriveGameplayCueTagFromAssetName();
}

bool AGameplayCueNotify_Actor::HandlesEvent(EGameplayCueEvent::Type EventType) const
{
	return true;
}

void AGameplayCueNotify_Actor::K2_EndGameplayCue()
{
	GameplayCueFinishedCallback();
}

int32 GameplayCueNotifyTagCheckOnRemove = 1;
static FAutoConsoleVariableRef CVarGameplayCueNotifyActorStacking(TEXT("AbilitySystem.GameplayCueNotifyTagCheckOnRemove"), GameplayCueNotifyTagCheckOnRemove, TEXT("Check that target no longer has tag when removing GameplayCues"), ECVF_Default );

void AGameplayCueNotify_Actor::HandleGameplayCue(AActor* MyTarget, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters)
{
	SCOPE_CYCLE_COUNTER(STAT_HandleGameplayCueNotifyActor);

	if (Parameters.MatchedTagName.IsValid() == false)
	{
		ABILITY_LOG(Warning, TEXT("GameplayCue parameter is none for %s"), *GetNameSafe(this));
	}

	// Handle multiple event gating
	{
		if (EventType == EGameplayCueEvent::OnActive && !bAllowMultipleOnActiveEvents && bHasHandledOnActiveEvent)
		{
			return;
		}

		if (EventType == EGameplayCueEvent::WhileActive && !bAllowMultipleWhileActiveEvents && bHasHandledWhileActiveEvent)
		{
			ABILITY_LOG(Log, TEXT("GameplayCue Notify %s WhileActive already handled, skipping this one."), *GetName());
			return;
		}

		if (EventType == EGameplayCueEvent::Removed && bHasHandledOnRemoveEvent)
		{
			return;
		}
	}

	// If cvar is enabled, check that the target no longer has the matched tag before doing remove logic. This is a simple way of supporting stacking, such that if an actor has two sources giving it the same GC tag, it will not be removed when the first one is removed.
	if (GameplayCueNotifyTagCheckOnRemove > 0 && EventType == EGameplayCueEvent::Removed)
	{
		if (IGameplayTagAssetInterface* TagInterface = Cast<IGameplayTagAssetInterface>(MyTarget))
		{
			if (TagInterface->HasMatchingGameplayTag(Parameters.MatchedTagName))
			{
				return;
			}			
		}
	}

	if (IsValid(MyTarget))
	{
		K2_HandleGameplayCue(MyTarget, EventType, Parameters);

		// Clear any pending auto-destroy that may have occurred from a previous OnRemove
		SetLifeSpan(0.f);

		switch (EventType)
		{
		case EGameplayCueEvent::OnActive:
			OnActive(MyTarget, Parameters);
			bHasHandledOnActiveEvent = true;
			break;

		case EGameplayCueEvent::WhileActive:
			WhileActive(MyTarget, Parameters);
			bHasHandledWhileActiveEvent = true;
			if (CVarGameplayCueNotifyMultipleRemoveEvents.GetValueOnGameThread())
			{
				bHasHandledOnRemoveEvent = false;
			}
			break;

		case EGameplayCueEvent::Executed:
			OnExecute(MyTarget, Parameters);
			break;

		case EGameplayCueEvent::Removed:
			bHasHandledOnRemoveEvent = true;
			OnRemove(MyTarget, Parameters);

			if (bAutoDestroyOnRemove)
			{
				if (AutoDestroyDelay > 0.f)
				{
					FTimerDelegate Delegate = FTimerDelegate::CreateUObject(this, &AGameplayCueNotify_Actor::GameplayCueFinishedCallback);
					GetWorld()->GetTimerManager().SetTimer(FinishTimerHandle, Delegate, AutoDestroyDelay, false);
				}
				else
				{
					GameplayCueFinishedCallback();
				}
			}
			break;
		};
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("Null Target called for event %d on GameplayCueNotifyActor %s"), (int32)EventType, *GetName() );
		if (EventType == EGameplayCueEvent::Removed)
		{
			// Make sure the removed event is handled so that we don't leak GC notify actors
			GameplayCueFinishedCallback();
		}
	}
}

void AGameplayCueNotify_Actor::OnOwnerDestroyed(AActor* DestroyedActor)
{
	if (bInRecycleQueue)
	{
		// We are already done
		return;
	}

	// May need to do extra cleanup in child classes
	GameplayCueFinishedCallback();
}

bool AGameplayCueNotify_Actor::OnExecute_Implementation(AActor* MyTarget, const FGameplayCueParameters& Parameters)
{
	return false;
}

bool AGameplayCueNotify_Actor::OnActive_Implementation(AActor* MyTarget, const FGameplayCueParameters& Parameters)
{
	return false;
}

bool AGameplayCueNotify_Actor::WhileActive_Implementation(AActor* MyTarget, const FGameplayCueParameters& Parameters)
{
	if (IsHidden())
	{
		SetActorHiddenInGame(false);
	}

	return false;
}

bool AGameplayCueNotify_Actor::OnRemove_Implementation(AActor* MyTarget, const FGameplayCueParameters& Parameters)
{
	if (!IsHidden())
	{
		SetActorHiddenInGame(true);
	}

	return false;
}

void AGameplayCueNotify_Actor::GameplayCueFinishedCallback()
{
	UWorld* MyWorld = GetWorld();
	if (MyWorld) // Teardown cases in PIE may cause the world to be invalid
	{
		if (FinishTimerHandle.IsValid())
		{
			MyWorld->GetTimerManager().ClearTimer(FinishTimerHandle);
			FinishTimerHandle.Invalidate();
		}

		// Make sure OnRemoved has been called at least once if WhileActive was called (for possible cleanup)
		if (bHasHandledWhileActiveEvent && !bHasHandledOnRemoveEvent)
		{
			// Force onremove to be called with null parameters
			bHasHandledOnRemoveEvent = true;
			OnRemove(nullptr, FGameplayCueParameters());
		}
	}
	
	UAbilitySystemGlobals::Get().GetGameplayCueManager()->NotifyGameplayCueActorFinished(this);
}

bool AGameplayCueNotify_Actor::GameplayCuePendingRemove()
{
	return GetLifeSpan() > 0.f || FinishTimerHandle.IsValid() || !IsValidChecked(this);
}

bool AGameplayCueNotify_Actor::Recycle()
{
	bHasHandledOnActiveEvent = false;
	bHasHandledWhileActiveEvent = false;
	bHasHandledOnRemoveEvent = false;
	CueInstigator = nullptr;
	CueSourceObject = nullptr;

	ClearOwnerDestroyedDelegate();
	if (FinishTimerHandle.IsValid())
	{
		FinishTimerHandle.Invalidate();
	}

	// End timeline components
	TInlineComponentArray<UTimelineComponent*> TimelineComponents(this);
	for (UTimelineComponent* Timeline : TimelineComponents)
	{
		if (Timeline)
		{
			// May be too spammy, but want to call visibility to this. Maybe make this editor only?
			if (Timeline->IsPlaying() && WarnIfTimelineIsStillRunning)
			{
				ABILITY_LOG(Warning, TEXT("GameplayCueNotify_Actor %s had active timelines when it was recycled."), *GetName());
			}

			Timeline->SetPlaybackPosition(0.f, false, false);
			Timeline->Stop();
		}
	}

	UWorld* MyWorld = GetWorld();
	if (MyWorld)
	{
		// Note, ::Recycle is called on CDOs too, so that even "new" GCs start off in a recycled state.
		// So, its ok if there is no valid world here, just skip the stuff that has to do with worlds.
		if (MyWorld->GetLatentActionManager().GetNumActionsForObject(this) && WarnIfLatentActionIsStillRunning)
		{
			// May be too spammy, but want ot call visibility to this. Maybe make this editor only?
			ABILITY_LOG(Warning, TEXT("GameplayCueNotify_Actor %s has active latent actions (Delays, etc) when it was recycled."), *GetName());
		}

		// End latent actions
		MyWorld->GetLatentActionManager().RemoveActionsForObject(this);

		if (FAbilitySystemTweaks::ClearCueNotifyTimers)
		{
			// End all timers
			MyWorld->GetTimerManager().ClearAllTimersForObject(this);
		}
	}

	// Clear owner, hide, detach from parent
	SetOwner(nullptr);
	SetActorHiddenInGame(true);
	DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);

	return true;
}

void AGameplayCueNotify_Actor::ReuseAfterRecycle()
{
	SetActorHiddenInGame(false);
}

#undef LOCTEXT_NAMESPACE
