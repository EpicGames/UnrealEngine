// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueNotify_Burst.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueNotify_Burst)


//////////////////////////////////////////////////////////////////////////
// UGameplayCueNotify_Burst
//////////////////////////////////////////////////////////////////////////
UGameplayCueNotify_Burst::UGameplayCueNotify_Burst()
{
}

bool UGameplayCueNotify_Burst::OnExecute_Implementation(AActor* Target, const FGameplayCueParameters& Parameters) const
{
	UWorld* World = (Target ? Target->GetWorld() : GetWorld());

	FGameplayCueNotify_SpawnContext SpawnContext(World, Target, Parameters);
	SpawnContext.SetDefaultSpawnCondition(&DefaultSpawnCondition);
	SpawnContext.SetDefaultPlacementInfo(&DefaultPlacementInfo);

	if (DefaultSpawnCondition.ShouldSpawn(SpawnContext))
	{
		FGameplayCueNotify_SpawnResult SpawnResult;

		BurstEffects.ExecuteEffects(SpawnContext, SpawnResult);

		OnBurst(Target, Parameters, SpawnResult);
	}

	return false;
}

#if WITH_EDITOR
EDataValidationResult UGameplayCueNotify_Burst::IsDataValid(FDataValidationContext& Context) const
{
	BurstEffects.ValidateAssociatedAssets(this, TEXT("BurstEffects"), Context);

	return ((Context.GetNumErrors() > 0) ? EDataValidationResult::Invalid : EDataValidationResult::Valid);
}
#endif // #if WITH_EDITOR

