// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueNotify_BurstLatent.h"
#include "Engine/World.h"
#include "TimerManager.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueNotify_BurstLatent)


const float DefaultBurstLatentLifetime = 5.0f;


//////////////////////////////////////////////////////////////////////////
// AGameplayCueNotify_BurstLatent
//////////////////////////////////////////////////////////////////////////
AGameplayCueNotify_BurstLatent::AGameplayCueNotify_BurstLatent()
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent"));

	PrimaryActorTick.bStartWithTickEnabled = false;
	bAutoDestroyOnRemove = true;
	NumPreallocatedInstances = 3;

	Recycle();
}

bool AGameplayCueNotify_BurstLatent::Recycle()
{
	Super::Recycle();

	BurstSpawnResults.Reset();

	return true;
}

bool AGameplayCueNotify_BurstLatent::OnExecute_Implementation(AActor* Target, const FGameplayCueParameters& Parameters)
{
	UWorld* World = GetWorld();

	FGameplayCueNotify_SpawnContext SpawnContext(World, Target, Parameters);
	SpawnContext.SetDefaultSpawnCondition(&DefaultSpawnCondition);
	SpawnContext.SetDefaultPlacementInfo(&DefaultPlacementInfo);

	if (DefaultSpawnCondition.ShouldSpawn(SpawnContext))
	{
		BurstEffects.ExecuteEffects(SpawnContext, BurstSpawnResults);

		OnBurst(Target, Parameters, BurstSpawnResults);
	}

	// Handle GC removal by default. This is a simple default to handle all cases we can currently think of.
	// If we didn't do this, we'd be relying on every BurstLatent GC manually setting up its removal within BP graphs,
	// or some inference based on parameters.
	if (World)
	{
		const float Lifetime = FMath::Max<float>(AutoDestroyDelay, DefaultBurstLatentLifetime);
		World->GetTimerManager().SetTimer(FinishTimerHandle, this, &AGameplayCueNotify_Actor::GameplayCueFinishedCallback, Lifetime);
	}

	return false;
}

#if WITH_EDITOR
EDataValidationResult AGameplayCueNotify_BurstLatent::IsDataValid(FDataValidationContext& Context) const
{
	BurstEffects.ValidateAssociatedAssets(this, TEXT("BurstEffects"), Context);

	return ((Context.GetNumErrors() > 0) ? EDataValidationResult::Invalid : EDataValidationResult::Valid);
}
#endif // #if WITH_EDITOR

