// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueNotify_HitImpact.h"
#include "Kismet/GameplayStatics.h"
#include "GameplayCueManager.h"
#include "AbilitySystemGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueNotify_HitImpact)

UGameplayCueNotify_HitImpact::UGameplayCueNotify_HitImpact(const FObjectInitializer& PCIP)
: Super(PCIP)
{

}

bool UGameplayCueNotify_HitImpact::HandlesEvent(EGameplayCueEvent::Type EventType) const
{
	return (EventType == EGameplayCueEvent::Executed);
}

void UGameplayCueNotify_HitImpact::HandleGameplayCue(AActor* Self, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters)
{
	check(EventType == EGameplayCueEvent::Executed);
	
	const UObject* WorldContextObject = Self;
	if (!WorldContextObject)
	{
		WorldContextObject = UAbilitySystemGlobals::Get().GetGameplayCueManager();
	}

	if (ParticleSystem && WorldContextObject)
	{
		const FHitResult* HitResult = Parameters.EffectContext.GetHitResult();
		if (HitResult)
		{
			UGameplayStatics::SpawnEmitterAtLocation(WorldContextObject, ParticleSystem, HitResult->ImpactPoint, HitResult->ImpactNormal.Rotation(), true);
		}
		else
		{
			FVector Location = FVector::ZeroVector;
			FRotator Rotation = FRotator::ZeroRotator;
			if (Self)
			{
				Location = Self->GetActorLocation();
				Rotation = Self->GetActorRotation();
			}
			else
			{
				Location = Parameters.Location;
				Rotation = Parameters.Normal.Rotation();
			}
			UGameplayStatics::SpawnEmitterAtLocation(WorldContextObject, ParticleSystem, Location, Rotation, true);
		}
	}
}

