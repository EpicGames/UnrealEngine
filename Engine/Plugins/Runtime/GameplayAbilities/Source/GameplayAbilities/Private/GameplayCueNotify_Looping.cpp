// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueNotify_Looping.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueNotify_Looping)


//////////////////////////////////////////////////////////////////////////
// AGameplayCueNotify_Looping
//////////////////////////////////////////////////////////////////////////
AGameplayCueNotify_Looping::AGameplayCueNotify_Looping()
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent"));

	PrimaryActorTick.bStartWithTickEnabled = false;
	bAutoDestroyOnRemove = true;
	bAllowMultipleWhileActiveEvents = false;
	NumPreallocatedInstances = 3;

	DefaultPlacementInfo.AttachPolicy = EGameplayCueNotify_AttachPolicy::AttachToTarget;

	Recycle();
}

void AGameplayCueNotify_Looping::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (EndPlayReason == EEndPlayReason::Destroyed)
	{
		RemoveLoopingEffects();
	}

	Super::EndPlay(EndPlayReason);
}

bool AGameplayCueNotify_Looping::Recycle()
{
	Super::Recycle();

	// Extra check to make sure looping effects have been removed.  Normally they will have been removed in the OnRemove event.
	RemoveLoopingEffects();

	ApplicationSpawnResults.Reset();
	LoopingSpawnResults.Reset();
	RecurringSpawnResults.Reset();
	RemovalSpawnResults.Reset();

	bLoopingEffectsRemoved = true;

	return true;
}

bool AGameplayCueNotify_Looping::OnActive_Implementation(AActor* Target, const FGameplayCueParameters& Parameters)
{
	UWorld* World = GetWorld();

	FGameplayCueNotify_SpawnContext SpawnContext(World, Target, Parameters);
	SpawnContext.SetDefaultSpawnCondition(&DefaultSpawnCondition);
	SpawnContext.SetDefaultPlacementInfo(&DefaultPlacementInfo);

	if (DefaultSpawnCondition.ShouldSpawn(SpawnContext))
	{
		ApplicationEffects.ExecuteEffects(SpawnContext, ApplicationSpawnResults);

		OnApplication(Target, Parameters, ApplicationSpawnResults);
	}

	return false;
}

bool AGameplayCueNotify_Looping::WhileActive_Implementation(AActor* Target, const FGameplayCueParameters& Parameters)
{
	UWorld* World = GetWorld();

	FGameplayCueNotify_SpawnContext SpawnContext(World, Target, Parameters);
	SpawnContext.SetDefaultSpawnCondition(&DefaultSpawnCondition);
	SpawnContext.SetDefaultPlacementInfo(&DefaultPlacementInfo);

	if (DefaultSpawnCondition.ShouldSpawn(SpawnContext))
	{
		bLoopingEffectsRemoved = false;

		LoopingEffects.StartEffects(SpawnContext, LoopingSpawnResults);	

		OnLoopingStart(Target, Parameters, LoopingSpawnResults);
	}

	return false;
}

bool AGameplayCueNotify_Looping::OnExecute_Implementation(AActor* Target, const FGameplayCueParameters& Parameters)
{
	UWorld* World = GetWorld();

	FGameplayCueNotify_SpawnContext SpawnContext(World, Target, Parameters);
	SpawnContext.SetDefaultSpawnCondition(&DefaultSpawnCondition);
	SpawnContext.SetDefaultPlacementInfo(&DefaultPlacementInfo);

	if (DefaultSpawnCondition.ShouldSpawn(SpawnContext))
	{
		RecurringEffects.ExecuteEffects(SpawnContext, RecurringSpawnResults);

		OnRecurring(Target, Parameters, RecurringSpawnResults);
	}

	return false;
}

bool AGameplayCueNotify_Looping::OnRemove_Implementation(AActor* Target, const FGameplayCueParameters& Parameters)
{
	RemoveLoopingEffects();

	// Don't spawn removal effects if our target is gone.
	if (Target)
	{
		UWorld* World = GetWorld();

		FGameplayCueNotify_SpawnContext SpawnContext(World, Target, Parameters);
		SpawnContext.SetDefaultSpawnCondition(&DefaultSpawnCondition);
		SpawnContext.SetDefaultPlacementInfo(&DefaultPlacementInfo);

		if (DefaultSpawnCondition.ShouldSpawn(SpawnContext))
		{
			RemovalEffects.ExecuteEffects(SpawnContext, RemovalSpawnResults);
		}
	}

	// Always call OnRemoval(), even if target is bad, so it can clean up BP-spawned things.
	OnRemoval(Target, Parameters, RemovalSpawnResults);

	return false;
}

void AGameplayCueNotify_Looping::RemoveLoopingEffects()
{
	if (bLoopingEffectsRemoved)
	{
		return;
	}

	bLoopingEffectsRemoved = true;

	LoopingEffects.StopEffects(LoopingSpawnResults);
}

#if WITH_EDITOR
EDataValidationResult AGameplayCueNotify_Looping::IsDataValid(FDataValidationContext& Context) const
{
	TArray<FText> ValidationErrors;
	ApplicationEffects.ValidateAssociatedAssets(this, TEXT("ApplicationEffects"), Context);
	LoopingEffects.ValidateAssociatedAssets(this, TEXT("LoopingEffects"), Context);
	RecurringEffects.ValidateAssociatedAssets(this, TEXT("RecurringEffects"), Context);
	RemovalEffects.ValidateAssociatedAssets(this, TEXT("RemovalEffects"), Context);

	return ((ValidationErrors.Num() > 0) ? EDataValidationResult::Invalid : EDataValidationResult::Valid);
}
#endif // #if WITH_EDITOR

