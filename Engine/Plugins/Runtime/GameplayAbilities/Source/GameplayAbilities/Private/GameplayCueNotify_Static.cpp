// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueNotify_Static.h"
#include "Engine/Blueprint.h"
#include "AbilitySystemStats.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"
#include "GameplayCueManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueNotify_Static)

UGameplayCueNotify_Static::UGameplayCueNotify_Static(const FObjectInitializer& PCIP)
: Super(PCIP)
{
	IsOverride = true;
}

#if WITH_EDITOR
void UGameplayCueNotify_Static::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(GetClass());

	if (PropertyThatChanged && PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(UGameplayCueNotify_Static, GameplayCueTag))
	{
		DeriveGameplayCueTagFromAssetName();
		UAbilitySystemGlobals::Get().GetGameplayCueManager()->HandleAssetDeleted(Blueprint);
		UAbilitySystemGlobals::Get().GetGameplayCueManager()->HandleAssetAdded(Blueprint);
	}
}
#endif

void UGameplayCueNotify_Static::DeriveGameplayCueTagFromAssetName()
{
	UAbilitySystemGlobals::DeriveGameplayCueTagFromClass<UGameplayCueNotify_Static>(this);
}

void UGameplayCueNotify_Static::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		DeriveGameplayCueTagFromAssetName();
	}

	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		DeriveGameplayCueTagFromAssetName();
	}
}

void UGameplayCueNotify_Static::PostInitProperties()
{
	Super::PostInitProperties();
	DeriveGameplayCueTagFromAssetName();
}

bool UGameplayCueNotify_Static::HandlesEvent(EGameplayCueEvent::Type EventType) const
{
	return true;
}

void UGameplayCueNotify_Static::HandleGameplayCue(AActor* MyTarget, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters)
{
	SCOPE_CYCLE_COUNTER(STAT_HandleGameplayCueNotifyStatic);

	if (IsValid(MyTarget))
	{
		K2_HandleGameplayCue(MyTarget, EventType, Parameters);

		switch (EventType)
		{
		case EGameplayCueEvent::OnActive:
			OnActive(MyTarget, Parameters);
			break;

		case EGameplayCueEvent::WhileActive:
			WhileActive(MyTarget, Parameters);
			break;

		case EGameplayCueEvent::Executed:
			OnExecute(MyTarget, Parameters);
			break;

		case EGameplayCueEvent::Removed:
			OnRemove(MyTarget, Parameters);
			break;
		};
	}
	else
	{
		ABILITY_LOG(Warning, TEXT("Null Target"));
	}
}

void UGameplayCueNotify_Static::OnOwnerDestroyed()
{
}

bool UGameplayCueNotify_Static::OnExecute_Implementation(AActor* MyTarget, const FGameplayCueParameters& Parameters) const
{
	return false;
}

bool UGameplayCueNotify_Static::OnActive_Implementation(AActor* MyTarget, const FGameplayCueParameters& Parameters) const
{
	return false;
}

bool UGameplayCueNotify_Static::WhileActive_Implementation(AActor* MyTarget, const FGameplayCueParameters& Parameters) const
{
	return false;
}

bool UGameplayCueNotify_Static::OnRemove_Implementation(AActor* MyTarget, const FGameplayCueParameters& Parameters) const
{
	return false;
}

UWorld* UGameplayCueNotify_Static::GetWorld() const
{
	return UGameplayCueManager::GetCachedWorldForGameplayCueNotifies();
}

