// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueSet.h"
#include "GameplayTagsManager.h"
#include "GameplayTagsModule.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemLog.h"
#include "GameplayCueNotify_Actor.h"
#include "GameplayCueNotify_Static.h"
#include "GameplayCueManager.h"
#include "Misc/CoreMiscDefines.h"
#include "NativeGameplayTags.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueSet)

UE_DEFINE_GAMEPLAY_TAG_STATIC(StaticTag_GameplayCue, TEXT("GameplayCue"));

namespace GameplayCueDebug
{
	static FGameplayTagContainer DebugGameplayCueFilter;
	FAutoConsoleCommand ConCommandDebugGameplayCueFilter(
		TEXT("GameplayCue.FilterCuesByTag"),
		TEXT("Adds or removes a GameplayCue tag to the debug filter list. If the filter is populated, only GameplayCues with tags in the filter will be invoked."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args) 
			{
				for (const FString& TagToFilter : Args)
				{
					const FGameplayTag ExistingTag = FGameplayTag::RequestGameplayTag(FName(TagToFilter));
					if (!ExistingTag.IsValid())
					{
						continue;
					}

					if (DebugGameplayCueFilter.HasTagExact(ExistingTag))
					{
						DebugGameplayCueFilter.RemoveTag(ExistingTag);
					}
					else
					{
						DebugGameplayCueFilter.AddTagFast(ExistingTag);
					}
				}
			})
	);
}

// Let's use this to create a path for testing what happens when GameplayCues are failing to load (or are loading slowly due to IO contention)
static TAutoConsoleVariable<bool> CVarGameplayCueFailLoads(TEXT("AbilitySystem.GameplayCueFailLoads"), false, TEXT("Pretend all GameplayCues are unloaded (and keep requesting loads) while true. Set to false to allow GameplayCues to play."), ECVF_Default);

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	UGameplayCueSet
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------


UGameplayCueSet::UGameplayCueSet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

bool UGameplayCueSet::HandleGameplayCue(AActor* TargetActor, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters)
{
#if WITH_SERVER_CODE
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GameplayCueSet_HandleGameplayCue);
#endif
	
#if !UE_BUILD_SHIPPING
	if (!GameplayCueDebug::DebugGameplayCueFilter.IsEmpty())
	{
		if (!GameplayCueDebug::DebugGameplayCueFilter.HasTagExact(GameplayCueTag))
		{
			return false;
		}
	}
#endif

	// GameplayCueTags could have been removed from the dictionary but not content. When the content is resaved the old tag will be cleaned up, but it could still come through here
	// at runtime. Since we only populate the map with dictionary gameplaycue tags, we may not find it here.
	int32* Ptr = GameplayCueDataMap.Find(GameplayCueTag);
	if (Ptr && *Ptr != INDEX_NONE)
	{
		int32 DataIdx = *Ptr;

		// TODO - resolve internal handler modifying params before passing them on with new const-ref params.
		FGameplayCueParameters writableParameters = Parameters;
		return HandleGameplayCueNotify_Internal(TargetActor, DataIdx, EventType, writableParameters);
	}

	return false;
}

void UGameplayCueSet::AddCues(const TArray<FGameplayCueReferencePair>& CuesToAdd)
{
	if (CuesToAdd.Num() > 0)
	{
		GameplayCueData.Reserve(GameplayCueData.Num() + CuesToAdd.Num());
		GameplayCueDataMap.Reserve(GameplayCueDataMap.Num() + CuesToAdd.Num());
		for (const FGameplayCueReferencePair& CueRefPair : CuesToAdd)
		{
			const FGameplayTag& GameplayCueTag = CueRefPair.GameplayCueTag;
			const FSoftObjectPath& StringRef = CueRefPair.StringRef;

			const int32* ExistingId = GameplayCueDataMap.Find(GameplayCueTag);
			if (ExistingId && *ExistingId != INDEX_NONE)
			{
				const FGameplayCueNotifyData& ExistingData = GameplayCueData[*ExistingId];
				// GameplayCueDataMap also have a redirection for child tags, but we allow to override those
				if (ExistingData.GameplayCueTag == GameplayCueTag)
				{
					TStringBuilder<256> AddingCueName;
					TStringBuilder<256> ExistingCueName;
					StringRef.ToString(AddingCueName);
					ExistingData.GameplayCueNotifyObj.ToString(ExistingCueName);
					if (AddingCueName.ToView() != ExistingCueName.ToView())
					{
						ABILITY_LOG(Warning, 
							TEXT("Multiple gameplay cues are triggered using tag '%s'. ")
							TEXT("No two gameplay cues may be triggered by the same tag. ")
							TEXT("Cue '%s' and '%s' are triggered on this tag. Skipping."), 
							*GameplayCueTag.ToString(), 
							*ExistingCueName, 
							*AddingCueName);
					}
					continue;
				}
			}

			FGameplayCueNotifyData NewData;
			NewData.GameplayCueNotifyObj = StringRef;
			NewData.GameplayCueTag = GameplayCueTag;

			GameplayCueData.Add(NewData);

			// GameplayCueDataMap will be rebuilt after the loop,
			// so it does not matter if child tags incorrectly point to the parent.
			// We will use the map only as a duplicate lookup during the loop.
			GameplayCueDataMap.Add(GameplayCueTag, GameplayCueData.Num() - 1);
		}

		BuildAccelerationMap_Internal();
	}
}

void UGameplayCueSet::RemoveCuesByTags(const FGameplayTagContainer& TagsToRemove)
{
	bool bAnyRemoved = false;
	for (auto It = GameplayCueData.CreateIterator(); It; ++It)
	{
		if (TagsToRemove.HasAnyExact(FGameplayTagContainer{It->GameplayCueTag}))
		{
			bAnyRemoved = true;
			It.RemoveCurrentSwap();
		}
	}
	if (bAnyRemoved)
	{
		BuildAccelerationMap_Internal();
	}
}

void UGameplayCueSet::RemoveCuesByStringRefs(const TArray<FSoftObjectPath>& CuesToRemove)
{
	for (const FSoftObjectPath& StringRefToRemove : CuesToRemove)
	{
		for (int32 idx = 0; idx < GameplayCueData.Num(); ++idx)
		{
			if (GameplayCueData[idx].GameplayCueNotifyObj == StringRefToRemove)
			{
				GameplayCueData.RemoveAt(idx);
				BuildAccelerationMap_Internal();
				break;
			}
		}
	}
}

void UGameplayCueSet::RemoveLoadedClass(UClass* Class)
{
	for (int32 idx = 0; idx < GameplayCueData.Num(); ++idx)
	{
		if (GameplayCueData[idx].LoadedGameplayCueClass == Class)
		{
			GameplayCueData[idx].LoadedGameplayCueClass = nullptr;
		}
	}
}

void UGameplayCueSet::GetFilenames(TArray<FString>& Filenames) const
{
	Filenames.Reserve(GameplayCueData.Num());
	for (const FGameplayCueNotifyData& Data : GameplayCueData)
	{
		Filenames.Add(Data.GameplayCueNotifyObj.GetLongPackageName());
	}
}

void UGameplayCueSet::GetSoftObjectPaths(TArray<FSoftObjectPath>& List) const
{
	List.Reserve(GameplayCueData.Num());
	for (const FGameplayCueNotifyData& Data : GameplayCueData)
	{
		List.Add(Data.GameplayCueNotifyObj);
	}
}

#if WITH_EDITOR
void UGameplayCueSet::UpdateCueByStringRefs(const FSoftObjectPath& CueToRemove, FString NewPath)
{
	for (int32 idx = 0; idx < GameplayCueData.Num(); ++idx)
	{
		if (GameplayCueData[idx].GameplayCueNotifyObj == CueToRemove)
		{
			GameplayCueData[idx].GameplayCueNotifyObj = NewPath;
			BuildAccelerationMap_Internal();
			break;
		}
	}
}

void UGameplayCueSet::CopyCueDataToSetForEditorPreview(FGameplayTag Tag, UGameplayCueSet* DestinationSet)
{
	const int32 SourceIdx = GameplayCueData.IndexOfByPredicate([&Tag](const FGameplayCueNotifyData& Data) { return Data.GameplayCueTag == Tag; });
	if (SourceIdx == INDEX_NONE)
	{
		// Doesn't exist in source, so nothing to copy
		return;
	}

	int32 DestIdx = DestinationSet->GameplayCueData.IndexOfByPredicate([&Tag](const FGameplayCueNotifyData& Data) { return Data.GameplayCueTag == Tag; });
	if (DestIdx == INDEX_NONE)
	{
		// wholesale copy
		DestIdx = DestinationSet->GameplayCueData.Num();
		DestinationSet->GameplayCueData.Add(GameplayCueData[SourceIdx]);

		DestinationSet->BuildAccelerationMap_Internal();
	}
	else
	{
		// Update only if we need to
		if (DestinationSet->GameplayCueData[DestIdx].GameplayCueNotifyObj.IsValid() == false)
		{
			DestinationSet->GameplayCueData[DestIdx].GameplayCueNotifyObj = GameplayCueData[SourceIdx].GameplayCueNotifyObj;
			DestinationSet->GameplayCueData[DestIdx].LoadedGameplayCueClass = GameplayCueData[SourceIdx].LoadedGameplayCueClass;
		}

	}

	// Start async load
	UGameplayCueManager* CueManager = UAbilitySystemGlobals::Get().GetGameplayCueManager();
	if (ensure(CueManager))
	{
		CueManager->StreamableManager.RequestAsyncLoad(DestinationSet->GameplayCueData[DestIdx].GameplayCueNotifyObj);
	}
}

#endif

void UGameplayCueSet::Empty()
{
	GameplayCueData.Empty();
	GameplayCueDataMap.Empty();
}

void UGameplayCueSet::PrintCues() const
{
	FGameplayTagContainer AllGameplayCueTags = UGameplayTagsManager::Get().RequestGameplayTagChildren(BaseGameplayCueTag());

	for (FGameplayTag ThisGameplayCueTag : AllGameplayCueTags)
	{
		const int32* idx = GameplayCueDataMap.Find(ThisGameplayCueTag);
		if (!idx)
		{
			ABILITY_LOG(Warning, TEXT("   %s -> index not found"), *ThisGameplayCueTag.ToString());
		}
		else if (*idx != INDEX_NONE)
		{
			ABILITY_LOG(Warning, TEXT("   %s -> %d"), *ThisGameplayCueTag.ToString(), *idx);
		}
		else
		{
			ABILITY_LOG(Warning, TEXT("   %s -> unmapped"), *ThisGameplayCueTag.ToString());
		}
	}
}

bool UGameplayCueSet::HandleGameplayCueNotify_Internal(AActor* TargetActor, int32 DataIdx, EGameplayCueEvent::Type EventType, FGameplayCueParameters& Parameters)
{	
	bool bReturnVal = false;

	UGameplayCueManager* CueManager = UAbilitySystemGlobals::Get().GetGameplayCueManager();
	if (!ensure(CueManager))
	{
		return false;
	}

	if (DataIdx != INDEX_NONE)
	{
		check(GameplayCueData.IsValidIndex(DataIdx));

		FGameplayCueNotifyData& CueData = GameplayCueData[DataIdx];

		Parameters.MatchedTagName = CueData.GameplayCueTag;

		const bool bDebugFailLoads = CVarGameplayCueFailLoads.GetValueOnGameThread();

		// If object is not loaded yet
		if (CueData.LoadedGameplayCueClass == nullptr || bDebugFailLoads)
		{
			// See if the object is loaded but just not hooked up here. Only do this just-in-time
			// detection of GCN class when the CueManager isn't waiting for an async load of the
			// class. Otherwise, defer to CueManager->HandleMissingGameplayCue so that it can queue 
			// events and execute them in order on async load completion. This addresses a race 
			// condition where a UClass is resolvable but the async load callback hasn't fired yet 
			// since those are deferred to the next frame. Previously this resulted in cues being
			// executed out of order (remove before add).
			if (!CueManager->IsAsyncLoadingGameplayCueNotifyClass(CueData.GameplayCueNotifyObj))
			{
				CueData.LoadedGameplayCueClass = Cast<UClass>(CueData.GameplayCueNotifyObj.ResolveObject());
			}

			if (CueData.LoadedGameplayCueClass == nullptr || bDebugFailLoads)
			{
				if (!CueManager->HandleMissingGameplayCue(this, CueData, TargetActor, EventType, Parameters))
				{
					return false;
				}
			}
		}

		check(CueData.LoadedGameplayCueClass);

		// Handle the Notify if we found something
		if (UGameplayCueNotify_Static* NonInstancedCue = Cast<UGameplayCueNotify_Static>(CueData.LoadedGameplayCueClass->GetDefaultObject(false)))
		{
			if (NonInstancedCue->HandlesEvent(EventType))
			{
				NonInstancedCue->HandleGameplayCue(TargetActor, EventType, Parameters);
				bReturnVal = true;
				if (!NonInstancedCue->IsOverride)
				{
					HandleGameplayCueNotify_Internal(TargetActor, CueData.ParentDataIdx, EventType, Parameters);
				}
			}
			else
			{
				//Didn't even handle it, so IsOverride should not apply.
				HandleGameplayCueNotify_Internal(TargetActor, CueData.ParentDataIdx, EventType, Parameters);
			}
		}
		else if (AGameplayCueNotify_Actor* InstancedCue = Cast<AGameplayCueNotify_Actor>(CueData.LoadedGameplayCueClass->GetDefaultObject(false)))
		{
			bool bShouldDestroy = false;
			if (EventType == EGameplayCueEvent::Executed && !Parameters.bGameplayEffectActive && InstancedCue->bAutoDestroyOnRemove)
			{
				bShouldDestroy = true;
			}

			if (InstancedCue->HandlesEvent(EventType))
			{
				if (TargetActor)
				{
					TSubclassOf<AGameplayCueNotify_Actor> InstancedClass = InstancedCue->GetClass();

					//Get our instance. We should probably have a flag or something to determine if we want to reuse or stack instances. That would mean changing our map to have a list of active instances.
					AGameplayCueNotify_Actor* SpawnedInstancedCue = CueManager->GetInstancedCueActor(TargetActor, InstancedClass, Parameters);
					if (ensure(SpawnedInstancedCue))
					{
						SpawnedInstancedCue->HandleGameplayCue(TargetActor, EventType, Parameters);
						bReturnVal = true;
						if (!SpawnedInstancedCue->IsOverride)
						{
							HandleGameplayCueNotify_Internal(TargetActor, CueData.ParentDataIdx, EventType, Parameters);
						}

						if (bShouldDestroy)
						{
							SpawnedInstancedCue->HandleGameplayCue(TargetActor, EGameplayCueEvent::Removed, Parameters);
						}
					}
				}
			}
			else
			{
				//Didn't even handle it, so IsOverride should not apply.
				HandleGameplayCueNotify_Internal(TargetActor, CueData.ParentDataIdx, EventType, Parameters);
			}
		}
	}

	return bReturnVal;
}

void UGameplayCueSet::BuildAccelerationMap_Internal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGameplayCueSet_BuildAccelerationMap);
	// ---------------------------------------------------------
	//	Build up the rest of the acceleration map: every GameplayCue tag should have an entry in the map that points to the index into GameplayCueData to use when it is invoked.
	//	(or to -1 if no GameplayCueNotify is associated with that tag)
	// 
	// ---------------------------------------------------------

	GameplayCueDataMap.Reset();
	GameplayCueDataMap.Add(BaseGameplayCueTag()) = INDEX_NONE;

	for (int32 idx = 0; idx < GameplayCueData.Num(); ++idx)
	{
		GameplayCueDataMap.FindOrAdd(GameplayCueData[idx].GameplayCueTag) = idx;
	}

	FGameplayTagContainer AllGameplayCueTags = UGameplayTagsManager::Get().RequestGameplayTagChildren(BaseGameplayCueTag());


	// Create entries for children.
	// E.g., if "a.b" notify exists but "a.b.c" does not, point "a.b.c" entry to "a.b"'s notify.
	for (FGameplayTag ThisGameplayCueTag : AllGameplayCueTags)
	{
		if (GameplayCueDataMap.Contains(ThisGameplayCueTag))
		{
			continue;
		}

		FGameplayTag Parent = ThisGameplayCueTag.RequestDirectParent();

		int32 ParentValue = GameplayCueDataMap.FindChecked(Parent);
		GameplayCueDataMap.Add(ThisGameplayCueTag, ParentValue);
	}
	GameplayCueDataMap.Shrink();


	// Build up parentIdx on each item in GameplayCUeData
	for (FGameplayCueNotifyData& Data : GameplayCueData)
	{
		FGameplayTag Parent = Data.GameplayCueTag.RequestDirectParent();
		while (Parent != BaseGameplayCueTag() && Parent.IsValid())
		{
			int32* idxPtr = GameplayCueDataMap.Find(Parent);
			if (idxPtr)
			{
				Data.ParentDataIdx = *idxPtr;
				break;
			}
			Parent = Parent.RequestDirectParent();
			if (Parent.GetTagName() == NAME_None)
			{
				break;
			}
		}
	}

	// PrintGameplayCueNotifyMap();
}

FGameplayTag UGameplayCueSet::BaseGameplayCueTag()
{
	// Note we should not cache this off as a static variable, since for new projects the GameplayCue tag will not be found until one is created.
	return StaticTag_GameplayCue.GetTag();
}

