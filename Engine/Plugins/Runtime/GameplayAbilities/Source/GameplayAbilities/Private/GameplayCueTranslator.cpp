// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueTranslator.h"
#include "HAL/IConsoleManager.h"
#include "GameplayCueSet.h"
#include "GameplayTagsManager.h"
#include "GameplayTagsModule.h"
#include "Stats/StatsMisc.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueTranslator)

DEFINE_LOG_CATEGORY_STATIC(LogGameplayCueTranslator, Display, All);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static FAutoConsoleVariable CVarGameplyCueTranslatorDebugTag(TEXT("GameplayCue.Translator.DebugTag"), TEXT(""), TEXT("Debug Tag in gameplay cue translation"), ECVF_Default	);
#endif

FGameplayCueTranslatorNodeIndex FGameplayCueTranslationManager::GetTranslationIndexForName(FName Name, bool CreateIfInvalid)
{
	FGameplayCueTranslatorNodeIndex Idx;
	if (CreateIfInvalid)
	{
		FGameplayCueTranslatorNodeIndex& MapIndex = TranslationNameToIndexMap.FindOrAdd(Name);
		if (MapIndex.IsValid() == false)
		{
			MapIndex = TranslationLUT.AddDefaulted();			
		}
		
		Idx = MapIndex;

		if (TranslationLUT[Idx].CachedIndex.IsValid() == false)
		{
			TranslationLUT[Idx].CachedIndex = Idx;
			TranslationLUT[Idx].CachedGameplayTag = TagManager->RequestGameplayTag(Name, false);
			TranslationLUT[Idx].CachedGameplayTagName = Name;
		}
	}
	else
	{
		if (FGameplayCueTranslatorNodeIndex* IdxPtr = TranslationNameToIndexMap.Find(Name))
		{
			Idx = *IdxPtr;
		}
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (Idx.IsValid() && TranslationLUT[Idx].CachedGameplayTagName.ToString().Contains(CVarGameplyCueTranslatorDebugTag->GetString()))
	{
		UE_LOGF(LogGameplayCueTranslator, Log, ".....");
	}
#endif


	ensureAlways(!Idx.IsValid() || TranslationLUT[Idx].CachedGameplayTagName != NAME_None);

#if WITH_EDITOR
	// In the editor tags can be created after the initial creation of the translation data structures.
	// This will update the tag in subsequent requests
	if (Idx.IsValid() && TranslationLUT[Idx].CachedGameplayTag.IsValid() == false)
	{
		TranslationLUT[Idx].CachedGameplayTag = TagManager->RequestGameplayTag(Name, false);
	}
#endif

	return Idx;
}

FGameplayCueTranslatorNode* FGameplayCueTranslationManager::GetTranslationNodeForName(FName Name, bool CreateIfInvalid)
{
	FGameplayCueTranslatorNodeIndex Idx = GetTranslationIndexForName(Name, CreateIfInvalid);
	if (TranslationLUT.IsValidIndex(Idx))
	{
		return &TranslationLUT[Idx];
	}
	return nullptr;
}

FGameplayCueTranslatorNodeIndex FGameplayCueTranslationManager::GetTranslationIndexForTag(const FGameplayTag& Tag, bool CreateIfInvalid)
{
	return GetTranslationIndexForName(Tag.GetTagName(), CreateIfInvalid);
}

FGameplayCueTranslatorNode* FGameplayCueTranslationManager::GetTranslationNodeForTag(const FGameplayTag& Tag, bool CreateIfInvalid)
{
	FGameplayCueTranslatorNodeIndex Idx = GetTranslationIndexForTag(Tag, CreateIfInvalid);
	if (TranslationLUT.IsValidIndex(Idx))
	{
		return &TranslationLUT[Idx];
	}
	return nullptr;
}

FGameplayCueTranslationLink& FGameplayCueTranslatorNode::FindOrCreateLink(const UGameplayCueTranslator* RuleClassCDO, int32 LookupSize)
{
	int32 InsertIdx = 0;
	int32 NewPriority = RuleClassCDO->GetPriority();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (CVarGameplyCueTranslatorDebugTag->GetString().IsEmpty() == false && this->CachedGameplayTagName.ToString().Contains(CVarGameplyCueTranslatorDebugTag->GetString()))
	{
		UE_LOGF(LogGameplayCueTranslator, Log, ".....");
	}
#endif

	for (int32 LinkIdx=0; LinkIdx < Links.Num(); ++LinkIdx)
	{
		if (Links[LinkIdx].RulesCDO == RuleClassCDO)
		{
			// Already here, return
			return Links[LinkIdx];
		}

		if (Links[LinkIdx].RulesCDO->GetPriority() > NewPriority)
		{
			// Insert after the existing one with higher priority
			InsertIdx = LinkIdx+1;
		}
	}

	check(InsertIdx <= Links.Num());

	FGameplayCueTranslationLink& NewLink = Links[Links.Insert(FGameplayCueTranslationLink(), InsertIdx)];
	NewLink.RulesCDO = RuleClassCDO;
	NewLink.NodeLookup.SetNum( LookupSize );
	return NewLink;
}

void FGameplayCueTranslationManager::RefreshNameSwaps()
{
	AllNameSwaps.Reset();
	TArray<UGameplayCueTranslator*> CDOList;

	// Gather CDOs
	for( TObjectIterator<UClass> It ; It ; ++It )
	{
		UClass* Class = *It;
		if( !Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated) )
		{
			if( Class->IsChildOf(UGameplayCueTranslator::StaticClass()) )
			{
				UGameplayCueTranslator* CDO = Class->GetDefaultObject<UGameplayCueTranslator>();
				if (CDO->IsEnabled())
				{
					CDOList.Add(CDO);
				}
			}
		}
	}

	// Sort and get translated names
	CDOList.Sort([](const UGameplayCueTranslator& A, const UGameplayCueTranslator& B) { return (A.GetPriority() > B.GetPriority()); });

	for (UGameplayCueTranslator* CDO : CDOList)
	{
		FNameSwapData& Data = AllNameSwaps[AllNameSwaps.AddDefaulted()];
		CDO->GetTranslationNameSpawns(Data.NameSwaps);
		if (Data.NameSwaps.Num() > 0)
		{
			Data.ClassCDO = CDO;
		}
		else
		{
			AllNameSwaps.Pop(EAllowShrinking::No);
		}
	}

#if WITH_EDITOR
	// Give UniqueID to each rule
	int32 ID=1;
	for (FNameSwapData& GroupData : AllNameSwaps)
	{
		for (FGameplayCueTranslationNameSwap& SwapData : GroupData.NameSwaps)
		{
			SwapData.EditorData.UniqueID = ID++;
		}
	}
#endif
}

void FGameplayCueTranslationManager::ResetTranslationLUT()
{
	TranslationNameToIndexMap.Reset();
	TranslationLUT.Reset();
}

void FGameplayCueTranslationManager::BuildTagTranslationTable()
{
#if WITH_EDITOR
	//SCOPE_LOG_TIME_IN_SECONDS(*FString::Printf(TEXT("FGameplayCueTranslatorManager::BuildTagTranslationTables")), nullptr)
#endif

	TagManager = &UGameplayTagsManager::Get();
	check(TagManager);
	
	FGameplayTagContainer AllGameplayCueTags = TagManager->RequestGameplayTagChildren(UGameplayCueSet::BaseGameplayCueTag());
	
	ResetTranslationLUT();
	RefreshNameSwaps();

	// ----------------------------------------------------------------------------------------------
	
	// Find what tags may be derived from swap rules. Note how we work backwards.
	// If we worked forward, by expanding out all possible tags and then seeing if they exist, 
	// this would take much much longer!

	TArray<FName> SplitNames;
	SplitNames.Reserve(10);
	
	// All gameplaycue tags
	for (const FGameplayTag& Tag : AllGameplayCueTags)
	{
		SplitNames.Reset();
		TagManager->SplitGameplayTagFName(Tag, SplitNames);

		BuildTagTranslationTable_r(Tag.GetTagName(), SplitNames);
	}

	// ----------------------------------------------------------------------------------------------
}

bool FGameplayCueTranslationManager::BuildTagTranslationTable_r(const FName& TagName, const TArray<FName>& SplitNames)
{

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (CVarGameplyCueTranslatorDebugTag->GetString().IsEmpty() == false && TagName.ToString().Contains(CVarGameplyCueTranslatorDebugTag->GetString()))
	{
		// 
		UE_LOGF(LogGameplayCueTranslator, Log, ".....");
	}
#endif

	bool HasValidRootTag = false;

	TArray<FName> SwappedNames;
	SwappedNames.Reserve(10);

	// Every NameSwap Rule/Class that gave us data
	for (FNameSwapData& NameSwapData : AllNameSwaps)
	{
		// Avoid rule recursion
		{
			if (FGameplayCueTranslatorNode* ChildNode = GetTranslationNodeForName(TagName, false))
			{
				if (ChildNode->UsedTranslators.Contains(NameSwapData.ClassCDO))
				{
					continue;
				}
			}
		}

		// Every Swap that this Rule/Class gave us
		for (int32 SwapRuleIdx=0; SwapRuleIdx < NameSwapData.NameSwaps.Num(); ++SwapRuleIdx)
		{
			const FGameplayCueTranslationNameSwap& SwapRule = NameSwapData.NameSwaps[SwapRuleIdx];

#if WITH_EDITOR
			if (SwapRule.EditorData.Enabled == false)
			{
				continue;
			}
#endif

			// Walk through the original tag's elements
			for (int32 TagIdx=0; TagIdx < SplitNames.Num(); ++TagIdx)
			{
				// Walk through the potential new tag's elemnts
				for (int32 ToNameIdx=0; ToNameIdx < SwapRule.ToNames.Num() && TagIdx < SplitNames.Num(); ++ToNameIdx)
				{
					// If they match
					if (SwapRule.ToNames[ToNameIdx] == SplitNames[TagIdx])
					{
						// If we are at the end
						if (ToNameIdx == SwapRule.ToNames.Num()-1)
						{
							// *Possible* tag translation found! This tag can be derived from out name swapping rules, 
							// but we don't know if there actually is a tag that matches the tag which it would be translated *from*

							// Don't operator on SplitNames, since subsequent rules and name swaps use the same array
							SwappedNames = SplitNames;

							// Remove the "To Names" from the SwappedNames array, replace with the single "From Name"
							// E.g, GC.{Steel.Primary} -> GC.{Hero}
							int32 NumRemoves = SwapRule.ToNames.Num(); // We are going to remove as many tags 
							int32 RemoveAtIdx = TagIdx - (SwapRule.ToNames.Num() - 1);
							check(SwappedNames.IsValidIndex(RemoveAtIdx));

							SwappedNames.RemoveAt(RemoveAtIdx, NumRemoves, EAllowShrinking::No);
							SwappedNames.Insert(SwapRule.FromName, RemoveAtIdx);

							// Compose a string from the new name
							FString ComposedString = SwappedNames[0].ToString();							
							for (int32 ComposeIdx=1; ComposeIdx < SwappedNames.Num(); ++ ComposeIdx)
							{
								ComposedString += FString::Printf(TEXT(".%s"), *SwappedNames[ComposeIdx].ToString());
							}
								
							UE_LOGF(LogGameplayCueTranslator, Log, "Found possible expanded tag. Original Child Tag: %ls. Possible Parent Tag: %ls", *TagName.ToString(), *ComposedString);
							FName ComposedName = FName(*ComposedString);

							// Look for this tag - is it an actual real tag? If not, continue on
							{
								FGameplayTag ComposedTag = TagManager->RequestGameplayTag(ComposedName, false);
								if (ComposedTag.IsValid() == false)
								{
									UE_LOGF(LogGameplayCueTranslator, Log, "   No tag match found, recursing...");
									
									FGameplayCueTranslatorNodeIndex ParentIdx = GetTranslationIndexForName( ComposedName, false );
									if (ParentIdx.IsValid() == false)
									{
										ParentIdx = GetTranslationIndexForName( ComposedName, true );
										check(ParentIdx.IsValid());
										TranslationLUT[ParentIdx].UsedTranslators.Add( NameSwapData.ClassCDO );
																		
										HasValidRootTag |= BuildTagTranslationTable_r(ComposedName, SwappedNames);
									}
								}
								else
								{
									HasValidRootTag = true;
								}
							}

							if (HasValidRootTag)
							{
								// Add it to our data structures
								FGameplayCueTranslatorNodeIndex ParentIdx = GetTranslationIndexForName(ComposedName, true);
								check(ParentIdx.IsValid());
								
								UE_LOGF(LogGameplayCueTranslator, Log, "   Matches real tags! Adding to translation tree");

								FGameplayCueTranslatorNodeIndex ChildIdx = GetTranslationIndexForName(TagName, true);
								ensure(ChildIdx.IsValid());

								// Note: important to do this after getting ChildIdx since allocating idx can move TranslationMap memory around
								FGameplayCueTranslatorNode& ParentNode = TranslationLUT[ParentIdx];

								FGameplayCueTranslationLink& NewLink = ParentNode.FindOrCreateLink(NameSwapData.ClassCDO, NameSwapData.NameSwaps.Num());
										
								// Verify this link hasn't already been established
								ensure(NewLink.NodeLookup[SwapRuleIdx] != ChildIdx);

								// Setup the link
								NewLink.NodeLookup[SwapRuleIdx] = ChildIdx;

								// Now make sure we don't reapply this rule to this child node or any of its child nodes
								FGameplayCueTranslatorNode& ChildNode = TranslationLUT[ChildIdx];
								ChildNode.UsedTranslators.Append( ParentNode.UsedTranslators );
								ChildNode.UsedTranslators.Add( NameSwapData.ClassCDO );
							}
							else
							{
								UE_LOGF(LogGameplayCueTranslator, Log, "   No tag match found after recursing. Dead end.");
							}

							break;
						}
						else
						{
							// Keep going, advance TagIdx
							TagIdx++;
							continue;
						}
					}
					else
					{
						// Match failed
						break;
					}
				}
			}
		}
	}

	return HasValidRootTag;
}

void FGameplayCueTranslationManager::BuildTagTranslationTable_Forward()
{
#if WITH_EDITOR
	SCOPE_LOG_TIME_IN_SECONDS(*FString::Printf(TEXT("FGameplayCueTranslatorManager::BuildTagTranslationTable_Forward")), nullptr)
#endif

	// Build the normal TranslationLUT first. This is only done to make sure that UsedTranslators are filled in, giving "real" tags higher priority.
	// Example:
	//	1) GC.Rampage.Enraged
	//	2) GC.Rampage.Elemental.Enraged
	//	
	//	2 is am override for 1, but comes first alphabetically. In the _Forward method, 2 would be handled first and expanded again to GC.Rampage.Elemental.Elemental.Enraged.
	//	rule recursion wouldn't have been hit yet because 2 actually exists and would be encountered before 1.
	//
	//	Since BuildTagTranslationTable_Forward is only called by the editor and BuildTagTranslationTable is already fast, this is the simplest way to avoid the above example.
	//	_Forward() could be made more complicated to test for this itself, but doesn't seem like a good trade off for how it would complicate the function.
	BuildTagTranslationTable();

	TArray<FName> SplitNames;
	SplitNames.Reserve(10);
	
	FGameplayTagContainer AllGameplayCueTags = TagManager->RequestGameplayTagChildren(UGameplayCueSet::BaseGameplayCueTag());

	// Each GameplayCueTag
	for (const FGameplayTag& Tag : AllGameplayCueTags)
	{
		SplitNames.Reset();
		TagManager->SplitGameplayTagFName(Tag, SplitNames);

		BuildTagTranslationTable_Forward_r(Tag.GetTagName(), SplitNames);
	}
}

void FGameplayCueTranslationManager::BuildTagTranslationTable_Forward_r(const FName& TagName, const TArray<FName>& SplitNames)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (CVarGameplyCueTranslatorDebugTag->GetString().IsEmpty() == false && TagName.ToString().Contains(CVarGameplyCueTranslatorDebugTag->GetString()))
	{
		// 
		UE_LOGF(LogGameplayCueTranslator, Log, ".....");
	}
#endif

	TArray<FName> SwappedNames;
	SwappedNames.Reserve(10);

	// Each nameswap rule group
	for (FNameSwapData& NameSwapData : AllNameSwaps)
	{
		// Avoid rule recursion
		{
			if (FGameplayCueTranslatorNode* ChildNode = GetTranslationNodeForName(TagName, false))
			{
				if (ChildNode->UsedTranslators.Contains(NameSwapData.ClassCDO))
				{
					continue;
				}
			}
		}

		// Each swaprule 
		for (int32 SwapRuleIdx=0; SwapRuleIdx < NameSwapData.NameSwaps.Num(); ++SwapRuleIdx)
		{
			const FGameplayCueTranslationNameSwap& SwapRule = NameSwapData.NameSwaps[SwapRuleIdx];

#if WITH_EDITOR
			if (SwapRule.EditorData.Enabled == false)
			{
				continue;
			}
#endif

			// Each subtag within this GameplayTag
			for (int32 TagIdx=0; TagIdx < SplitNames.Num(); ++TagIdx)
			{
				if (SplitNames[TagIdx] == SwapRule.FromName)
				{
					SwappedNames = SplitNames;

					// Possible match!
					// Done - full match found
					SwappedNames.RemoveAt(TagIdx, EAllowShrinking::No);
					for (int32 ToIdx=0; ToIdx < SwapRule.ToNames.Num(); ++ToIdx)
					{
						SwappedNames.Insert(SwapRule.ToNames[ToIdx], TagIdx + ToIdx);
					}

					FString ComposedString = SwappedNames[0].ToString();
					for (int32 ComposeIdx=1; ComposeIdx < SwappedNames.Num(); ++ ComposeIdx)
					{
						ComposedString += FString::Printf(TEXT(".%s"), *SwappedNames[ComposeIdx].ToString());
					}
						
					UE_LOGF(LogGameplayCueTranslator, Log, "Found possible new expanded tag. Original: %ls. Parent: %ls", *TagName.ToString(), *ComposedString);
					FName ComposedName = FName(*ComposedString);

					FGameplayCueTranslatorNodeIndex ChildIdx = GetTranslationIndexForName( ComposedName, true );
					if ( ChildIdx.IsValid() )
					{
						FGameplayCueTranslatorNodeIndex ParentIdx = GetTranslationIndexForName( TagName, true );
						if (ParentIdx.IsValid())
						{
							FGameplayCueTranslatorNode& ParentNode = TranslationLUT[ParentIdx];
							FGameplayCueTranslatorNode& ChildNode = TranslationLUT[ChildIdx];

							// Find or create the link structure out of the parent node
							FGameplayCueTranslationLink& NewLink = ParentNode.FindOrCreateLink(NameSwapData.ClassCDO, NameSwapData.NameSwaps.Num());
							
							NewLink.NodeLookup[SwapRuleIdx] = ChildNode.CachedIndex;

							ChildNode.UsedTranslators.Append(ParentNode.UsedTranslators);
							ChildNode.UsedTranslators.Add(NameSwapData.ClassCDO);
						}
					}

					BuildTagTranslationTable_Forward_r(ComposedName, SwappedNames);
				}
			}
		}
	}
}

void FGameplayCueTranslationManager::TranslateTag(FGameplayTag& Tag, AActor* TargetActor, const FGameplayCueParameters& Parameters)
{
	if (FGameplayCueTranslatorNode* Node = GetTranslationNodeForTag(Tag))
	{
		TranslateTag_Internal(*Node, Tag, Tag.GetTagName(), TargetActor, Parameters);
	}
}

bool FGameplayCueTranslationManager::TranslateTag_Internal(FGameplayCueTranslatorNode& Node, FGameplayTag& OutTag, const FName& TagName, AActor* TargetActor, const FGameplayCueParameters& Parameters)
{
	for (FGameplayCueTranslationLink& Link : Node.Links)
	{
		// Have CDO give us TranslationIndex. This is 0 - (number of name swaps this class gave us)
		int32 TranslationIndex = Link.RulesCDO->GameplayCueToTranslationIndex(TagName, TargetActor, Parameters);
		if (TranslationIndex != INDEX_NONE)
		{
			if (Link.NodeLookup.IsValidIndex(TranslationIndex) == false)
			{
				UE_LOGF(LogGameplayCueTranslator, Error, "FGameplayCueTranslationManager::TranslateTag_Internal %ls invalid index %d was returned from GameplayCueToTranslationIndex. NodeLookup.Num=%d. Tag %ls", *GetNameSafe(Link.RulesCDO), TranslationIndex, Link.NodeLookup.Num(), *TagName.ToString());
				continue;
			}

			// Use the link's NodeLookup to get the real NodeIndex
			FGameplayCueTranslatorNodeIndex NodeIndex = Link.NodeLookup[TranslationIndex];
			if (NodeIndex.IsValid())
			{
				if (TranslationLUT.IsValidIndex(NodeIndex) == false)
				{
					UE_LOGF(LogGameplayCueTranslator, Error, "FGameplayCueTranslationManager::TranslateTag_Internal %ls invalid index %d was returned from NodeLookup. NodeLookup.Num=%d. Tag %ls", *GetNameSafe(Link.RulesCDO), NodeIndex.Index, TranslationLUT.Num(), *TagName.ToString());
					continue;
				}

				// Warn if more links?
				FGameplayCueTranslatorNode& InnerNode = TranslationLUT[NodeIndex];

				UE_LOGF(LogGameplayCueTranslator, Verbose, "Translating %ls --> %ls (via %ls)", *TagName.ToString(), *InnerNode.CachedGameplayTagName.ToString(), *GetNameSafe(Link.RulesCDO));

				OutTag = InnerNode.CachedGameplayTag;
				
				TranslateTag_Internal(InnerNode, OutTag, InnerNode.CachedGameplayTagName, TargetActor, Parameters);
				return true;
			}
		}
	}

	return false;
}

void FGameplayCueTranslationManager::PrintTranslationTable()
{
	UE_LOGF(LogGameplayCueTranslator, Display, "Printing GameplayCue Translation Table. * means tag is not created but could be.");

	TotalNumTranslations = 0;
	TotalNumTheoreticalTranslations = 0;
	for (FGameplayCueTranslatorNode& Node : TranslationLUT)
	{
		PrintTranslationTable_r(Node);
	}

	UE_LOGF(LogGameplayCueTranslator, Display, "");
	UE_LOGF(LogGameplayCueTranslator, Display, "Total Number of Translations with valid tags: %d", TotalNumTranslations);
	UE_LOGF(LogGameplayCueTranslator, Display, "Total Number of Translations without  valid tags: %d (theoretical translations)", TotalNumTheoreticalTranslations);
}

void FGameplayCueTranslationManager::PrintTranslationTable_r(FGameplayCueTranslatorNode& Node, FString IdentStr)
{
	if (Node.Links.Num() > 0)
	{
		if (IdentStr.IsEmpty())
		{
			UE_LOGF(LogGameplayCueTranslator, Display, "%ls %ls", *Node.CachedGameplayTagName.ToString(), Node.CachedGameplayTag.IsValid() ? TEXT("") : TEXT("*"));
		}

		for (FGameplayCueTranslationLink& Link : Node.Links)
		{
			for (FGameplayCueTranslatorNodeIndex& Index : Link.NodeLookup)
			{
				if (Index.IsValid())
				{
					FGameplayCueTranslatorNode& InnerNode = TranslationLUT[Index];

					if (InnerNode.CachedGameplayTag.IsValid())
					{
						UE_LOGF(LogGameplayCueTranslator, Display, "%ls -> %ls [%ls]", *IdentStr, *InnerNode.CachedGameplayTag.ToString(), *GetNameSafe(Link.RulesCDO) );
						TotalNumTranslations++;
					}
					else
					{
						UE_LOGF(LogGameplayCueTranslator, Display, "%ls -> %ls [%ls] *", *IdentStr, *InnerNode.CachedGameplayTagName.ToString(), *GetNameSafe(Link.RulesCDO) );
						TotalNumTheoreticalTranslations++;
					}

					PrintTranslationTable_r(InnerNode, IdentStr + TEXT("  "));
				}
			}
		}

		UE_LOGF(LogGameplayCueTranslator, Display, "");
	}
}

// --------------------------------------------------------------------------------------------------
#if WITH_EDITOR
bool FGameplayCueTranslationManager::GetTranslatedTags(const FName& ParentTag, TArray<FGameplayCueTranslationEditorInfo>& Children)
{
	if (const FGameplayCueTranslatorNode* Node = GetTranslationNodeForName(ParentTag))
	{
		for (const FGameplayCueTranslationLink& Link : Node->Links)
		{
			for (int32 LinkIdx=0; LinkIdx < Link.NodeLookup.Num(); ++LinkIdx)
			{
				const FGameplayCueTranslatorNodeIndex& Index = Link.NodeLookup[LinkIdx];
				if (Index.IsValid())
				{
					FGameplayCueTranslatorNode& ChildNode = TranslationLUT[Index];

					// Find the description of the rule this came from
					for (FNameSwapData& SwapData : AllNameSwaps)
					{
						if (SwapData.ClassCDO == Link.RulesCDO)
						{
							check(SwapData.NameSwaps.IsValidIndex(LinkIdx));

							FGameplayCueTranslationEditorInfo& Info = Children[Children.AddDefaulted()];
							Info.GameplayTagName = ChildNode.CachedGameplayTagName;
							Info.GameplayTag = ChildNode.CachedGameplayTag;
							Info.EditorData = SwapData.NameSwaps[LinkIdx].EditorData;
							break;
						}
					}
				}
			}
		}
	}
	return Children.Num() > 0;
}


FGameplayTag FGameplayCueTranslationManager::SearchSlowForTranslationParent(FGameplayTag Tag)
{
	TFunction<FGameplayTag(int32)> FindParent_r = [&](int32 FindIndex) -> FGameplayTag
	{
		for (int32 idx=0; idx < TranslationLUT.Num(); ++idx)
		{
			for (FGameplayCueTranslationLink& Link : TranslationLUT[idx].Links)
			{
				for (FGameplayCueTranslatorNodeIndex& LinkIdx : Link.NodeLookup)
				{
					if (LinkIdx.Index == FindIndex)
					{
						FGameplayCueTranslatorNode& Node = TranslationLUT[idx];
						//UE_LOGF(LogGameplayCueTranslator, Display, "%ls links to %ls", *Node.CachedGameplayTag.ToString(), *TranslationLUT[FindIndex].CachedGameplayTag.ToString());

						FGameplayTag ParentTag = FindParent_r(idx);
						if (ParentTag.IsValid())
						{
							// Found a higher level parent
							return ParentTag;
						}
						else
						{
							// Topmost tag
							return Node.CachedGameplayTag;
						}
					}
				}
			}
		}

		// Found nothing
		return FGameplayTag();

	};

	for (int32 RootIdx=0; RootIdx < TranslationLUT.Num(); ++RootIdx)
	{
		FGameplayCueTranslatorNode& Node = TranslationLUT[RootIdx];
		if (Node.CachedGameplayTag == Tag)
		{
			return FindParent_r(RootIdx);
		}
	}

	// No match, return invalid tag
	return FGameplayTag();
}
#endif

