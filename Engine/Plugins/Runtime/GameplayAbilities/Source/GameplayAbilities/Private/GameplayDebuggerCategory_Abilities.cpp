// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerCategory_Abilities.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU

#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"
#include "Engine/ActorChannel.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/NetConnection.h"
#include "GameFramework/PlayerController.h"
#include "GameplayTagContainer.h"
#include "GameplayAbilitySpec.h"
#include "GameplayEffect.h"
#include "Net/Subsystems/NetworkSubsystem.h"
#include "Net/RepLayout.h"

namespace UE::AbilitySystem::Debug
{
	// This is the longest name we can use for the UI (string format truncate with %.35s).  We use a variety of letters because MeasureString depends on kerning.
	const FString LongestDebugObjectName{ TEXT("ABCDEFGHIJKLMNOPQRSTUVWXYZ_ ABCDEFGH") };

	// Let's define some names for consistent look across all of the categories
	const TCHAR* BothColor = TEXT("{yellow}");
	const TCHAR* ServerColor = TEXT("{cyan}");
	const TCHAR* LocalColor = TEXT("{green}");
	const TCHAR* NonReplicatedColor = TEXT("{violetred}");

	/** Given a string, print it out using the color legend based on the NetworkStatus */
	inline FString ColorNetworkString(FGameplayDebuggerCategory_Abilities::ENetworkStatus NetworkStatus, const FStringView DisplayString)
	{
		const TCHAR* Colors[+FGameplayDebuggerCategory_Abilities::ENetworkStatus::MAX] =
		{
			ServerColor, LocalColor, BothColor, NonReplicatedColor
		};

		return FString::Printf(TEXT("%s%.*s"), Colors[+NetworkStatus], DisplayString.Len(), DisplayString.GetData());
	}
}

FGameplayDebuggerCategory_Abilities::FGameplayDebuggerCategory_Abilities()
{
	SetDataPackReplication<FRepData>(&DataPack);

	// Hard coding these to avoid needing to import InputCore just for EKeys::GetFName().
	const FName KeyNameOne{ "One" };
	const FName KeyNameTwo{ "Two" };
	const FName KeyNameThree{ "Three" };
	const FName KeyNameFour{ "Four" };

	BindKeyPress(KeyNameOne, FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Abilities::OnShowGameplayTagsToggle, EGameplayDebuggerInputMode::Local);
	BindKeyPress(KeyNameTwo, FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Abilities::OnShowGameplayAbilitiesToggle, EGameplayDebuggerInputMode::Local);
	BindKeyPress(KeyNameThree, FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Abilities::OnShowGameplayEffectsToggle, EGameplayDebuggerInputMode::Local);
	BindKeyPress(KeyNameFour, FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Abilities::OnShowGameplayAttributesToggle, EGameplayDebuggerInputMode::Local);
}

TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_Abilities::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerCategory_Abilities());
}

void FGameplayDebuggerCategory_Abilities::OnShowGameplayTagsToggle()
{
	bShowGameplayTags = !bShowGameplayTags;
}

void FGameplayDebuggerCategory_Abilities::OnShowGameplayAbilitiesToggle()
{
	bShowGameplayAbilities = !bShowGameplayAbilities;
}

void FGameplayDebuggerCategory_Abilities::OnShowGameplayEffectsToggle()
{
	bShowGameplayEffects = !bShowGameplayEffects;
}

void FGameplayDebuggerCategory_Abilities::OnShowGameplayAttributesToggle()
{
	bShowGameplayAttributes = !bShowGameplayAttributes;
}

void FGameplayDebuggerCategory_Abilities::FRepData::Serialize(FArchive& Ar)
{
	Ar << TagCounts;

	FString StrOwnedTags = OwnedTags.ToString();
	Ar << StrOwnedTags;
	if (Ar.IsLoading())
	{
		OwnedTags.FromExportString(StrOwnedTags);
	}

	int32 NumAbilities = Abilities.Num();
	Ar << NumAbilities;
	if (Ar.IsLoading())
	{
		Abilities.SetNum(NumAbilities);
	}

	for (int32 Idx = 0; Idx < NumAbilities; Idx++)
	{
		Ar << Abilities[Idx].Ability;
		Ar << Abilities[Idx].Source;
		Ar << Abilities[Idx].Level;
		Ar << Abilities[Idx].bIsActive;
	}

	int32 NumGE = GameplayEffects.Num();
	Ar << NumGE;
	if (Ar.IsLoading())
	{
		GameplayEffects.SetNum(NumGE);
	}

	for (int32 Idx = 0; Idx < NumGE; Idx++)
	{
		Ar << GameplayEffects[Idx].ReplicationID;
		Ar << GameplayEffects[Idx].Effect;
		Ar << GameplayEffects[Idx].Context;
		Ar << GameplayEffects[Idx].Duration;
		Ar << GameplayEffects[Idx].Period;
		Ar << GameplayEffects[Idx].Stacks;
		Ar << GameplayEffects[Idx].Level;
		Ar << GameplayEffects[Idx].bInhibited;
	}

	int32 NumAttrib = Attributes.Num();
	Ar << NumAttrib;
	if (Ar.IsLoading())
	{
		Attributes.SetNum(NumAttrib);
	}

	for (int32 Idx = 0; Idx < NumAttrib; ++Idx)
	{
		Ar << Attributes[Idx].AttributeName;
		Ar << Attributes[Idx].BaseValue;
		Ar << Attributes[Idx].CurrentValue;
		Ar << Attributes[Idx].NetworkStatus;
	}
}

void FGameplayDebuggerCategory_Abilities::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
	if (const UAbilitySystemComponent* AbilityComp = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(DebugActor))
	{
		AbilityComp->GetOwnedGameplayTags(DataPack.OwnedTags);

		// Copy over the tag counts
		DataPack.TagCounts.Empty();
		for (const FGameplayTag& Tag : DataPack.OwnedTags)
		{
			DataPack.TagCounts.Add(AbilityComp->GetTagCount(Tag));
		}

		// Abilities
		const TArray<FGameplayAbilitySpec>& AbilitySpecs = AbilityComp->GetActivatableAbilities();
		for (int32 Idx = 0; Idx < AbilitySpecs.Num(); Idx++)
		{
			const FGameplayAbilitySpec& AbilitySpec = AbilitySpecs[Idx];
			FRepData::FGameplayAbilityDebug ItemData;

			ItemData.Ability = GetNameSafe(AbilitySpec.Ability);
			ItemData.Ability.RemoveFromStart(DEFAULT_OBJECT_PREFIX);
			ItemData.Ability.RemoveFromEnd(TEXT("_C"));

			ItemData.Source = GetNameSafe(AbilitySpec.SourceObject.Get());
			ItemData.Source.RemoveFromStart(DEFAULT_OBJECT_PREFIX);

			ItemData.Level = AbilitySpec.Level;
			ItemData.bIsActive = AbilitySpec.IsActive();

			DataPack.Abilities.Add(ItemData);
		}

		// Gameplay Effects
		DataPack.GameplayEffects = CollectEffectsData(OwnerPC, AbilityComp);

		// Attributes
		DataPack.Attributes = CollectAttributeData(OwnerPC, AbilityComp);
	}
}

TArray<FGameplayDebuggerCategory_Abilities::FRepData::FGameplayEffectDebug> FGameplayDebuggerCategory_Abilities::CollectEffectsData(const APlayerController* OwnerPC, const UAbilitySystemComponent* AbilityComp) const
{
	TArray<FRepData::FGameplayEffectDebug> DebugEffects;

	for (const FActiveGameplayEffect& ActiveGE : &AbilityComp->GetActiveGameplayEffects())
	{
		FRepData::FGameplayEffectDebug& ItemData = DebugEffects.AddDefaulted_GetRef();
		ItemData.ReplicationID = ActiveGE.ReplicationID;
		ItemData.bInhibited = ActiveGE.bIsInhibited;
		ItemData.Duration = ActiveGE.GetDuration();
		ItemData.Period = ActiveGE.GetPeriod();

		const FGameplayEffectSpec& EffectSpec = ActiveGE.Spec;
		ItemData.Effect = EffectSpec.ToSimpleString();
		ItemData.Effect.RemoveFromStart(DEFAULT_OBJECT_PREFIX);
		ItemData.Effect.RemoveFromEnd(TEXT("_C"));

		ItemData.Context = EffectSpec.GetContext().ToString();
		ItemData.Stacks = EffectSpec.GetStackCount();
		ItemData.Level = EffectSpec.GetLevel();

		ItemData.NetworkStatus = OwnerPC != nullptr && OwnerPC->HasAuthority() ? ENetworkStatus::ServerOnly : ENetworkStatus::LocalOnly;
	}

	return DebugEffects;
}

TArray<FGameplayDebuggerCategory_Abilities::FRepData::FGameplayAttributeDebug> FGameplayDebuggerCategory_Abilities::CollectAttributeData(const APlayerController* OwnerPC, const UAbilitySystemComponent* AbilityComp) const
{
	TArray<FRepData::FGameplayAttributeDebug> DebugAttributes;

	const AActor* Actor = AbilityComp ? AbilityComp->GetOwner() : nullptr;
	if (!Actor || !Actor->GetWorld())
	{
		FRepData::FGameplayAttributeDebug& DebugAttribute = DebugAttributes.AddDefaulted_GetRef();
		DebugAttribute.AttributeName = TEXT("Invalid AbilitySystemComponent");
		return DebugAttributes;
	}

	// We need to do a lot of work to detect if the AttributeSet is replicated which only occurs of AbilitySystemComponent is replicated
	const ENetRole LocalRole = Actor->GetLocalRole();
	const ELifetimeCondition NetCondition = Actor->AllowActorComponentToReplicate(AbilityComp);

	bool bASCReplicates = (NetCondition != ELifetimeCondition::COND_Never) && AbilityComp->IsSupportedForNetworking() && AbilityComp->IsNameStableForNetworking();
	if (bASCReplicates)
	{
		// Lots of extra work due to COND_NetGroup
		if (const UNetworkSubsystem* NetConditionGroupSubsystem = Actor->GetWorld()->GetSubsystem<UNetworkSubsystem>())
		{
			FReplicationFlags RepFlags;
			RepFlags.bNetOwner = OwnerPC != nullptr && (Actor->GetNetConnection() == OwnerPC->GetNetConnection());
			RepFlags.bNetSimulated = !RepFlags.bNetOwner && Actor->GetNetConnection();
			RepFlags.bRolesOnly = true;
			const TStaticBitArray<COND_Max> ConditionMap = UE::Net::BuildConditionMapFromRepFlags(RepFlags);

			bASCReplicates = UActorChannel::CanSubObjectReplicateToClient(OwnerPC, NetCondition, AbilityComp, ConditionMap, NetConditionGroupSubsystem->GetNetConditionGroupManager());
		}
	}

	// Grab the AttributeSet rather than the Attributes themselves so we can check the network functionality
	for (const UAttributeSet* AttributeSet : AbilityComp->GetSpawnedAttributes())
	{
		const TSubclassOf<UAttributeSet> AttributeSetClass = AttributeSet ? AttributeSet->GetClass() : nullptr;
		if (!AttributeSet || !AttributeSetClass) // !AttributeSet needed for static analysis
		{
			continue;
		}

		// These are all of the replication conditions per variable
		TArray<FLifetimeProperty> LifetimeProps;
		AttributeSet->GetLifetimeReplicatedProps(LifetimeProps);

		// Network status can change per AttributeSet, so figure it out
		ENetworkStatus LocalNetworkStatus = (LocalRole == ENetRole::ROLE_Authority) ? ENetworkStatus::ServerOnly : ENetworkStatus::LocalOnly;
		const bool bAttributeSetReplicates = bASCReplicates && AttributeSet->IsSupportedForNetworking();

		// Now just gather the debug data
		TArray<FGameplayAttribute> LocalAttributes;
		UAttributeSet::GetAttributesFromSetClass(AttributeSetClass, LocalAttributes);
		for (const FGameplayAttribute& Attrib : LocalAttributes)
		{
			ENetworkStatus NetworkStatus = LocalNetworkStatus;

			// Override the status to network if all replication tests pass
			if (bAttributeSetReplicates)
			{
				if (const FProperty* Property = Attrib.GetUProperty())
				{
					if (Property->HasAnyPropertyFlags(EPropertyFlags::CPF_Net))
					{
						FLifetimeProperty* RepProperty = LifetimeProps.FindByPredicate([RepIndex = Property->RepIndex](const FLifetimeProperty& Item) { return Item.RepIndex == RepIndex; });
						if (RepProperty && RepProperty->Condition != ELifetimeCondition::COND_Never)
						{
							// InvalidRepIndex maps to INDEX_NONE even though unsigned (wrap-around)
							NetworkStatus = ENetworkStatus::Networked;
						}
					}
				}
			}

			// Add the data to our returned array
			FRepData::FGameplayAttributeDebug& DebugAttribute = DebugAttributes.Add_GetRef(
				{
					.AttributeName = Attrib.AttributeName,
					.BaseValue = AbilityComp->GetNumericAttributeBase(Attrib),
					.CurrentValue = Attrib.GetNumericValue(AttributeSet),
					.NetworkStatus = NetworkStatus
				});
		}
	}

	return DebugAttributes;
}

bool FGameplayDebuggerCategory_Abilities::WrapStringAccordingToViewport(const FString& StrIn, FString& StrOut, FGameplayDebuggerCanvasContext& CanvasContext, float ViewportWitdh) const
{
	if (!StrIn.IsEmpty())
	{
		// Clamp the Width
		ViewportWitdh = FMath::Max(ViewportWitdh, 10.0f);

		float StrWidth = 0.0f, StrHeight = 0.0f;
		// Calculate the length(in pixel) of the tags
		CanvasContext.MeasureString(StrIn, StrWidth, StrHeight);

		int32 SubDivision = FMath::CeilToInt(StrWidth / ViewportWitdh);
		if (SubDivision > 1)
		{
			// Copy the string
			StrOut = StrIn;
			const int32 Step = StrOut.Len() / SubDivision;
			// Start sub divide if needed
			for (int32 i = SubDivision - 1; i > 0; --i)
			{
				// Insert Line Feed
				StrOut.InsertAt(i * Step - 1, '\n');
			}
			return true;
		}
		else
		{
			StrOut = StrIn;
		}
	}
	// No need to wrap the text 
	return false;
}

void FGameplayDebuggerCategory_Abilities::DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext)
{
	// Draw the sub-category bindings inline with the category header
	{
		CanvasContext.CursorX += 200.0f;
		CanvasContext.CursorY -= CanvasContext.GetLineHeight();
		const TCHAR* Active = TEXT("{green}");
		const TCHAR* Inactive = TEXT("{grey}");
		CanvasContext.Printf(TEXT("Tags [%s%s{white}]\tAbilities [%s%s{white}]\tEffects [%s%s{white}]\tAttributes [%s%s{white}]"),
			bShowGameplayTags ? Active : Inactive, *GetInputHandlerDescription(0),
			bShowGameplayAbilities ? Active : Inactive, *GetInputHandlerDescription(1),
			bShowGameplayEffects ? Active : Inactive, *GetInputHandlerDescription(2),
			bShowGameplayAttributes ? Active : Inactive, *GetInputHandlerDescription(3));
	}

	if (LastDrawDataEndSize <= 0.0f)
	{
		// Default to the full frame size
		LastDrawDataEndSize = CanvasContext.Canvas->SizeY - CanvasContext.CursorY - CanvasContext.CursorX;
	}

	float ThisDrawDataStartPos = CanvasContext.CursorY;

	const FLinearColor BackgroundColor(0.1f, 0.1f, 0.1f, 0.8f);
	const FVector2D BackgroundPos{ CanvasContext.CursorX, CanvasContext.CursorY };
	const FVector2D BackgroundSize(CanvasContext.Canvas->SizeX - (2.0f * CanvasContext.CursorX), LastDrawDataEndSize);

	// Draw a transparent dark background so that the text is easier to look at
	FCanvasTileItem Background(FVector2D(0.0f), BackgroundSize, BackgroundColor);
	Background.BlendMode = SE_BLEND_Translucent;

	CanvasContext.DrawItem(Background, BackgroundPos.X, BackgroundPos.Y);

	if (bShowGameplayTags)
	{
		DrawGameplayTags(CanvasContext, OwnerPC);
	}

	if (bShowGameplayAbilities)
	{
		DrawGameplayAbilities(CanvasContext, OwnerPC);
	}

	if (bShowGameplayEffects)
	{
		DrawGameplayEffects(CanvasContext, OwnerPC);
	}

	if (bShowGameplayAttributes)
	{
		DrawGameplayAttributes(CanvasContext, OwnerPC);
	}

	LastDrawDataEndSize = CanvasContext.CursorY - ThisDrawDataStartPos;
}

void FGameplayDebuggerCategory_Abilities::DrawGameplayTags(FGameplayDebuggerCanvasContext& CanvasContext, const APlayerController* OwnerPC) const
{
	using namespace UE::AbilitySystem::Debug;
	const float CanvasWidth = CanvasContext.Canvas->SizeX;
	constexpr float Padding = 10.0f;

	const AActor* LocalDebugActor = FindLocalDebugActor();
	if (const UAbilitySystemComponent* AbilityComp = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(LocalDebugActor))
	{
		FGameplayTagContainer ServerOnlyTags = DataPack.OwnedTags;
		const TArray<int32>& ServerTagCounts = DataPack.TagCounts;
		ensureMsgf(ServerTagCounts.Num() == ServerOnlyTags.Num(), TEXT("Server's OwnedTags Array and ServerTagCounts Array did not match in size. Check FGameplayDebuggerCategory_Abilities's CollectData or NetSerialize functions."));

		// Fill in the map for easy access
		TMap<FGameplayTag, int32> ServerTagCountMap;
		for (int32 Idx = 0 ; Idx < ServerOnlyTags.Num() ; ++Idx)
		{
			const FGameplayTag& Tag = ServerOnlyTags.GetByIndex(Idx);
			const int32 ServerTagCount = Idx < ServerTagCounts.Num() ? ServerTagCounts[Idx] : 0;

			ServerTagCountMap.Add(Tag, ServerTagCount);
		}

		auto BuildTagsString = [&](const FGameplayTagContainer& TagContainer, const TCHAR* DefaultColorName) -> FString
			{
				if (TagContainer.IsEmpty())
				{
					return FString{};
				}

				TStringBuilder<1024> StringBuilder;
				StringBuilder.Append(DefaultColorName);

				for (const FGameplayTag& Tag : TagContainer)
				{
					const int32 LocalTagCount = AbilityComp->GetTagCount(Tag);
					const int32 ServerTagCount = ServerTagCountMap.FindRef(Tag, 0);

					// If we're out of sync, then we should print both
					if (DefaultColorName == BothColor && LocalTagCount != ServerTagCount)
					{
						StringBuilder.Appendf(TEXT(" %s %s(x%d) %s(x%d)%s,"), *Tag.ToString(), ServerColor, ServerTagCount, LocalColor, LocalTagCount, DefaultColorName);
						continue;
					}

					// We're in sync (or only exist in one location), just print the numbers
					const int32 TagCount = DefaultColorName == ServerColor ? ServerTagCount : LocalTagCount;
					if (TagCount == 1)
					{
						StringBuilder.Appendf(TEXT(" %s,"), *Tag.ToString());
					}
					else if (TagCount > 1)
					{
						StringBuilder.Appendf(TEXT(" %s (x%d),"), *Tag.ToString(), TagCount);
					}
					else if (TagCount <= 0)
					{
						// Special case where the count is zero.  We should call this out.
						StringBuilder.Appendf(TEXT(" {red}%s (x%d)%s,"), *Tag.ToString(), TagCount, DefaultColorName);
					}
				}

				// Remove the last comma
				StringBuilder.RemoveSuffix(1);
				return StringBuilder.ToString();
			};

		// If we're not the authority, we should represent the tags in such a way that the user can see the difference between agreed upon server/client tags
		// and ones where they disagree.
		if (!AbilityComp->IsOwnerActorAuthoritative())
		{
			FGameplayTagContainer LocalOnlyTags;
			AbilityComp->GetOwnedGameplayTags(LocalOnlyTags);

			const FGameplayTagContainer MatchingTags = ServerOnlyTags.FilterExact(LocalOnlyTags);
			ServerOnlyTags.RemoveTags(MatchingTags);
			LocalOnlyTags.RemoveTags(MatchingTags);

			// Build up the strings
			const FString MatchingTagsStr = BuildTagsString(MatchingTags, BothColor);
			const FString ServerOnlyTagsStr = BuildTagsString(ServerOnlyTags, ServerColor);
			const FString LocalOnlyTagsStr = BuildTagsString(LocalOnlyTags, LocalColor);

			CanvasContext.Printf(TEXT("Owned Tags [%d]:"), MatchingTags.Num() + ServerOnlyTags.Num() + LocalOnlyTags.Num());
			CanvasContext.CursorX += 200.0f;
			CanvasContext.CursorY -= CanvasContext.GetLineHeight();
			CanvasContext.Printf(TEXT("Legend:  %sBoth [%d]    %sServer [%d]    %sLocal [%d]"),
				BothColor, MatchingTags.Num(),
				ServerColor, ServerOnlyTags.Num(),
				LocalColor, LocalOnlyTags.Num());

			FString WrappedDebugText;
			WrapStringAccordingToViewport(FString::Printf(TEXT("%s%s%s"), *MatchingTagsStr, *ServerOnlyTagsStr, *LocalOnlyTagsStr), WrappedDebugText, CanvasContext, CanvasWidth);

			CanvasContext.CursorX += Padding;
			CanvasContext.Print(WrappedDebugText);
		}
		else
		{
			// As the authority, the source of the truth should be the ServerTags we already gathered
			FString ServerOnlyTagsStr;
			WrapStringAccordingToViewport(BuildTagsString(ServerOnlyTags, BothColor), ServerOnlyTagsStr, CanvasContext, CanvasWidth);
			CanvasContext.Printf(TEXT("Owned Tags [%d]"), ServerOnlyTags.Num());

			CanvasContext.CursorX += Padding;
			CanvasContext.Print(ServerOnlyTagsStr);
		}
	}

	// End with a newline to separate from the other categories
	CanvasContext.Print(TEXT(""));
}

void FGameplayDebuggerCategory_Abilities::DrawGameplayEffects(FGameplayDebuggerCanvasContext& CanvasContext, const APlayerController* OwnerPC) const
{
	using namespace UE::AbilitySystem::Debug;
	using FGameplayEffectDebug = FRepData::FGameplayEffectDebug;

	// Find some stable naming sizes
	constexpr float Padding = 10.0f;
	static float ObjNameSize = 0.0f, SrcNameSize = 0.0f;
	if (ObjNameSize <= 0.0f)
	{
		float TempSizeY = 0.0f;
		CanvasContext.MeasureString(*LongestDebugObjectName, ObjNameSize, TempSizeY);
		CanvasContext.MeasureString(FString::Printf(TEXT("source: %.30s"), *LongestDebugObjectName), SrcNameSize, TempSizeY);
		ObjNameSize += Padding;
		SrcNameSize += Padding;
	}

	// Merge together the server and local data
	const UAbilitySystemComponent* LocalASC = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(FindLocalDebugActor());
	const bool bConsiderLocalStatus = OwnerPC != nullptr && !OwnerPC->IsNetMode(ENetMode::NM_Standalone) && LocalASC && (LocalASC->ReplicationMode != EGameplayEffectReplicationMode::Full);
	const TArray<FGameplayEffectDebug>& ServerEffects = DataPack.GameplayEffects;
	TArray<FGameplayEffectDebug> LocalEffects = bConsiderLocalStatus ? CollectEffectsData(OwnerPC, LocalASC) : ServerEffects;

	int NumEffectCounts[+ENetworkStatus::MAX] = { 0 };

	auto GetKeyHash = [](const FGameplayEffectDebug& GameplayEffectDebug, TArray<FGameplayEffectDebug> SourceArray) -> uint32
		{
			// If we're networked, this will be the same between the client and server (assuming it's been replicated)
			if (GameplayEffectDebug.ReplicationID != INDEX_NONE)
				return GameplayEffectDebug.ReplicationID;

			// If we're not networked/not yet replicated, rely on the index of the item in the array & name for good measure
			ptrdiff_t ArrayIdx = (&GameplayEffectDebug - SourceArray.GetData());
			return HashCombineFast(GetTypeHash(GameplayEffectDebug.Effect), ArrayIdx);
		};

	TMap<uint32, FGameplayEffectDebug> Effects;
	for (const FGameplayEffectDebug& ServerEffect : ServerEffects)
	{
		Effects.Add(GetKeyHash(ServerEffect, ServerEffects)) = ServerEffect;
	}

	for (const FGameplayEffectDebug& LocalEffect : LocalEffects)
	{
		FGameplayEffectDebug& Effect = Effects.FindOrAdd(GetKeyHash(LocalEffect, LocalEffects));
		if (!Effect.Effect.IsEmpty())
		{
			Effect.NetworkStatus = ENetworkStatus::Networked;
		}
		else
		{
			Effect = LocalEffect;
			Effect.NetworkStatus = ENetworkStatus::LocalOnly;
		}

		++NumEffectCounts[+Effect.NetworkStatus];
	}

	// Easier to do this calculation than to keep track during those loops
	NumEffectCounts[+ENetworkStatus::ServerOnly] = Effects.Num() - NumEffectCounts[+ENetworkStatus::Networked] - NumEffectCounts[+ENetworkStatus::LocalOnly];

	CanvasContext.Printf(TEXT("Gameplay Effects [%d]:"), Effects.Num());
	CanvasContext.CursorX += 200.0f;
	CanvasContext.CursorY -= CanvasContext.GetLineHeight();
	CanvasContext.Printf(TEXT("Legend:  %sBoth [%d]    %sServer [%d]    %sLocal [%d]    {red}!! Inhibited"),
		BothColor, NumEffectCounts[+ENetworkStatus::Networked],
		ServerColor, NumEffectCounts[+ENetworkStatus::ServerOnly],
		LocalColor, NumEffectCounts[+ENetworkStatus::LocalOnly]);

	CanvasContext.CursorX += Padding;
	for (const auto& Pair : Effects)
	{
		const FGameplayEffectDebug& ItemData = Pair.Value;

		float CursorX = CanvasContext.CursorX;
		float CursorY = CanvasContext.CursorY;

		const FString EffectName = ColorNetworkString(ItemData.NetworkStatus, *ItemData.Effect.Left(LongestDebugObjectName.Len()));
		CanvasContext.PrintfAt(CursorX, CursorY, TEXT("%s%s"), ItemData.bInhibited ? TEXT("{red}!!") : TEXT(""), *EffectName);
		CanvasContext.PrintfAt(CursorX + ObjNameSize, CursorY, FColor::Silver, TEXT("source: {white}%.30s"), *ItemData.Context);

		TStringBuilder<1024> Desc;
		Desc.Appendf(TEXT(" {grey}duration: {white}"), *ItemData.Context);
		if (ItemData.Duration > 0.0f)
		{
			Desc.Appendf(TEXT("%.3f"), ItemData.Duration);
		}
		else
		{
			Desc.Appendf(TEXT("INF"));
		}

		if (ItemData.Period > 0.0f)
		{
			Desc.Appendf(TEXT(" {grey}period: {white}%.3f"), ItemData.Period);
		}

		if (ItemData.Stacks > 1)
		{
			Desc.Appendf(TEXT(" {grey}stacks: {white}%d"), ItemData.Stacks);
		}

		if (ItemData.Level > 1.0f)
		{
			Desc.Appendf(TEXT(" {grey}level: {white}%.2f"), ItemData.Level);
		}

		CanvasContext.PrintAt(CursorX + ObjNameSize + SrcNameSize, CursorY, *Desc);
		CanvasContext.MoveToNewLine();
		CanvasContext.CursorX += Padding;
	}
	
	// End with a newline to separate from the other categories
	CanvasContext.MoveToNewLine();
}

void FGameplayDebuggerCategory_Abilities::DrawGameplayAbilities(FGameplayDebuggerCanvasContext& CanvasContext, const APlayerController* OwnerPC) const
{
	using namespace UE::AbilitySystem::Debug;

	const float CanvasWidth = CanvasContext.Canvas->SizeX / CanvasContext.Canvas->GetDPIScale();
	Algo::Sort(DataPack.Abilities, [](const FRepData::FGameplayAbilityDebug& ItemOne, const FRepData::FGameplayAbilityDebug& ItemTwo) { return ItemOne.Ability < ItemTwo.Ability; });

	int32 NumActive = 0;
	for (const FRepData::FGameplayAbilityDebug& ItemData : DataPack.Abilities)
	{
		NumActive += ItemData.bIsActive;
	}

	// Measure the individual string sizes, so that we can size the columns properly
	// We're picking a long-enough name for the object name sizes
	constexpr float Padding = 10.0f;
	static float ObjNameSize = 0.0f, SourceNameSize = 0.0f, LevelNameSize = 0.0f;
	if (ObjNameSize <= 0.0f)
	{
		float TempSizeY = 0.0f;

		// We have to actually use representative strings because of the kerning
		CanvasContext.MeasureString(*LongestDebugObjectName, ObjNameSize, TempSizeY);
		CanvasContext.MeasureString(TEXT("source: "), SourceNameSize, TempSizeY);
		CanvasContext.MeasureString(TEXT("level: 00"), LevelNameSize, TempSizeY);
		ObjNameSize += Padding;
	}
	const float ColumnWidth = ObjNameSize * 2 + SourceNameSize + LevelNameSize;
	const int NumColumns = FMath::Max(1, FMath::FloorToInt(CanvasWidth / ColumnWidth));

	CanvasContext.Print(TEXT("Gameplay Abilities:"));
	CanvasContext.CursorX += 200.0f;
	CanvasContext.CursorY -= CanvasContext.GetLineHeight();
	CanvasContext.Printf(TEXT("Legend:  {yellow}Granted [%d]    {cyan}Active [%d]"), DataPack.Abilities.Num(), NumActive);

	CanvasContext.CursorX += Padding;
	for (const FRepData::FGameplayAbilityDebug& ItemData : DataPack.Abilities)
	{
		float CursorX = CanvasContext.CursorX;
		float CursorY = CanvasContext.CursorY;

		// Print positions manually to align them properly
		CanvasContext.PrintAt(CursorX + ObjNameSize * 0, CursorY, ItemData.bIsActive ? FColor::Cyan : FColor::Yellow, ItemData.Ability.Left(35));
		CanvasContext.PrintAt(CursorX + ObjNameSize * 1, CursorY, FString::Printf(TEXT("{grey}source: {white}%.35s"), *ItemData.Source));
		CanvasContext.PrintAt(CursorX + ObjNameSize * 2 + SourceNameSize, CursorY, FString::Printf(TEXT("{grey}level: {white}%02d"), ItemData.Level));

		// PrintAt would have reset these values, restore them.
		CanvasContext.CursorX = CursorX + (CanvasWidth / NumColumns);
		CanvasContext.CursorY = CursorY;

		// If we're going to overflow, go to the next line...
		if (CanvasContext.CursorX + ColumnWidth >= CanvasWidth)
		{
			CanvasContext.MoveToNewLine();
			CanvasContext.CursorX += Padding;
		}
	}

	// End the row with a newline
	if (CanvasContext.CursorX != CanvasContext.DefaultX)
	{
		CanvasContext.MoveToNewLine();
	}

	// End the category with a newline to separate from the other categories
	CanvasContext.MoveToNewLine();
}

void FGameplayDebuggerCategory_Abilities::DrawGameplayAttributes(FGameplayDebuggerCanvasContext& CanvasContext, const APlayerController* OwnerPC) const
{
	using namespace UE::AbilitySystem::Debug;

	struct FDebugAttributeData
	{
		FStringView AttributeName;

		float ServerBaseValue = 0.0f;
		float ServerCurrentValue = 0.0f;
		float LocalBaseValue = 0.0f;
		float LocalCurrentValue = 0.0f;

		ENetworkStatus NetworkStatus = ENetworkStatus::LocalOnly;
	};

	const bool bConsiderNetworkStatus = OwnerPC != nullptr && !OwnerPC->IsNetMode(ENetMode::NM_Standalone);
	const UAbilitySystemComponent* LocalASC = bConsiderNetworkStatus ? UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(FindLocalDebugActor()) : nullptr;
	const TArray<FRepData::FGameplayAttributeDebug> LocalAttributes = LocalASC ? CollectAttributeData(OwnerPC, LocalASC) : TArray<FRepData::FGameplayAttributeDebug>{};
	TArray<FRepData::FGameplayAttributeDebug> ServerAttributes = DataPack.Attributes;

	int NumAttributeCounts[+ENetworkStatus::MAX] = { 0 };

	// Reverse the order of iteration so RemoveAt has a better chance at removing near the end
	TArray<FDebugAttributeData> AttributeDebugData;
	for (int32 Index = LocalAttributes.Num() - 1 ; Index >= 0 ; --Index)
	{
		const FRepData::FGameplayAttributeDebug& LocalAttribute = LocalAttributes[Index];

		FDebugAttributeData& DebugDatum = AttributeDebugData.Add_GetRef(
			FDebugAttributeData{
				.AttributeName = LocalAttribute.AttributeName,
				.LocalBaseValue = LocalAttribute.BaseValue,
				.LocalCurrentValue = LocalAttribute.CurrentValue,
				.NetworkStatus = ENetworkStatus::LocalOnly
			});

		int32 ServerIndex = ServerAttributes.FindLastByPredicate([FindName = LocalAttribute.AttributeName](const FRepData::FGameplayAttributeDebug& Item) { return FindName == Item.AttributeName; });
		if (ServerIndex != INDEX_NONE)
		{
			const FRepData::FGameplayAttributeDebug& ServerAttribute = ServerAttributes[ServerIndex];

			DebugDatum.ServerBaseValue = ServerAttribute.BaseValue;
			DebugDatum.ServerCurrentValue = ServerAttribute.CurrentValue;
			DebugDatum.NetworkStatus = (ServerAttribute.NetworkStatus == ENetworkStatus::Networked) ? ENetworkStatus::Networked : ENetworkStatus::Detached;

			// Remove them from server-only array
			ServerAttributes.RemoveAtSwap(ServerIndex, EAllowShrinking::No);
		}
		++NumAttributeCounts[+DebugDatum.NetworkStatus];
	}

	// If any entries are still in this array, they are server only and mark them as such
	for (const FRepData::FGameplayAttributeDebug& ServerAttribute : ServerAttributes)
	{
		FDebugAttributeData& DebugDatum = AttributeDebugData.AddDefaulted_GetRef();
		DebugDatum.AttributeName = ServerAttribute.AttributeName;
		DebugDatum.ServerBaseValue = ServerAttribute.BaseValue;
		DebugDatum.ServerCurrentValue = ServerAttribute.CurrentValue;
		DebugDatum.NetworkStatus = ENetworkStatus::ServerOnly;
		++NumAttributeCounts[+DebugDatum.NetworkStatus];
	}

	// Finally sort to keep everything in alphabetical order
	auto SortByAttributeName = [](const FDebugAttributeData& ItemOne, const FDebugAttributeData& ItemTwo) { return ItemOne.AttributeName < ItemTwo.AttributeName; };
	AttributeDebugData.Sort(SortByAttributeName);

	// Measure a large string once and save the value, so that we can size the columns properly and not continually resize it
	static float MaxLineSize = 0.0f;
	if (MaxLineSize <= 0.0f)
	{
		float TempSizeY;
		const FString MaxLineString = FString::Printf(TEXT("%.35s: 0.000 [0.000]"), *UE::AbilitySystem::Debug::LongestDebugObjectName);
		CanvasContext.MeasureString(MaxLineString, MaxLineSize, TempSizeY);
	}

	constexpr float Padding = 10.0f;
	const float ColumnWidth = MaxLineSize + Padding;
	const float CanvasWidth = CanvasContext.Canvas->SizeX;
	const int NumColumns = FMath::Max(1, FMath::FloorToInt(CanvasWidth / ColumnWidth));

	// Figure out the colors we're going to be using and print the legend
	FString LegendText;
	const bool bAllAttributesNetworked = (0 == (NumAttributeCounts[+ENetworkStatus::ServerOnly] + NumAttributeCounts[+ENetworkStatus::LocalOnly] + NumAttributeCounts[+ENetworkStatus::Detached]));
	if (!bConsiderNetworkStatus || bAllAttributesNetworked)
	{
		// Normal case: no legend required
	}
	else if (!bAllAttributesNetworked)
	{
		// Not all attributes are networked; we should display all cases for easier debugging
		LegendText = FString::Printf(TEXT("Legend:  %sBoth [%d]    %sServer [%d]    %sLocal [%d]    %sNonReplicated [%d]"),
			BothColor, NumAttributeCounts[+ENetworkStatus::Networked],
			ServerColor, NumAttributeCounts[+ENetworkStatus::ServerOnly],
			LocalColor, NumAttributeCounts[+ENetworkStatus::LocalOnly],
			NonReplicatedColor, NumAttributeCounts[+ENetworkStatus::Detached]);
	}
	else
	{
		// All attributes are networked; the values may differ but assume the server is correct
		LegendText = FString::Printf(TEXT("Legend:  %sServer    %sLocal"), ServerColor, LocalColor);
	}
	CanvasContext.Printf(TEXT("Attributes [%d]:"), AttributeDebugData.Num());
	CanvasContext.CursorX += 200.0f;
	CanvasContext.CursorY -= CanvasContext.GetLineHeight();
	CanvasContext.Print(LegendText);

	CanvasContext.CursorX += Padding;
	for (const FDebugAttributeData& AttributeData : AttributeDebugData)
	{
		const bool bServerValueMatch = (AttributeData.ServerBaseValue == AttributeData.ServerCurrentValue);
		const bool bLocalValueMatch = (AttributeData.LocalBaseValue == AttributeData.LocalCurrentValue);
		const bool bModified = !bServerValueMatch || !bLocalValueMatch;
		const bool bNetworkValueMatch = (AttributeData.LocalBaseValue == AttributeData.ServerBaseValue) && (AttributeData.LocalCurrentValue == AttributeData.ServerCurrentValue);

		// Let's build up the attribute value string which is just trying to represent the four states: srv cur [srv base] local cur [local base] in as little text as possible
		TStringBuilder<64> AttributeValueStr;
		if (bNetworkValueMatch && bServerValueMatch && bLocalValueMatch)
		{
			// Everything matches, let's choose white and any one of the values (since they match)
			AttributeValueStr.Appendf(TEXT("{white}%.4g "), AttributeData.ServerBaseValue);
		}
		else
		{
			const bool bDisplayServerValue = (AttributeData.NetworkStatus != ENetworkStatus::LocalOnly);
			const bool bDisplayClientValue = (AttributeData.NetworkStatus != ENetworkStatus::ServerOnly) && !bNetworkValueMatch;

			// Append status that is happening on the Server
			if (bDisplayServerValue)
			{
				// If we are going to also display a client value, color it in the ServerColor, otherwise white.
				const TCHAR* ServerValueColor = bDisplayClientValue ? ServerColor : TEXT("{white}");
				if (bServerValueMatch)
				{
					AttributeValueStr.Appendf(TEXT("%s%.4g "), ServerValueColor, AttributeData.ServerCurrentValue);
				}
				else
				{
					AttributeValueStr.Appendf(TEXT("%s%.4g [%.4g] "), ServerValueColor, AttributeData.ServerCurrentValue, AttributeData.ServerBaseValue);
				}
			}

			// Append status that is happening on the Client (locally)
			if (bDisplayClientValue)
			{
				if (bLocalValueMatch)
				{
					AttributeValueStr.Appendf(TEXT("%s%.4g "), LocalColor, AttributeData.LocalCurrentValue);
				}
				else
				{
					AttributeValueStr.Appendf(TEXT("%s%.4g [%.4g] "), LocalColor, AttributeData.LocalCurrentValue, AttributeData.LocalBaseValue);
				}
			}
		}

		// Print positions manually to align things properly
		const float CursorX = CanvasContext.CursorX;
		const float CursorY = CanvasContext.CursorY;

		const ENetworkStatus NetworkStatus = bConsiderNetworkStatus ? AttributeData.NetworkStatus : ENetworkStatus::Networked;
		const FString ColoredAttributeName = ColorNetworkString(NetworkStatus, AttributeData.AttributeName.Left(LongestDebugObjectName.Len()));
		const FString AttributeDebugText = FString::Printf(TEXT("%s%s: %.*s"), bModified ? TEXT("*") : TEXT(""), *ColoredAttributeName, AttributeValueStr.Len(), AttributeValueStr.GetData());
		CanvasContext.PrintAt(CursorX, CursorY, AttributeDebugText);

		// PrintAt would have reset these values, restore them.
		CanvasContext.CursorX = CursorX + (CanvasWidth / NumColumns);
		CanvasContext.CursorY = CursorY;

		// If we're going to overflow, go to the next line...
		if (CanvasContext.CursorX + ColumnWidth >= CanvasWidth)
		{
			CanvasContext.MoveToNewLine();
			CanvasContext.CursorX += Padding;
		}
	}

	// End the row with a newline
	if (CanvasContext.CursorX != CanvasContext.DefaultX)
	{
		CanvasContext.MoveToNewLine();
	}

	// End the category with a newline to separate from the other categories
	CanvasContext.MoveToNewLine();
}

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
