// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffect.h"

#include "TimerManager.h"
#include "AbilitySystemLog.h"
#include "GameFramework/GameStateBase.h"
#include "Engine/ChildConnection.h"
#include "Engine/NetConnection.h"
#include "Engine/PackageMapClient.h"
#include "AbilitySystemStats.h"
#include "GameplayTagsModule.h"
#include "AbilitySystemGlobals.h"
#include "GameplayEffectExtension.h"
#include "AbilitySystemComponent.h"
#include "GameplayModMagnitudeCalculation.h"
#include "GameplayEffectExecutionCalculation.h"
#include "GameplayCueManager.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectThreadContext.h" // FUObjectSerializeContext
#include "GameplayEffectTypes.h"
#include "GameplayEffectCustomApplicationRequirement.h"
#include "GameplayEffectUIData.h"
#include "Misc/DataValidation.h"
#include "HAL/IConsoleManager.h"

#include "GameplayEffectComponents/AbilitiesGameplayEffectComponent.h"
#include "GameplayEffectComponents/AssetTagsGameplayEffectComponent.h"
#include "GameplayEffectComponents/BlockAbilityTagsGameplayEffectComponent.h"
#include "GameplayEffectComponents/ChanceToApplyGameplayEffectComponent.h"
#include "GameplayEffectComponents/AdditionalEffectsGameplayEffectComponent.h"
#include "GameplayEffectComponents/CustomCanApplyGameplayEffectComponent.h"
#include "GameplayEffectComponents/ImmunityGameplayEffectComponent.h"
#include "GameplayEffectComponents/RemoveOtherGameplayEffectComponent.h"
#include "GameplayEffectComponents/TargetTagRequirementsGameplayEffectComponent.h"
#include "GameplayEffectComponents/TargetTagsGameplayEffectComponent.h"
#include "Iris/IrisConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayEffect)

#define LOCTEXT_NAMESPACE "GameplayEffect"

#if ENABLE_VISUAL_LOG
//#include "VisualLogger/VisualLogger.h"
#endif // ENABLE_VISUAL_LOG

const float FGameplayEffectConstants::INFINITE_DURATION = -1.f;
const float FGameplayEffectConstants::INSTANT_APPLICATION = 0.f;
const float FGameplayEffectConstants::NO_PERIOD = 0.f;
const float FGameplayEffectConstants::INVALID_LEVEL = -1.f;

const float UGameplayEffect::INFINITE_DURATION = FGameplayEffectConstants::INFINITE_DURATION;
const float UGameplayEffect::INSTANT_APPLICATION =FGameplayEffectConstants::INSTANT_APPLICATION;
const float UGameplayEffect::NO_PERIOD = FGameplayEffectConstants::NO_PERIOD;
const float UGameplayEffect::INVALID_LEVEL = FGameplayEffectConstants::INVALID_LEVEL;

DECLARE_CYCLE_STAT(TEXT("MakeQuery"), STAT_MakeGameplayEffectQuery, STATGROUP_AbilitySystem);

extern TAutoConsoleVariable<bool> CVarReplicateTagCountContainerWithIris;
extern TAutoConsoleVariable<bool> CVarGESimOnlyTags;

namespace UE::GameplayEffect
{
	enum class EActiveGameplayEffectFix : int32
	{
		None = 0,									// No additional fixes to UE5.3 are attempted
		MostRecentArrayReplicationKey	= (1 << 0),	// MostRecentArrayReplicationKey is also copied during move/copy
		OmitPendingNextOnCopy			= (1 << 1),	// PendingNext should never be set according to the data structure
		MoveClientCachedStackCount		= (1 << 2), // Copy the ClientCachedStackCount in the move constructor
		CleanupAllPendingActiveGEs		= (1 << 3), // Clean-up the entire Active Pending GE list on destruction

		Current = MostRecentArrayReplicationKey | OmitPendingNextOnCopy | MoveClientCachedStackCount | CleanupAllPendingActiveGEs
	};

	int32 ActiveGameplayEffectReplicationFix = (int32)EActiveGameplayEffectFix::Current;
	FAutoConsoleVariableRef CVarActiveGameplayEffectReplicationFix{ TEXT("AbilitySystem.Fix.ActiveGEReplicationFix"), ActiveGameplayEffectReplicationFix, TEXT("Experimental code mask for fixing Active Gameplay Effects (set to 0 to disable)"), ECVF_Default };
	inline bool HasActiveGameplayEffectFix(EActiveGameplayEffectFix Flag) { return (ActiveGameplayEffectReplicationFix & static_cast<int32>(Flag)) != 0; }

	enum class EActiveGameplayEffectOverflowBehavior : int32
	{
		None = 0,
		HandleOverflowingStacksAlways		= (1 << 0), // Instead of checking the overflow if the ActiveGE stacks are at the limit, we check it whenever there are overflowing stacks
		ApplyRemainingStacksWhenOverflow	= (1 << 1), // Instead of returning when we deny the application because of overflowing stacks, we apply the stacks up to the limit and ignore the surplus of stacks (stacks over the StackLimit)
		
		Legacy = None,							// Default behavior without those new changes
		Current = HandleOverflowingStacksAlways | ApplyRemainingStacksWhenOverflow,
	};

	int32 ActiveGameplayEffectOverflowBehavior = (int32)EActiveGameplayEffectOverflowBehavior::Current;
	FAutoConsoleVariableRef CVarActiveGameplayEffectOverflowBehavior{ TEXT("AbilitySystem.ActiveGameplayEffectOverflowBehavior"), ActiveGameplayEffectOverflowBehavior, TEXT("Change the behavior of overflowing stacks for Active Gameplay Effects. Use 0 to keep legacy functionality"), ECVF_Default };
	inline bool HasActiveGameplayEffectOverflowBehavior(EActiveGameplayEffectOverflowBehavior Flag) { return (ActiveGameplayEffectOverflowBehavior & static_cast<int32>(Flag)) != 0; }

	// Prediction of stacking gameplay effects
	bool bAllowPredictiveApplicationOfStackingGameplayEffects = true;
	FAutoConsoleVariableRef CVarAllowPredictiveStackingGEs{ TEXT("AbilitySystem.AllowPredictiveStackingGEs"), bAllowPredictiveApplicationOfStackingGameplayEffects, TEXT("Allow the predictive application of stacking gameplay effects. The removal of stacks is not predicted (Only application is)"), ECVF_Default };

	TAutoConsoleVariable<int32> CVarGameplayEffectMaxVersion(TEXT("AbilitySystem.GameplayEffects.MaxVersion"), (int32)EGameplayEffectVersion::Current, TEXT("Override the Gameplay Effect Current Version (disabling upgrade code paths)"), ECVF_Default);

	// Fix introduced in UE5.4
	bool bUseModifierTagRequirementsOnAllGameplayEffects = true;
	FAutoConsoleVariableRef CVarUseModTagReqsOnAllGE{ TEXT("AbilitySystem.Fix.UseModTagReqsOnAllGE"), bUseModifierTagRequirementsOnAllGameplayEffects, TEXT("Fix an issue where MustHave/MustNotHave tags did not apply to Instant and Periodic Gameplay Effects"), ECVF_Default };

	// Fix introduced in UE5.4
	bool bSkipUnmappedReferencesCheckForGameplayCues = true;
	FAutoConsoleVariableRef CVarSkipUnmappedReferencesCheckForGameplayCues{ TEXT("AbilitySystem.Fix.SkipUnmappedReferencesCheckForGameplayCues"), bSkipUnmappedReferencesCheckForGameplayCues,
		TEXT("Skip the bHasMoreUnmappedReferences check for GameplayCues which never worked as intended and causes issues when set properly (may be deprecated soon)"), ECVF_Default };

	int32 ReturnAssetTagsViaGetOwnedGameplayTags = 2;
	FAutoConsoleVariableRef CVarReturnAssetTagsViaGetOwnedGameplayTags{ TEXT("AbilitySystem.Fix.GameplayEffects.ReturnAssetTagsViaGetOwnedGameplayTags"), ReturnAssetTagsViaGetOwnedGameplayTags ,
		TEXT("0 = Return the asset tags as per the function name in get owned gameplay tags. 1 =  Use legacy behavior. If legacy is desired, change code to GetGrantedTags. 2 = Legacy behavior but ensure to catch bad use. 3. Legacy behavior but always ensure to catch multiple issues."), ECVF_Default };

	TAutoConsoleVariable<bool> CVarFixPredictedTagApplication(
		TEXT("AbilitySystem.Fix.PredictedTagApplication"),
		true,
		TEXT("Default: True.  When enabled Gameplay Effect tag application usesthe same logic as Gameplay Cues so that application and removal works correctly for predicted effects."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarFixPendingGameplayEffectHeadNextAtTeardown(
		TEXT("AbilitySystem.Fix.PendingGameplayEffectHeadNextAtTeardown"),
		true,
		TEXT("Default: True.  When enabled we will prevent the Pending Gameplay Effect Head from pointing to itself through Next pointer during teardown to prevent a double call to delete."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarFixAllPendingGameplayEffectsHeadNextsAtTeardown(
		TEXT("AbilitySystem.Fix.AllPendingGameplayEffectsHeadNextsAtTeardown"),
		true,
		TEXT("Default: True.  When enabled we will prevent all the Pending Gameplay Effects Head from pointing to itself through Next pointers during teardown to prevent any double call to delete."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarFixVerboseLogDeletionOfPendingGameplayEffectsHead(
		TEXT("AbilitySystem.Fix.VerboseLogDeletionOfPendingGameplayEffectsHead"),
		true,
		TEXT("Default: True.  When enabled we will verbose log each gameplay effect deletion."),
		ECVF_Default);

#if WITH_EDITOR
	namespace EditorOnly
	{
		// Helper function that allows us to guard against upgrading versions (in case a bug is discovered while the product is live)
		static bool ShouldUpgradeVersion(EGameplayEffectVersion FromVersion, EGameplayEffectVersion ToVersion)
		{
			EGameplayEffectVersion MaxVersion = static_cast<EGameplayEffectVersion>(CVarGameplayEffectMaxVersion.GetValueOnAnyThread() & 0xff);
			if (MaxVersion < ToVersion)
			{
				return false;
			}

			return FromVersion < ToVersion;
		}

		static void ConformGameplayEffectTransactionality(TArrayView<TObjectPtr<UGameplayEffectComponent>> Components)
		{
			// This logic is compensating for FObjectInstancingGraph::GetInstancedSubobject's failure
			// to propagate template object's flags - specifically RF_Transactional. If our CDO
			// were reliably RF_Transactional we would also not need to do this. For now, let's
			// just conform here:
			for (const TObjectPtr<UGameplayEffectComponent>& GEComponent : Components)
			{
				if (GEComponent)
				{
					GEComponent->SetFlags(RF_Transactional);
				}
			}
		}
	}
#endif
}

// Versioning code
bool FGameplayEffectVersion::Serialize(FStructuredArchive::FSlot Slot)
{
	Slot << CurrentVersion;
	return true;
}

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	UGameplayEffect
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

UGameplayEffect::UGameplayEffect(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	DurationPolicy = EGameplayEffectDurationType::Instant;
	bExecutePeriodicEffectOnApplication = true;
	PeriodicInhibitionPolicy = EGameplayEffectPeriodInhibitionRemovedPolicy::NeverReset;
	ChanceToApplyToTarget_DEPRECATED.SetValue(1.f);
	StackingType = EGameplayEffectStackingType::None;
	StackLimitCount = 0;
	StackDurationRefreshPolicy = EGameplayEffectStackingDurationPolicy::RefreshOnSuccessfulApplication;
	StackPeriodResetPolicy = EGameplayEffectStackingPeriodPolicy::ResetOnSuccessfulApplication;
	bRequireModifierSuccessToTriggerCues = true;
	bFactorInStackCount = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

// Needed for versioning code
void UGameplayEffect::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	// On the first load of this package, the version will be inherited from the parent class.
	// We want to always stomp that value to say it's a Monolithic version (pre-UE5.3), ensuring a child class always
	// executes the upgrade path at least once.  Once the package has been fully loaded, in PostLoad we will correctly
	// set the package version.  However, this function will again be called on recompilation of the Blueprint.
	// In order to account for that, we should check if the Package has a Linker:
	// This will be True if it's already existed; False if being created (new asset).
	if (const UPackage* Package = GetPackage())
	{
		const bool bAlreadyExists = (Package->GetLinker() != nullptr);
		if (!bAlreadyExists)
		{
			constexpr bool bForceGameThreadValue = true;
			SetVersion(static_cast<EGameplayEffectVersion>(UE::GameplayEffect::CVarGameplayEffectMaxVersion.GetValueOnAnyThread(bForceGameThreadValue)));
		}
		else
		{
			SetVersion(EGameplayEffectVersion::Monolithic);
		}
	}

	// Let's cover some easy-to-overlook issues when implementing a Gameplay Effect as a native class
	// First, ensure we've actually caught all of the native GEComponents
	if (!IsInBlueprint())
	{
		TArray<UObject*> PossibleComponents;
		GetDefaultSubobjects(PossibleComponents);
		for (UObject* Obj : PossibleComponents)
		{
			if (UGameplayEffectComponent* GEComponent = Cast<UGameplayEffectComponent>(Obj))
			{
				if (!ensureMsgf(GEComponents.Contains(GEComponent), TEXT("%s: %s should be added to GEComponents during the constructor or in PostInitProperties"), *GetName(), *GEComponent->GetName()))
				{
					GEComponents.Add(GEComponent);
				}
			}
		}

		// Catch any GEComponent configuration issues.  We usually want to do this late (after deserialization or BP compilation), but
		// we've already determined this is a native class, so let's do that now (we won't get PostLoad or BP compilation).
		FDataValidationContext DataValidationContext;
		IsDataValid(DataValidationContext);

		// Now spit out any of those issues in the log
		TArray<FText> Warnings, Errors;
		DataValidationContext.SplitIssues(Warnings, Errors);
		for (const FText& WarningText : Warnings)
		{
			UE_LOGF(LogGameplayEffects, Warning, "%ls: %ls", *GetName(), *WarningText.ToString());
		}
		for (const FText& ErrorText : Errors)
		{
			UE_LOGF(LogGameplayEffects, Error, "%ls: %ls", *GetName(), *ErrorText.ToString());
		}
	}
#endif
}

void UGameplayEffect::GetOwnedGameplayTags(FGameplayTagContainer& TagContainer) const
{
	if (UE::GameplayEffect::ReturnAssetTagsViaGetOwnedGameplayTags == 0)
	{
		TagContainer.AppendTags(GetAssetTags());
	}
	else
	{	
		// Default Engine Behavior, to be deprecated. Set to ensure to warn of potential complications 
		if (UE::GameplayEffect::ReturnAssetTagsViaGetOwnedGameplayTags == 2)
		{
			// Warn once. To check for additional accessors, set ReturnAssetTagsViaGetOwnedGameplayTags = 3
			ensureMsgf(false, TEXT("%hs on %s: The implementation and method name did not match.  Use GetGrantedTags() to get the tags Granted to the Actor this GameplayEffect is applied to. Change ReturnAssetTagsViaGetOwnedGameplayTags to get the Asset Tags on the GE, or to modify the behavior of this ensure"), __func__, *GetName());
		}
		else if (UE::GameplayEffect::ReturnAssetTagsViaGetOwnedGameplayTags == 3)
		{
			ensureAlwaysMsgf(false, TEXT("%hs on %s: The implementation and method name did not match.  Use GetGrantedTags() to get the tags Granted to the Actor this GameplayEffect is applied to. Change ReturnAssetTagsViaGetOwnedGameplayTags to get the Asset Tags on the GE, or to modify the behavior of this ensure"), __func__, *GetName());
		}

		TagContainer.AppendTags(GetGrantedTags());
	}
}

bool UGameplayEffect::HasMatchingGameplayTag(FGameplayTag TagToCheck) const
{
	UE_LOGF(LogGameplayEffects, Warning, "%s on %ls: The implementation and method name did not match.  Use GetGrantedTags().HasTag() to check against the tags this GameplayEffect will Grant to the Actor.", __func__, *GetName());
	return IGameplayTagAssetInterface::HasMatchingGameplayTag(TagToCheck);
}

bool UGameplayEffect::HasAllMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const
{
	UE_LOGF(LogGameplayEffects, Warning, "%s on %ls: The implementation and method name did not match.  Use GetGrantedTags().HasAll() to check against the tags this GameplayEffect will Grant to the Actor.", __func__, *GetName());
	return IGameplayTagAssetInterface::HasAllMatchingGameplayTags(TagContainer);
}

bool UGameplayEffect::HasAnyMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const
{
	UE_LOGF(LogGameplayEffects, Warning, "%s on %ls: The implementation and method name did not match.  Use GetGrantedTags().HasAny() to check against the tags this GameplayEffect will Grant to the Actor.", __func__, *GetName());
	return IGameplayTagAssetInterface::HasAnyMatchingGameplayTags(TagContainer);
}

void UGameplayEffect::GetBlockedAbilityTags(FGameplayTagContainer& OutTagContainer) const
{
	OutTagContainer.AppendTags(CachedBlockedAbilityTags);
}

#if WITH_EDITOR
EDataValidationResult UGameplayEffect::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult ValidationResult = Super::IsDataValid(Context);

	// Report these errors here & now, even though they're not using the DataValidationContext because
	// these will show-up when saving/presubmit rather than during more extensive testing done at cook time.
	FString AssetPathName = GetPathName();

	if (DurationPolicy == EGameplayEffectDurationType::HasDuration)
	{
		ValidationResult = CombineDataValidationResults(ValidationResult, DurationMagnitude.IsDataValid(Context, AssetPathName + TEXT(".DurationMagnitude")));
		ValidationResult = CombineDataValidationResults(ValidationResult, MaxDurationMagnitude.IsDataValid(Context, AssetPathName + TEXT(".MaxDurationMagnitude")));
	}

	ValidationResult = CombineDataValidationResults(ValidationResult, Period.IsDataValid(Context, AssetPathName + TEXT(".Period")));

	for (const FGameplayModifierInfo& CurModInfo : Modifiers)
	{
		ValidationResult = CombineDataValidationResults(ValidationResult, CurModInfo.ModifierMagnitude.IsDataValid(Context, AssetPathName + TEXT(".") + CurModInfo.Attribute.GetName()));
	}

	for (const UGameplayEffectComponent* GEComponent : GEComponents)
	{
		if (GEComponent)
		{
			ValidationResult = CombineDataValidationResults(ValidationResult, GEComponent->IsDataValid(Context));
		}
		else
		{
			Context.AddWarning(LOCTEXT("GEIsNull", "Null entry in GEComponents"));
		}
	}

	if (DurationPolicy == EGameplayEffectDurationType::HasDuration && GetStackingType() != EGameplayEffectStackingType::None &&
		StackDurationRefreshPolicy == EGameplayEffectStackingDurationPolicy::ExtendDuration && StackExpirationPolicy != EGameplayEffectStackingExpirationPolicy::ClearEntireStack)
	{
		Context.AddWarning(LOCTEXT("ExtendAndRefreshDurationBehaviour", "Stack Duration and Expiration Policy: When duration is extended on new stack but GE expiration does not clear the whole stack, duration will be refreshed using the total extended duration."));
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DurationPolicy == EGameplayEffectDurationType::Instant && StackingType != EGameplayEffectStackingType::None)
	{
		Context.AddWarning(LOCTEXT("InstantNonStackingBehaviour", "Instant GameplayEffect's will not Stack."));
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TArray<FText> Warnings, Errors;
	Context.SplitIssues(Warnings, Errors);

	if (Errors.Num() > 0)
	{
		EditorStatusText = FText::FormatOrdered(LOCTEXT("ErrorsFmt", "Error: {0}"), FText::Join(FText::FromStringView(TEXT(", ")), Errors));
	}
	else if (Warnings.Num() > 0)
	{
		EditorStatusText = FText::FormatOrdered(LOCTEXT("WarningsFmt", "Warning: {0}"), FText::Join(FText::FromStringView(TEXT(", ")), Warnings));
	}
	else
	{
		EditorStatusText = LOCTEXT("AllOk", "All Ok");
	}

	return ValidationResult;
}
#endif

void UGameplayEffect::PostLoad()
{
	Super::PostLoad();

	OnGameplayEffectChanged();

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HasRemoveGameplayEffectsQuery = !RemoveGameplayEffectQuery.IsEmpty();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// We're done loading (and therefore upgrading), boost the version.
	SetVersion( static_cast<EGameplayEffectVersion>(UE::GameplayEffect::CVarGameplayEffectMaxVersion.GetValueOnGameThread()) );
#endif // WITH_EDITOR
}

void UGameplayEffect::OnGameplayEffectChanged()
{
	if (HasAnyFlags(RF_NeedPostLoad))
	{
		ensureMsgf(false, TEXT("%s: OnGameplayEffectChanged can only be called after the GameplayEffect is fully loaded"), *GetName());
		return;
	}

	// Reset these tags so we can reaggregate them properly from the GEComponents
	CachedAssetTags.Reset();
	CachedGrantedTags.Reset();
	CachedBlockedAbilityTags.Reset();

	// Call PostLoad on components and cache tag info from components for more efficient lookups
	for (UGameplayEffectComponent* GEComponent : GEComponents)
	{
		if (GEComponent)
		{
			// Ensure the SubObject is fully loaded
			GEComponent->ConditionalPostLoad();
			GEComponent->OnGameplayEffectChanged();
		}
	}
}

#if WITH_EDITOR
/**
 * We now support GameplayEffectComponents which are Subobjects.
 * 
 * When we're loading a Blueprint (including Data Only Blueprints), we go through a bunch of steps:  Preload, Serialize (which does the instancing), Init, CDO Compile, PostLoad.
 * However, we are not guaranteed that our Parent Class (Archetype) is fully loaded until PostLoad.  So during load (or cooking, where we see this most) is that
 * a Child will request to load, then it's possible the Parent has not fully loaded, so they may also request its Parent load and they go through these steps *together*.
 * When we get to PostCDOCompiled, the Parent may create some Subobjects (for us, this happens during the Monolithic -> Modular upgrade) and yet the Child is also at the same step,
 * so it hasn't actually seen those Subobjects and instanced them, but it *would have* instanced them if Parent Class was loaded first through some other means.
 * So our job here is to ensure the Subobjects that exist in the Parent also exist in the Child, so that the order of loading the classes doesn't matter.
 *  
 * There are other issues that will pop-up:
 *	1. You cannot remove a Parent's Subobject in the Child (this code will just recreate it).
 *	2. If you remove a Subobject from the Parent, the Child will continue to own it and it will be delta serialized *from the Grandparent*
 */
void UGameplayEffect::PostCDOCompiledFixupSubobjects()
{
	const UGameplayEffect* Archetype = Cast<UGameplayEffect>(GetArchetype());
	if (!Archetype)
	{
		return;
	}

	// Clear out any bad inherited components
	GEComponents.Remove(nullptr);

	for (const UGameplayEffectComponent* ParentComponent : Archetype->GEComponents)
	{
		if (!ParentComponent)
		{
			continue;
		}

		bool bFound = false;
		const FName ParentComponentName = ParentComponent->GetFName();
		for (const UGameplayEffectComponent* ChildComponent : GEComponents)
		{
			// When the SubObject code decides how to delta serialize from its Archetype
			// The Archetype is determined by name. Let's match-up specifically by name rather than type alone.
			if (ChildComponent && ChildComponent->GetFName() == ParentComponentName)
			{
				bFound = true;
				break;
			}
		}

		// We already have the Component that matches the Archetype's Component
		if (bFound)
		{
			continue;
		}

		// This will find components inherited from the parent that are missing from the array
		UGameplayEffectComponent* ChildComponent = FindObjectFast<UGameplayEffectComponent>(this, *ParentComponentName.ToString());
		if (!ChildComponent)
		{
			// We don't already have the Archetype's Component, so add it here using the exact same name so we link up.
			UE_LOGF(LogGameplayEffects, Verbose, "%ls is manually duplicating Archetype %ls because they were not inherited through automatic instancing", *GetFullNameSafe(this), *GetFullNameSafe(ParentComponent));
			ChildComponent = DuplicateObject(ParentComponent, this, ParentComponentName);
		}
		else
		{
			UE_LOGF(LogGameplayEffects, Verbose, "%ls is re-adding inherited copy of Archetype %ls to components array", *GetFullNameSafe(this), *GetFullNameSafe(ParentComponent));
		}

		GEComponents.Add(ChildComponent);
	}
}

void UGameplayEffect::PostCDOCompiled(const FPostCDOCompiledContext& Context)
{
	Super::PostCDOCompiled(Context);
	UE_LOGF(LogGameplayEffects, VeryVerbose, "%ls: Running Upgrade code. Current Version = %d", *GetName(), int(GetVersion()));

	// Make sure everything is fixed up properly before going through upgrade code
	PostCDOCompiledFixupSubobjects();

	// Move any data from deprecated properties into components, create components as needed
	ConvertAbilitiesComponent();
	ConvertRemoveOtherComponent(); // This should come before AdditionalEffects for legacy compatibility
	ConvertAdditionalEffectsComponent();
	ConvertAssetTagsComponent();
	ConvertBlockByTagsComponent();
	ConvertChanceToApplyComponent();
	ConvertCustomCanApplyComponent();
	ConvertImmunityComponent();
	ConvertTagRequirementsComponent();
	ConvertTargetTagsComponent();
	ConvertUIComponent();
	#if WITH_EDITOR
	using namespace UE::GameplayEffect::EditorOnly;
	ConformGameplayEffectTransactionality(GEComponents);
	#endif

	const bool bAlreadyLoaded = !HasAnyFlags(RF_NeedPostLoad);
	if (bAlreadyLoaded)
	{
		OnGameplayEffectChanged();
	}
}

// Upgrade Code Section.  Lots of uses of deprecated variables (because we're upgrading them)
// So just disable it throughout this section.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UGameplayEffect::ConvertAbilitiesComponent()
{
	// Get the archetype to compare to
	const UGameplayEffect* Archetype = CastChecked<UGameplayEffect>(GetArchetype());

	// Due to the delta serialization, this will only be true if we're not inheriting the parent's values
	const bool bChanged = !(GrantedAbilities == Archetype->GrantedAbilities);
	if (bChanged && UE::GameplayEffect::EditorOnly::ShouldUpgradeVersion(GetVersion(), EGameplayEffectVersion::AbilitiesComponent53))
	{
		UAbilitiesGameplayEffectComponent& AbilitiesComponent = FindOrAddComponent<UAbilitiesGameplayEffectComponent>();

		// Since we've already determined we're not inheriting the parent values, clobber the array
		AbilitiesComponent.GrantAbilityConfigs.Empty();
		for (const FGameplayAbilitySpecDef& GASpecDef : GrantedAbilities)
		{
			if (!GASpecDef.Ability.Get())
			{
				continue;
			}

			FGameplayAbilitySpecConfig GASpecConfig;
			GASpecConfig.Ability = GASpecDef.Ability;
			GASpecConfig.InputID = GASpecDef.InputID;
			GASpecConfig.LevelScalableFloat = GASpecDef.LevelScalableFloat;
			GASpecConfig.RemovalPolicy = GASpecDef.RemovalPolicy;

			// This function is smart enough to not add duplicates (handles the inheritance case)
			AbilitiesComponent.AddGrantedAbilityConfig(GASpecConfig);
		}

		// We shouldn't just empty out the deprecated GrantedAbilities because then we wouldn't know if a Child's Empty array was an override or inheritance.
		// Instead, let's null-out the entries so they won't affect gameplay, but the child will still inherit the nulled-out list rather than empty list.
		for (FGameplayAbilitySpecDef& Entry : GrantedAbilities)
		{
			Entry.Ability = nullptr;
		}
	}
}

void UGameplayEffect::ConvertCustomCanApplyComponent()
{
	// Get the archetype to compare to
	const UGameplayEffect* Archetype = CastChecked<UGameplayEffect>(GetArchetype());

	const bool bChanged = !(ApplicationRequirements_DEPRECATED == Archetype->ApplicationRequirements_DEPRECATED);
	if (bChanged && UE::GameplayEffect::EditorOnly::ShouldUpgradeVersion(GetVersion(), EGameplayEffectVersion::Modular53))
	{
		UCustomCanApplyGameplayEffectComponent& ApplicationComponent = FindOrAddComponent<UCustomCanApplyGameplayEffectComponent>();
		ApplicationComponent.ApplicationRequirements = ApplicationRequirements_DEPRECATED;
	}

	// Keep backwards compatibility (at least in terms of reading from the data)
	const UCustomCanApplyGameplayEffectComponent* ApplicationComponent = FindComponent<UCustomCanApplyGameplayEffectComponent>();
	if (ApplicationComponent)
	{
		ApplicationRequirements_DEPRECATED = ApplicationComponent->ApplicationRequirements;
	}
}

void UGameplayEffect::ConvertChanceToApplyComponent()
{
	// Get the archetype to compare to
	const UGameplayEffect* Archetype = CastChecked<UGameplayEffect>(GetArchetype());

	bool bChanged = (ChanceToApplyToTarget_DEPRECATED != Archetype->ChanceToApplyToTarget_DEPRECATED);
	bChanged = bChanged || !(ApplicationRequirements_DEPRECATED == Archetype->ApplicationRequirements_DEPRECATED);

	if (bChanged && UE::GameplayEffect::EditorOnly::ShouldUpgradeVersion(GetVersion(), EGameplayEffectVersion::Modular53))
	{
		UChanceToApplyGameplayEffectComponent& ChanceToApplyComponent = FindOrAddComponent<UChanceToApplyGameplayEffectComponent>();
		ChanceToApplyComponent.SetChanceToApplyToTarget(ChanceToApplyToTarget_DEPRECATED);
	}

	// Keep backwards compatibility (at least in terms of reading from the data)
	const UChanceToApplyGameplayEffectComponent* ChanceToApplyComponent = FindComponent<UChanceToApplyGameplayEffectComponent>();
	if (ChanceToApplyComponent)
	{
		ChanceToApplyToTarget_DEPRECATED = ChanceToApplyComponent->GetChanceToApplyToTarget();
	}
}

void UGameplayEffect::ConvertAssetTagsComponent()
{
	// Get the archetype to compare to
	const UGameplayEffect* Archetype = Cast<UGameplayEffect>(GetArchetype());

	bool bChanged = InheritableGameplayEffectTags.CombinedTags != Archetype->InheritableGameplayEffectTags.CombinedTags;
	if (bChanged && UE::GameplayEffect::EditorOnly::ShouldUpgradeVersion(GetVersion(), EGameplayEffectVersion::Modular53))
	{
		UAssetTagsGameplayEffectComponent& AssetTagsComponent = FindOrAddComponent<UAssetTagsGameplayEffectComponent>();
		AssetTagsComponent.SetAndApplyAssetTagChanges(InheritableGameplayEffectTags);
	}

	// Keep backwards compatibility (at least in terms of reading from the data)
	const UAssetTagsGameplayEffectComponent* AssetTagsComponent = FindComponent<UAssetTagsGameplayEffectComponent>();
	if (AssetTagsComponent)
	{
		InheritableGameplayEffectTags = AssetTagsComponent->GetConfiguredAssetTagChanges();
		InheritableGameplayEffectTags.UpdateInheritedTagProperties(&Archetype->InheritableGameplayEffectTags);
	}
}

void UGameplayEffect::ConvertAdditionalEffectsComponent()
{
	// Get the archetype to compare to
	const UGameplayEffect* Archetype = Cast<UGameplayEffect>(GetArchetype());

	bool bConditionalChanged = (ConditionalGameplayEffects.Num() != Archetype->ConditionalGameplayEffects.Num());
	bool bPrematureChanged = (PrematureExpirationEffectClasses != Archetype->PrematureExpirationEffectClasses);
	bool bRoutineChanged = (RoutineExpirationEffectClasses != Archetype->RoutineExpirationEffectClasses);

	bool bChanged = bConditionalChanged || bPrematureChanged || bRoutineChanged;
	if (bChanged && UE::GameplayEffect::EditorOnly::ShouldUpgradeVersion(GetVersion(), EGameplayEffectVersion::Modular53))
	{
		UAdditionalEffectsGameplayEffectComponent& ConditionalEffectsComponent = FindOrAddComponent<UAdditionalEffectsGameplayEffectComponent>();

		if (bConditionalChanged)
		{
			ConditionalEffectsComponent.OnApplicationGameplayEffects = ConditionalGameplayEffects;
		}

		if (bPrematureChanged)
		{
			ConditionalEffectsComponent.OnCompletePrematurely = PrematureExpirationEffectClasses;
		}

		if (bRoutineChanged)
		{
			ConditionalEffectsComponent.OnCompleteNormal = RoutineExpirationEffectClasses;
		}
	}

	// Keep backwards compatibility (at least in terms of reading from the data)
	const UAdditionalEffectsGameplayEffectComponent* ConditionalEffectsComponent = FindComponent<UAdditionalEffectsGameplayEffectComponent>();
	if (ConditionalEffectsComponent)
	{
		ConditionalGameplayEffects = ConditionalEffectsComponent->OnApplicationGameplayEffects;
		PrematureExpirationEffectClasses = ConditionalEffectsComponent->OnCompletePrematurely;
		RoutineExpirationEffectClasses = ConditionalEffectsComponent->OnCompleteNormal;
	}
}

void UGameplayEffect::ConvertBlockByTagsComponent()
{
	// Get the archetype to compare to
	const UGameplayEffect* Archetype = Cast<UGameplayEffect>(GetArchetype());

	bool bChanged = InheritableBlockedAbilityTagsContainer.CombinedTags != Archetype->InheritableBlockedAbilityTagsContainer.CombinedTags;
	if (bChanged && UE::GameplayEffect::EditorOnly::ShouldUpgradeVersion(GetVersion(), EGameplayEffectVersion::Modular53))
	{
		UBlockAbilityTagsGameplayEffectComponent& BlockByTagsComponent = FindOrAddComponent<UBlockAbilityTagsGameplayEffectComponent>();
		BlockByTagsComponent.SetAndApplyBlockedAbilityTagChanges(InheritableBlockedAbilityTagsContainer);
	}

	// Keep backwards compatibility (at least in terms of reading from the data)
	const UBlockAbilityTagsGameplayEffectComponent* BlockByTagsComponent = FindComponent<UBlockAbilityTagsGameplayEffectComponent>();
	if (BlockByTagsComponent)
	{
		InheritableBlockedAbilityTagsContainer = BlockByTagsComponent->GetConfiguredBlockedAbilityTagChanges();
		InheritableBlockedAbilityTagsContainer.UpdateInheritedTagProperties(&Archetype->InheritableBlockedAbilityTagsContainer);
	}
}

void UGameplayEffect::ConvertTargetTagsComponent()
{
	// Get the archetype to compare to
	const UGameplayEffect* Archetype = Cast<UGameplayEffect>(GetArchetype());

	bool bChanged = InheritableOwnedTagsContainer.CombinedTags != Archetype->InheritableOwnedTagsContainer.CombinedTags;
	if (bChanged && UE::GameplayEffect::EditorOnly::ShouldUpgradeVersion(GetVersion(), EGameplayEffectVersion::Modular53))
	{
		UTargetTagsGameplayEffectComponent& TargetTagsComponent = FindOrAddComponent<UTargetTagsGameplayEffectComponent>();
		TargetTagsComponent.SetAndApplyTargetTagChanges(InheritableOwnedTagsContainer);
	}

	// Keep backwards compatibility (at least in terms of reading from the data)
	const UTargetTagsGameplayEffectComponent* TargetTagsComponent = FindComponent<UTargetTagsGameplayEffectComponent>();
	if (TargetTagsComponent)
	{
		InheritableOwnedTagsContainer = TargetTagsComponent->GetConfiguredTargetTagChanges();
		InheritableOwnedTagsContainer.UpdateInheritedTagProperties(&Archetype->InheritableOwnedTagsContainer);
	}
}

void UGameplayEffect::ConvertImmunityComponent()
{
	// Get the archetype to compare to
	const UGameplayEffect* Archetype = Cast<UGameplayEffect>(GetArchetype());

	const bool bTagsChanged = GrantedApplicationImmunityTags != Archetype->GrantedApplicationImmunityTags;
	const bool bQueryChanged (GrantedApplicationImmunityQuery != Archetype->GrantedApplicationImmunityQuery);

	const bool bChanged = bTagsChanged || bQueryChanged;
	if (bChanged && UE::GameplayEffect::EditorOnly::ShouldUpgradeVersion(GetVersion(), EGameplayEffectVersion::Modular53))
	{
		UImmunityGameplayEffectComponent& ImmunityComponent = FindOrAddComponent<UImmunityGameplayEffectComponent>();

		// Our upgrade path is going to have to be conservative because of the Archetype inheritance rules.
		// Let's always make sure there's 2 slots (originating Tags and originating Queries).
		while (ImmunityComponent.ImmunityQueries.Num() < 2)
		{
			ImmunityComponent.ImmunityQueries.AddDefaulted();
		}

		if (bTagsChanged)
		{
			// Build up a query that matches the equivalent GrantedApplicationImmunityTags
			FGameplayEffectQuery TagQuery;

			const bool bHasRequireTags = !GrantedApplicationImmunityTags.RequireTags.IsEmpty();
			const bool bHasIgnoreTags = !GrantedApplicationImmunityTags.IgnoreTags.IsEmpty();
			if (bHasRequireTags || bHasIgnoreTags)
			{
				// Previously the Tags query was GrantedApplicationImmunityTags.RequirementsMet. FGameplayTagContainer::RequirementsMet is HasAll(RequireTags) && !HasAny(IgnoreTags);
				FGameplayTagQueryExpression RequiredTagsQueryExpression = FGameplayTagQueryExpression().AllTagsMatch().AddTags(GrantedApplicationImmunityTags.RequireTags);
				FGameplayTagQueryExpression IgnoreTagsQueryExpression = FGameplayTagQueryExpression().NoTagsMatch().AddTags(GrantedApplicationImmunityTags.IgnoreTags);

				FGameplayTagQueryExpression RootQueryExpression;
				if (bHasRequireTags && bHasIgnoreTags)
				{
					RootQueryExpression = FGameplayTagQueryExpression().AllExprMatch().AddExpr(RequiredTagsQueryExpression).AddExpr(IgnoreTagsQueryExpression);
				}
				else if (bHasRequireTags)
				{
					RootQueryExpression = RequiredTagsQueryExpression;
				}
				else // bHasIgnoreTags
				{
					RootQueryExpression = IgnoreTagsQueryExpression;
				}

				// Build the expression
				TagQuery.SourceTagQuery.Build(RootQueryExpression,TEXT("GrantedApplicationImmunityTags"));
			}
			ImmunityComponent.ImmunityQueries[0] = MoveTemp(TagQuery);
		}

		if (bQueryChanged)
		{
			ImmunityComponent.ImmunityQueries[1] = GrantedApplicationImmunityQuery;
		}
	}

	// Keep backwards compatibility (at least in terms of reading from the data)
	// Note: This isn't quite right because we've removed the Tags and converted it into a Query.
	const UImmunityGameplayEffectComponent* ImmunityComponent = FindComponent<UImmunityGameplayEffectComponent>();
	if (ImmunityComponent && !ImmunityComponent->ImmunityQueries.IsEmpty())
	{
		GrantedApplicationImmunityQuery = ImmunityComponent->ImmunityQueries.Last();
	}
}

void UGameplayEffect::ConvertRemoveOtherComponent()
{
	// Get the archetype to compare to
	const UGameplayEffect* Archetype = Cast<UGameplayEffect>(GetArchetype());

	const bool bTagsChanged = RemoveGameplayEffectsWithTags != Archetype->RemoveGameplayEffectsWithTags;
	const bool bQueryChanged = RemoveGameplayEffectQuery != Archetype->RemoveGameplayEffectQuery;

	const bool bChanged = bTagsChanged || bQueryChanged;
	if (bChanged && UE::GameplayEffect::EditorOnly::ShouldUpgradeVersion(GetVersion(), EGameplayEffectVersion::Modular53))
	{
		URemoveOtherGameplayEffectComponent& RemoveOtherComponent = FindOrAddComponent<URemoveOtherGameplayEffectComponent>();
		
		// Our upgrade path is going to have to be conservative because of the Archetype inheritance rules.
		// Let's always make sure there's 2 slots (originating Tags and originating Queries).
		while (RemoveOtherComponent.RemoveGameplayEffectQueries.Num() < 2)
		{
			RemoveOtherComponent.RemoveGameplayEffectQueries.AddDefaulted();
		}

		if (bTagsChanged)
		{
			// Note: This is keeping the old functionality which incorrectly conflates Asset Tags with Owned Tags.
			RemoveOtherComponent.RemoveGameplayEffectQueries[0] = FGameplayEffectQuery::MakeQuery_MatchAnyOwningTags(RemoveGameplayEffectsWithTags.CombinedTags);
		}

		if (bQueryChanged)
		{
			RemoveOtherComponent.RemoveGameplayEffectQueries[1] = RemoveGameplayEffectQuery;
		}
	}

	// Keep backwards compatibility (at least in terms of reading from the data)
	// Note: This isn't quite right because we've removed the Tags and converted it into a Query.
	const URemoveOtherGameplayEffectComponent* RemoveOtherComponent = FindComponent<URemoveOtherGameplayEffectComponent>();
	if (RemoveOtherComponent && !RemoveOtherComponent->RemoveGameplayEffectQueries.IsEmpty())
	{
		RemoveGameplayEffectQuery = RemoveOtherComponent->RemoveGameplayEffectQueries.Last();
	}
}

void UGameplayEffect::ConvertTagRequirementsComponent()
{
	// Get the archetype to compare to
	const UGameplayEffect* Archetype = Cast<UGameplayEffect>(GetArchetype());

	bool bApplicationChanged = ApplicationTagRequirements != Archetype->ApplicationTagRequirements;
	bool bRemovalChanged = RemovalTagRequirements != Archetype->RemovalTagRequirements;
	bool bOngoingChanged = OngoingTagRequirements != Archetype->OngoingTagRequirements;

	const bool bChanged = bApplicationChanged || bRemovalChanged || bOngoingChanged;
	if (bChanged && UE::GameplayEffect::EditorOnly::ShouldUpgradeVersion(GetVersion(), EGameplayEffectVersion::Modular53))
	{
		UTargetTagRequirementsGameplayEffectComponent& TagRequirementsComponent = FindOrAddComponent<UTargetTagRequirementsGameplayEffectComponent>();

		if (bApplicationChanged)
		{
			TagRequirementsComponent.ApplicationTagRequirements = ApplicationTagRequirements;
		}

		if (bRemovalChanged)
		{
			TagRequirementsComponent.RemovalTagRequirements = RemovalTagRequirements;
		}

		if (bOngoingChanged)
		{
			TagRequirementsComponent.OngoingTagRequirements = OngoingTagRequirements;
		}
	}

	// Keep backwards compatibility (at least in terms of reading from the data)
	const UTargetTagRequirementsGameplayEffectComponent* TagRequirementsComponent = FindComponent<UTargetTagRequirementsGameplayEffectComponent>();
	if (TagRequirementsComponent)
	{
		ApplicationTagRequirements = TagRequirementsComponent->ApplicationTagRequirements;
		RemovalTagRequirements = TagRequirementsComponent->RemovalTagRequirements;
		OngoingTagRequirements = TagRequirementsComponent->OngoingTagRequirements;
	}
}

void UGameplayEffect::ConvertUIComponent()
{
	// Get the archetype to compare to
	const UGameplayEffect* Archetype = Cast<UGameplayEffect>(GetArchetype());

	const bool bChanged = (UIData != Archetype->UIData);
	if (bChanged && UE::GameplayEffect::EditorOnly::ShouldUpgradeVersion(GetVersion(), EGameplayEffectVersion::Modular53))
	{
		const UGameplayEffectUIData* UIComponent = FindComponent<UGameplayEffectUIData>();
		if (!UIComponent && UIData)
		{
			GEComponents.Add(MoveTemp(UIData));
			UIData = nullptr;
		}
	}

	// Note: We cannot keep backwards compatibility.  Since UGameplayEffectUIData (and UIData by extension) are instanced,
	// we would end up with multiple references to a single GEComponent.
	if (UIData)
	{
		UIData = nullptr;
	}
}

void UGameplayEffect::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);

	// Don't need any more data updates because during Cook the data shouldn't have changed
	if (SaveContext.IsCooking())
	{
		return;
	}

	// Since we've deprecated these fields in favor of GEComponents, we should also manually
	// clear them out of their gameplay tag references, then rebuild them using the new data, 
	// otherwise we would end up with stale references when a GEComponent is removed.
	if (GetVersion() >= EGameplayEffectVersion::Modular53)
	{
		InheritableGameplayEffectTags = FInheritedTagContainer{};
		InheritableOwnedTagsContainer = FInheritedTagContainer{};
		InheritableBlockedAbilityTagsContainer = FInheritedTagContainer{};
		OngoingTagRequirements = FGameplayTagRequirements{};
		ApplicationTagRequirements = FGameplayTagRequirements{};
		RemovalTagRequirements = FGameplayTagRequirements{};
		RemoveGameplayEffectsWithTags = FInheritedTagContainer{};
		GrantedApplicationImmunityTags = FGameplayTagRequirements{};
		GrantedApplicationImmunityQuery = FGameplayEffectQuery{};
		RemoveGameplayEffectQuery = FGameplayEffectQuery{};
		GrantedAbilities.Empty();

		// Now that we've removed all of the stale data, run through the upgrade path again which will copy the new components
		// back into the deprecated variables.  This allows us to keep for extended backwards compatibility.
		FPostCDOCompiledContext PostCDOCompiledContext;
		PostCDOCompiled(PostCDOCompiledContext);
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Let's keep track of the version so that we can upgrade the Components properly.  This will be set properly in PostLoad after upgrades. */
EGameplayEffectVersion UGameplayEffect::GetVersion() const
{
	return DataVersion.CurrentVersion;
}

/** Sets the Version of the class to denote it's been upgraded */
void UGameplayEffect::SetVersion(EGameplayEffectVersion Version)
{
	DataVersion.CurrentVersion = Version;
}

void UGameplayEffect::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UAbilitySystemGlobals::Get().GameplayEffectPostEditChangeProperty(this, PropertyChangedEvent);

	// Update the Status Text
	FDataValidationContext DataValidationContext;
	IsDataValid(DataValidationContext);
}

#endif // #if WITH_EDITOR

EGameplayEffectStackingType UGameplayEffect::GetStackingType() const
{
	if (DurationPolicy == EGameplayEffectDurationType::Instant)
	{
		return EGameplayEffectStackingType::None;
	}
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return StackingType;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR
void UGameplayEffect::SetStackingType(EGameplayEffectStackingType InType)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	StackingType = InType;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif

bool UGameplayEffect::CanApply(const FActiveGameplayEffectsContainer& ActiveGEContainer, const FGameplayEffectSpec& GESpec) const
{
	for (const UGameplayEffectComponent* GEComponent : GEComponents)
	{
		if (GEComponent && !GEComponent->CanGameplayEffectApply(ActiveGEContainer, GESpec))
		{
			UE_VLOG_UELOG(ActiveGEContainer.Owner, LogGameplayEffects, Verbose, TEXT("%s could not apply. Blocked by %s"), *GetNameSafe(GESpec.Def), *GetNameSafe(GEComponent));
			return false;
		}
	}

	return true;
}

bool UGameplayEffect::OnAddedToActiveContainer(FActiveGameplayEffectsContainer& ActiveGEContainer, FActiveGameplayEffect& ActiveGE) const
{
	bool bShouldBeActive = true;
	for (const UGameplayEffectComponent* GEComponent : GEComponents)
	{
		if (GEComponent)
		{
			bShouldBeActive = GEComponent->OnActiveGameplayEffectAdded(ActiveGEContainer, ActiveGE) && bShouldBeActive;
		}
	}

	UE_VLOG_UELOG(ActiveGEContainer.Owner->GetOwnerActor(), LogGameplayEffects, Log, TEXT("Added: %s. Auth: %d. ReplicationID: %d. ShouldBeActive: %d"), *ActiveGE.GetDebugString(), ActiveGEContainer.IsNetAuthority(), ActiveGE.ReplicationID, bShouldBeActive);

	return bShouldBeActive;
}

void UGameplayEffect::OnExecuted(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const
{
	for (const UGameplayEffectComponent* GEComponent : GEComponents)
	{
		if (GEComponent)
		{
			GEComponent->OnGameplayEffectExecuted(ActiveGEContainer, GESpec, PredictionKey);
		}
	}

	UE_VLOG_UELOG(ActiveGEContainer.Owner->GetOwnerActor(), LogGameplayEffects, Log, TEXT("Executed: %s"), *GetNameSafe(GESpec.Def));
}

void UGameplayEffect::OnApplied(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const
{
	for (const UGameplayEffectComponent* GEComponent : GEComponents)
	{
		if (GEComponent)
		{
			GEComponent->OnGameplayEffectApplied(ActiveGEContainer, GESpec, PredictionKey);
		}
	}

	UE_LOGF(LogGameplayEffects, Verbose, "Applied: %ls", *GetNameSafe(GESpec.Def));
}

int32 UGameplayEffect::GetStackLimitCount() const
{
	return StackLimitCount;
}

EGameplayEffectStackingExpirationPolicy UGameplayEffect::GetStackExpirationPolicy() const
{
	return StackExpirationPolicy;
}

const UGameplayEffectComponent* UGameplayEffect::FindComponent(TSubclassOf<UGameplayEffectComponent> ClassToFind) const
{
	for (const TObjectPtr<UGameplayEffectComponent>& GEComponent : GEComponents)
	{
		if (GEComponent && GEComponent->IsA(ClassToFind))
		{
			return GEComponent;
		}
	}

	return nullptr;
}

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	FAttributeBasedFloat
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

float FAttributeBasedFloat::CalculateMagnitude(const FGameplayEffectSpec& InRelevantSpec) const
{
	const FGameplayEffectAttributeCaptureSpec* CaptureSpec = InRelevantSpec.CapturedRelevantAttributes.FindCaptureSpecByDefinition(BackingAttribute, true);
	checkf(CaptureSpec, TEXT("Attempted to calculate an attribute-based float from spec: %s that did not have the required captured attribute: %s"), 
		*InRelevantSpec.ToSimpleString(), *BackingAttribute.ToSimpleString());

	float AttribValue = 0.f;

	// Base value can be calculated w/o evaluation parameters
	if (AttributeCalculationType == EAttributeBasedFloatCalculationType::AttributeBaseValue)
	{
		CaptureSpec->AttemptCalculateAttributeBaseValue(AttribValue);
	}
	// Set up eval params to handle magnitude or bonus magnitude calculations
	else
	{
		FAggregatorEvaluateParameters EvaluationParameters;
		EvaluationParameters.SourceTags = InRelevantSpec.CapturedSourceTags.GetAggregatedTags();
		EvaluationParameters.TargetTags = InRelevantSpec.CapturedTargetTags.GetAggregatedTags();
		EvaluationParameters.AppliedSourceTagFilter = SourceTagFilter;
		EvaluationParameters.AppliedTargetTagFilter = TargetTagFilter;

		if (AttributeCalculationType == EAttributeBasedFloatCalculationType::AttributeMagnitude)
		{
			CaptureSpec->AttemptCalculateAttributeMagnitude(EvaluationParameters, AttribValue);
		}
		else if (AttributeCalculationType == EAttributeBasedFloatCalculationType::AttributeBonusMagnitude)
		{
			CaptureSpec->AttemptCalculateAttributeBonusMagnitude(EvaluationParameters, AttribValue);
		}
		else if (AttributeCalculationType == EAttributeBasedFloatCalculationType::AttributeMagnitudeEvaluatedUpToChannel)
		{
			const bool bRequestingValidChannel = UAbilitySystemGlobals::Get().IsGameplayModEvaluationChannelValid(FinalChannel);
			ensure(bRequestingValidChannel);
			const EGameplayModEvaluationChannel ChannelToUse = bRequestingValidChannel ? FinalChannel : EGameplayModEvaluationChannel::Channel0;

			CaptureSpec->AttemptCalculateAttributeMagnitudeUpToChannel(EvaluationParameters, ChannelToUse, AttribValue);
		}
	}

	// if a curve table entry is specified, use the attribute value as a lookup into the curve instead of using it directly
	static const FString CalculateMagnitudeContext(TEXT("FAttributeBasedFloat::CalculateMagnitude"));
	if (AttributeCurve.IsValid(CalculateMagnitudeContext))
	{
		AttributeCurve.Eval(AttribValue, &AttribValue, CalculateMagnitudeContext);
	}

	const float SpecLvl = InRelevantSpec.GetLevel();
	FString ContextString = FString::Printf(TEXT("FAttributeBasedFloat::CalculateMagnitude from spec %s"), *InRelevantSpec.ToSimpleString());
	return ((Coefficient.GetValueAtLevel(SpecLvl, &ContextString) * (AttribValue + PreMultiplyAdditiveValue.GetValueAtLevel(SpecLvl, &ContextString))) + PostMultiplyAdditiveValue.GetValueAtLevel(SpecLvl, &ContextString));
}

bool FAttributeBasedFloat::operator==(const FAttributeBasedFloat& Other) const
{
	if (Coefficient != Other.Coefficient ||
		PreMultiplyAdditiveValue != Other.PreMultiplyAdditiveValue ||
		PostMultiplyAdditiveValue != Other.PostMultiplyAdditiveValue ||
		BackingAttribute != Other.BackingAttribute ||
		AttributeCurve != Other.AttributeCurve ||
		AttributeCalculationType != Other.AttributeCalculationType ||
		FinalChannel != Other.FinalChannel)
	{
		return false;
	}
	if (SourceTagFilter.Num() != Other.SourceTagFilter.Num() ||
		!SourceTagFilter.HasAllExact(Other.SourceTagFilter))
	{
		return false;
	}
	if (TargetTagFilter.Num() != Other.TargetTagFilter.Num() ||
		!TargetTagFilter.HasAllExact(Other.TargetTagFilter))
	{
		return false;
	}

	return true;
}

bool FAttributeBasedFloat::operator!=(const FAttributeBasedFloat& Other) const
{
	return !(*this == Other);
}

#if WITH_EDITOR
EDataValidationResult FAttributeBasedFloat::IsDataValid(FDataValidationContext& Context, const FString& PathName) const
{
	EDataValidationResult Result = Coefficient.IsDataValid(Context, FString::Printf(TEXT("%s.%s"), *PathName, TEXT("Coefficient")));
	Result = CombineDataValidationResults(Result, PreMultiplyAdditiveValue.IsDataValid(Context, FString::Printf(TEXT("%s.%s"), *PathName, TEXT("PreMultiplyAdditiveValue"))));
	Result = CombineDataValidationResults(Result, PostMultiplyAdditiveValue.IsDataValid(Context, FString::Printf(TEXT("%s.%s"), *PathName, TEXT("PostMultiplyAdditiveValue"))));

	return Result;
}
#endif

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	FCustomCalculationBasedFloat
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

float FCustomCalculationBasedFloat::CalculateMagnitude(const FGameplayEffectSpec& InRelevantSpec) const
{
	const UGameplayModMagnitudeCalculation* CalcCDO = CalculationClassMagnitude->GetDefaultObject<UGameplayModMagnitudeCalculation>();
	check(CalcCDO);

	float CustomBaseValue = CalcCDO->CalculateBaseMagnitude(InRelevantSpec);

	const float SpecLvl = InRelevantSpec.GetLevel();
	FString ContextString = FString::Printf(TEXT("FCustomCalculationBasedFloat::CalculateMagnitude from effect %s"), *CalcCDO->GetName());

	float FinalValue = ((Coefficient.GetValueAtLevel(SpecLvl, &ContextString) * (CustomBaseValue + PreMultiplyAdditiveValue.GetValueAtLevel(SpecLvl, &ContextString))) + PostMultiplyAdditiveValue.GetValueAtLevel(SpecLvl, &ContextString));
	if (FinalLookupCurve.IsValid(ContextString))
	{
		FinalValue = FinalLookupCurve.Eval(FinalValue, ContextString);
	}

	return FinalValue;
}

/** Equality/Inequality operators */
bool FCustomCalculationBasedFloat::operator==(const FCustomCalculationBasedFloat& Other) const
{
	if (CalculationClassMagnitude != Other.CalculationClassMagnitude)
	{
		return false;
	}
	if (Coefficient != Other.Coefficient ||
		PreMultiplyAdditiveValue != Other.PreMultiplyAdditiveValue ||
		PostMultiplyAdditiveValue != Other.PostMultiplyAdditiveValue ||
		FinalLookupCurve != Other.FinalLookupCurve)
	{
		return false;
	}

	return true;
}

bool FCustomCalculationBasedFloat::operator!=(const FCustomCalculationBasedFloat& Other) const
{
	return !(*this == Other);
}

#if WITH_EDITOR
EDataValidationResult FCustomCalculationBasedFloat::IsDataValid(FDataValidationContext& Context, const FString& PathName) const
{
	EDataValidationResult Result = Coefficient.IsDataValid(Context, FString::Printf(TEXT("%s.%s"), *PathName, TEXT("Coefficient")));
	Result = CombineDataValidationResults(Result, PreMultiplyAdditiveValue.IsDataValid(Context, FString::Printf(TEXT("%s.%s"), *PathName, TEXT("PreMultiplyAdditiveValue"))));
	Result = CombineDataValidationResults(Result, PostMultiplyAdditiveValue.IsDataValid(Context, FString::Printf(TEXT("%s.%s"), *PathName, TEXT("PostMultiplyAdditiveValue"))));

	return Result;
}
#endif

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	FGameplayEffectMagnitude
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

bool FGameplayEffectModifierMagnitude::CanCalculateMagnitude(const FGameplayEffectSpec& InRelevantSpec) const
{
	// Only can calculate magnitude properly if all required capture definitions are fulfilled by the spec
	TArray<FGameplayEffectAttributeCaptureDefinition> ReqCaptureDefs;
	ReqCaptureDefs.Reset();

	GetAttributeCaptureDefinitions(ReqCaptureDefs);

	return InRelevantSpec.HasValidCapturedAttributes(ReqCaptureDefs);
}

bool FGameplayEffectModifierMagnitude::AttemptCalculateMagnitude(const FGameplayEffectSpec& InRelevantSpec, OUT float& OutCalculatedMagnitude, bool WarnIfSetByCallerFail, float DefaultSetbyCaller) const
{
	const bool bCanCalc = CanCalculateMagnitude(InRelevantSpec);
	if (bCanCalc)
	{
		FString ContextString = FString::Printf(TEXT("FGameplayEffectModifierMagnitude::AttemptCalculateMagnitude from effect %s"), *InRelevantSpec.ToSimpleString());

		switch (MagnitudeCalculationType)
		{
			case EGameplayEffectMagnitudeCalculation::ScalableFloat:
			{
				OutCalculatedMagnitude = ScalableFloatMagnitude.GetValueAtLevel(InRelevantSpec.GetLevel(), &ContextString);
			}
			break;

			case EGameplayEffectMagnitudeCalculation::AttributeBased:
			{
				OutCalculatedMagnitude = AttributeBasedMagnitude.CalculateMagnitude(InRelevantSpec);
			}
			break;

			case EGameplayEffectMagnitudeCalculation::CustomCalculationClass:
			{
				OutCalculatedMagnitude = CustomMagnitude.CalculateMagnitude(InRelevantSpec);
			}
			break;

			case EGameplayEffectMagnitudeCalculation::SetByCaller:
			{
				if (SetByCallerMagnitude.DataTag.IsValid())
				{
					OutCalculatedMagnitude = InRelevantSpec.GetSetByCallerMagnitude(SetByCallerMagnitude.DataTag, WarnIfSetByCallerFail, DefaultSetbyCaller);
				}
				else
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS

					OutCalculatedMagnitude = InRelevantSpec.GetSetByCallerMagnitude(SetByCallerMagnitude.DataName, WarnIfSetByCallerFail, DefaultSetbyCaller);

					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}
			break;

			default:
				UE_LOGF(LogGameplayEffects, Error, "Unknown MagnitudeCalculationType %d in AttemptCalculateMagnitude", (int32)MagnitudeCalculationType);
				OutCalculatedMagnitude = 0.f;
				break;
		}
	}
	else
	{
		OutCalculatedMagnitude = 0.f;
	}

	return bCanCalc;
}

bool FGameplayEffectModifierMagnitude::AttemptRecalculateMagnitudeFromDependentAggregatorChange(const FGameplayEffectSpec& InRelevantSpec, OUT float& OutCalculatedMagnitude, const FAggregator* ChangedAggregator) const
{
	TArray<FGameplayEffectAttributeCaptureDefinition > ReqCaptureDefs;
	ReqCaptureDefs.Reset();

	GetAttributeCaptureDefinitions(ReqCaptureDefs);

	// We could have many potential captures. If a single one matches our criteria, then we call AttemptCalculateMagnitude once and return.
	for (const FGameplayEffectAttributeCaptureDefinition& CaptureDef : ReqCaptureDefs)
	{
		if (CaptureDef.bSnapshot == false)
		{
			const FGameplayEffectAttributeCaptureSpec* CapturedSpec = InRelevantSpec.CapturedRelevantAttributes.FindCaptureSpecByDefinition(CaptureDef, true);
			if (CapturedSpec && CapturedSpec->ShouldRefreshLinkedAggregator(ChangedAggregator))
			{
				return AttemptCalculateMagnitude(InRelevantSpec, OutCalculatedMagnitude);
			}
		}
	}

	return false;
}

void FGameplayEffectModifierMagnitude::GetAttributeCaptureDefinitions(OUT TArray<FGameplayEffectAttributeCaptureDefinition>& OutCaptureDefs) const
{
	OutCaptureDefs.Empty();

	switch (MagnitudeCalculationType)
	{
		case EGameplayEffectMagnitudeCalculation::AttributeBased:
		{
			OutCaptureDefs.Add(AttributeBasedMagnitude.BackingAttribute);
		}
		break;

		case EGameplayEffectMagnitudeCalculation::CustomCalculationClass:
		{
			if (CustomMagnitude.CalculationClassMagnitude)
			{
				const UGameplayModMagnitudeCalculation* CalcCDO = CustomMagnitude.CalculationClassMagnitude->GetDefaultObject<UGameplayModMagnitudeCalculation>();
				check(CalcCDO);

				OutCaptureDefs.Append(CalcCDO->GetAttributeCaptureDefinitions());
			}
		}
		break;
	}
}

bool FGameplayEffectModifierMagnitude::GetStaticMagnitudeIfPossible(float InLevel, float& OutMagnitude, const FString* ContextString) const
{
	if (MagnitudeCalculationType == EGameplayEffectMagnitudeCalculation::ScalableFloat)
	{
		OutMagnitude = ScalableFloatMagnitude.GetValueAtLevel(InLevel, ContextString);
		return true;
	}

	return false;
}

bool FGameplayEffectModifierMagnitude::GetSetByCallerDataNameIfPossible(FName& OutDataName) const
{
	if (MagnitudeCalculationType == EGameplayEffectMagnitudeCalculation::SetByCaller)
	{
		OutDataName = SetByCallerMagnitude.DataName;
		return true;
	}

	return false;
}

TSubclassOf<UGameplayModMagnitudeCalculation> FGameplayEffectModifierMagnitude::GetCustomMagnitudeCalculationClass() const
{
	TSubclassOf<UGameplayModMagnitudeCalculation> CustomCalcClass = nullptr;

	if (MagnitudeCalculationType == EGameplayEffectMagnitudeCalculation::CustomCalculationClass)
	{
		CustomCalcClass = CustomMagnitude.CalculationClassMagnitude;
	}

	return CustomCalcClass;
}

bool FGameplayEffectModifierMagnitude::Serialize(FArchive& Ar)
{
	// Clear properties that are not needed for the chosen calculation type
	if (Ar.IsSaving() && Ar.IsPersistent() && !Ar.IsTransacting())
	{
		if (MagnitudeCalculationType != EGameplayEffectMagnitudeCalculation::ScalableFloat)
		{
			ScalableFloatMagnitude = FScalableFloat();
		}

		if (MagnitudeCalculationType != EGameplayEffectMagnitudeCalculation::AttributeBased)
		{
			AttributeBasedMagnitude = FAttributeBasedFloat();
		}

		if (MagnitudeCalculationType != EGameplayEffectMagnitudeCalculation::CustomCalculationClass)
		{
			CustomMagnitude = FCustomCalculationBasedFloat();
		}

		if (MagnitudeCalculationType != EGameplayEffectMagnitudeCalculation::SetByCaller)
		{
			SetByCallerMagnitude = FSetByCallerFloat();
		}
	}

	// Return false to let normal tagged serialization occur
	return false;
}

bool FGameplayEffectModifierMagnitude::operator==(const FGameplayEffectModifierMagnitude& Other) const
{
	if (MagnitudeCalculationType != Other.MagnitudeCalculationType)
	{
		return false;
	}

	switch (MagnitudeCalculationType)
	{
	case EGameplayEffectMagnitudeCalculation::ScalableFloat:
		if (ScalableFloatMagnitude != Other.ScalableFloatMagnitude)
		{
			return false;
		}
		break;
	case EGameplayEffectMagnitudeCalculation::AttributeBased:
		if (AttributeBasedMagnitude != Other.AttributeBasedMagnitude)
		{
			return false;
		}
		break;
	case EGameplayEffectMagnitudeCalculation::CustomCalculationClass:
		if (CustomMagnitude != Other.CustomMagnitude)
		{
			return false;
		}
		break;
	case EGameplayEffectMagnitudeCalculation::SetByCaller:
		if (SetByCallerMagnitude.DataName != Other.SetByCallerMagnitude.DataName)
		{
			return false;
		}
		break;
	}

	return true;
}

bool FGameplayEffectModifierMagnitude::operator!=(const FGameplayEffectModifierMagnitude& Other) const
{
	return !(*this == Other);
}

#if WITH_EDITOR
FText FGameplayEffectModifierMagnitude::GetValueForEditorDisplay() const
{
	switch (MagnitudeCalculationType)
	{
		case EGameplayEffectMagnitudeCalculation::ScalableFloat:
			return FText::Format(NSLOCTEXT("GameplayEffect", "ScalableFloatModifierMagnitude", "{0} s"), FText::AsNumber(ScalableFloatMagnitude.Value));
			
		case EGameplayEffectMagnitudeCalculation::AttributeBased:
			return NSLOCTEXT("GameplayEffect", "AttributeBasedModifierMagnitude", "Attribute Based");

		case EGameplayEffectMagnitudeCalculation::CustomCalculationClass:
			return NSLOCTEXT("GameplayEffect", "CustomCalculationClassModifierMagnitude", "Custom Calculation");

		case EGameplayEffectMagnitudeCalculation::SetByCaller:
			return NSLOCTEXT("GameplayEffect", "SetByCallerModifierMagnitude", "Set by Caller");
	}

	return NSLOCTEXT("GameplayEffect", "UnknownModifierMagnitude", "Unknown");
}

void FGameplayEffectModifierMagnitude::ReportErrors(const FString& PathName) const
{
	// Deprecated, use IsDataValid
}

EDataValidationResult FGameplayEffectModifierMagnitude::IsDataValid(FDataValidationContext& Context, const FString& PathName) const
{
	if (MagnitudeCalculationType == EGameplayEffectMagnitudeCalculation::ScalableFloat)
	{
		return ScalableFloatMagnitude.IsDataValid(Context, FString::Printf(TEXT("%s.%s"), *PathName, TEXT("ScalableFloatMagnitude")));
	}
	else if (MagnitudeCalculationType == EGameplayEffectMagnitudeCalculation::AttributeBased)
	{
		return AttributeBasedMagnitude.IsDataValid(Context, FString::Printf(TEXT("%s.%s"), *PathName, TEXT("AttributeBasedMagnitude")));
	}
	else if (MagnitudeCalculationType == EGameplayEffectMagnitudeCalculation::CustomCalculationClass)
	{
		return CustomMagnitude.IsDataValid(Context, FString::Printf(TEXT("%s.%s"), *PathName, TEXT("CustomMagnitude")));
	}

	return EDataValidationResult::Valid;
}
#endif // WITH_EDITOR


// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	FGameplayEffectExecutionDefinition
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

void FGameplayEffectExecutionDefinition::GetAttributeCaptureDefinitions(OUT TArray<FGameplayEffectAttributeCaptureDefinition>& OutCaptureDefs) const
{
	OutCaptureDefs.Empty();

	if (CalculationClass)
	{
		const UGameplayEffectExecutionCalculation* CalculationCDO = Cast<UGameplayEffectExecutionCalculation>(CalculationClass->GetDefaultObject(false));
		check(CalculationCDO);

		OutCaptureDefs.Append(CalculationCDO->GetAttributeCaptureDefinitions());
	}

	// Scoped modifiers might have custom magnitude calculations, requiring additional captured attributes
	for (const FGameplayEffectExecutionScopedModifierInfo& CurScopedMod : CalculationModifiers)
	{
		TArray<FGameplayEffectAttributeCaptureDefinition> ScopedModMagDefs;
		CurScopedMod.ModifierMagnitude.GetAttributeCaptureDefinitions(ScopedModMagDefs);

		OutCaptureDefs.Append(ScopedModMagDefs);
	}
}

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	FConditionalGameplayEffect
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

bool FConditionalGameplayEffect::CanApply(const FGameplayTagContainer& SourceTags, float SourceLevel) const
{
	// Right now we're just using the tags but in the future we may gate this by source level as well
	return SourceTags.HasAll(RequiredSourceTags);
}

FGameplayEffectSpecHandle FConditionalGameplayEffect::CreateSpec(FGameplayEffectContextHandle EffectContext, float SourceLevel) const
{
	const UGameplayEffect* EffectCDO = EffectClass ? EffectClass->GetDefaultObject<UGameplayEffect>() : nullptr;
	return EffectCDO ? FGameplayEffectSpecHandle(new FGameplayEffectSpec(EffectCDO, EffectContext, SourceLevel)) : FGameplayEffectSpecHandle();
}

bool FConditionalGameplayEffect::operator==(const FConditionalGameplayEffect& Other) const
{
	return EffectClass == Other.EffectClass && RequiredSourceTags == Other.RequiredSourceTags && RemovalPolicy == Other.RemovalPolicy && StackCountToRemove == Other.StackCountToRemove;
}

bool FConditionalGameplayEffect::operator!=(const FConditionalGameplayEffect& Other) const
{
	return !(*this == Other);
}


// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	FGameplayEffectSpec
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

FGameplayEffectSpec::FGameplayEffectSpec()
	: Def(nullptr)
	, Duration(UGameplayEffect::INSTANT_APPLICATION)
	, Period(UGameplayEffect::NO_PERIOD)
	, StackCount(1)
	, bCompletedSourceAttributeCapture(false)
	, bCompletedTargetAttributeCapture(false)
	, bDurationLocked(false)
	, Level(UGameplayEffect::INVALID_LEVEL)
{
}

FGameplayEffectSpec::FGameplayEffectSpec(const UGameplayEffect* InDef, const FGameplayEffectContextHandle& InEffectContext, float InLevel)
	: Def(InDef)
	, Duration(UGameplayEffect::INSTANT_APPLICATION)
	, Period(UGameplayEffect::NO_PERIOD)
	, StackCount(1)
	, bCompletedSourceAttributeCapture(false)
	, bCompletedTargetAttributeCapture(false)
	, bDurationLocked(false)
{
	Initialize(InDef, InEffectContext, InLevel);
}

FGameplayEffectSpec::FGameplayEffectSpec(const FGameplayEffectSpec& Other)
{
	*this = Other;
}

FGameplayEffectSpec::FGameplayEffectSpec(const FGameplayEffectSpec& Other, const FGameplayEffectContextHandle& InEffectContext)
{
	*this = Other;
	EffectContext = InEffectContext;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FGameplayEffectSpec::FGameplayEffectSpec(FGameplayEffectSpec&& Other)
	: Def(Other.Def)
	, ModifiedAttributes(MoveTemp(Other.ModifiedAttributes))
	, CapturedRelevantAttributes(MoveTemp(Other.CapturedRelevantAttributes))
	, TargetEffectSpecs(MoveTemp(Other.TargetEffectSpecs))
	, Duration(Other.Duration)
	, Period(Other.Period)
	, CapturedSourceTags(MoveTemp(Other.CapturedSourceTags))
	, CapturedTargetTags(MoveTemp(Other.CapturedTargetTags))
	, DynamicGrantedTags(MoveTemp(Other.DynamicGrantedTags))
	, DynamicAssetTags(MoveTemp(Other.DynamicAssetTags))
	, Modifiers(MoveTemp(Other.Modifiers))
	, StackCount(Other.StackCount)
	, bCompletedSourceAttributeCapture(Other.bCompletedSourceAttributeCapture)
	, bCompletedTargetAttributeCapture(Other.bCompletedTargetAttributeCapture)
	, bDurationLocked(Other.bDurationLocked)
	, GrantedAbilitySpecs(MoveTemp(Other.GrantedAbilitySpecs))
	, SetByCallerNameMagnitudes(MoveTemp(Other.SetByCallerNameMagnitudes))
	, SetByCallerTagMagnitudes(MoveTemp(Other.SetByCallerTagMagnitudes))
	, EffectContext(Other.EffectContext)
	, Level(Other.Level)
{

}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FGameplayEffectSpec& FGameplayEffectSpec::operator=(FGameplayEffectSpec&& Other)
{
	Def = Other.Def;
	ModifiedAttributes = MoveTemp(Other.ModifiedAttributes);
	CapturedRelevantAttributes = MoveTemp(Other.CapturedRelevantAttributes);
	TargetEffectSpecs = MoveTemp(Other.TargetEffectSpecs);
	Duration = Other.Duration;
	Period = Other.Period;
	CapturedSourceTags = MoveTemp(Other.CapturedSourceTags);
	CapturedTargetTags = MoveTemp(Other.CapturedTargetTags);
	DynamicGrantedTags = MoveTemp(Other.DynamicGrantedTags);
	DynamicAssetTags = MoveTemp(Other.DynamicAssetTags);
	Modifiers = MoveTemp(Other.Modifiers);
	StackCount = Other.StackCount;
	bCompletedSourceAttributeCapture = Other.bCompletedSourceAttributeCapture;
	bCompletedTargetAttributeCapture = Other.bCompletedTargetAttributeCapture;
	bDurationLocked = Other.bDurationLocked;
	GrantedAbilitySpecs = MoveTemp(Other.GrantedAbilitySpecs);
	SetByCallerNameMagnitudes = MoveTemp(Other.SetByCallerNameMagnitudes);
	SetByCallerTagMagnitudes = MoveTemp(Other.SetByCallerTagMagnitudes);
	EffectContext = Other.EffectContext;
	Level = Other.Level;

	return *this;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FGameplayEffectSpec& FGameplayEffectSpec::operator=(const FGameplayEffectSpec& Other)
{
	Def = Other.Def;
	ModifiedAttributes = Other.ModifiedAttributes;
	CapturedRelevantAttributes = Other.CapturedRelevantAttributes;
	TargetEffectSpecs = Other.TargetEffectSpecs;
	Duration = Other.Duration;
	Period = Other.Period;
	CapturedSourceTags = Other.CapturedSourceTags;
	CapturedTargetTags = Other.CapturedTargetTags;
	DynamicGrantedTags = Other.DynamicGrantedTags;
	DynamicAssetTags = Other.DynamicAssetTags;
	Modifiers = Other.Modifiers;
	StackCount = Other.StackCount;
	bCompletedSourceAttributeCapture = Other.bCompletedSourceAttributeCapture;
	bCompletedTargetAttributeCapture = Other.bCompletedTargetAttributeCapture;
	bDurationLocked = Other.bDurationLocked;
	GrantedAbilitySpecs =  Other.GrantedAbilitySpecs;
	SetByCallerNameMagnitudes = Other.SetByCallerNameMagnitudes;
	SetByCallerTagMagnitudes = Other.SetByCallerTagMagnitudes;
	EffectContext = Other.EffectContext;
	Level = Other.Level;

	return *this;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FGameplayEffectSpecForRPC::FGameplayEffectSpecForRPC()
	: Def(nullptr)
	, Level(UGameplayEffect::INVALID_LEVEL)
	, AbilityLevel(1)
{
}

FGameplayEffectSpecForRPC::FGameplayEffectSpecForRPC(const FGameplayEffectSpec& InSpec)
	: Def(InSpec.Def)
	, ModifiedAttributes()
	, EffectContext(InSpec.GetEffectContext())
	, AggregatedSourceTags(*InSpec.CapturedSourceTags.GetAggregatedTags())
	, AggregatedTargetTags(*InSpec.CapturedTargetTags.GetAggregatedTags())
	, Level(InSpec.GetLevel())
	, AbilityLevel(InSpec.GetEffectContext().GetAbilityLevel())
{
	// Only copy attributes that are in the gameplay cue info
	for (int32 i = InSpec.ModifiedAttributes.Num() - 1; i >= 0; i--)
	{
		if (Def)
		{
			for (const FGameplayEffectCue& CueInfo : Def->GameplayCues)
			{
				if (CueInfo.MagnitudeAttribute == InSpec.ModifiedAttributes[i].Attribute)
				{
					ModifiedAttributes.Add(InSpec.ModifiedAttributes[i]);
				}
			}
		}
	}
}

void FGameplayEffectSpec::Initialize(const UGameplayEffect* InDef, const FGameplayEffectContextHandle& InEffectContext, float InLevel)
{
	Def = InDef;
	check(Def);	

	// SetContext requires the level to be set before it runs
	// however, there are code paths in SetLevel that can potentially (depends on game data setup) require the context to be set
	Level = InLevel;
	SetContext(InEffectContext);
	SetLevel(InLevel);

	// Init our ModifierSpecs
	Modifiers.SetNum(Def->Modifiers.Num());

	// Prep the spec with all of the attribute captures it will need to perform
	SetupAttributeCaptureDefinitions();
	
	// Add the GameplayEffect asset tags to the source Spec tags
	CapturedSourceTags.GetSpecTags().AppendTags(Def->GetAssetTags());

	// Prepare source tags before accessing them in the Components
	CaptureDataFromSource();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// ------------------------------------------------
	//	Granted Abilities
	// ------------------------------------------------

	// Make Granted AbilitySpecs (caller may modify these specs after creating spec, which is why we dont just reference them from the def)
	// Note: GrantedAbilitySpecs is going to be removed in the future in favor of immutable GameplayEffectComponents.  Don't rely on this (dynamic) functionality.
	GrantedAbilitySpecs.Empty();

	// if we're granting abilities and they don't specify a source object use the source of this GE
	for (const FGameplayAbilitySpecDef& AbilitySpecDef : InDef->GrantedAbilities)
	{
		// Don't copy null entries over, these would have been nulled during the conversion to use GEComponents.
		if (!AbilitySpecDef.Ability.Get())
		{
			continue;
		}

		UE_LOGF(LogGameplayEffects, Error, "%ls had GrantedAbilities dynamically added to it. This functionality is being deprecated", *GetNameSafe(InDef));
		FGameplayAbilitySpecDef& OurCopy = GrantedAbilitySpecs.Emplace_GetRef(AbilitySpecDef);

		if (OurCopy.SourceObject == nullptr)
		{
			OurCopy.SourceObject = InEffectContext.GetSourceObject();
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FGameplayEffectSpec::InitializeFromLinkedSpec(const UGameplayEffect* InDef, const FGameplayEffectSpec& OriginalSpec)
{
	// We need to manually initialize the new GE spec. We want to pass on all of the tags from the originating GE *Except* for that GE's asset tags. (InheritableGameplayEffectTags).
	// But its very important that the ability tags and anything else that was added to the source tags in the originating GE carries over

	// Duplicate GE context
	const FGameplayEffectContextHandle& ExpiringSpecContextHandle = OriginalSpec.GetEffectContext();
	FGameplayEffectContextHandle NewContextHandle = ExpiringSpecContextHandle.Duplicate();

	// Make a full copy
	CapturedSourceTags = OriginalSpec.CapturedSourceTags;

	// But then remove the tags the originating GE added
	CapturedSourceTags.GetSpecTags().RemoveTags(OriginalSpec.Def->GetAssetTags());

	// Now initialize like the normal cstor would have. Note that this will add the new GE's asset tags (in case they were removed in the line above / e.g., shared asset tags with the originating GE)					
	Initialize(InDef, NewContextHandle, OriginalSpec.GetLevel());

	// Finally, copy over set by called magnitudes
	CopySetByCallerMagnitudes(OriginalSpec);
}

void FGameplayEffectSpec::CopySetByCallerMagnitudes(const FGameplayEffectSpec& OriginalSpec)
{
	SetByCallerNameMagnitudes = OriginalSpec.SetByCallerNameMagnitudes;
	SetByCallerTagMagnitudes = OriginalSpec.SetByCallerTagMagnitudes;
}

void FGameplayEffectSpec::MergeSetByCallerMagnitudes(const TMap<FGameplayTag, float>& Magnitudes)
{
	for (auto It : Magnitudes)
	{
		if (SetByCallerTagMagnitudes.Contains(It.Key) == false)
		{
			SetByCallerTagMagnitudes.Add(It.Key) = It.Value;
		}
	}
}

void FGameplayEffectSpec::SetupAttributeCaptureDefinitions()
{
	// Add duration if required
	if (Def->DurationPolicy == EGameplayEffectDurationType::HasDuration)
	{
		CapturedRelevantAttributes.AddCaptureDefinition(UAbilitySystemComponent::GetOutgoingDurationCapture());
		CapturedRelevantAttributes.AddCaptureDefinition(UAbilitySystemComponent::GetIncomingDurationCapture());
	}

	TArray<FGameplayEffectAttributeCaptureDefinition> CaptureDefs;

	// Gather capture definitions from duration	
	{
		CaptureDefs.Reset();
		Def->DurationMagnitude.GetAttributeCaptureDefinitions(CaptureDefs);
		for (const FGameplayEffectAttributeCaptureDefinition& CurDurationCaptureDef : CaptureDefs)
		{
			CapturedRelevantAttributes.AddCaptureDefinition(CurDurationCaptureDef);
		}

		CaptureDefs.Reset();
		Def->MaxDurationMagnitude.GetAttributeCaptureDefinitions(CaptureDefs);
		for (const FGameplayEffectAttributeCaptureDefinition& CurDurationCaptureDef : CaptureDefs)
		{
			CapturedRelevantAttributes.AddCaptureDefinition(CurDurationCaptureDef);
		}
	}

	// Gather all capture definitions from modifiers
	for (int32 ModIdx = 0; ModIdx < Modifiers.Num(); ++ModIdx)
	{
		const FGameplayModifierInfo& ModDef = Def->Modifiers[ModIdx];
		const FModifierSpec& ModSpec = Modifiers[ModIdx];

		CaptureDefs.Reset();
		ModDef.ModifierMagnitude.GetAttributeCaptureDefinitions(CaptureDefs);
		
		for (const FGameplayEffectAttributeCaptureDefinition& CurCaptureDef : CaptureDefs)
		{
			CapturedRelevantAttributes.AddCaptureDefinition(CurCaptureDef);
		}
	}

	// Gather all capture definitions from executions
	for (const FGameplayEffectExecutionDefinition& Exec : Def->Executions)
	{
		CaptureDefs.Reset();
		Exec.GetAttributeCaptureDefinitions(CaptureDefs);
		for (const FGameplayEffectAttributeCaptureDefinition& CurExecCaptureDef : CaptureDefs)
		{
			CapturedRelevantAttributes.AddCaptureDefinition(CurExecCaptureDef);
		}
	}
}

void FGameplayEffectSpec::CaptureAttributeDataFromTarget(UAbilitySystemComponent* TargetAbilitySystemComponent)
{
	CapturedRelevantAttributes.CaptureAttributes(TargetAbilitySystemComponent, EGameplayEffectAttributeCaptureSource::Target);
	bCompletedTargetAttributeCapture = true;
}

void FGameplayEffectSpec::CaptureDataFromSource(bool bSkipRecaptureSourceActorTags /*= false*/)
{
	// Capture source actor tags
	if (!bSkipRecaptureSourceActorTags)
	{
		RecaptureSourceActorTags();
	}

	// Capture source Attributes
	// Is this the right place to do it? Do we ever need to create spec and capture attributes at a later time? If so, this will need to move.
	CapturedRelevantAttributes.CaptureAttributes(EffectContext.GetInstigatorAbilitySystemComponent(), EGameplayEffectAttributeCaptureSource::Source);

	// Now that we have source attributes captured, re-evaluate the duration since it could be based on the captured attributes.
	float DefCalcDuration = 0.f;
	if (AttemptCalculateDurationFromDef(DefCalcDuration))
	{
		SetDuration(DefCalcDuration, false);
	}

	bCompletedSourceAttributeCapture = true;
}

void FGameplayEffectSpec::RecaptureSourceActorTags()
{
	CapturedSourceTags.GetActorTags().Reset();
	EffectContext.GetOwnedGameplayTags(CapturedSourceTags.GetActorTags(), CapturedSourceTags.GetSpecTags());
}

bool FGameplayEffectSpec::AttemptCalculateDurationRelatedMagnitude(const FGameplayEffectModifierMagnitude& MagnitudeDef, OUT float& OutDurationValue) const
{
	bool bCalculatedDuration = true;

	const EGameplayEffectDurationType DurType = Def->DurationPolicy;
	if (DurType == EGameplayEffectDurationType::Infinite)
	{
		OutDurationValue = UGameplayEffect::INFINITE_DURATION;
	}
	else if (DurType == EGameplayEffectDurationType::Instant)
	{
		OutDurationValue = UGameplayEffect::INSTANT_APPLICATION;
	}
	else
	{
		// The last parameters (false, 1.f) are so that if SetByCaller hasn't been set yet, we don't warn and default
		// to 1.f. This is so that the rest of the system doesn't treat the effect as an instant effect. 1.f is arbitrary
		// and this makes it illegal to SetByCaller something into an instant effect.
		bCalculatedDuration = MagnitudeDef.AttemptCalculateMagnitude(*this, OutDurationValue, false, 1.f);
	}

	return bCalculatedDuration;
}

bool FGameplayEffectSpec::AttemptCalculateDurationFromDef(OUT float& OutDefDuration) const
{
	check(Def);
	return AttemptCalculateDurationRelatedMagnitude(Def->DurationMagnitude, OutDefDuration);
}

bool FGameplayEffectSpec::AttemptCalculateMaxDurationFromDef(OUT float& OutDefMaxDuration) const
{
	check(Def);
	return AttemptCalculateDurationRelatedMagnitude(Def->MaxDurationMagnitude, OutDefMaxDuration);
}

void FGameplayEffectSpec::SetLevel(float InLevel)
{
	Level = InLevel;
	if (Def)
	{
		float DefCalcDuration = 0.f;
		if (AttemptCalculateDurationFromDef(DefCalcDuration))
		{
			SetDuration(DefCalcDuration, false);
		}

		FString ContextString = Def->GetName();
		Period = Def->Period.GetValueAtLevel(InLevel, &ContextString);
	}
}

float FGameplayEffectSpec::GetLevel() const
{
	return Level;
}

float FGameplayEffectSpec::GetDuration() const
{
	return Duration;
}

void FGameplayEffectSpec::SetDuration(float NewDuration, bool bLockDuration)
{
	if (!bDurationLocked)
	{
		Duration = NewDuration;
		bDurationLocked = bLockDuration;
		if (Duration > 0.f)
		{
			// We may have potential problems one day if a game is applying duration based gameplay effects from instantaneous effects
			// (E.g., every time fire damage is applied, a DOT is also applied). We may need to for Duration to always be captured.
			CapturedRelevantAttributes.AddCaptureDefinition(UAbilitySystemComponent::GetOutgoingDurationCapture());
		}
	}
}

float FGameplayEffectSpec::CalculateModifiedDuration() const
{
	FAggregator DurationAgg;

	const FGameplayEffectAttributeCaptureSpec* OutgoingCaptureSpec = CapturedRelevantAttributes.FindCaptureSpecByDefinition(UAbilitySystemComponent::GetOutgoingDurationCapture(), true);
	if (OutgoingCaptureSpec)
	{
		OutgoingCaptureSpec->AttemptAddAggregatorModsToAggregator(DurationAgg);
	}

	const FGameplayEffectAttributeCaptureSpec* IncomingCaptureSpec = CapturedRelevantAttributes.FindCaptureSpecByDefinition(UAbilitySystemComponent::GetIncomingDurationCapture(), true);
	if (IncomingCaptureSpec)
	{
		IncomingCaptureSpec->AttemptAddAggregatorModsToAggregator(DurationAgg);
	}

	FAggregatorEvaluateParameters Params;
	Params.SourceTags = CapturedSourceTags.GetAggregatedTags();
	Params.TargetTags = CapturedTargetTags.GetAggregatedTags();

	return DurationAgg.EvaluateWithBase(GetDuration(), Params);
}

void FGameplayEffectSpec::AddDynamicAssetTag(const FGameplayTag& TagToAdd)
{
	DynamicAssetTags.AddTag(TagToAdd);
	CapturedSourceTags.GetSpecTags().AddTag(TagToAdd);
}

void FGameplayEffectSpec::AppendDynamicAssetTags(const FGameplayTagContainer& TagsToAppend)
{
	DynamicAssetTags.AppendTags(TagsToAppend);
	CapturedSourceTags.GetSpecTags().AppendTags(TagsToAppend);
}

const FGameplayTagContainer& FGameplayEffectSpec::GetDynamicAssetTags() const
{
	return DynamicAssetTags;
}

float FGameplayEffectSpec::GetPeriod() const
{
	if (Def && Def->DurationPolicy == EGameplayEffectDurationType::Instant)
	{
		return UGameplayEffect::NO_PERIOD;
	}
	
	return Period;
}

void FGameplayEffectSpec::SetStackCount(int32 NewStackCount)
{
	StackCount = NewStackCount;
}

int32 FGameplayEffectSpec::GetStackCount() const
{
	return StackCount;
}

float FGameplayEffectSpec::GetModifierMagnitude(int32 ModifierIdx, bool bFactorInStackCount) const
{
	check(Modifiers.IsValidIndex(ModifierIdx) && Def && Def->Modifiers.IsValidIndex(ModifierIdx));

	const float SingleEvaluatedMagnitude = Modifiers[ModifierIdx].GetEvaluatedMagnitude();

	float ModMagnitude = SingleEvaluatedMagnitude;
	if (bFactorInStackCount)
	{
		ModMagnitude = GameplayEffectUtilities::ComputeStackedModifierMagnitude(SingleEvaluatedMagnitude, GetStackCount(), Def->Modifiers[ModifierIdx].ModifierOp);
	}

	return ModMagnitude;
}

float FGameplayEffectSpec::GetModifierMagnitude(int32 ModifierIdx) const
{
	check(Modifiers.IsValidIndex(ModifierIdx) && Def && Def->Modifiers.IsValidIndex(ModifierIdx));

	const float SingleEvaluatedMagnitude = Modifiers[ModifierIdx].GetEvaluatedMagnitude();

	float ModMagnitude = SingleEvaluatedMagnitude;
	if (Def->bFactorInStackCount)
	{
		ModMagnitude = GameplayEffectUtilities::ComputeStackedModifierMagnitude(SingleEvaluatedMagnitude, GetStackCount(), Def->Modifiers[ModifierIdx].ModifierOp);
	}

	return ModMagnitude;
}

void FGameplayEffectSpec::CalculateModifierMagnitudes()
{
	for(int32 ModIdx = 0; ModIdx < Modifiers.Num(); ++ModIdx)
	{
		const FGameplayModifierInfo& ModDef = Def->Modifiers[ModIdx];
		FModifierSpec& ModSpec = Modifiers[ModIdx];

		if (ModDef.ModifierMagnitude.AttemptCalculateMagnitude(*this, ModSpec.EvaluatedMagnitude) == false)
		{
			ModSpec.EvaluatedMagnitude = 0.f;
			UE_LOGF(LogGameplayEffects, Warning, "Modifier on spec: %ls was asked to CalculateMagnitude and failed, falling back to 0.", *ToSimpleString());
		}
	}
}

bool FGameplayEffectSpec::HasValidCapturedAttributes(const TArray<FGameplayEffectAttributeCaptureDefinition>& InCaptureDefsToCheck) const
{
	return CapturedRelevantAttributes.HasValidCapturedAttributes(InCaptureDefsToCheck);
}

void FGameplayEffectSpec::RecaptureAttributeDataForClone(UAbilitySystemComponent* OriginalASC, UAbilitySystemComponent* NewASC)
{
	if (bCompletedSourceAttributeCapture == false)
	{
		// Only do this if we are the source
		if (EffectContext.GetInstigatorAbilitySystemComponent() == OriginalASC)
		{
			// Flip the effect context
			EffectContext.AddInstigator(NewASC->GetOwner(), EffectContext.GetEffectCauser());
			CaptureDataFromSource();
		}
	}

	if (bCompletedTargetAttributeCapture == false)
	{
		CaptureAttributeDataFromTarget(NewASC);
	}
}

#if ENABLE_VISUAL_LOG
FVisualLogStatusCategory FGameplayEffectSpec::GrabVisLogStatus() const
{
	FVisualLogStatusCategory SpecStatus;
	SpecStatus.Category = FString::Printf(TEXT("Spec: %s"), *GetNameSafe(Def));

	SpecStatus.Add(TEXT("Level"), FString::Printf(TEXT("%.2f"), Level));

	if (Duration == UGameplayEffect::INSTANT_APPLICATION)
	{
		SpecStatus.Add(TEXT("Duration"), TEXT("Instant"));
	}
	else if (Duration == UGameplayEffect::INFINITE_DURATION)
	{
		SpecStatus.Add(TEXT("Duration"), TEXT("Infinite"));
	}
	else
	{
		SpecStatus.Add(TEXT("Duration"), FString::Printf(TEXT("%.3f"), Duration));
	}

	if (Period > 0.0f)
	{
		SpecStatus.Add(TEXT("Period"), FString::Printf(TEXT("%.3f"), Period));
	}

	int32 LocalStackCount = GetStackCount();
	if (LocalStackCount > 0)
	{
		SpecStatus.Add(TEXT("StackCount"), FString::Printf(TEXT("%d"), LocalStackCount));
	}

	if (DynamicGrantedTags.Num() > 0)
	{
		SpecStatus.Add(TEXT("DynamicGrantedTags"), DynamicGrantedTags.ToStringSimple());
	}

	if (GetDynamicAssetTags().Num() > 0)
	{
		SpecStatus.Add(TEXT("DynamicAssetTags"), GetDynamicAssetTags().ToStringSimple());
	}

	auto AddTagContainerAggregator = [&](FString&& InName, const FTagContainerAggregator& InContainer)
		{
			const FGameplayTagContainer* AggregatedTags = InContainer.GetAggregatedTags();
			if (InContainer.GetActorTags().Num() || InContainer.GetSpecTags().Num() || (AggregatedTags && AggregatedTags->Num()))
			{
				FVisualLogStatusCategory Status;
				Status.Category = MoveTemp(InName);

				Status.Add(TEXT("ActorTags"), InContainer.GetActorTags().ToStringSimple());
				Status.Add(TEXT("AggregatedTags"), AggregatedTags ? AggregatedTags->ToStringSimple() : FString{});
				Status.Add(TEXT("SpecTags"), InContainer.GetSpecTags().ToStringSimple());

				SpecStatus.AddChild(Status);
			}
		};
	AddTagContainerAggregator(TEXT("CapturedSourceTags"), CapturedSourceTags);
	AddTagContainerAggregator(TEXT("CapturedTargetTags"), CapturedTargetTags);

	// Handle the SetByCallers
	{
		FVisualLogStatusCategory SetByCallersStatus;
		SetByCallersStatus.Category = TEXT("SetByCallers");
		for (const auto& Entry : SetByCallerNameMagnitudes)
		{
			SetByCallersStatus.Add(Entry.Key.ToString(), FString::Printf(TEXT("%.3f"), Entry.Value));
		}

		for (const auto& Entry : SetByCallerTagMagnitudes)
		{
			SetByCallersStatus.Add(Entry.Key.ToString(), FString::Printf(TEXT("%.3f"), Entry.Value));
		}

		// If we had any data, attach us
		if (SetByCallersStatus.Data.Num() > 0)
		{
			SpecStatus.AddChild(SetByCallersStatus);
		}
	}

	// Handle the EffectContext
	{
		FVisualLogStatusCategory EffectContextStatus;
		EffectContextStatus.Category = TEXT("Effect Context");

		const FGameplayEffectContextHandle& LocalContext = GetEffectContext();

		if (const UGameplayAbility* GrantingAbility = LocalContext.GetAbility())
		{
			EffectContextStatus.Add(TEXT("Ability"), *GrantingAbility->GetName());
		}

		if (const UObject* SourceObject = LocalContext.GetSourceObject())
		{
			EffectContextStatus.Add(TEXT("SourceObject"), *SourceObject->GetName());
		}

		if (const AActor* Instigator = LocalContext.GetInstigator())
		{
			EffectContextStatus.Add(TEXT("Instigator"), *Instigator->GetName());
		}

		if (const AActor* EffectCauser = LocalContext.GetEffectCauser())
		{
			EffectContextStatus.Add(TEXT("EffectCauser"), *EffectCauser->GetName());
		}

		if (LocalContext.HasOrigin())
		{
			EffectContextStatus.Add(TEXT("Origin"), *LocalContext.GetOrigin().ToString());
		}

		if (const FHitResult* HitResult = LocalContext.GetHitResult())
		{
			EffectContextStatus.Add(TEXT("HitResult"), *HitResult->ToString());
		}

		// If we had any data, attach us
		if (EffectContextStatus.Data.Num() > 0)
		{
			SpecStatus.AddChild(EffectContextStatus);
		}
	}

	return SpecStatus;
}
#endif

const FGameplayEffectModifiedAttribute* FGameplayEffectSpec::GetModifiedAttribute(const FGameplayAttribute& Attribute) const
{
	for (const FGameplayEffectModifiedAttribute& ModifiedAttribute : ModifiedAttributes)
	{
		if (ModifiedAttribute.Attribute == Attribute)
		{
			return &ModifiedAttribute;
		}
	}
	return NULL;
}

FGameplayEffectModifiedAttribute* FGameplayEffectSpec::GetModifiedAttribute(const FGameplayAttribute& Attribute)
{
	for (FGameplayEffectModifiedAttribute& ModifiedAttribute : ModifiedAttributes)
	{
		if (ModifiedAttribute.Attribute == Attribute)
		{
			return &ModifiedAttribute;
		}
	}
	return NULL;
}

const FGameplayEffectModifiedAttribute* FGameplayEffectSpecForRPC::GetModifiedAttribute(const FGameplayAttribute& Attribute) const
{
	for (const FGameplayEffectModifiedAttribute& ModifiedAttribute : ModifiedAttributes)
	{
		if (ModifiedAttribute.Attribute == Attribute)
		{
			return &ModifiedAttribute;
		}
	}
	return NULL;
}

FString FGameplayEffectSpecForRPC::ToSimpleString() const
{
	return FString::Printf(TEXT("%s"), *Def->GetName());
}

FGameplayEffectModifiedAttribute* FGameplayEffectSpec::AddModifiedAttribute(const FGameplayAttribute& Attribute)
{
	FGameplayEffectModifiedAttribute NewAttribute;
	NewAttribute.Attribute = Attribute;
	return &ModifiedAttributes[ModifiedAttributes.Add(NewAttribute)];
}

void FGameplayEffectSpec::SetContext(FGameplayEffectContextHandle NewEffectContext, bool bSkipRecaptureSourceActorTags /*= false*/)
{
	bool bWasAlreadyInit = EffectContext.IsValid();
	EffectContext = NewEffectContext;	
	if (bWasAlreadyInit)
	{
		CaptureDataFromSource(bSkipRecaptureSourceActorTags);
	}
}

void FGameplayEffectSpec::GetAllGrantedTags(OUT FGameplayTagContainer& OutContainer) const
{
	OutContainer.AppendTags(DynamicGrantedTags);
	if (Def)
	{
		OutContainer.AppendTags(Def->GetGrantedTags());
	}
}

void FGameplayEffectSpec::GetAllBlockedAbilityTags(OUT FGameplayTagContainer& OutContainer) const
{
	if (Def)
	{
		OutContainer.AppendTags(Def->GetBlockedAbilityTags());
	}
}

void FGameplayEffectSpec::GetAllAssetTags(OUT FGameplayTagContainer& OutContainer) const
{
	OutContainer.AppendTags(GetDynamicAssetTags());
	if (Def)
	{
		OutContainer.AppendTags(Def->GetAssetTags());
	}
}

void FGameplayEffectSpec::SetSetByCallerMagnitude(FName DataName, float Magnitude)
{
	if (DataName != NAME_None)
	{
		SetByCallerNameMagnitudes.FindOrAdd(DataName) = Magnitude;
	}
}

void FGameplayEffectSpec::SetSetByCallerMagnitude(FGameplayTag DataTag, float Magnitude)
{
	if (DataTag.IsValid())
	{
		SetByCallerTagMagnitudes.FindOrAdd(DataTag) = Magnitude;
	}
}

float FGameplayEffectSpec::GetSetByCallerMagnitude(FName DataName, bool WarnIfNotFound, float DefaultIfNotFound) const
{
	float Magnitude = DefaultIfNotFound;
	const float* Ptr = nullptr;
	
	if (DataName != NAME_None)
	{
		Ptr = SetByCallerNameMagnitudes.Find(DataName);
	}
	
	if (Ptr)
	{
		Magnitude = *Ptr;
	}
	else if (WarnIfNotFound)
	{
		UE_LOGF(LogGameplayEffects, Error, "FGameplayEffectSpec::GetMagnitude called for Data %ls on Def %ls when magnitude had not yet been set by caller.", *DataName.ToString(), *Def->GetName());
	}

	return Magnitude;
}

float FGameplayEffectSpec::GetSetByCallerMagnitude(FGameplayTag DataTag, bool WarnIfNotFound, float DefaultIfNotFound) const
{
	float Magnitude = DefaultIfNotFound;
	const float* Ptr = nullptr;

	if (DataTag.IsValid())
	{
		Ptr = SetByCallerTagMagnitudes.Find(DataTag);
	}

	if (Ptr)
	{
		Magnitude = *Ptr;
	}
	else if (WarnIfNotFound)
	{
		UE_LOGF(LogGameplayEffects, Error, "FGameplayEffectSpec::GetMagnitude called for Data %ls on Def %ls when magnitude had not yet been set by caller.", *DataTag.ToString(), *Def->GetName());
	}

	return Magnitude;
}

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	FGameplayEffectAttributeCaptureSpec
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

FGameplayEffectAttributeCaptureSpec::FGameplayEffectAttributeCaptureSpec()
{
}

FGameplayEffectAttributeCaptureSpec::FGameplayEffectAttributeCaptureSpec(const FGameplayEffectAttributeCaptureDefinition& InDefinition)
	: BackingDefinition(InDefinition)
{
}

bool FGameplayEffectAttributeCaptureSpec::HasValidCapture() const
{
	return (AttributeAggregator.Get() != nullptr);
}

bool FGameplayEffectAttributeCaptureSpec::AttemptCalculateAttributeMagnitude(const FAggregatorEvaluateParameters& InEvalParams, OUT float& OutMagnitude) const
{
	FAggregator* Agg = AttributeAggregator.Get();
	if (Agg)
	{
		OutMagnitude = Agg->Evaluate(InEvalParams);
		return true;
	}

	return false;
}

bool FGameplayEffectAttributeCaptureSpec::AttemptCalculateAttributeMagnitudeUpToChannel(const FAggregatorEvaluateParameters& InEvalParams, EGameplayModEvaluationChannel FinalChannel, OUT float& OutMagnitude) const
{
	FAggregator* Agg = AttributeAggregator.Get();
	if (Agg)
	{
		OutMagnitude = Agg->EvaluateToChannel(InEvalParams, FinalChannel);
		return true;
	}

	return false;
}

bool FGameplayEffectAttributeCaptureSpec::AttemptCalculateAttributeMagnitudeWithBase(const FAggregatorEvaluateParameters& InEvalParams, float InBaseValue, OUT float& OutMagnitude) const
{
	FAggregator* Agg = AttributeAggregator.Get();
	if (Agg)
	{
		OutMagnitude = Agg->EvaluateWithBase(InBaseValue, InEvalParams);
		return true;
	}

	return false;
}

bool FGameplayEffectAttributeCaptureSpec::AttemptCalculateAttributeBaseValue(OUT float& OutBaseValue) const
{
	FAggregator* Agg = AttributeAggregator.Get();
	if (Agg)
	{
		OutBaseValue = Agg->GetBaseValue();
		return true;
	}

	return false;
}

bool FGameplayEffectAttributeCaptureSpec::AttemptCalculateAttributeBonusMagnitude(const FAggregatorEvaluateParameters& InEvalParams, OUT float& OutBonusMagnitude) const
{
	FAggregator* Agg = AttributeAggregator.Get();
	if (Agg)
	{
		OutBonusMagnitude = Agg->EvaluateBonus(InEvalParams);
		return true;
	}

	return false;
}

bool FGameplayEffectAttributeCaptureSpec::AttemptCalculateAttributeContributionMagnitude(const FAggregatorEvaluateParameters& InEvalParams, FActiveGameplayEffectHandle ActiveHandle, OUT float& OutBonusMagnitude) const
{
	FAggregator* Agg = AttributeAggregator.Get();
	if (Agg && ActiveHandle.IsValid())
	{
		OutBonusMagnitude = Agg->EvaluateContribution(InEvalParams, ActiveHandle);
		return true;
	}

	return false;
}

bool FGameplayEffectAttributeCaptureSpec::AttemptGetAttributeAggregatorSnapshot(OUT FAggregator& OutAggregatorSnapshot) const
{
	FAggregator* Agg = AttributeAggregator.Get();
	if (Agg)
	{
		OutAggregatorSnapshot.TakeSnapshotOf(*Agg);
		return true;
	}

	return false;
}

bool FGameplayEffectAttributeCaptureSpec::AttemptAddAggregatorModsToAggregator(OUT FAggregator& OutAggregatorToAddTo) const
{
	FAggregator* Agg = AttributeAggregator.Get();
	if (Agg)
	{
		OutAggregatorToAddTo.AddModsFrom(*Agg);
		return true;
	}

	return false;
}

bool FGameplayEffectAttributeCaptureSpec::AttemptGatherAttributeMods(const FAggregatorEvaluateParameters& InEvalParams, OUT TMap<EGameplayModEvaluationChannel, const TArray<FAggregatorMod>*>& OutModMap) const
{
	FAggregator* Agg = AttributeAggregator.Get();
	if (Agg)
	{
		Agg->EvaluateQualificationForAllMods(InEvalParams);
		Agg->GetAllAggregatorMods(OutModMap);
		return true;
	}

	return false;
}

void FGameplayEffectAttributeCaptureSpec::RegisterLinkedAggregatorCallback(FActiveGameplayEffectHandle Handle) const
{
	if (BackingDefinition.bSnapshot == false)
	{
		// Its possible the linked Aggregator is already gone.
		FAggregator* Agg = AttributeAggregator.Get();
		if (Agg)
		{
			Agg->AddDependent(Handle);
		}
	}
}

void FGameplayEffectAttributeCaptureSpec::UnregisterLinkedAggregatorCallback(FActiveGameplayEffectHandle Handle) const
{
	FAggregator* Agg = AttributeAggregator.Get();
	if (Agg)
	{
		Agg->RemoveDependent(Handle);
	}
}

bool FGameplayEffectAttributeCaptureSpec::ShouldRefreshLinkedAggregator(const FAggregator* ChangedAggregator) const
{
	return (BackingDefinition.bSnapshot == false && (ChangedAggregator == nullptr || AttributeAggregator.Get() == ChangedAggregator));
}

void FGameplayEffectAttributeCaptureSpec::SwapAggregator(FAggregatorRef From, FAggregatorRef To)
{
	if (AttributeAggregator.Get() == From.Get())
	{
		AttributeAggregator = To;
	}
}

const FGameplayEffectAttributeCaptureDefinition& FGameplayEffectAttributeCaptureSpec::GetBackingDefinition() const
{
	return BackingDefinition;
}

FGameplayEffectAttributeCaptureSpecContainer::FGameplayEffectAttributeCaptureSpecContainer()
	: bHasNonSnapshottedAttributes(false)
{
}

FGameplayEffectAttributeCaptureSpecContainer::FGameplayEffectAttributeCaptureSpecContainer(FGameplayEffectAttributeCaptureSpecContainer&& Other)
	: SourceAttributes(MoveTemp(Other.SourceAttributes))
	, TargetAttributes(MoveTemp(Other.TargetAttributes))
	, bHasNonSnapshottedAttributes(Other.bHasNonSnapshottedAttributes)
{
}

FGameplayEffectAttributeCaptureSpecContainer::FGameplayEffectAttributeCaptureSpecContainer(const FGameplayEffectAttributeCaptureSpecContainer& Other)
	: SourceAttributes(Other.SourceAttributes)
	, TargetAttributes(Other.TargetAttributes)
	, bHasNonSnapshottedAttributes(Other.bHasNonSnapshottedAttributes)
{
}

FGameplayEffectAttributeCaptureSpecContainer& FGameplayEffectAttributeCaptureSpecContainer::operator=(FGameplayEffectAttributeCaptureSpecContainer&& Other)
{
	SourceAttributes = MoveTemp(Other.SourceAttributes);
	TargetAttributes = MoveTemp(Other.TargetAttributes);
	bHasNonSnapshottedAttributes = Other.bHasNonSnapshottedAttributes;
	return *this;
}

FGameplayEffectAttributeCaptureSpecContainer& FGameplayEffectAttributeCaptureSpecContainer::operator=(const FGameplayEffectAttributeCaptureSpecContainer& Other)
{
	SourceAttributes = Other.SourceAttributes;
	TargetAttributes = Other.TargetAttributes;
	bHasNonSnapshottedAttributes = Other.bHasNonSnapshottedAttributes;
	return *this;
}

void FGameplayEffectAttributeCaptureSpecContainer::AddCaptureDefinition(const FGameplayEffectAttributeCaptureDefinition& InCaptureDefinition)
{
	const bool bSourceAttribute = (InCaptureDefinition.AttributeSource == EGameplayEffectAttributeCaptureSource::Source);
	TArray<FGameplayEffectAttributeCaptureSpec>& AttributeArray = (bSourceAttribute ? SourceAttributes : TargetAttributes);

	// Only add additional captures if this exact capture definition isn't already being handled
	if (!AttributeArray.ContainsByPredicate(
			[&](const FGameplayEffectAttributeCaptureSpec& Element)
			{
				return Element.GetBackingDefinition() == InCaptureDefinition;
			}))
	{
		AttributeArray.Add(FGameplayEffectAttributeCaptureSpec(InCaptureDefinition));

		if (!InCaptureDefinition.bSnapshot)
		{
			bHasNonSnapshottedAttributes = true;
		}
	}
}

void FGameplayEffectAttributeCaptureSpecContainer::CaptureAttributes(UAbilitySystemComponent* InAbilitySystemComponent, EGameplayEffectAttributeCaptureSource InCaptureSource)
{
	if (InAbilitySystemComponent)
	{
		const bool bSourceComponent = (InCaptureSource == EGameplayEffectAttributeCaptureSource::Source);
		TArray<FGameplayEffectAttributeCaptureSpec>& AttributeArray = (bSourceComponent ? SourceAttributes : TargetAttributes);

		// Capture every spec's requirements from the specified component
		for (FGameplayEffectAttributeCaptureSpec& CurCaptureSpec : AttributeArray)
		{
			InAbilitySystemComponent->CaptureAttributeForGameplayEffect(CurCaptureSpec);
		}
	}
}

const FGameplayEffectAttributeCaptureSpec* FGameplayEffectAttributeCaptureSpecContainer::FindCaptureSpecByDefinition(const FGameplayEffectAttributeCaptureDefinition& InDefinition, bool bOnlyIncludeValidCapture) const
{
	const bool bSourceAttribute = (InDefinition.AttributeSource == EGameplayEffectAttributeCaptureSource::Source);
	const TArray<FGameplayEffectAttributeCaptureSpec>& AttributeArray = (bSourceAttribute ? SourceAttributes : TargetAttributes);

	const FGameplayEffectAttributeCaptureSpec* MatchingSpec = AttributeArray.FindByPredicate(
		[&](const FGameplayEffectAttributeCaptureSpec& Element)
	{
		return Element.GetBackingDefinition() == InDefinition;
	});

	// Null out the found results if the caller only wants valid captures and we don't have one yet
	if (MatchingSpec && bOnlyIncludeValidCapture && !MatchingSpec->HasValidCapture())
	{
		MatchingSpec = nullptr;
	}

	return MatchingSpec;
}

bool FGameplayEffectAttributeCaptureSpecContainer::HasValidCapturedAttributes(const TArray<FGameplayEffectAttributeCaptureDefinition>& InCaptureDefsToCheck) const
{
	bool bHasValid = true;

	for (const FGameplayEffectAttributeCaptureDefinition& CurDef : InCaptureDefsToCheck)
	{
		const FGameplayEffectAttributeCaptureSpec* CaptureSpec = FindCaptureSpecByDefinition(CurDef, true);
		if (!CaptureSpec)
		{
			bHasValid = false;
			break;
		}
	}

	return bHasValid;
}

bool FGameplayEffectAttributeCaptureSpecContainer::HasNonSnapshottedAttributes() const
{
	return bHasNonSnapshottedAttributes;
}

void FGameplayEffectAttributeCaptureSpecContainer::RegisterLinkedAggregatorCallbacks(FActiveGameplayEffectHandle Handle) const
{
	for (const FGameplayEffectAttributeCaptureSpec& CaptureSpec : SourceAttributes)
	{
		CaptureSpec.RegisterLinkedAggregatorCallback(Handle);
	}

	for (const FGameplayEffectAttributeCaptureSpec& CaptureSpec : TargetAttributes)
	{
		CaptureSpec.RegisterLinkedAggregatorCallback(Handle);
	}
}

void FGameplayEffectAttributeCaptureSpecContainer::UnregisterLinkedAggregatorCallbacks(FActiveGameplayEffectHandle Handle) const
{
	for (const FGameplayEffectAttributeCaptureSpec& CaptureSpec : SourceAttributes)
	{
		CaptureSpec.UnregisterLinkedAggregatorCallback(Handle);
	}

	for (const FGameplayEffectAttributeCaptureSpec& CaptureSpec : TargetAttributes)
	{
		CaptureSpec.UnregisterLinkedAggregatorCallback(Handle);
	}
}

void FGameplayEffectAttributeCaptureSpecContainer::SwapAggregator(FAggregatorRef From, FAggregatorRef To)
{
	for (FGameplayEffectAttributeCaptureSpec& CaptureSpec : SourceAttributes)
	{
		CaptureSpec.SwapAggregator(From, To);
	}

	for (FGameplayEffectAttributeCaptureSpec& CaptureSpec : TargetAttributes)
	{
		CaptureSpec.SwapAggregator(From, To);
	}
}

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	FActiveGameplayEffect
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

FActiveGameplayEffect::FActiveGameplayEffect(const FActiveGameplayEffect& Other)
{
	*this = Other;
}

FActiveGameplayEffect::FActiveGameplayEffect(FActiveGameplayEffect&& Other)
{
	*this = MoveTemp(Other);
}

FActiveGameplayEffect::FActiveGameplayEffect(FActiveGameplayEffectHandle InHandle, const FGameplayEffectSpec &InSpec, float InCurrentWorldTime, float InStartServerWorldTime, FPredictionKey InPredictionKey)
	: Handle(InHandle)
	, Spec(InSpec)
	, PredictionKey(InPredictionKey)
	, StartServerWorldTime(InStartServerWorldTime)
	, CachedStartServerWorldTime(InStartServerWorldTime)
	, StartWorldTime(InCurrentWorldTime)
{
}

FActiveGameplayEffect& FActiveGameplayEffect::operator=(FActiveGameplayEffect&& Other)
{
	using namespace UE::GameplayEffect;

	Handle = Other.Handle;
	Spec = MoveTemp(Other.Spec);
	PredictionKey = Other.PredictionKey;
	GrantedAbilityHandles = MoveTemp(Other.GrantedAbilityHandles);
	StartServerWorldTime = Other.StartServerWorldTime;
	CachedStartServerWorldTime = Other.CachedStartServerWorldTime;
	StartWorldTime = Other.StartWorldTime;
	bIsInhibited = Other.bIsInhibited;
	bPendingRepOnActiveGC = Other.bPendingRepOnActiveGC;
	bPendingRepWhileActiveGC = Other.bPendingRepWhileActiveGC;
	IsPendingRemove = Other.IsPendingRemove;
	ClientCachedStackCount = HasActiveGameplayEffectFix(EActiveGameplayEffectFix::MoveClientCachedStackCount) ? Other.ClientCachedStackCount : 0;
	PeriodHandle = Other.PeriodHandle;
	DurationHandle = Other.DurationHandle;
	// Note: purposefully not copying PendingNext pointer.
	// PendingNext = Other.PendingNext;
	EventSet = Other.EventSet;

	// FFastArraySerializerItem properties
	ReplicationID = Other.ReplicationID;
	ReplicationKey = Other.ReplicationKey;

	if (HasActiveGameplayEffectFix(EActiveGameplayEffectFix::MostRecentArrayReplicationKey))
	{
		MostRecentArrayReplicationKey = Other.MostRecentArrayReplicationKey;
	}
	
	bPostPredictObject = Other.bPostPredictObject;

	return *this;
}

FActiveGameplayEffect& FActiveGameplayEffect::operator=(const FActiveGameplayEffect& Other)
{
	Handle = Other.Handle;
	Spec = Other.Spec;
	PredictionKey = Other.PredictionKey;
	GrantedAbilityHandles = Other.GrantedAbilityHandles;
	StartServerWorldTime = Other.StartServerWorldTime;
	CachedStartServerWorldTime = Other.CachedStartServerWorldTime;
	StartWorldTime = Other.StartWorldTime;
	bIsInhibited = Other.bIsInhibited;
	bPendingRepOnActiveGC = Other.bPendingRepOnActiveGC;
	bPendingRepWhileActiveGC = Other.bPendingRepWhileActiveGC;
	IsPendingRemove = Other.IsPendingRemove;
	ClientCachedStackCount = Other.ClientCachedStackCount;
	PeriodHandle = Other.PeriodHandle;
	DurationHandle = Other.DurationHandle;
	EventSet = Other.EventSet;

	// Note: purposefully not copying PendingNext pointer unless the fix is disabled
	using namespace UE::GameplayEffect;
	if (!HasActiveGameplayEffectFix(EActiveGameplayEffectFix::OmitPendingNextOnCopy))
	{
		PendingNext = Other.PendingNext;
	}

	// FFastArraySerializerItem properties
	ReplicationID = Other.ReplicationID;
	ReplicationKey = Other.ReplicationKey;

	if (HasActiveGameplayEffectFix(EActiveGameplayEffectFix::MostRecentArrayReplicationKey))
	{
		MostRecentArrayReplicationKey = Other.MostRecentArrayReplicationKey;
	}
	
	bPostPredictObject = Other.bPostPredictObject;

	return *this;
}

void FActiveGameplayEffect::PreReplicatedRemove(const struct FActiveGameplayEffectsContainer &InArray)
{
	ensureMsgf(InArray.ScopedLockCount > 0, TEXT("%hs needs to be ScopeLocked.  Previously from UAbilitySystemComponent::PreNetReceive."), __func__);

	const int32 PreRemoveIdx = this - InArray.GameplayEffects_Internal.GetData();
	ensureMsgf(InArray.GameplayEffects_Internal.IsValidIndex(PreRemoveIdx), TEXT("%hs had invalid 'this' pointer which was not part of the passed-in FActiveGameplayEffectsContainer"), __func__);

	if (Spec.Def == nullptr)
	{
		UE_LOGF(LogGameplayEffects, Error, "Received PreReplicatedRemove with no UGameplayEffect def.");
		return;
	}

	FGameplayEffectRemovalInfo GameplayEffectRemovalInfo;
	GameplayEffectRemovalInfo.ActiveEffect = this;
	GameplayEffectRemovalInfo.StackCount = ClientCachedStackCount;
	GameplayEffectRemovalInfo.OwningASC = InArray.Owner;
	//Check duration to set bPrematureRemoval as req.
	if (DurationHandle.IsValid())
	{
		float SecondsRemaining = GetTimeRemaining(const_cast<FActiveGameplayEffectsContainer&>(InArray).GetWorldTime());

		if (SecondsRemaining > 0.f)
		{
			GameplayEffectRemovalInfo.bPrematureRemoval = true;
		}
	}
	GameplayEffectRemovalInfo.EffectContext = Spec.GetEffectContext();

	UE_VLOG_UELOG(InArray.Owner->GetOwnerActor(), LogGameplayEffects, Verbose, TEXT("%hs (Non-Auth): %s. Premature: %d Inhibited: %d. Pending( Remove: %d OnActive: %d WhileActive: %d )"), __func__, *GetDebugString(), GameplayEffectRemovalInfo.bPrematureRemoval, bIsInhibited, IsPendingRemove, bPendingRepOnActiveGC, bPendingRepWhileActiveGC);

	const_cast<FActiveGameplayEffectsContainer&>(InArray).InternalOnActiveGameplayEffectRemoved(*this, !bIsInhibited, GameplayEffectRemovalInfo);	// Const cast is ok. It is there to prevent mutation of the GameplayEffects array, which this wont do.

	// Make sure no calls caused a realloc of InArray while holding this pointer
	ensureMsgf(InArray.GameplayEffects_Internal.IsValidIndex(PreRemoveIdx) && &InArray.GameplayEffects_Internal[PreRemoveIdx] == this, TEXT("%hs had realloc of the array holding the item pointed to by 'this'. This will eventually crash"), __func__);
}

void FActiveGameplayEffect::PostReplicatedAdd(const struct FActiveGameplayEffectsContainer &InArray)
{
	ensureMsgf(InArray.ScopedLockCount > 0, TEXT("%hs needs to be ScopeLocked.  Previously from UAbilitySystemComponent::PreNetReceive."), __func__);

	const int32 PreAddIdx = this - InArray.GameplayEffects_Internal.GetData();
	ensureMsgf(InArray.GameplayEffects_Internal.IsValidIndex(PreAddIdx), TEXT("%hs had invalid 'this' pointer which was not part of the passed-in FActiveGameplayEffectsContainer"), __func__);

	if (Spec.Def == nullptr)
	{
		UE_LOGF(LogGameplayEffects, Error, "FActiveGameplayEffect::PostReplicatedAdd Received ReplicatedGameplayEffect with no UGameplayEffect def. (%ls)", *Spec.GetEffectContext().ToString());
		return;
	}

	if (Spec.Modifiers.Num() != Spec.Def->Modifiers.Num())
	{
		// This can happen with older replays, where the replicated Spec.Modifiers size changed in the newer Spec.Def
		UE_LOGF(LogGameplayEffects, Error, "FActiveGameplayEffect::PostReplicatedAdd: Spec.Modifiers.Num() != Spec.Def->Modifiers.Num(). Spec: %ls", *Spec.ToSimpleString());
		Spec.Modifiers.Empty();
		return;
	}

	bool ShouldInvokeGameplayCueEvents = true;
	bPostPredictObject = PredictionKey.IsLocalClientKey();
	if (bPostPredictObject)
	{
		// PredictionKey will only be valid on the client that predicted it. So if this has a valid PredictionKey, we can assume we already predicted it and shouldn't invoke GameplayCues.
		// We may need to do more bookkeeping here in the future. Possibly give the predicted gameplayeffect a chance to pass something off to the new replicated gameplay effect.
		if (InArray.HasPredictedEffectWithPredictedKey(PredictionKey))
		{
			ShouldInvokeGameplayCueEvents = false;
		}
	}

	bPendingRepOnActiveGC = false;

	// Adjust start time for local clock
	if (InArray.IsServerWorldTimeAvailable())
	{
		static const float MAX_DELTA_TIME = 3.f;

		// Was this actually just activated, or are we just finding out about it due to relevancy/join in progress?
		float WorldTimeSeconds = InArray.GetWorldTime();
		float ServerWorldTime = InArray.GetServerWorldTime();

		float DeltaServerWorldTime = ServerWorldTime - StartServerWorldTime;	// How long we think the effect has been playing

		// Set our local start time accordingly
		RecomputeStartWorldTime(WorldTimeSeconds, ServerWorldTime);
		CachedStartServerWorldTime = StartServerWorldTime;

		// Determine if we should invoke the OnActive GameplayCue event
		if (ShouldInvokeGameplayCueEvents)
		{
			// These events will get invoked if, after the parent array has been completely updated, this GE is still not inhibited
			bPendingRepOnActiveGC = (ServerWorldTime > 0 && FMath::Abs(DeltaServerWorldTime) < MAX_DELTA_TIME);
		}
	}

	if (ShouldInvokeGameplayCueEvents)
	{
		bPendingRepWhileActiveGC = true;
	}

	// Cache off StackCount
	ClientCachedStackCount = Spec.GetStackCount();

	// Handles are not replicated, so create a new one.
	Handle = FActiveGameplayEffectHandle::GenerateNewHandle(InArray.Owner);

	UE_VLOG_UELOG(
		IsValid(InArray.Owner) ? InArray.Owner->GetOwnerActor() : nullptr,
		LogGameplayEffects,
		Verbose,
		TEXT("%s (Non-Auth): %s. Pending( OnActive: %d WhileActive: %d )"),
		ANSI_TO_TCHAR(__func__),
		*GetDebugString(),
		bPendingRepOnActiveGC,
		bPendingRepWhileActiveGC);

	// Do stuff for adding GEs (add mods, tags, *invoke callbacks*).  But do NOT invoke the GameplayCues or tags as we don't know if this GE ends up inhibited or not (thus the bPendingRepOnActiveGC variables).
	constexpr bool bInvokePredictedEffects = false;
	const_cast<FActiveGameplayEffectsContainer&>(InArray).InternalOnActiveGameplayEffectAdded(*this, bInvokePredictedEffects);	// Const cast is ok. It is there to prevent mutation of the GameplayEffects array, which this wont do.

	// Make sure no calls caused a realloc of InArray while holding this pointer
	ensureMsgf(InArray.GameplayEffects_Internal.IsValidIndex(PreAddIdx) && &InArray.GameplayEffects_Internal[PreAddIdx] == this, TEXT("%hs had realloc of the array holding the item pointed to by 'this'. This will eventually crash"), __func__);
}

void FActiveGameplayEffect::PostReplicatedChange(const struct FActiveGameplayEffectsContainer &InArray)
{
	ensureMsgf(InArray.ScopedLockCount > 0, TEXT("%hs needs to be ScopeLocked.  Previously from UAbilitySystemComponent::PreNetReceive."), __func__);

	const int32 PreChangeIdx = this - InArray.GameplayEffects_Internal.GetData();
	ensureMsgf(InArray.GameplayEffects_Internal.IsValidIndex(PreChangeIdx), TEXT("%hs had invalid 'this' pointer which was not part of the passed-in FActiveGameplayEffectsContainer"), __func__);

	if (Spec.Def == nullptr)
	{
		UE_LOGF(LogGameplayEffects, Error, "FActiveGameplayEffect::PostReplicatedChange Received ReplicatedGameplayEffect with no UGameplayEffect def. (%ls)", *Spec.GetEffectContext().ToString());
		return;
	}

	if (Spec.Modifiers.Num() != Spec.Def->Modifiers.Num())
	{
		// This can happen with older replays, where the replicated Spec.Modifiers size changed in the newer Spec.Def
		Spec.Modifiers.Empty();
		return;
	}

	UE_VLOG_UELOG(InArray.Owner->GetOwnerActor(), LogGameplayEffects, Verbose, TEXT("%hs (Non-Auth): %s. Pending( OnActive: %d. WhileActive: %d )"), __func__, *GetDebugString(), bPendingRepOnActiveGC, bPendingRepWhileActiveGC);

	// Handle potential duration refresh
	if (CachedStartServerWorldTime != StartServerWorldTime)
	{
		RecomputeStartWorldTime(InArray);
		CachedStartServerWorldTime = StartServerWorldTime;

		// Const cast is ok. It is there to prevent mutation of the GameplayEffects array, which we believe this wont do due to scope lock.
		const_cast<FActiveGameplayEffectsContainer&>(InArray).OnDurationChange(*this);
	}
	
	int32 StackCount = Spec.GetStackCount();
	if (ClientCachedStackCount != StackCount)
	{
		// If its a stack count change, we just call OnStackCountChange and it will broadcast delegates and update attribute aggregators
		// Const cast is ok. It is there to prevent mutation of the GameplayEffects array, which we believe this wont do due to scope lock.
		const_cast<FActiveGameplayEffectsContainer&>(InArray).OnStackCountChange(*this, ClientCachedStackCount, StackCount);
		ClientCachedStackCount = StackCount;
	}
	else
	{
		// Stack count didn't change, but something did (like a modifier magnitude). We need to update our attribute aggregators
		// Const cast is ok. It is there to prevent mutation of the GameplayEffects array, which we believe this wont do due to scope lock.
		const_cast<FActiveGameplayEffectsContainer&>(InArray).UpdateAllAggregatorModMagnitudes(*this);
	}

	// Make sure no calls caused a realloc of InArray while holding this pointer
	ensureMsgf(InArray.GameplayEffects_Internal.IsValidIndex(PreChangeIdx) && &InArray.GameplayEffects_Internal[PreChangeIdx] == this, TEXT("%hs had realloc of the array holding the item pointed to by 'this'. This will eventually crash"), __func__);
}

FString FActiveGameplayEffect::GetDebugString()
{
	return FString::Printf(TEXT("Def: %s. Handle: %s. PredictionKey: %s"), *GetNameSafe(Spec.Def), *Handle.ToString(), *PredictionKey.ToString());
}

void FActiveGameplayEffect::RecomputeStartWorldTime(const FActiveGameplayEffectsContainer& InArray)
{
	RecomputeStartWorldTime(InArray.GetWorldTime(), InArray.GetServerWorldTime());
}

void FActiveGameplayEffect::RecomputeStartWorldTime(const float WorldTime, const float ServerWorldTime)
{
	StartWorldTime = WorldTime - (ServerWorldTime - StartServerWorldTime);
}

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	FActiveGameplayEffectsContainer
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

FActiveGameplayEffectsContainer::FActiveGameplayEffectsContainer()
	: Owner(nullptr)
	, OwnerIsNetAuthority(false)
	, ScopedLockCount(0)
	, PendingRemoves(0)
	, PendingGameplayEffectHead(nullptr)
	, bIsUsingReplicationCondition(false)
{
	PendingGameplayEffectNext = &PendingGameplayEffectHead;
	SetDeltaSerializationEnabled(true);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FActiveGameplayEffectsContainer::~FActiveGameplayEffectsContainer()
{
	using namespace UE::GameplayEffect;

	if (CVarFixAllPendingGameplayEffectsHeadNextsAtTeardown.GetValueOnAnyThread())
	{
		if (HasActiveGameplayEffectFix(EActiveGameplayEffectFix::CleanupAllPendingActiveGEs))
		{
			while (PendingGameplayEffectHead)
			{
				DeletePendingActiveGameplayEffect();
			}
		}
		else
		{
			DeletePendingActiveGameplayEffect();
		}
	}
	else
	{
		if (!ensureAlwaysMsgf(!PendingGameplayEffectHead || PendingGameplayEffectHead != PendingGameplayEffectHead->PendingNext, TEXT("Pending Gameplay Effect Head is pointing to the same location as Pending Next this shouldn't happen'")))
		{
			ensure(HasActiveGameplayEffectFix(EActiveGameplayEffectFix::OmitPendingNextOnCopy));

			if (CVarFixPendingGameplayEffectHeadNextAtTeardown.GetValueOnAnyThread())
			{
				// setting next to null so that we don't call delete on the same memory twice in the code below
				PendingGameplayEffectHead->PendingNext = nullptr;
			}
		}

		if (HasActiveGameplayEffectFix(EActiveGameplayEffectFix::CleanupAllPendingActiveGEs))
		{
			while (PendingGameplayEffectHead)
			{
				FActiveGameplayEffect* Next = PendingGameplayEffectHead->PendingNext;
				delete PendingGameplayEffectHead;
				PendingGameplayEffectHead = Next;
			}
		}
		else
		{
			if (PendingGameplayEffectHead)
			{
				FActiveGameplayEffect* Next = PendingGameplayEffectHead->PendingNext;
				delete PendingGameplayEffectHead;
				PendingGameplayEffectHead = Next;
			}
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FActiveGameplayEffectsContainer::DeletePendingActiveGameplayEffect()
{
	using namespace UE::GameplayEffect;

	if (!PendingGameplayEffectHead)
	{
		return;
	}

	if (!ensureAlwaysMsgf(PendingGameplayEffectHead != PendingGameplayEffectHead->PendingNext, TEXT("Pending Gameplay Effect Head is pointing to the same location as Pending Next this shouldn't happen'")))
	{
		ensure(HasActiveGameplayEffectFix(EActiveGameplayEffectFix::OmitPendingNextOnCopy));

		if (CVarFixPendingGameplayEffectHeadNextAtTeardown.GetValueOnAnyThread())
		{
			UE_LOGF(LogGameplayEffects, Warning, "Setting PendingNext to null for pointing to self on Pending ActiveGE: %ls", *PendingGameplayEffectHead->GetDebugString());
			// setting next to null so that we don't call delete on the same memory twice in the code below
			PendingGameplayEffectHead->PendingNext = nullptr;
		}
	}

	if (CVarFixVerboseLogDeletionOfPendingGameplayEffectsHead.GetValueOnAnyThread())
	{
		UE_LOG(LogGameplayEffects, Log, TEXT("Deleting pending gameplay effect: %s. Pending next effect: %s. Owner: %s."), *PendingGameplayEffectHead->GetDebugString(), PendingGameplayEffectHead->PendingNext ? *PendingGameplayEffectHead->PendingNext->GetDebugString() : TEXT("Next is null"), Owner ? *Owner->GetFullName() : TEXT("Owner is null"));
	}

	FActiveGameplayEffect* Next = PendingGameplayEffectHead->PendingNext;
	delete PendingGameplayEffectHead;
	PendingGameplayEffectHead = Next;
}

void FActiveGameplayEffectsContainer::RegisterWithOwner(UAbilitySystemComponent* InOwner)
{
	if (Owner != InOwner && InOwner != nullptr)
	{
		Owner = InOwner;
		OwnerIsNetAuthority = Owner->IsOwnerActorAuthoritative();
	}
}

void FActiveGameplayEffectsContainer::PredictivelyExecuteEffectSpec(FGameplayEffectSpec& Spec, FPredictionKey PredictionKey, const bool bPredictGameplayCues /*= false*/)
{
	if (!Owner)
	{
		return;
	}

	if (!PredictionKey.IsValidForMorePrediction())
	{
		return;
	}

	// Should only ever execute effects that are instant application or periodic application
	// Effects with no period and that aren't instant application should never be executed
	const bool bNotInstantEffect = (Spec.GetDuration() > UGameplayEffect::INSTANT_APPLICATION);
	const bool bNoPeriodEffect = (Spec.GetPeriod() != UGameplayEffect::NO_PERIOD);
	if (bNotInstantEffect && bNoPeriodEffect)
	{
		return;
	}

	FGameplayEffectSpec& SpecToUse = Spec;

	// Capture our own tags.
	// TODO: We should only capture them if we need to. We may have snapshotted target tags (?) (in the case of dots with exotic setups?)

	SpecToUse.CapturedTargetTags.GetActorTags().Reset();
	Owner->GetOwnedGameplayTags(SpecToUse.CapturedTargetTags.GetActorTags());

	SpecToUse.CalculateModifierMagnitudes();

	// ------------------------------------------------------
	//	Modifiers
	//		These will modify the base value of attributes
	// ------------------------------------------------------

	bool ModifierSuccessfullyExecuted = false;

	ensureMsgf(SpecToUse.Modifiers.Num() == SpecToUse.Def->Modifiers.Num(), TEXT("GE Spec %s modifiers did not match the Definition.  This indicates an error much earlier (when setting up the GESpec)"), *SpecToUse.ToSimpleString());
	for (int32 ModIdx = 0; ModIdx < SpecToUse.Modifiers.Num(); ++ModIdx)
	{
		const FGameplayModifierInfo& ModDef = SpecToUse.Def->Modifiers[ModIdx];

		if (UE::GameplayEffect::bUseModifierTagRequirementsOnAllGameplayEffects)
		{
			// Check tag requirements. This code path is for Instant & Periodic effects.
			// Duration effects are a separate code path; they use aggregators and their requirements checks are in FAggregatorMod::UpdateQualifies.
			if (!ModDef.SourceTags.IsEmpty() && !ModDef.SourceTags.RequirementsMet(Spec.CapturedSourceTags.GetActorTags()))
			{
				continue;
			}

			if (!ModDef.TargetTags.IsEmpty() && !ModDef.TargetTags.RequirementsMet(Spec.CapturedTargetTags.GetActorTags()))
			{
				continue;
			}
		}

		FGameplayModifierEvaluatedData EvalData(ModDef.Attribute, ModDef.ModifierOp, SpecToUse.GetModifierMagnitude(ModIdx));
		ModifierSuccessfullyExecuted |= InternalExecuteMod(SpecToUse, EvalData);
	}

	// ------------------------------------------------------
	//	Executions
	//		This will run custom code to 'do stuff'
	// ------------------------------------------------------

	bool GameplayCuesWereManuallyHandled = false;

	for (const FGameplayEffectExecutionDefinition& CurExecDef : SpecToUse.Def->Executions)
	{
		if (CurExecDef.CalculationClass)
		{
			const UGameplayEffectExecutionCalculation* ExecCDO = CurExecDef.CalculationClass->GetDefaultObject<UGameplayEffectExecutionCalculation>();
			check(ExecCDO);

			// Run the custom execution
			FGameplayEffectCustomExecutionParameters ExecutionParams(SpecToUse, CurExecDef.CalculationModifiers, Owner, CurExecDef.PassedInTags, PredictionKey);
			FGameplayEffectCustomExecutionOutput ExecutionOutput;
			ExecCDO->Execute(ExecutionParams, ExecutionOutput);

			// Execute any mods the custom execution yielded
			TArray<FGameplayModifierEvaluatedData>& OutModifiers = ExecutionOutput.GetOutputModifiersRef();

			const bool bApplyStackCountToEmittedMods = !ExecutionOutput.IsStackCountHandledManually();
			const int32 SpecStackCount = SpecToUse.GetStackCount();

			for (FGameplayModifierEvaluatedData& CurExecMod : OutModifiers)
			{
				// If the execution didn't manually handle the stack count, automatically apply it here
				if (bApplyStackCountToEmittedMods && SpecStackCount > 1)
				{
					CurExecMod.Magnitude = GameplayEffectUtilities::ComputeStackedModifierMagnitude(CurExecMod.Magnitude, SpecStackCount, CurExecMod.ModifierOp);
				}
				ModifierSuccessfullyExecuted |= InternalExecuteMod(SpecToUse, CurExecMod);
			}

			// If execution handled GameplayCues, we dont have to.
			if (ExecutionOutput.AreGameplayCuesHandledManually())
			{
				GameplayCuesWereManuallyHandled = true;
			}
		}
	}

	// ------------------------------------------------------
	//	Invoke GameplayCue events
	// ------------------------------------------------------
	if (bPredictGameplayCues)
	{
		// If there are no modifiers or we don't require modifier success to trigger, we apply the GameplayCue.
		const bool bHasModifiers = SpecToUse.Modifiers.Num() > 0;
		const bool bHasExecutions = SpecToUse.Def->Executions.Num() > 0;
		const bool bHasModifiersOrExecutions = bHasModifiers || bHasExecutions;

		// If there are no modifiers or we don't require modifier success to trigger, we apply the GameplayCue.
		bool InvokeGameplayCueExecute = (!bHasModifiersOrExecutions) || !Spec.Def->bRequireModifierSuccessToTriggerCues;

		if (bHasModifiersOrExecutions && ModifierSuccessfullyExecuted)
		{
			InvokeGameplayCueExecute = true;
		}

		// Don't trigger gameplay cues if one of the executions says it manually handled them
		if (GameplayCuesWereManuallyHandled)
		{
			InvokeGameplayCueExecute = false;
		}

		if (InvokeGameplayCueExecute && SpecToUse.Def->GameplayCues.Num())
		{
			// TODO: check replication policy. Right now we will replicate every execute via a multicast RPC

			UE_LOGF(LogGameplayEffects, Log, "Invoking Execute GameplayCue for %ls", *SpecToUse.ToSimpleString());

			UAbilitySystemGlobals::Get().GetGameplayCueManager()->InvokeGameplayCueExecuted_FromSpec(Owner, SpecToUse, PredictionKey);
		}
	}
}

/** This is the main function that executes a GameplayEffect on Attributes and ActiveGameplayEffects */
void FActiveGameplayEffectsContainer::ExecuteActiveEffectsFrom(FGameplayEffectSpec &Spec, FPredictionKey PredictionKey)
{
#if WITH_SERVER_CODE
	SCOPE_CYCLE_COUNTER(STAT_ExecuteActiveEffectsFrom);
#endif

	if (!Owner)
	{
		return;
	}

	FGameplayEffectSpec& SpecToUse = Spec;

	// Capture our own tags.
	// TODO: We should only capture them if we need to. We may have snapshotted target tags (?) (in the case of dots with exotic setups?)

	SpecToUse.CapturedTargetTags.GetActorTags().Reset();
	Owner->GetOwnedGameplayTags(SpecToUse.CapturedTargetTags.GetActorTags());

	SpecToUse.CalculateModifierMagnitudes();

	// ------------------------------------------------------
	//	Modifiers
	//		These will modify the base value of attributes
	// ------------------------------------------------------
	
	bool ModifierSuccessfullyExecuted = false;
	
	ensureMsgf(SpecToUse.Modifiers.Num() == SpecToUse.Def->Modifiers.Num(), TEXT("GE Spec %s modifiers did not match the Definition.  This indicates an error much earlier (when setting up the GESpec)"), *SpecToUse.ToSimpleString());
	for (int32 ModIdx = 0; ModIdx < SpecToUse.Modifiers.Num(); ++ModIdx)
	{
		const FGameplayModifierInfo& ModDef = SpecToUse.Def->Modifiers[ModIdx];

		if (UE::GameplayEffect::bUseModifierTagRequirementsOnAllGameplayEffects)
		{
			// Check tag requirements. This code path is for Instant & Periodic effects.
			// Duration effects are a separate code path; they use aggregators and their requirements checks are in FAggregatorMod::UpdateQualifies.
			if (!ModDef.SourceTags.IsEmpty() && !ModDef.SourceTags.RequirementsMet(Spec.CapturedSourceTags.GetActorTags()))
			{
				continue;
			}

			if (!ModDef.TargetTags.IsEmpty() && !ModDef.TargetTags.RequirementsMet(Spec.CapturedTargetTags.GetActorTags()))
			{
				continue;
			}
		}
		
		FGameplayModifierEvaluatedData EvalData(ModDef.Attribute, ModDef.ModifierOp, SpecToUse.GetModifierMagnitude(ModIdx));
		ModifierSuccessfullyExecuted |= InternalExecuteMod(SpecToUse, EvalData);
	}

	// ------------------------------------------------------
	//	Executions
	//		This will run custom code to 'do stuff'
	// ------------------------------------------------------
	
	TArray< FGameplayEffectSpecHandle, TInlineAllocator<4> > ConditionalEffectSpecs;

	bool GameplayCuesWereManuallyHandled = false;

	for (const FGameplayEffectExecutionDefinition& CurExecDef : SpecToUse.Def->Executions)
	{
		bool bRunConditionalEffects = true; // Default to true if there is no CalculationClass specified.

		if (CurExecDef.CalculationClass)
		{
			const UGameplayEffectExecutionCalculation* ExecCDO = CurExecDef.CalculationClass->GetDefaultObject<UGameplayEffectExecutionCalculation>();
			check(ExecCDO);

			// Run the custom execution
			FGameplayEffectCustomExecutionParameters ExecutionParams(SpecToUse, CurExecDef.CalculationModifiers, Owner, CurExecDef.PassedInTags, PredictionKey);
			FGameplayEffectCustomExecutionOutput ExecutionOutput;
			ExecCDO->Execute(ExecutionParams, ExecutionOutput);

			bRunConditionalEffects = ExecutionOutput.ShouldTriggerConditionalGameplayEffects();

			// Execute any mods the custom execution yielded
			TArray<FGameplayModifierEvaluatedData>& OutModifiers = ExecutionOutput.GetOutputModifiersRef();

			const bool bApplyStackCountToEmittedMods = !ExecutionOutput.IsStackCountHandledManually();
			const int32 SpecStackCount = SpecToUse.GetStackCount();

			for (FGameplayModifierEvaluatedData& CurExecMod : OutModifiers)
			{
				// If the execution didn't manually handle the stack count, automatically apply it here
				if (bApplyStackCountToEmittedMods && SpecStackCount > 1)
				{
					CurExecMod.Magnitude = GameplayEffectUtilities::ComputeStackedModifierMagnitude(CurExecMod.Magnitude, SpecStackCount, CurExecMod.ModifierOp);
				}
				ModifierSuccessfullyExecuted |= InternalExecuteMod(SpecToUse, CurExecMod);
			}

			// If execution handled GameplayCues, we dont have to.
			if (ExecutionOutput.AreGameplayCuesHandledManually())
			{
				GameplayCuesWereManuallyHandled = true;
			}
		}

		if (bRunConditionalEffects)
		{
			// If successful, apply conditional specs
			for (const FConditionalGameplayEffect& ConditionalEffect : CurExecDef.ConditionalGameplayEffects)
			{
				if (ConditionalEffect.CanApply(SpecToUse.CapturedSourceTags.GetActorTags(), SpecToUse.GetLevel()))
				{
					FGameplayEffectSpecHandle SpecHandle = ConditionalEffect.CreateSpec(SpecToUse.GetEffectContext(), SpecToUse.GetLevel());
					if (SpecHandle.IsValid())
					{
						ConditionalEffectSpecs.Add(SpecHandle);
					}
				}
			}
		}
	}

	// ------------------------------------------------------
	//	Invoke GameplayCue events
	// ------------------------------------------------------
	
	// If there are no modifiers or we don't require modifier success to trigger, we apply the GameplayCue.
	const bool bHasModifiers = SpecToUse.Modifiers.Num() > 0;
	const bool bHasExecutions = SpecToUse.Def->Executions.Num() > 0;
	const bool bHasModifiersOrExecutions = bHasModifiers || bHasExecutions;

	// If there are no modifiers or we don't require modifier success to trigger, we apply the GameplayCue.
	bool InvokeGameplayCueExecute = (!bHasModifiersOrExecutions) || !Spec.Def->bRequireModifierSuccessToTriggerCues;

	if (bHasModifiersOrExecutions && ModifierSuccessfullyExecuted)
	{
		InvokeGameplayCueExecute = true;
	}

	// Don't trigger gameplay cues if one of the executions says it manually handled them
	if (GameplayCuesWereManuallyHandled)
	{
		InvokeGameplayCueExecute = false;
	}

	if (InvokeGameplayCueExecute && SpecToUse.Def->GameplayCues.Num())
	{
		// TODO: check replication policy. Right now we will replicate every execute via a multicast RPC

		UE_LOGF(LogGameplayEffects, Log, "Invoking Execute GameplayCue for %ls", *SpecToUse.ToSimpleString());

		UAbilitySystemGlobals::Get().GetGameplayCueManager()->InvokeGameplayCueExecuted_FromSpec(Owner, SpecToUse, PredictionKey);
	}

	// Apply any conditional linked effects
	for (const FGameplayEffectSpecHandle& TargetSpec : ConditionalEffectSpecs)
	{
		if (TargetSpec.IsValid())
		{
			Owner->ApplyGameplayEffectSpecToSelf(*TargetSpec.Data.Get(), PredictionKey);
		}
	}

	// Notify the Gameplay Effect that it's been executed.  This will allow the GameplayEffectComponents to augment behavior.
	Spec.Def->OnExecuted(*this, Spec, PredictionKey);
}

void FActiveGameplayEffectsContainer::ExecutePeriodicGameplayEffect(FActiveGameplayEffectHandle Handle)
{
	GAMEPLAYEFFECT_SCOPE_LOCK();

	FActiveGameplayEffect* ActiveEffect = GetActiveGameplayEffect(Handle);

	if (ActiveEffect != nullptr)
	{
		InternalExecutePeriodicGameplayEffect(*ActiveEffect);
	}
}

FActiveGameplayEffect* FActiveGameplayEffectsContainer::GetActiveGameplayEffect(const FActiveGameplayEffectHandle Handle)
{
	for (FActiveGameplayEffect& Effect : this)
	{
		if (Effect.Handle == Handle)
		{
			return &Effect;
		}
	}
	return nullptr;
}

const FActiveGameplayEffect* FActiveGameplayEffectsContainer::GetActiveGameplayEffect(const FActiveGameplayEffectHandle Handle) const
{
	for (const FActiveGameplayEffect& Effect : this)
	{
		if (Effect.Handle == Handle)
		{
			return &Effect;
		}
	}
	return nullptr;
}

FAggregatorRef& FActiveGameplayEffectsContainer::FindOrCreateAttributeAggregator(const FGameplayAttribute& Attribute)
{
	FAggregatorRef* RefPtr = AttributeAggregatorMap.Find(Attribute);
	if (RefPtr)
	{
		return *RefPtr;
	}

	// Create a new aggregator for this attribute.
	float CurrentBaseValueOfProperty = Owner->GetNumericAttributeBase(Attribute);
	UE_LOGF(LogGameplayEffects, Log, "Creating new entry in AttributeAggregatorMap for %ls. CurrentValue: %.2f", *Attribute.GetName(), CurrentBaseValueOfProperty);

	FAggregator* NewAttributeAggregator = new FAggregator(CurrentBaseValueOfProperty);
	
	if (Attribute.IsSystemAttribute() == false)
	{
		NewAttributeAggregator->OnDirty.AddUObject(Owner, &UAbilitySystemComponent::OnAttributeAggregatorDirty, Attribute, false);
		NewAttributeAggregator->OnDirtyRecursive.AddUObject(Owner, &UAbilitySystemComponent::OnAttributeAggregatorDirty, Attribute, true);

		// Callback in case the set wants to do something
		const UAttributeSet* Set = Owner->GetAttributeSubobject(Attribute.GetAttributeSetClass());
		Set->OnAttributeAggregatorCreated(Attribute, NewAttributeAggregator);
	}

	return AttributeAggregatorMap.Add(Attribute, FAggregatorRef(NewAttributeAggregator));
}

void FActiveGameplayEffectsContainer::CleanupAttributeAggregator(const FGameplayAttribute& Attribute)
{
	FAggregatorRef* RefPtr = AttributeAggregatorMap.Find(Attribute);
	if (RefPtr)
	{
		UE_LOGF(LogGameplayEffects, Log, "Removing entry in AttributeAggregatorMap for %ls.", *Attribute.GetName());

		// No longer interested in OnDirty events for this aggregator, in case other sources call it.
		RefPtr->Data->OnDirty.RemoveAll(Owner);
		RefPtr->Data->OnDirtyRecursive.RemoveAll(Owner);

		// Remove the aggregator from the map, we no longer use it. If an attribute set gets added again and gameplay effect requires an aggregator
		// for this attribute, a new one would be created via FindOrCreateAttributeAggregator with a clean state.
		AttributeAggregatorMap.Remove(Attribute);
	}
}

void FActiveGameplayEffectsContainer::OnAttributeAggregatorDirty(FAggregator* Aggregator, FGameplayAttribute Attribute, bool bFromRecursiveCall)
{
	check(AttributeAggregatorMap.FindChecked(Attribute).Get() == Aggregator);

	// Our Aggregator has changed, we need to reevaluate this aggregator and update the current value of the attribute.
	// Note that this is not an execution, so there are no 'source' and 'target' tags to fill out in the FAggregatorEvaluateParameters.
	// ActiveGameplayEffects that have required owned tags will be turned on/off via delegates, and will add/remove themselves from attribute
	// aggregators when that happens.
	
	FAggregatorEvaluateParameters EvaluationParameters;

	if (Owner->IsNetSimulating())
	{
		if (FScopedAggregatorOnDirtyBatch::GlobalFromNetworkUpdate && Aggregator->NetUpdateID != FScopedAggregatorOnDirtyBatch::NetUpdateID)
		{
			// We are a client. The current value of this attribute is the replicated server's "final" value. We dont actually know what the 
			// server's base value is. But we can calculate it with ReverseEvaluate(). Then, we can call Evaluate with IncludePredictiveMods=true
			// to apply our mods and get an accurate predicted value.
			// 
			// It is very important that we only do this exactly one time when we get a new value from the server. Once we set the new local value for this
			// attribute below, recalculating the base would give us the wrong server value. We should only do this when we are coming directly from a network update.
			// 
			// Unfortunately there are two ways we could get here from a network update: from the ActiveGameplayEffect container being updated or from a traditional
			// OnRep on the actual attribute uproperty. Both of these could happen in a single network update, or potentially only one could happen
			// (and in fact it could be either one! the AGE container could change in a way that doesnt change the final attribute value, or we could have the base value
			// of the attribute actually be modified (e.g,. losing health or mana which only results in an OnRep and not in a AGE being applied).
			// 
			// So both paths need to lead to this function, but we should only do it one time per update. Once we update the base value, we need to make sure we dont do it again
			// until we get a new network update. GlobalFromNetworkUpdate and NetUpdateID are what do this.
			// 
			// GlobalFromNetworkUpdate - only set to true when we are coming from an OnRep or when we are coming from an ActiveGameplayEffect container net update.
			// NetUpdateID - updated once whenever an AttributeSet is received over the network. It will be incremented one time per actor that gets an update.
			//
			// See UAttributeSet::PostNetReceive().

			if (!FGameplayAttribute::IsGameplayAttributeDataProperty(Attribute.GetUProperty()))
			{
				// Legacy float attribute case requires the base value to be deduced from the final value, as it is not replicated
				const float FinalValue = Owner->GetNumericAttribute(Attribute);
				const float BaseValue = Aggregator->ReverseEvaluate(FinalValue, EvaluationParameters);
				UE_LOGF(LogGameplayEffects, Log, "Reverse Evaluated %ls. FinalValue: %.2f  BaseValue: %2.f.  Setting BaseValue.  (Role: %ls)", *Attribute.GetName(), FinalValue, BaseValue, *UEnum::GetValueAsString(Owner->GetOwnerRole()));

				Aggregator->SetBaseValue(BaseValue, false);
			}

			Aggregator->NetUpdateID = FScopedAggregatorOnDirtyBatch::NetUpdateID;
		}

		EvaluationParameters.IncludePredictiveMods = true;
	}

	const float NewValue = Aggregator->Evaluate(EvaluationParameters);
	if (EvaluationParameters.IncludePredictiveMods)
	{
		const float OldValue = Owner->GetNumericAttribute(Attribute);
		UE_LOGF(LogGameplayEffects, Log, "[%ls] Aggregator Evaluated %ls. OldValue: %.2f  NewValue: %.2f", *UEnum::GetValueAsString(Owner->GetOwnerRole()), *Attribute.GetName(), OldValue, NewValue);
	}

	InternalUpdateNumericalAttribute(Attribute, NewValue, nullptr, bFromRecursiveCall);
}

void FActiveGameplayEffectsContainer::OnMagnitudeDependencyChange(FActiveGameplayEffectHandle Handle, const FAggregator* ChangedAgg)
{
	if (Handle.IsValid())
	{
		GAMEPLAYEFFECT_SCOPE_LOCK();
		FActiveGameplayEffect* ActiveEffect = GetActiveGameplayEffect(Handle);
		if (ActiveEffect)
		{
			// This handle registered with the ChangedAgg to be notified when the aggregator changed.
			// At this point we don't know what actually needs to be updated inside this active gameplay effect.
			FGameplayEffectSpec& Spec = ActiveEffect->Spec;

			// We must update attribute aggregators only if we are actually 'on' right now, and if we are non periodic (periodic effects do their thing on execute callbacks)
			const bool MustUpdateAttributeAggregators = (ActiveEffect->bIsInhibited == false && (Spec.GetPeriod() <= UGameplayEffect::NO_PERIOD));

			// As we update our modifier magnitudes, we will update our owner's attribute aggregators. When we do this, we have to clear them first of all of our (Handle's) previous mods.
			// Since we could potentially have two mods to the same attribute, one that gets updated, and one that doesnt - we need to do this in two passes.
			TSet<FGameplayAttribute> AttributesToUpdate;

			bool bMarkedDirty = false;

			// First pass: update magnitudes of our modifiers that changed
			for(int32 ModIdx = 0; ModIdx < Spec.Modifiers.Num(); ++ModIdx)
			{
				const FGameplayModifierInfo& ModDef = Spec.Def->Modifiers[ModIdx];
				FModifierSpec& ModSpec = Spec.Modifiers[ModIdx];

				float RecalculatedMagnitude = 0.f;
				if (ModDef.ModifierMagnitude.AttemptRecalculateMagnitudeFromDependentAggregatorChange(Spec, RecalculatedMagnitude, ChangedAgg))
				{
					// If this is the first pending magnitude change, need to mark the container item dirty as well as
					// wake the owner actor from dormancy so replication works properly
					if (!bMarkedDirty)
					{
						bMarkedDirty = true;
						AActor* OwnerActor = Owner ? Owner->GetOwnerActor() : nullptr;
						if (IsNetAuthority() && OwnerActor)
						{
							OwnerActor->FlushNetDormancy();
						}
						MarkItemDirty(*ActiveEffect);
					}

					ModSpec.EvaluatedMagnitude = RecalculatedMagnitude;

					// We changed, so we need to reapply/update our spot in the attribute aggregator map
					if (MustUpdateAttributeAggregators)
					{
						AttributesToUpdate.Add(ModDef.Attribute);
					}
				}
			}

			// Second pass, update the aggregators that we need to
			UpdateAggregatorModMagnitudes(AttributesToUpdate, *ActiveEffect);
		}
	}
}

void FActiveGameplayEffectsContainer::OnStackCountChange(FActiveGameplayEffect& ActiveEffect, int32 OldStackCount, int32 NewStackCount)
{
	UE_VLOG_UELOG(Owner->GetOwnerActor(), LogGameplayEffects, Verbose, TEXT("OnStackCountChange: %s. OldStackCount: %d. NewStackCount: %d"), *ActiveEffect.GetDebugString(), OldStackCount, NewStackCount);

	MarkItemDirty(ActiveEffect);
	if (OldStackCount != NewStackCount)
	{
		// Only update attributes if stack count actually changed.
		UpdateAllAggregatorModMagnitudes(ActiveEffect);
	}

	if (ActiveEffect.Spec.Def != nullptr)
	{
		Owner->NotifyTagMap_StackCountChange(ActiveEffect.Spec.Def->GetGrantedTags());
	}

	Owner->NotifyTagMap_StackCountChange(ActiveEffect.Spec.DynamicGrantedTags);

	ActiveEffect.EventSet.OnStackChanged.Broadcast(ActiveEffect.Handle, ActiveEffect.Spec.GetStackCount(), OldStackCount);
}

void FActiveGameplayEffectsContainer::OnPredictiveGameplayEffectStackCaughtUp(FActiveGameplayEffectHandle Handle)
{
	const FActiveGameplayEffect* ActiveGE = GetActiveGameplayEffect(Handle);
	if (ActiveGE)
	{
		const int32 CurrentStackCount = ActiveGE->Spec.GetStackCount();
		const int32 LastServerReplicatedStackCount = ActiveGE->ClientCachedStackCount;

		// If there is a delta then remove the predicted stacks. 
		const int32 DeltaStacks = CurrentStackCount - LastServerReplicatedStackCount;
		if (DeltaStacks != 0)
		{
			// We can't predictively remove stacks, so this should always be true
			ensure(CurrentStackCount > LastServerReplicatedStackCount);

			// We can assume that if the StackCount is different from the ClientCachedStackedCount then the application of the new stacks has been rejected.
			// Otherwise, PostReplicatedChange would have been called first and would have set the new StackCount and its ClientCachedStackCount
			Owner->RemoveActiveGameplayEffect_AllowClientRemoval(Handle, DeltaStacks);
			UE_LOGF(LogGameplayEffects, Log, "Removing local predicted stacks [%d] in %s. Last Client's stack count [%d] and Current stack count [%d]", DeltaStacks, __func__, LastServerReplicatedStackCount, CurrentStackCount);
		}
	}
}

/** Called when the duration or starttime of an AGE has changed */
void FActiveGameplayEffectsContainer::OnDurationChange(FActiveGameplayEffect& Effect)
{
	Effect.EventSet.OnTimeChanged.Broadcast(Effect.Handle, Effect.StartWorldTime, Effect.GetDuration());
	Owner->OnGameplayEffectDurationChange(Effect);
}

void FActiveGameplayEffectsContainer::UpdateAllAggregatorModMagnitudes(FActiveGameplayEffect& ActiveEffect)
{
	// We should never be doing this for periodic effects since their mods are not persistent on attribute aggregators
	if (ActiveEffect.Spec.GetPeriod() > UGameplayEffect::NO_PERIOD)
	{
		return;
	}

	// we don't need to update inhibited effects
	if (ActiveEffect.bIsInhibited)
	{
		return;
	}

	const FGameplayEffectSpec& Spec = ActiveEffect.Spec;

	if (Spec.Def == nullptr)
	{
		UE_LOGF(LogGameplayEffects, Error, "UpdateAllAggregatorModMagnitudes called with no UGameplayEffect def.");
		return;
	}

	TSet<FGameplayAttribute> AttributesToUpdate;

	for (int32 ModIdx = 0; ModIdx < Spec.Modifiers.Num(); ++ModIdx)
	{
		const FGameplayModifierInfo& ModDef = Spec.Def->Modifiers[ModIdx];
		AttributesToUpdate.Add(ModDef.Attribute);
	}

	UpdateAggregatorModMagnitudes(AttributesToUpdate, ActiveEffect);
}

void FActiveGameplayEffectsContainer::UpdateAggregatorModMagnitudes(const TSet<FGameplayAttribute>& AttributesToUpdate, FActiveGameplayEffect& ActiveEffect)
{
	const FGameplayEffectSpec& Spec = ActiveEffect.Spec;
	for (const FGameplayAttribute& Attribute : AttributesToUpdate)
	{
		// skip over any modifiers for attributes that we don't have
		if (!Owner || Owner->HasAttributeSetForAttribute(Attribute) == false)
		{
			continue;
		}

		FAggregator* Aggregator = FindOrCreateAttributeAggregator(Attribute).Get();
		check(Aggregator);

		// Update the aggregator Mods.
		Aggregator->UpdateAggregatorMod(ActiveEffect.Handle, Attribute, Spec, ActiveEffect.PredictionKey.WasLocallyGenerated(), ActiveEffect.Handle);
	}
}

FActiveGameplayEffect* FActiveGameplayEffectsContainer::FindStackableActiveGameplayEffect(const FGameplayEffectSpec& Spec)
{
	FActiveGameplayEffect* StackableGE = nullptr;
	const UGameplayEffect* GEDef = Spec.Def;
	EGameplayEffectStackingType StackingType = GEDef->GetStackingType();

	if ((StackingType != EGameplayEffectStackingType::None) && (GEDef->DurationPolicy != EGameplayEffectDurationType::Instant))
	{
		// Iterate through GameplayEffects to see if we find a match. Note that we could cache off a handle in a map but we would still
		// do a linear search through GameplayEffects to find the actual FActiveGameplayEffect (due to unstable nature of the GameplayEffects array).
		// If this becomes a slow point in the profiler, the map may still be useful as an early out to avoid an unnecessary sweep.
		UAbilitySystemComponent* SourceASC = Spec.GetContext().GetInstigatorAbilitySystemComponent();
		for (FActiveGameplayEffect& ActiveEffect: this)
		{
			// Aggregate by source stacking additionally requires the source ability component to match
			if (ActiveEffect.Spec.Def == Spec.Def && ((StackingType == EGameplayEffectStackingType::AggregateByTarget) || (SourceASC && SourceASC == ActiveEffect.Spec.GetContext().GetInstigatorAbilitySystemComponent())))
			{
				StackableGE = &ActiveEffect;
				break;
			}
		}
	}

	return StackableGE;
}

bool FActiveGameplayEffectsContainer::HandleActiveGameplayEffectStackOverflow(const FActiveGameplayEffect& ActiveStackableGE, const FGameplayEffectSpec& OldSpec, const FGameplayEffectSpec& OverflowingSpec, FPredictionKey PredictionKey)
{
	const UGameplayEffect* StackedGE = OldSpec.Def;
	const bool bAllowOverflowApplication = !(StackedGE->bDenyOverflowApplication);
	
	// allow refreshing duration and context if the new application would cause us to overflow but we're not currently at max stacks
	using namespace UE::GameplayEffect;
	const bool bAtStackLimit = OldSpec.GetStackCount() == StackedGE->StackLimitCount;
	const bool bRefreshToLimit = !bAtStackLimit && !StackedGE->bClearStackOnOverflow && HasActiveGameplayEffectOverflowBehavior(EActiveGameplayEffectOverflowBehavior::ApplyRemainingStacksWhenOverflow);

	for (TSubclassOf<UGameplayEffect> OverflowEffect : StackedGE->OverflowEffects)
	{
		if (const UGameplayEffect* CDO = OverflowEffect.GetDefaultObject())
		{
			FGameplayEffectSpec NewGESpec;
			NewGESpec.InitializeFromLinkedSpec(CDO, OverflowingSpec);
			Owner->ApplyGameplayEffectSpecToSelf(NewGESpec, PredictionKey);
		}
	}

	// don't allow clients to predict removal of active effects due to overflow.
	if (!PredictionKey.IsLocalClientKey() || IsNetAuthority())
	{
		if (!bAllowOverflowApplication && StackedGE->bClearStackOnOverflow)
		{
			Owner->RemoveActiveGameplayEffect(ActiveStackableGE.Handle);
		}
	}

	return bAllowOverflowApplication || bRefreshToLimit;
}

bool FActiveGameplayEffectsContainer::ShouldUseMinimalReplication()
{
	return IsNetAuthority() && (Owner->ReplicationMode == EGameplayEffectReplicationMode::Minimal || Owner->ReplicationMode == EGameplayEffectReplicationMode::Mixed);
}

void FActiveGameplayEffectsContainer::SetBaseAttributeValueFromReplication(const FGameplayAttribute& Attribute, float NewBaseValue, float OldBaseValue)
{
	SetBaseAttributeValueFromReplication(Attribute, FGameplayAttributeData(NewBaseValue), FGameplayAttributeData(OldBaseValue));
}

void FActiveGameplayEffectsContainer::SetBaseAttributeValueFromReplication(const FGameplayAttribute& Attribute, const FGameplayAttributeData& NewValue, const FGameplayAttributeData& OldValue)
{
	FAggregatorRef* RefPtr = AttributeAggregatorMap.Find(Attribute);
	if (RefPtr && RefPtr->Get())
	{
		FAggregator* Aggregator = RefPtr->Get();
		if (FGameplayAttribute::IsGameplayAttributeDataProperty(Attribute.GetUProperty()))
		{
			const float ServerBaseValue = NewValue.GetBaseValue();
			const float OldBaseValue = OldValue.GetBaseValue();
			
			// Reset to the server's old value
			constexpr bool bDoNotExecuteCallbacksValue = false;
			Aggregator->SetBaseValue(OldBaseValue, bDoNotExecuteCallbacksValue);

			// Evaluate what the old value would have resulted in...  We do this to ensure the correct "old value" for the callbacks.
			FAggregatorEvaluateParameters EvaluationParameters;
			EvaluationParameters.IncludePredictiveMods = true;
			float OldEvaluatedValue = Aggregator->Evaluate(EvaluationParameters);
			Owner->SetNumericAttribute_Internal(Attribute, OldEvaluatedValue);

			// Now set the new value and go through all of the aggregations...
			Aggregator->SetBaseValue(ServerBaseValue, bDoNotExecuteCallbacksValue);
			UE_LOGF(LogGameplayEffects, Log, "SetBaseAttributeValueFromReplication [%ls]: %ls rewound to state NewBaseValue: %.2f  OldCurrentValue: %.2f", OwnerIsNetAuthority ? TEXT("Authority") : TEXT("Client"), *Attribute.AttributeName, ServerBaseValue, OldEvaluatedValue);
		}

		FScopedAggregatorOnDirtyBatch::GlobalFromNetworkUpdate = true;
		OnAttributeAggregatorDirty(Aggregator, Attribute);
		FScopedAggregatorOnDirtyBatch::GlobalFromNetworkUpdate = false;
	}
	else
	{
		// No aggregators on the client but still broadcast the dirty delegate
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (FOnGameplayAttributeChange* LegacyDelegate = AttributeChangeDelegates.Find(Attribute))
		{
			LegacyDelegate->Broadcast(NewValue.GetCurrentValue(), nullptr);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (FOnGameplayAttributeValueChange* Delegate = AttributeValueChangeDelegates.Find(Attribute))
		{
			FOnAttributeChangeData CallbackData;
			CallbackData.Attribute = Attribute;
			CallbackData.NewValue = NewValue.GetCurrentValue();
			CallbackData.OldValue = OldValue.GetCurrentValue();
			CallbackData.GEModData = nullptr;

			Delegate->Broadcast(CallbackData);
		}
	}
}

void FActiveGameplayEffectsContainer::GetAllActiveGameplayEffectSpecs(TArray<FGameplayEffectSpec>& OutSpecCopies) const
{
	for (const FActiveGameplayEffect& ActiveEffect : this)	
	{
		OutSpecCopies.Add(ActiveEffect.Spec);
	}
}

void FActiveGameplayEffectsContainer::GetGameplayEffectStartTimeAndDuration(FActiveGameplayEffectHandle Handle, float& EffectStartTime, float& EffectDuration) const
{
	EffectStartTime = UGameplayEffect::INFINITE_DURATION;
	EffectDuration = UGameplayEffect::INFINITE_DURATION;

	if (Handle.IsValid())
	{
		for (const FActiveGameplayEffect& ActiveEffect : this)
		{
			if (ActiveEffect.Handle == Handle)
			{
				EffectStartTime = ActiveEffect.StartWorldTime;
				EffectDuration = ActiveEffect.GetDuration();
				return;
			}
		}
	}

	UE_LOGF(LogGameplayEffects, Warning, "GetGameplayEffectStartTimeAndDuration called with invalid Handle: %ls", *Handle.ToString());
}

void FActiveGameplayEffectsContainer::RecomputeStartWorldTimes(const float WorldTime, const float ServerWorldTime)
{
	for (FActiveGameplayEffect& ActiveEffect : this)
	{
		ActiveEffect.RecomputeStartWorldTime(WorldTime, ServerWorldTime);
	}
}

float FActiveGameplayEffectsContainer::GetGameplayEffectMagnitude(FActiveGameplayEffectHandle Handle, FGameplayAttribute Attribute) const
{
	for (const FActiveGameplayEffect& Effect : this)
	{
		if (Effect.Handle == Handle)
		{
			for(int32 ModIdx = 0; ModIdx < Effect.Spec.Modifiers.Num(); ++ModIdx)
			{
				const FGameplayModifierInfo& ModDef = Effect.Spec.Def->Modifiers[ModIdx];
				const FModifierSpec& ModSpec = Effect.Spec.Modifiers[ModIdx];
			
				if (ModDef.Attribute == Attribute)
				{
					return ModSpec.GetEvaluatedMagnitude();
				}
			}
			
			UE_LOGF(LogGameplayEffects, Warning, "GetGameplayEffectMagnitude with Handle: %ls did not find Attribute %ls in the modifiers.", *Handle.ToString(), *Attribute.GetName());
			return -1.f;
		}
	}

	UE_LOGF(LogGameplayEffects, Warning, "GetGameplayEffectMagnitude called with invalid Handle: %ls", *Handle.ToString());
	return -1.f;
}

void FActiveGameplayEffectsContainer::SetActiveGameplayEffectLevel(FActiveGameplayEffectHandle ActiveHandle, int32 NewLevel)
{
	for (FActiveGameplayEffect& Effect : this)
	{
		if (Effect.Handle == ActiveHandle)
		{
			if (Effect.Spec.GetLevel() != NewLevel)
			{
				Effect.Spec.SetLevel(NewLevel);
				MarkItemDirty(Effect);
			
				Effect.Spec.CalculateModifierMagnitudes();
				UpdateAllAggregatorModMagnitudes(Effect);
			}
			break;
		}
	}
}

void FActiveGameplayEffectsContainer::UpdateActiveGameplayEffectSetByCallerMagnitude(FActiveGameplayEffectHandle ActiveHandle, const FGameplayTag& SetByCallerTag, float NewValue)
{
	if (FActiveGameplayEffect* Effect = GetActiveGameplayEffect(ActiveHandle))
	{
		Effect->Spec.SetSetByCallerMagnitude(SetByCallerTag, NewValue);
		Effect->Spec.CalculateModifierMagnitudes();
		MarkItemDirty(*Effect);

		UpdateAllAggregatorModMagnitudes(*Effect);
	}
}

void FActiveGameplayEffectsContainer::UpdateActiveGameplayEffectSetByCallerMagnitudes(FActiveGameplayEffectHandle ActiveHandle, const TMap<FGameplayTag, float>& NewSetByCallerValues)
{
	if (FActiveGameplayEffect* Effect = GetActiveGameplayEffect(ActiveHandle))
	{
		for (const TPair<FGameplayTag, float>& CurPair : NewSetByCallerValues)
		{
			Effect->Spec.SetSetByCallerMagnitude(CurPair.Key, CurPair.Value);
		}
		Effect->Spec.CalculateModifierMagnitudes();
		MarkItemDirty(*Effect);

		UpdateAllAggregatorModMagnitudes(*Effect);
	}
}

const FGameplayTagContainer* FActiveGameplayEffectsContainer::GetGameplayEffectSourceTagsFromHandle(FActiveGameplayEffectHandle Handle) const
{
	// @todo: Need to consider this with tag changes
	for (const FActiveGameplayEffect& Effect : this)
	{
		if (Effect.Handle == Handle)
		{
			return Effect.Spec.CapturedSourceTags.GetAggregatedTags();
		}
	}

	return nullptr;
}

const FGameplayTagContainer* FActiveGameplayEffectsContainer::GetGameplayEffectTargetTagsFromHandle(FActiveGameplayEffectHandle Handle) const
{
	// @todo: Need to consider this with tag changes
	const FActiveGameplayEffect* Effect = GetActiveGameplayEffect(Handle);
	if (Effect)
	{
		return Effect->Spec.CapturedTargetTags.GetAggregatedTags();
	}

	return nullptr;
}

void FActiveGameplayEffectsContainer::CaptureAttributeForGameplayEffect(OUT FGameplayEffectAttributeCaptureSpec& OutCaptureSpec)
{
	FAggregatorRef& AttributeAggregator = FindOrCreateAttributeAggregator(OutCaptureSpec.BackingDefinition.AttributeToCapture);
	
	if (OutCaptureSpec.BackingDefinition.bSnapshot)
	{
		OutCaptureSpec.AttributeAggregator.TakeSnapshotOf(AttributeAggregator);
	}
	else
	{
		OutCaptureSpec.AttributeAggregator = AttributeAggregator;
	}
}

void FActiveGameplayEffectsContainer::InternalUpdateNumericalAttribute(FGameplayAttribute Attribute, float NewValue, const FGameplayEffectModCallbackData* ModData, bool bFromRecursiveCall)
{
	const float OldValue = Owner->GetNumericAttribute(Attribute);
	UE_VLOG_UELOG(Owner->GetOwnerActor(), LogGameplayEffects, Log, TEXT("[%s] InternalUpdateNumericalAttribute %s OldValue = %.2f  NewValue = %.2f."), *UEnum::GetValueAsString(Owner->GetOwnerRole()), *Attribute.GetName(), OldValue, NewValue);
	Owner->SetNumericAttribute_Internal(Attribute, NewValue);
	
	if (!bFromRecursiveCall)
	{
		// Grab the actual value to broadcast from the attribute since SetCurrentValue could have been overriden
		NewValue = Owner->GetNumericAttribute(Attribute);

		// We should only have one: either cached CurrentModcallbackData, or explicit callback data passed directly in.
		if (ModData && CurrentModcallbackData)
		{
			UE_LOGF(LogGameplayEffects, Warning, "Had passed in ModData and cached CurrentModcallbackData in FActiveGameplayEffectsContainer::InternalUpdateNumericalAttribute. For attribute %ls on %ls.", *Attribute.GetName(), *Owner->GetFullName() );
		}
		
		const FGameplayEffectModCallbackData* DataToShare = ModData ? ModData : CurrentModcallbackData;

		// DEPRECATED Delegate
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (FOnGameplayAttributeChange* LegacyDelegate = AttributeChangeDelegates.Find(Attribute))
		{
			LegacyDelegate->Broadcast(NewValue, DataToShare);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// NEW Delegate
		if (FOnGameplayAttributeValueChange* NewDelegate = AttributeValueChangeDelegates.Find(Attribute))
		{
			FOnAttributeChangeData CallbackData;
			CallbackData.Attribute = Attribute;
			CallbackData.NewValue = NewValue;
			CallbackData.OldValue = OldValue;
			CallbackData.GEModData = DataToShare;
			NewDelegate->Broadcast(CallbackData);
		}
	}
	CurrentModcallbackData = nullptr;
}

void FActiveGameplayEffectsContainer::SetAttributeBaseValue(FGameplayAttribute Attribute, float NewBaseValue)
{
	if (!ensureMsgf(Owner, TEXT("%hs: This ActiveGameplayEffectsContainer has an invalid owner. Unable to set attribute %s"), __FUNCTION__, *Attribute.AttributeName))
	{
		return;
	}

	const UAttributeSet* Set = Owner->GetAttributeSubobject(Attribute.GetAttributeSetClass());
	if (!ensureMsgf(Set, TEXT("FActiveGameplayEffectsContainer::SetAttributeBaseValue: Unable to get attribute set for attribute %s"), *Attribute.AttributeName))
	{
		return;
	}

	float OldBaseValue = 0.0f;

	Set->PreAttributeBaseChange(Attribute, NewBaseValue);

	// if we're using the new attributes we should always update their base value
	bool bIsGameplayAttributeDataProperty = FGameplayAttribute::IsGameplayAttributeDataProperty(Attribute.GetUProperty());
	if (bIsGameplayAttributeDataProperty)
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(Attribute.GetUProperty());
		check(StructProperty);
		FGameplayAttributeData* DataPtr = StructProperty->ContainerPtrToValuePtr<FGameplayAttributeData>(const_cast<UAttributeSet*>(Set));
		if (ensure(DataPtr))
		{
			OldBaseValue = DataPtr->GetBaseValue();
			DataPtr->SetBaseValue(NewBaseValue);
		}
	}
	else
	{
		// If it's a simple float value, then base value == current value (unless there's an aggregator, below)
		OldBaseValue = Owner->GetNumericAttribute(Attribute);
	}

	FAggregatorRef* RefPtr = AttributeAggregatorMap.Find(Attribute);
	if (RefPtr)
	{
		FAggregator* Aggregator = RefPtr->Get();
		check(Aggregator);

		// There is an aggregator for this attribute, so set the base value. The dirty callback chain
		// will update the actual AttributeSet property value for us.		
		OldBaseValue = Aggregator->GetBaseValue();
		Aggregator->SetBaseValue(NewBaseValue);
	}
	// if there is no aggregator set the current value (base == current in this case)
	else
	{
		InternalUpdateNumericalAttribute(Attribute, NewBaseValue, nullptr);
	}

	Set->PostAttributeBaseChange(Attribute, OldBaseValue, GetAttributeBaseValue(Attribute));
}

float FActiveGameplayEffectsContainer::GetAttributeBaseValue(FGameplayAttribute Attribute) const
{
	float BaseValue = 0.f;
	if (Owner)
	{
		const UAttributeSet* AttributeSet = Owner->GetAttributeSubobject(Attribute.GetAttributeSetClass());
		if (!ensureMsgf(AttributeSet, TEXT("FActiveGameplayEffectsContainer::SetAttributeBaseValue: Unable to get attribute set for attribute %s"), *Attribute.AttributeName))
		{
			return BaseValue;
		}

		const FAggregatorRef* RefPtr = AttributeAggregatorMap.Find(Attribute);
		// if this attribute is of type FGameplayAttributeData then use the base value stored there
		if (FGameplayAttribute::IsGameplayAttributeDataProperty(Attribute.GetUProperty()))
		{
			const FStructProperty* StructProperty = CastField<FStructProperty>(Attribute.GetUProperty());
			check(StructProperty);
			const FGameplayAttributeData* DataPtr = StructProperty->ContainerPtrToValuePtr<FGameplayAttributeData>(AttributeSet);
			if (DataPtr)
			{
				BaseValue = DataPtr->GetBaseValue();
			}
		}
		// otherwise, if we have an aggregator use the base value in the aggregator
		else if (RefPtr)
		{
			BaseValue = RefPtr->Get()->GetBaseValue();
		}
		// if the attribute is just a float and there is no aggregator then the base value is the current value
		else
		{
			BaseValue = Owner->GetNumericAttribute(Attribute);
		}
	}
	else
	{
		UE_LOGF(LogGameplayEffects, Warning, "No Owner for FActiveGameplayEffectsContainer in GetAttributeBaseValue");
	}

	return BaseValue;
}

float FActiveGameplayEffectsContainer::GetEffectContribution(const FAggregatorEvaluateParameters& Parameters, FActiveGameplayEffectHandle ActiveHandle, FGameplayAttribute Attribute)
{
	FAggregatorRef Aggregator = FindOrCreateAttributeAggregator(Attribute);
	return Aggregator.Get()->EvaluateContribution(Parameters, ActiveHandle);
}

bool FActiveGameplayEffectsContainer::InternalExecuteMod(FGameplayEffectSpec& Spec, FGameplayModifierEvaluatedData& ModEvalData)
{
	SCOPE_CYCLE_COUNTER(STAT_InternalExecuteMod);

	check(Owner);

	bool bExecuted = false;

	UAttributeSet* AttributeSet = nullptr;
	UClass* AttributeSetClass = ModEvalData.Attribute.GetAttributeSetClass();
	if (AttributeSetClass && AttributeSetClass->IsChildOf(UAttributeSet::StaticClass()))
	{
		AttributeSet = const_cast<UAttributeSet*>(Owner->GetAttributeSubobject(AttributeSetClass));
	}

	if (AttributeSet)
	{
		FGameplayEffectModCallbackData ExecuteData(Spec, ModEvalData, *Owner);

		/**
		 *  This should apply 'gamewide' rules. Such as clamping Health to MaxHealth or granting +3 health for every point of strength, etc
		 */
		if (AttributeSet->PreGameplayEffectExecute(ExecuteData))
		{
			float OldValueOfProperty = Owner->GetNumericAttribute(ModEvalData.Attribute);
			ApplyModToAttribute(ModEvalData.Attribute, ModEvalData.ModifierOp, ModEvalData.Magnitude, &ExecuteData);

			FGameplayEffectModifiedAttribute* ModifiedAttribute = Spec.GetModifiedAttribute(ModEvalData.Attribute);
			if (!ModifiedAttribute)
			{
				// If we haven't already created a modified attribute holder, create it
				ModifiedAttribute = Spec.AddModifiedAttribute(ModEvalData.Attribute);
			}
			ModifiedAttribute->TotalMagnitude += ModEvalData.Magnitude;

			{
				SCOPE_CYCLE_COUNTER(STAT_PostGameplayEffectExecute);
				/** This should apply 'gamewide' rules. Such as clamping Health to MaxHealth or granting +3 health for every point of strength, etc */
				AttributeSet->PostGameplayEffectExecute(ExecuteData);
			}

#if ENABLE_VISUAL_LOG
			if (FVisualLogger::IsRecording())
			{
				DebugExecutedGameplayEffectData DebugData;
				DebugData.GameplayEffectName = Spec.Def->GetName();
				DebugData.ActivationState = "INSTANT";
				DebugData.Attribute = ModEvalData.Attribute;
				DebugData.Magnitude = Owner->GetNumericAttribute(ModEvalData.Attribute) - OldValueOfProperty;
				DebugExecutedGameplayEffects.Add(DebugData);
			}
#endif // ENABLE_VISUAL_LOG

			bExecuted = true;
		}
	}
	else
	{
		// Our owner doesn't have this attribute, so we can't do anything
		UE_LOGF(LogGameplayEffects, Log, "%ls does not have attribute %ls. Skipping modifier", *Owner->GetPathName(), *ModEvalData.Attribute.GetName());
	}

	return bExecuted;
}

void FActiveGameplayEffectsContainer::ApplyModToAttribute(const FGameplayAttribute &Attribute, TEnumAsByte<EGameplayModOp::Type> ModifierOp, float ModifierMagnitude, const FGameplayEffectModCallbackData* ModData)
{
	CurrentModcallbackData = ModData;
	float CurrentBase = GetAttributeBaseValue(Attribute);
	float NewBase = FAggregator::StaticExecModOnBaseValue(CurrentBase, ModifierOp, ModifierMagnitude);

	SetAttributeBaseValue(Attribute, NewBase);

	if (CurrentModcallbackData)
	{
		// We expect this to be cleared for us in InternalUpdateNumericalAttribute
		UE_LOGF(LogGameplayEffects, Warning, "FActiveGameplayEffectsContainer::ApplyModToAttribute CurrentModcallbackData was not consumed For attribute %ls on %ls.", *Attribute.GetName(), *Owner->GetFullName());
		CurrentModcallbackData = nullptr;
	}
}

FActiveGameplayEffect* FActiveGameplayEffectsContainer::ApplyGameplayEffectSpec(const FGameplayEffectSpec& Spec, FPredictionKey& InPredictionKey, bool& bFoundExistingStackableGE)
{
	SCOPE_CYCLE_COUNTER(STAT_ApplyGameplayEffectSpec);

	GAMEPLAYEFFECT_SCOPE_LOCK();

	if (!ensureMsgf(Spec.Def, TEXT("Tried to apply GE with no Def (context == %s)"), *Spec.GetEffectContext().ToString()))
	{
		return nullptr;
	}

	bFoundExistingStackableGE = false;

	AActor* OwnerActor = Owner ? Owner->GetOwnerActor() : nullptr;
	if (IsNetAuthority() && OwnerActor)
	{
		OwnerActor->FlushNetDormancy();
	}

	FActiveGameplayEffect* AppliedActiveGE = nullptr;
	FActiveGameplayEffect* ExistingStackableGE = FindStackableActiveGameplayEffect(Spec);

	// Whether to (re)schedule the duration check with world timer manager
	bool bSetDurationTimer = true;
	// Whether to (re)schedule the periodic tick with world timer manager
	bool bSetPeriodTimer = true;

	int32 StartingStackCount = 0;
	int32 NewStackCount = Spec.GetStackCount();

	// If we're applying a stack and the GE is configured to extend duration, 
	// we will remember how much duration was left of the existing application.
	float CarryOverDuration = 0.0f;

	// For HasActiveGameplayEffectOverflowBehavior() 
	// and EActiveGameplayEffectOverflowBehavior
	// and bAllowPredictiveApplicationOfStackingGameplayEffects
	using namespace UE::GameplayEffect;

	// Check if there's an active GE this application should stack upon
	if (ExistingStackableGE)
	{
		if (bAllowPredictiveApplicationOfStackingGameplayEffects == false)
		{
			if (!IsNetAuthority())
			{
				// Don't apply the stacking GE if we don't allow it
				return nullptr;
			}
			else
			{
				// Server invalidates the prediction key for this GE since client is not predicting it
				InPredictionKey = FPredictionKey();
			}
		}

		bFoundExistingStackableGE = true;

		FGameplayEffectSpec& ExistingSpec = ExistingStackableGE->Spec;
		StartingStackCount = ExistingSpec.GetStackCount();

		// This is now the global "being applied GE"
		UAbilitySystemGlobals::Get().SetCurrentAppliedGE(&ExistingSpec);
		
		const int32 ExistingStackLimit = ExistingSpec.Def->StackLimitCount;
		NewStackCount = ExistingSpec.GetStackCount() + Spec.GetStackCount();

		// Legacy behavior: call HandleActiveGameplayEffectStackOverflow only if we are ALREADY at the limit.
		const bool bStackCountAtLimit = ExistingSpec.GetStackCount() == ExistingStackLimit;

		// In this path, we call HandleActiveGameplayEffectStackOverflow if we have any overflowing stacks. 
		const bool bWillOverflowStacks = ExistingStackLimit > 0 && NewStackCount > ExistingSpec.Def->StackLimitCount && HasActiveGameplayEffectOverflowBehavior(EActiveGameplayEffectOverflowBehavior::HandleOverflowingStacksAlways);

		// We call HandleActiveGameplayEffectStackOverflow if we want to apply the remaining stacks up to the stack limit or for legacy behavior
		if (bStackCountAtLimit || bWillOverflowStacks)
		{
			if (!HandleActiveGameplayEffectStackOverflow(*ExistingStackableGE, ExistingSpec, Spec, InPredictionKey))
			{
				UE_VLOG_UELOG(Owner, LogGameplayEffects, Log, TEXT("Application of %s denied (StackLimit)"), *Spec.ToSimpleString());
				return nullptr;
			}
		}
		
		if (ExistingStackLimit > 0)
		{
			NewStackCount = FMath::Min(NewStackCount, ExistingStackLimit);
		}

		// Need to unregister callbacks because the source aggregators could potentially be different with the new application. They will be
		// re-registered later below, as necessary.
		ExistingSpec.CapturedRelevantAttributes.UnregisterLinkedAggregatorCallbacks(ExistingStackableGE->Handle);

		// @todo: If dynamically granted/asset tags differ (which they shouldn't), we'll actually have to diff them
		// and cause a removal and add of only the ones that have changed. For now, ensure on this happening and come
		// back to this later.
		ensureMsgf(ExistingSpec.DynamicGrantedTags == Spec.DynamicGrantedTags, TEXT("While adding a stack of the gameplay effect: %s, the old stack and the new application had different dynamically granted tags, which is currently not resolved properly!"), *Spec.Def->GetName());
		ensureMsgf(ExistingSpec.GetDynamicAssetTags() == Spec.GetDynamicAssetTags(), TEXT("While adding a stack of the gameplay effect: %s, the old stack and the new application had different dynamic asset tags, which is currently not resolved properly! Existing: %s. New: %s"), *Spec.Def->GetName(), *ExistingSpec.GetDynamicAssetTags().ToStringSimple(), *Spec.GetDynamicAssetTags().ToStringSimple());

		// If GE's stack duration policy is configured to extend duration, the remaining time and the new spec's time will be summed.
		// Store the remaining time for later use. Cache it now before overwriting existing spec's duration.
		if (ExistingSpec.Def->StackDurationRefreshPolicy == EGameplayEffectStackingDurationPolicy::ExtendDuration)
		{
			CarryOverDuration = ExistingStackableGE->GetTimeRemaining(GetWorldTime());
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// We only grant abilities on the first apply. So we *dont* want the new spec's GrantedAbilitySpecs list
		TArray<FGameplayAbilitySpecDef>	GrantedSpecTempArray(MoveTemp(ExistingStackableGE->Spec.GrantedAbilitySpecs));

		ExistingStackableGE->Spec = Spec;

		// Swap in old granted ability spec
		ExistingStackableGE->Spec.GrantedAbilitySpecs = MoveTemp(GrantedSpecTempArray);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		ExistingStackableGE->Spec.SetStackCount(NewStackCount);

		AppliedActiveGE = ExistingStackableGE;

		const UGameplayEffect* GEDef = ExistingSpec.Def;

		// Make sure the GE actually wants to refresh its duration
		// Client's predicted GE will be combined with the existing GE and we don't want to set any duration timer in this case
		if ((InPredictionKey.IsLocalClientKey() && InPredictionKey.WasLocallyGenerated()) || GEDef->StackDurationRefreshPolicy == EGameplayEffectStackingDurationPolicy::NeverRefresh)
		{
			bSetDurationTimer = false;
		}
		else
		{
			// Duration is going to be refreshed. Rebase the active GE's start time.
			RestartActiveGameplayEffectDuration(*ExistingStackableGE);
		}

		// Make sure the GE actually wants to reset its period
		if (GEDef->StackPeriodResetPolicy == EGameplayEffectStackingPeriodPolicy::NeverReset)
		{
			bSetPeriodTimer = false;
		}
	}
	else
	{
		FActiveGameplayEffectHandle NewHandle = FActiveGameplayEffectHandle::GenerateNewHandle(Owner);

		if (GameplayEffects_Internal.GetSlack() <= 0)
		{
			/**
			 *	Since we are scope locked, if we have no more slack we need to put this addition on our pending GE list, which will be moved
			 *	onto the real active GE list once the scope lock is over.  Otherwise we risk moving the pointer of the GE that scope-locked us.
			 *	
			 *	To avoid extra heap allocations, each active gameplayeffect container keeps a linked list of pending GEs. This list is allocated
			 *	on demand and re-used in subsequent pending adds. The code below will either 1) Alloc a new pending GE 2) reuse an existing pending GE.
			 *	The move operator is used to move stuff to and from these pending GEs to avoid deep copies.
			 */

			check(PendingGameplayEffectNext);
			const FActiveGameplayEffect* PreviousPendingNext = (*PendingGameplayEffectNext) ? (*PendingGameplayEffectNext)->PendingNext : nullptr;

			if (*PendingGameplayEffectNext == nullptr)
			{
				// We have no memory allocated to put our next pending GE, so make a new one.
				// [#1] If you change this, please change #1-3!!!
				AppliedActiveGE = new FActiveGameplayEffect(NewHandle, Spec, GetWorldTime(), GetServerWorldTime(), InPredictionKey);
				*PendingGameplayEffectNext = AppliedActiveGE;
			}
			else
			{
				// We already had memory allocated to put a pending GE, move in.
				// [#2] If you change this, please change #1-3!!!
				**PendingGameplayEffectNext = FActiveGameplayEffect(NewHandle, Spec, GetWorldTime(), GetServerWorldTime(), InPredictionKey);
				AppliedActiveGE = *PendingGameplayEffectNext;
			}

			// Let's check that our Pending Active GE Chain is still intact. If this triggers, the code is wrong, not the asset.
			ensureMsgf(AppliedActiveGE->PendingNext == PreviousPendingNext, TEXT("ApplyGameplayEffectSpec Code Leaked a Pending FActiveGameplayEffect while applying %s"), *AppliedActiveGE->Spec.ToSimpleString());

			// The next pending GameplayEffect goes to where our PendingNext points
			PendingGameplayEffectNext = &AppliedActiveGE->PendingNext;
		}
		else
		{

			// [#3] If you change this, please change #1-3!!!
			AppliedActiveGE = new(GameplayEffects_Internal) FActiveGameplayEffect(NewHandle, Spec, GetWorldTime(), GetServerWorldTime(), InPredictionKey);
		}

		// Make a copy of the spec with a count of 0 so that the handle overflow logic is accurate.
		FGameplayEffectSpec SpecCopy = Spec;
		SpecCopy.SetStackCount(0);
		// Account for applying multiple stacks in our initial application.
		if (!HandleActiveGameplayEffectStackOverflow(*AppliedActiveGE, SpecCopy, Spec, InPredictionKey))
		{
			UE_VLOG_UELOG(Owner, LogGameplayEffects, Log, TEXT("Application of %s denied (StackLimit)"), *Spec.ToSimpleString());
			
			return nullptr;
		}
		
		if (Spec.Def->StackLimitCount > 0)
		{
			NewStackCount = FMath::Min(NewStackCount, Spec.Def->StackLimitCount);
		}
		
		AppliedActiveGE->Spec.SetStackCount(NewStackCount);
	}

	// This is now the global "being applied GE"
	UAbilitySystemGlobals::Get().SetCurrentAppliedGE(&AppliedActiveGE->Spec);

	FGameplayEffectSpec& AppliedEffectSpec = AppliedActiveGE->Spec;
	UAbilitySystemGlobals::Get().GlobalPreGameplayEffectSpecApply(AppliedEffectSpec, Owner);

	// Make sure our target's tags are collected, so we can properly filter infinite effects
	AppliedEffectSpec.CapturedTargetTags.GetActorTags().Reset();
	Owner->GetOwnedGameplayTags(AppliedEffectSpec.CapturedTargetTags.GetActorTags());

	// Calc all of our modifier magnitudes now. Some may need to update later based on attributes changing, etc, but those should
	// be done through delegate callbacks.
	AppliedEffectSpec.CaptureAttributeDataFromTarget(Owner);
	AppliedEffectSpec.CalculateModifierMagnitudes();

	// Build ModifiedAttribute list so GCs can have magnitude info if non-period effect
	// Note: One day may want to not call gameplay cues unless ongoing tag requirements are met (will need to move this there)
	{
		const bool HasModifiedAttributes = AppliedEffectSpec.ModifiedAttributes.Num() > 0;
		const bool HasDurationAndNoPeriod = AppliedEffectSpec.Def->DurationPolicy == EGameplayEffectDurationType::HasDuration && AppliedEffectSpec.GetPeriod() == UGameplayEffect::NO_PERIOD;
		const bool HasPeriodAndNoDuration = AppliedEffectSpec.Def->DurationPolicy == EGameplayEffectDurationType::Instant &&
											AppliedEffectSpec.GetPeriod() > UGameplayEffect::NO_PERIOD;
		const bool ShouldBuildModifiedAttributeList = !HasModifiedAttributes && (HasDurationAndNoPeriod || HasPeriodAndNoDuration);
		if (ShouldBuildModifiedAttributeList)
		{
			int32 ModifierIndex = -1;
			for (const FGameplayModifierInfo& Mod : AppliedEffectSpec.Def->Modifiers)
			{
				++ModifierIndex;

				// Take magnitude from evaluated magnitudes
				float Magnitude = 0.0f;
				if (AppliedEffectSpec.Modifiers.IsValidIndex(ModifierIndex))
				{
					const FModifierSpec& ModSpec = AppliedEffectSpec.Modifiers[ModifierIndex];
					Magnitude = ModSpec.GetEvaluatedMagnitude();
				}
				
				// Add to ModifiedAttribute list if it doesn't exist already
				FGameplayEffectModifiedAttribute* ModifiedAttribute = AppliedEffectSpec.GetModifiedAttribute(Mod.Attribute);
				if (!ModifiedAttribute)
				{
					ModifiedAttribute = AppliedEffectSpec.AddModifiedAttribute(Mod.Attribute);
				}
				ModifiedAttribute->TotalMagnitude += Magnitude;
			}
		}
	}

	// Register Source and Target non snapshot capture delegates here
	AppliedEffectSpec.CapturedRelevantAttributes.RegisterLinkedAggregatorCallbacks(AppliedActiveGE->Handle);
	
	// Re-calculate the duration, as it could rely on target captured attributes
	float DefCalcDuration = 0.f;
	if (AppliedEffectSpec.AttemptCalculateDurationFromDef(DefCalcDuration))
	{
		AppliedEffectSpec.SetDuration(DefCalcDuration, false);
	}
	else if (AppliedEffectSpec.Def->DurationMagnitude.GetMagnitudeCalculationType() == EGameplayEffectMagnitudeCalculation::SetByCaller)
	{
		AppliedEffectSpec.Def->DurationMagnitude.AttemptCalculateMagnitude(AppliedEffectSpec, AppliedEffectSpec.Duration);
	}

	const float DurationBaseValue = AppliedEffectSpec.GetDuration();

	// Calculate Duration mods if we have a real duration
	if (DurationBaseValue > 0.f)
	{
		float FinalDuration = AppliedEffectSpec.CalculateModifiedDuration();

		// If GE is configured to extend duration when stacking, add the remaining time of existing effect to the new final duration
		if (CarryOverDuration > 0.0f)
		{
			FinalDuration += CarryOverDuration;
		}

		// Calculate max duration from the spec. If the new spec provides a different value for max duration, for example 
		// through different SetByCaller magnitudes, then that takes precedence over the previously applied spec's.
		float SpecMaxDuration = 0.0f;
		if (!AppliedEffectSpec.AttemptCalculateMaxDurationFromDef(SpecMaxDuration))
		{
			if (AppliedEffectSpec.Def->MaxDurationMagnitude.GetMagnitudeCalculationType() == EGameplayEffectMagnitudeCalculation::SetByCaller)
			{
				AppliedEffectSpec.Def->MaxDurationMagnitude.AttemptCalculateMagnitude(AppliedEffectSpec, SpecMaxDuration);
			}
		}

		// We cannot mod ourselves into an instant or infinite duration effect
		if (FinalDuration <= 0.f)
		{
			UE_LOGF(LogGameplayEffects, Error, "ActiveGE %ls Duration was modified to %.2f. Clamping to 0.1s duration.", *AppliedActiveGE->GetDebugString(), FinalDuration);
			FinalDuration = 0.1f;
		}
		else if (SpecMaxDuration > 0.0f && FinalDuration > SpecMaxDuration)
		{
			UE_LOGF(LogGameplayEffects, Verbose, "ActiveGE %ls Duration (%.2f) exceeds MaxDuration (%.2f). Clamping to MaxDuration.", *AppliedActiveGE->GetDebugString(), FinalDuration, SpecMaxDuration);
			FinalDuration = SpecMaxDuration;
		}

		// Store the final calculated duration on the applied spec
		AppliedEffectSpec.SetDuration(FinalDuration, /*bLockDuration=*/false);

		// Broadcast latest duration
		OnDurationChange(*AppliedActiveGE);

		// Register duration callbacks with the timer manager
		if (Owner && bSetDurationTimer)
		{
			FTimerManager& TimerManager = Owner->GetWorld()->GetTimerManager();
			FTimerDelegate Delegate = FTimerDelegate::CreateUObject(Owner, &UAbilitySystemComponent::CheckDurationExpired, AppliedActiveGE->Handle);
			TimerManager.SetTimer(AppliedActiveGE->DurationHandle, Delegate, FinalDuration, false);
			if (!ensureMsgf(AppliedActiveGE->DurationHandle.IsValid(), TEXT("Invalid Duration Handle after attempting to set duration for GE (%s) @ %.2f"), 
				*AppliedActiveGE->GetDebugString(), FinalDuration))
			{
				// Force this off next frame
				TimerManager.SetTimerForNextTick(Delegate);
			}
		}
	}
	
	// Register period callbacks with the timer manager
	if (bSetPeriodTimer && Owner && (AppliedEffectSpec.GetPeriod() > UGameplayEffect::NO_PERIOD))
	{
		FTimerManager& TimerManager = Owner->GetWorld()->GetTimerManager();
		FTimerDelegate Delegate = FTimerDelegate::CreateUObject(Owner, &UAbilitySystemComponent::ExecutePeriodicEffect, AppliedActiveGE->Handle);
			
		// The timer manager moves things from the pending list to the active list after checking the active list on the first tick so we need to execute here
		if (AppliedEffectSpec.Def->bExecutePeriodicEffectOnApplication)
		{
			TimerManager.SetTimerForNextTick(Delegate);
		}

		TimerManager.SetTimer(AppliedActiveGE->PeriodHandle, Delegate, AppliedEffectSpec.GetPeriod(), true);
	}

	if (InPredictionKey.IsLocalClientKey() == false || IsNetAuthority())	// Clients predicting a GameplayEffect must not call MarkItemDirty
	{
		MarkItemDirty(*AppliedActiveGE);
	}
	else
	{
		// Clients predicting should call MarkArrayDirty to force the internal replication map to be rebuilt.
		MarkArrayDirty();

		// Once replicated state has caught up to this prediction key, we must remove this gameplay effect.
		// This is either not a stackable GE or the first application of a stackable GE 
		if (ExistingStackableGE == nullptr)
		{
			// Once replicated state has caught up to this prediction key, we must remove this gameplay effect.
			constexpr int32 RemoveAllStacks = -1;
			if (!CVarFixPredictedTagApplication.GetValueOnAnyThread())
			{
				InPredictionKey.NewCaughtUpDelegate().BindUObject(Owner, &UAbilitySystemComponent::OnCaughtUpActiveGameplayEffect, AppliedActiveGE->Handle, RemoveAllStacks);
				InPredictionKey.NewRejectedDelegate().BindUObject(Owner, &UAbilitySystemComponent::OnRejectedActiveGameplayEffect, AppliedActiveGE->Handle, RemoveAllStacks);
			}
			else
			{
				InPredictionKey.NewRejectOrCaughtUpDelegate(FPredictionKeyEvent::CreateUObject(Owner, &UAbilitySystemComponent::RemoveActiveGameplayEffect_AllowClientRemoval, AppliedActiveGE->Handle, RemoveAllStacks));
			}
		}
		else // We have an existing Stackable GE
		{
			// Once replicated state has caught up to this prediction key, we need to readjust the stack count or at least make sure it has been handled in PostReplicatedChange
			InPredictionKey.NewRejectOrCaughtUpDelegate(FPredictionKeyEvent::CreateUObject(Owner, &UAbilitySystemComponent::OnPredictiveGameplayEffectStackCaughtUp, AppliedActiveGE->Handle));

			// @note: This should be the number of stacks that would have been removed if the application is rejected but we would have been relying on the FPredictionKey reject delegate.
			// However the reject delegate is not guaranteed to be called with 'bad' network conditions and we would end up with too many stacks on the client.
			const int32 StacksAppliedWithinThisApplication = NewStackCount - StartingStackCount;
			UE_LOGF(LogGameplayEffects, Log, "This Gameplay Effect [%ls] has applied [%d] stacks when it had to stack.", *AppliedActiveGE->GetDebugString(), StacksAppliedWithinThisApplication);
		}
	}

	// @note @todo: This is currently assuming (potentially incorrectly) that the inhibition state of a stacked GE won't change
	// as a result of stacking. In reality it could in complicated cases with differing sets of dynamically-granted tags.
	if (ExistingStackableGE)
	{
		OnStackCountChange(*ExistingStackableGE, StartingStackCount, NewStackCount);
	}
	else
	{
		// Since we are applying it locally (and possibly predictively) invoke the cues.  Unless it's an Instant Cue, in which case we're not invoking the OnActive/WhileActive cues.
		const bool bInvokePredictedEffects = (Spec.Def->DurationPolicy != EGameplayEffectDurationType::Instant);
		InternalOnActiveGameplayEffectAdded(*AppliedActiveGE, bInvokePredictedEffects);
	}

	return AppliedActiveGE;
}

/** This is called anytime a new ActiveGameplayEffect is added, on both client and server in all cases */
void FActiveGameplayEffectsContainer::InternalOnActiveGameplayEffectAdded(FActiveGameplayEffect& Effect, const bool bInvokePredictedEffects)
{
	SCOPE_CYCLE_COUNTER(STAT_OnActiveGameplayEffectAdded);

	const UGameplayEffect* EffectDef = Effect.Spec.Def;

	if (EffectDef == nullptr)
	{
		UE_LOGF(LogGameplayEffects, Error, "FActiveGameplayEffectsContainer serialized new GameplayEffect with NULL Def!");
		return;
	}

	SCOPE_CYCLE_UOBJECT(EffectDef, EffectDef);

	GAMEPLAYEFFECT_SCOPE_LOCK();

	// Add any external dependencies that might dirty the effect, if necessary
	AddCustomMagnitudeExternalDependencies(Effect);

	const bool bActive = EffectDef->OnAddedToActiveContainer(*this, Effect);
	Effect.bIsInhibited = true; // Effect has to start inhibited, so our call to Inhibit will trigger if we should be active

	FActiveGameplayEffectHandle EffectHandle = Effect.Handle;
	if (IsValid(Owner))
	{
		Owner->SetActiveGameplayEffectInhibit(MoveTemp(EffectHandle), !bActive, bInvokePredictedEffects);
	}
}

void FActiveGameplayEffectsContainer::AddActiveGameplayEffectGrantedTagsAndModifiers(FActiveGameplayEffect& Effect, bool bInvokePredictedEffects)
{
	if (Effect.Spec.Def == nullptr)
	{
		UE_LOGF(LogGameplayEffects, Error, "AddActiveGameplayEffectGrantedTagsAndModifiers called with null Def!");
		return;
	}

	if (!Owner)
	{
		UE_LOGF(LogGameplayEffects, Error, "AddActiveGameplayEffectGrantedTagsAndModifiers called with null Owner!");
		return;
	}

	GAMEPLAYEFFECT_SCOPE_LOCK();

	// Register this ActiveGameplayEffects modifiers with our Attribute Aggregators
	if (Effect.Spec.GetPeriod() <= UGameplayEffect::NO_PERIOD)
	{
		for (int32 ModIdx = 0; ModIdx < Effect.Spec.Modifiers.Num(); ++ModIdx)
		{
			if (Effect.Spec.Def->Modifiers.IsValidIndex(ModIdx) == false)
			{
				// This should not be possible but is happening for us in some replay scenerios. Possibly a backward compat issue: GE def has changed and removed modifiers, but replicated data sends the old number of mods
				ensureMsgf(false, TEXT("Spec Modifiers[%d] (max %d) is invalid with Def (%s) modifiers (max %d)"), ModIdx, Effect.Spec.Modifiers.Num(), *GetNameSafe(Effect.Spec.Def), Effect.Spec.Def ? Effect.Spec.Def->Modifiers.Num() : -1);
				continue;
			}

			const FGameplayModifierInfo &ModInfo = Effect.Spec.Def->Modifiers[ModIdx];

			// skip over any modifiers for attributes that we don't have
			if (Owner->HasAttributeSetForAttribute(ModInfo.Attribute) == false)
			{
				continue;
			}

			// Note we assume the EvaluatedMagnitude is up to do. There is no case currently where we should recalculate magnitude based on
			// Ongoing tags being met. We either calculate magnitude one time, or its done via OnDirty calls (or potentially a frequency timer one day)

			float EvaluatedMagnitude = Effect.Spec.GetModifierMagnitude(ModIdx);	// Note this could cause an attribute aggregator to be created, so must do this before calling/caching the Aggregator below!

			FAggregator* Aggregator = FindOrCreateAttributeAggregator(Effect.Spec.Def->Modifiers[ModIdx].Attribute).Get();
			if (ensure(Aggregator))
			{
				Aggregator->AddAggregatorMod(EvaluatedMagnitude, ModInfo.ModifierOp, ModInfo.EvaluationChannelSettings.GetEvaluationChannel(), &ModInfo.SourceTags, &ModInfo.TargetTags, Effect.PredictionKey.WasLocallyGenerated(), Effect.Handle);
			}
		}
	}
	else
	{
		if (Effect.Spec.Def->PeriodicInhibitionPolicy != EGameplayEffectPeriodInhibitionRemovedPolicy::NeverReset && Owner->IsOwnerActorAuthoritative())
		{
			FTimerManager& TimerManager = Owner->GetWorld()->GetTimerManager();
			FTimerDelegate Delegate = FTimerDelegate::CreateUObject(Owner, &UAbilitySystemComponent::ExecutePeriodicEffect, Effect.Handle);

			// The timer manager moves things from the pending list to the active list after checking the active list on the first tick so we need to execute here
			if (Effect.Spec.Def->PeriodicInhibitionPolicy == EGameplayEffectPeriodInhibitionRemovedPolicy::ExecuteAndResetPeriod)
			{
				TimerManager.SetTimerForNextTick(Delegate);
			}

			TimerManager.SetTimer(Effect.PeriodHandle, Delegate, Effect.Spec.GetPeriod(), true);
		}
	}
	
	if (CVarGESimOnlyTags.GetValueOnAnyThread())
	{
		// With Minimal Replication replicate tags to simulated proxies and the GE will manage the server and owning client.
		// Without Minimal Replication GE's are replicated fully to simulated proxies and will manage the tags on all clients.
		// Update our owner with the tags this GameplayEffect grants them
		Owner->UpdateTagMap(Effect.Spec.Def->GetGrantedTags(), 1, ShouldUseMinimalReplication() ? EGameplayTagReplicationState::SimulatedTagOnly : EGameplayTagReplicationState::None);
		Owner->UpdateTagMap(Effect.Spec.DynamicGrantedTags, 1, ShouldUseMinimalReplication() ? EGameplayTagReplicationState::SimulatedTagOnly : EGameplayTagReplicationState::None);
	}
	else
	{
		// Generic Replication will replicate Tags w/ Counts correctly.  So we should only apply the tags if we're predicting, otherwise we will get the tags of a Server-Initiated GE through GameplayTagCountContainer replication.
		// Iris Replication is more complicated. CVarReplicateTagCountContainerWithIris indicates that we do not have working Tag Count replication through GameplayTagCountContainerNetSerializer. Instead, we replicate using the legacy path of separated containers (Minimal/Loose).
		// The complex bShouldApplyTags case is we should still apply the tags of a Server-Initiated GE because we won't get those Tags through Iris GameplayTagCountContainerNetSerializer replication.  We should assume CVarFixPredictedTagApplication is true.
		const bool bIrisWithoutTagCountSerializer = UE::Net::ShouldUseIrisReplication() && !CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread();
		const bool bShouldApplyTags = UE::GameplayEffect::CVarFixPredictedTagApplication.GetValueOnAnyThread() ? (bInvokePredictedEffects || bIrisWithoutTagCountSerializer) : (!Effect.bPostPredictObject || bIrisWithoutTagCountSerializer);
		// Local application should be responsible for applying tags
		if (bShouldApplyTags)
		{
			// Update our owner with the tags this GameplayEffect grants them
			Owner->UpdateTagMap(Effect.Spec.Def->GetGrantedTags(), 1, bIrisWithoutTagCountSerializer ? EGameplayTagReplicationState::TagOnly : EGameplayTagReplicationState::CountToOwner);
			Owner->UpdateTagMap(Effect.Spec.DynamicGrantedTags, 1, bIrisWithoutTagCountSerializer ? EGameplayTagReplicationState::TagOnly : EGameplayTagReplicationState::CountToOwner);
		}
	}
	
	// Update our owner with the blocked ability tags this GameplayEffect adds to them
	Owner->BlockAbilitiesWithTags(Effect.Spec.Def->GetBlockedAbilityTags());

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Grant abilities
	if (IsNetAuthority() && !Owner->bSuppressGrantAbility)
	{
		for (FGameplayAbilitySpecDef& AbilitySpecDef : Effect.Spec.GrantedAbilitySpecs)
		{
			if (!AbilitySpecDef.Ability.Get())
			{
				continue;
			}

			// Only do this if we haven't assigned the ability yet! This prevents cases where stacking GEs
			// would regrant the ability every time the stack was applied
			if (AbilitySpecDef.AssignedHandle.IsValid() == false)
			{
				// Copy over SetByCaller Magnitudes (we can't do this in ::Initialize since these are set afterwards by the caller)
				AbilitySpecDef.SetByCallerTagMagnitudes = Effect.Spec.SetByCallerTagMagnitudes;

				Owner->GiveAbility( FGameplayAbilitySpec(AbilitySpecDef, Effect.Spec.GetLevel(), Effect.Handle) );
			}
		}	
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Update GameplayCue tags and events
	if (!Owner->bSuppressGameplayCues)
	{
		for (const FGameplayEffectCue& Cue : Effect.Spec.Def->GameplayCues)
		{
			// If we use Minimal/Mixed Replication, then this path will AddCue and broadcast the RPC that calls EGameplayCueEvent::OnActive (and WhileActive)
			// Note: This is going to ignore bInvokePredictedEffects and invoke them anyway (with a bunch of caveats e.g. you're the server and ignoring them)
			if (ShouldUseMinimalReplication()) // This can only be true on Authority
			{
				for (const FGameplayTag& CueTag : Cue.GameplayCueTags)
				{
					// We are now replicating the EffectContext in minimally replicated cues. It may be worth allowing this be determined on a per cue basis one day.
					// (not sending the EffectContext can make things wrong. E.g, the EffectCauser becomes the target of the GE rather than the source)
					Owner->AddGameplayCue_MinimalReplication(CueTag, Effect.Spec.GetEffectContext());
				}
			}
			else // ActiveGameplayEffects are replicating to everyone (this path can also execute on client)
			{
				// Do a pseudo-AddGameplayCue (but don't add to ActiveGameplayCues so it doesn't replicate in addition to the AGE we're replicating).
				Owner->UpdateTagMap(Cue.GameplayCueTags, 1, EGameplayTagReplicationState::None);

				if (bInvokePredictedEffects)
				{
					Owner->InvokeGameplayCueEvent(Effect.Spec, EGameplayCueEvent::OnActive);
					Owner->InvokeGameplayCueEvent(Effect.Spec, EGameplayCueEvent::WhileActive);
				}
			}
		}
	}

	// Generic notify for anyone listening
	Owner->OnActiveGameplayEffectAddedDelegateToSelf.Broadcast(Owner, Effect.Spec, Effect.Handle);
}

/** Called on server to remove a GameplayEffect and on owning client for Post-Prediction cleanup */
bool FActiveGameplayEffectsContainer::RemoveActiveGameplayEffect(FActiveGameplayEffectHandle Handle, int32 StacksToRemove, bool bPredictionRejected)
{
	// Iterating through manually since this is a removal operation and we need to pass the index into InternalRemoveActiveGameplayEffect
	int32 NumGameplayEffects = GetNumGameplayEffects();
	for (int32 ActiveGEIdx = 0; ActiveGEIdx < NumGameplayEffects; ++ActiveGEIdx)
	{
		FActiveGameplayEffect& Effect = *GetActiveGameplayEffect(ActiveGEIdx);
		if (Effect.Handle == Handle && Effect.IsPendingRemove == false)
		{
			InternalRemoveActiveGameplayEffect(ActiveGEIdx, StacksToRemove, true, bPredictionRejected);
			return true;
		}
	}

	UE_LOGF(LogGameplayEffects, Log, "RemoveActiveGameplayEffect called with invalid Handle: %ls", *Handle.ToString());
	return false;
}

void FActiveGameplayEffectsContainer::InternalExecutePeriodicGameplayEffect(FActiveGameplayEffect& ActiveEffect)
{
	GAMEPLAYEFFECT_SCOPE_LOCK();	
	if (!ActiveEffect.bIsInhibited)
	{
		FScopeCurrentGameplayEffectBeingApplied ScopedGEApplication(&ActiveEffect.Spec, Owner);

		UE_IFVLOG(
			AActor * OwnerActor = Owner->GetOwnerActor();
			UE_VLOG(OwnerActor, LogGameplayEffects, Log, TEXT("Executed Periodic Effect %s"), *ActiveEffect.Spec.Def->GetFName().ToString());
			for (const FGameplayModifierInfo& Modifier : ActiveEffect.Spec.Def->Modifiers)
			{
				float Magnitude = 0.f;
				Modifier.ModifierMagnitude.AttemptCalculateMagnitude(ActiveEffect.Spec, Magnitude);
				UE_VLOG(OwnerActor, LogGameplayEffects, Log, TEXT("         %s: %s %f"), *Modifier.Attribute.GetName(), *EGameplayModOpToString(Modifier.ModifierOp), Magnitude);
			}
		);

		// Clear modified attributes before each periodic execution
		ActiveEffect.Spec.ModifiedAttributes.Empty();

		// Execute
		ExecuteActiveEffectsFrom(ActiveEffect.Spec);

		// Invoke Delegates for periodic effects being executed
		UAbilitySystemComponent* SourceASC = ActiveEffect.Spec.GetContext().GetInstigatorAbilitySystemComponent();
		Owner->OnPeriodicGameplayEffectExecuteOnSelf(SourceASC, ActiveEffect.Spec, ActiveEffect.Handle);
		if (SourceASC)
		{
			SourceASC->OnPeriodicGameplayEffectExecuteOnTarget(Owner, ActiveEffect.Spec, ActiveEffect.Handle);
		}
	}
}

/** Called by server to actually remove a GameplayEffect */
bool FActiveGameplayEffectsContainer::InternalRemoveActiveGameplayEffect(int32 Idx, int32 StacksToRemove, bool bPrematureRemoval, bool bPredictionRejected)
{
	SCOPE_CYCLE_COUNTER(STAT_RemoveActiveGameplayEffect);

	bool IsLocked = (ScopedLockCount > 0);	// Cache off whether we were previously locked
	GAMEPLAYEFFECT_SCOPE_LOCK();			// Apply lock so no one else can change the AGE list (we may still change it if IsLocked is false)
	
	if (ensure(Idx < GetNumGameplayEffects()))
	{
		FActiveGameplayEffect& Effect = *GetActiveGameplayEffect(Idx);
		if (!ensure(!Effect.IsPendingRemove))
		{
			// This effect is already being removed. This probably means a bug at the callsite, but we can handle it gracefully here by earlying out and pretending the effect was removed.
			return true;
		}

		UE_LOGF(LogGameplayEffects, Verbose, "Removing: %ls. Auth: %d. NumToRemove: %d", *Effect.GetDebugString(), IsNetAuthority(), StacksToRemove);
		UE_IFVLOG(
			AActor * OwnerActor = Owner->GetOwnerActor();
			UE_VLOG(OwnerActor, LogGameplayEffects, Log, TEXT("Removing: %s. Auth: %d. NumToRemove: %d"), *Effect.GetDebugString(), IsNetAuthority(), StacksToRemove);
			for (const FGameplayModifierInfo& Modifier : Effect.Spec.Def->Modifiers)
			{
				float Magnitude = 0.f;
				Modifier.ModifierMagnitude.AttemptCalculateMagnitude(Effect.Spec, Magnitude);
				UE_VLOG(OwnerActor, LogGameplayEffects, Log, TEXT("         %s: %s %f"), *Modifier.Attribute.GetName(), *EGameplayModOpToString(Modifier.ModifierOp), Magnitude);
			}
		);


		FGameplayEffectRemovalInfo GameplayEffectRemovalInfo;
		GameplayEffectRemovalInfo.ActiveEffect = &Effect;
		GameplayEffectRemovalInfo.StackCount = Effect.Spec.GetStackCount();
		GameplayEffectRemovalInfo.bPrematureRemoval = bPrematureRemoval;
		GameplayEffectRemovalInfo.bPredictionRejected = bPredictionRejected;
		GameplayEffectRemovalInfo.EffectContext = Effect.Spec.GetEffectContext();
		GameplayEffectRemovalInfo.OwningASC = Owner;

		if (StacksToRemove > 0 && Effect.Spec.GetStackCount() > StacksToRemove)
		{
			// This won't be a full remove, only a change in StackCount.
			int32 StartingStackCount = Effect.Spec.GetStackCount();
			Effect.Spec.SetStackCount(StartingStackCount - StacksToRemove);
			OnStackCountChange(Effect, StartingStackCount, Effect.Spec.GetStackCount());
			return false;
		}

		// FlushNetDormancy needs to happen as early as possible.
		// Any changes made to Fast Arrays (like the Gameplay Effect Containers) won't be
		// tracked if made while the owner is still Dormant.
		const bool bIsNetAuthority = IsNetAuthority();
		AActor* OwnerActor = Owner->GetOwnerActor();
		if (bIsNetAuthority && OwnerActor)
		{
			OwnerActor->FlushNetDormancy();
		}
		
		// Invoke Remove GameplayCue event
		bool bShouldInvokeGameplayCueEvent = true;
		if (!bIsNetAuthority && Effect.PredictionKey.IsLocalClientKey() && Effect.PredictionKey.WasReceived() == false)
		{
			// This was an effect that we predicted. Don't invoke GameplayCue event if we have another GameplayEffect that shares the same predictionkey and was received from the server
			if (HasReceivedEffectWithPredictedKey(Effect.PredictionKey))
			{
				bShouldInvokeGameplayCueEvent = false;
			}
		}

		// Don't invoke the GC event if the effect is inhibited, and thus the GC is already not active
		bShouldInvokeGameplayCueEvent &= !Effect.bIsInhibited;

		// Mark the effect pending remove, and remove all side effects from the effect
		InternalOnActiveGameplayEffectRemoved(Effect, bShouldInvokeGameplayCueEvent, GameplayEffectRemovalInfo);

		// Check world validity in case RemoveActiveGameplayEffect is called during world teardown
		if (UWorld* World = Owner->GetWorld())
		{
			if (Effect.DurationHandle.IsValid())
			{
				World->GetTimerManager().ClearTimer(Effect.DurationHandle);
			}
			if (Effect.PeriodHandle.IsValid())
			{
				World->GetTimerManager().ClearTimer(Effect.PeriodHandle);
			}
		}

		bool ModifiedArray = false;

		// Finally remove the ActiveGameplayEffect
		if (IsLocked)
		{
			// We are locked, so this removal is now pending.
			PendingRemoves++;

			UE_LOGF(LogGameplayEffects, Verbose, "Begin Pending Remove: %ls. Auth: %d", *Effect.GetDebugString(), IsNetAuthority());
		}
		else
		{
			// Not locked, so do the removal right away.

			// If we are not scope locked, then there is no way this idx should be referring to something on the pending add list.
			// It is possible to remove a GE that is pending add, but it would happen while the scope lock is still in effect, resulting
			// in a pending remove being set.
			check(Idx < GameplayEffects_Internal.Num());

			GameplayEffects_Internal.RemoveAtSwap(Idx);
			ModifiedArray = true;
		}

		MarkArrayDirty();

		// Hack: force netupdate on owner. This isn't really necessary in real gameplay but is nice
		// during debugging where breakpoints or pausing can mess up network update times. Open issue
		// with network team.
		Owner->GetOwner()->ForceNetUpdate();
		
		return ModifiedArray;
	}

	UE_LOGF(LogGameplayEffects, Warning, "InternalRemoveActiveGameplayEffect called with invalid index: %d", Idx);
	return false;
}

/** Called by client and server: This does cleanup that has to happen whether the effect is being removed locally or due to replication */
void FActiveGameplayEffectsContainer::InternalOnActiveGameplayEffectRemoved(FActiveGameplayEffect& Effect, bool bInvokePredictedEffects, const FGameplayEffectRemovalInfo& GameplayEffectRemovalInfo)
{
	SCOPE_CYCLE_COUNTER(STAT_OnActiveGameplayEffectRemoved);
	SCOPE_CYCLE_UOBJECT(EffectDef, Effect.Spec.Def);

	// Mark the effect as pending removal
	Effect.IsPendingRemove = true;

	if (Effect.Spec.Def)
	{
		// Only Need to update tags and modifiers if the gameplay effect is active.
		if (!Effect.bIsInhibited)
		{
			RemoveActiveGameplayEffectGrantedTagsAndModifiers(Effect, bInvokePredictedEffects, GameplayEffectRemovalInfo.bPredictionRejected);
		}

		RemoveCustomMagnitudeExternalDependencies(Effect);
	}
	else
	{
		UE_LOGF(LogGameplayEffects, Warning, "InternalOnActiveGameplayEffectRemoved called with no GameplayEffect: %ls", *Effect.Handle.ToString());
	}

	Effect.EventSet.OnEffectRemoved.Broadcast(GameplayEffectRemovalInfo);

	OnActiveGameplayEffectRemovedDelegate.Broadcast(Effect);
}

void FActiveGameplayEffectsContainer::RemoveActiveGameplayEffectGrantedTagsAndModifiers(const FActiveGameplayEffect& Effect, bool bInvokePredictedEffects, bool bPredictionRejected)
{
	// Update AttributeAggregators: remove mods from this ActiveGE Handle
	if (Effect.Spec.GetPeriod() <= UGameplayEffect::NO_PERIOD)
	{
		for (const FGameplayModifierInfo& Mod : Effect.Spec.Def->Modifiers)
		{
			if (Mod.Attribute.IsValid())
			{
				if (const FAggregatorRef* RefPtr = AttributeAggregatorMap.Find(Mod.Attribute))
				{
					RefPtr->Get()->RemoveAggregatorMod(Effect.Handle);
				}
			}
		}
	}

	if (CVarGESimOnlyTags.GetValueOnAnyThread())
	{
		// With Minimal Replication replicate tags to simulated proxies and the GE will manage the server and owning client.
		// Without Minimal Replication GE's are replicated fully to simulated proxies and will manage the tags on all clients.
		// Update our owner with the tags this GameplayEffect grants them
		Owner->UpdateTagMap(Effect.Spec.Def->GetGrantedTags(), -1, ShouldUseMinimalReplication() ? EGameplayTagReplicationState::SimulatedTagOnly : EGameplayTagReplicationState::None);
		Owner->UpdateTagMap(Effect.Spec.DynamicGrantedTags, -1, ShouldUseMinimalReplication() ? EGameplayTagReplicationState::SimulatedTagOnly : EGameplayTagReplicationState::None);
	}
	else
	{
		// Generic Replication will replicate Tags w/ Counts correctly.  So we should only apply the tags if we're predicting, otherwise we will get the tags of a Server-Initiated GE through GameplayTagCountContainer replication.
		// Iris Replication is more complicated. CVarReplicateTagCountContainerWithIris indicates that we do not have working Tag Count replication through GameplayTagCountContainerNetSerializer. Instead, we replicate using the legacy path of separated containers (Minimal/Loose).
		// The complex bShouldApplyTags case is we should still apply the tags of a Server-Initiated GE because we won't get those Tags through Iris GameplayTagCountContainerNetSerializer replication.  We should assume CVarFixPredictedTagApplication is true.
		const bool bIrisWithoutTagCountSerializer = UE::Net::ShouldUseIrisReplication() && !CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread();
		const bool bShouldApplyTags = UE::GameplayEffect::CVarFixPredictedTagApplication.GetValueOnAnyThread() ? (bInvokePredictedEffects || bIrisWithoutTagCountSerializer) : (IsNetAuthority() || bPredictionRejected || bIrisWithoutTagCountSerializer);
		// Local application should be responsible for applying tags
		if (bShouldApplyTags)
		{
			// Update gameplaytag count and broadcast delegate if we are at 0
			Owner->UpdateTagMap(Effect.Spec.Def->GetGrantedTags(), -1, bIrisWithoutTagCountSerializer ? EGameplayTagReplicationState::TagOnly : EGameplayTagReplicationState::CountToOwner);
			Owner->UpdateTagMap(Effect.Spec.DynamicGrantedTags, -1, bIrisWithoutTagCountSerializer ? EGameplayTagReplicationState::TagOnly : EGameplayTagReplicationState::CountToOwner);
		}
	}

	// Update our owner with the blocked ability tags this GameplayEffect adds to them
	Owner->UnBlockAbilitiesWithTags(Effect.Spec.Def->GetBlockedAbilityTags());

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Cancel/remove granted abilities
	if (IsNetAuthority())
	{
		for (const FGameplayAbilitySpecDef& AbilitySpecDef : Effect.Spec.GrantedAbilitySpecs)
		{
			if (AbilitySpecDef.AssignedHandle.IsValid())
			{
				switch(AbilitySpecDef.RemovalPolicy)
				{
				case EGameplayEffectGrantedAbilityRemovePolicy::CancelAbilityImmediately:
					{
						Owner->ClearAbility(AbilitySpecDef.AssignedHandle);
						break;
					}
				case EGameplayEffectGrantedAbilityRemovePolicy::RemoveAbilityOnEnd:
					{
						Owner->SetRemoveAbilityOnEnd(AbilitySpecDef.AssignedHandle);
						break;
					}
				default:
					{
						// Do nothing to granted ability
						break;
					}
				}
			}
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Update GameplayCue tags and events
	if (!Owner->bSuppressGameplayCues)
	{
		for (const FGameplayEffectCue& Cue : Effect.Spec.Def->GameplayCues)
		{
			// If we use Minimal/Mixed Replication, then this will cause EGameplayCueEvent::Removed
			if (ShouldUseMinimalReplication())
			{
				for (const FGameplayTag& CueTag : Cue.GameplayCueTags)
				{
					Owner->RemoveGameplayCue_MinimalReplication(CueTag);
				}
			}
			else
			{
				// Perform pseudo-RemoveCue (without affecting ActiveGameplayCues, as we were not inserted there - see AddActiveGameplayEffectGrantedTagsAndModifiers)
				Owner->UpdateTagMap(Cue.GameplayCueTags, -1, EGameplayTagReplicationState::None);

				if (bInvokePredictedEffects)
				{
					Owner->InvokeGameplayCueEvent(Effect.Spec, EGameplayCueEvent::Removed);
				}
			}
		}
	}
}

void FActiveGameplayEffectsContainer::AddCustomMagnitudeExternalDependencies(FActiveGameplayEffect& Effect)
{
	const UGameplayEffect* GEDef = Effect.Spec.Def;
	if (GEDef)
	{
		const bool bIsNetAuthority = IsNetAuthority();

		// Check each modifier to see if it has a custom external dependency
		for (const FGameplayModifierInfo& CurMod : GEDef->Modifiers)
		{
			TSubclassOf<UGameplayModMagnitudeCalculation> ModCalcClass = CurMod.ModifierMagnitude.GetCustomMagnitudeCalculationClass();
			if (ModCalcClass)
			{
				const UGameplayModMagnitudeCalculation* ModCalcClassCDO = ModCalcClass->GetDefaultObject<UGameplayModMagnitudeCalculation>();
				if (ModCalcClassCDO)
				{
					// Only register the dependency if acting as net authority or if the calculation class has indicated it wants non-net authorities
					// to be allowed to perform the calculation as well
					UWorld* World = Owner ? Owner->GetWorld() : nullptr;
					FOnExternalGameplayModifierDependencyChange* ExternalDelegate = ModCalcClassCDO->GetExternalModifierDependencyMulticast(Effect.Spec, World);
					if (ExternalDelegate && (bIsNetAuthority || ModCalcClassCDO->ShouldAllowNonNetAuthorityDependencyRegistration()))
					{
						FObjectKey ModCalcClassKey(*ModCalcClass);
						FCustomModifierDependencyHandle* ExistingDependencyHandle = CustomMagnitudeClassDependencies.Find(ModCalcClassKey);
						
						// If the dependency has already been registered for this container, just add the handle of the effect to the existing list
						if (ExistingDependencyHandle)
						{
							ExistingDependencyHandle->ActiveEffectHandles.Add(Effect.Handle);
						}
						// If the dependency is brand new, bind an update to the delegate and cache off the handle
						else
						{
							FCustomModifierDependencyHandle& NewDependencyHandle = CustomMagnitudeClassDependencies.Add(ModCalcClassKey);
							NewDependencyHandle.ActiveDelegateHandle = ExternalDelegate->AddRaw(this, &FActiveGameplayEffectsContainer::OnCustomMagnitudeExternalDependencyFired, ModCalcClass);
							NewDependencyHandle.ActiveEffectHandles.Add(Effect.Handle);
						}
					}
				}
			}
		}
	}
}

void FActiveGameplayEffectsContainer::RemoveCustomMagnitudeExternalDependencies(FActiveGameplayEffect& Effect)
{
	const UGameplayEffect* GEDef = Effect.Spec.Def;
	if (GEDef && CustomMagnitudeClassDependencies.Num() > 0)
	{
		const bool bIsNetAuthority = IsNetAuthority();
		for (const FGameplayModifierInfo& CurMod : GEDef->Modifiers)
		{
			TSubclassOf<UGameplayModMagnitudeCalculation> ModCalcClass = CurMod.ModifierMagnitude.GetCustomMagnitudeCalculationClass();
			if (ModCalcClass)
			{
				const UGameplayModMagnitudeCalculation* ModCalcClassCDO = ModCalcClass->GetDefaultObject<UGameplayModMagnitudeCalculation>();
				if (ModCalcClassCDO)
				{
					UWorld* World = Owner ? Owner->GetWorld() : nullptr;
					FOnExternalGameplayModifierDependencyChange* ExternalDelegate = ModCalcClassCDO->GetExternalModifierDependencyMulticast(Effect.Spec, World);
					if (ExternalDelegate && (bIsNetAuthority || ModCalcClassCDO->ShouldAllowNonNetAuthorityDependencyRegistration()))
					{
						FObjectKey ModCalcClassKey(*ModCalcClass);
						FCustomModifierDependencyHandle* ExistingDependencyHandle = CustomMagnitudeClassDependencies.Find(ModCalcClassKey);
						
						// If this dependency was bound for this effect, remove it
						if (ExistingDependencyHandle)
						{
							ExistingDependencyHandle->ActiveEffectHandles.Remove(Effect.Handle);

							// If this was the last effect for this dependency, unbind the delegate and remove the dependency entirely
							if (ExistingDependencyHandle->ActiveEffectHandles.Num() == 0)
							{
								ExternalDelegate->Remove(ExistingDependencyHandle->ActiveDelegateHandle);
								CustomMagnitudeClassDependencies.Remove(ModCalcClassKey);
							}
						}
					}
				}
			}
		}
	}
}

void FActiveGameplayEffectsContainer::OnCustomMagnitudeExternalDependencyFired(TSubclassOf<UGameplayModMagnitudeCalculation> MagnitudeCalculationClass)
{
	if (MagnitudeCalculationClass)
	{
		FObjectKey ModCalcClassKey(*MagnitudeCalculationClass);
		FCustomModifierDependencyHandle* ExistingDependencyHandle = CustomMagnitudeClassDependencies.Find(ModCalcClassKey);
		if (ExistingDependencyHandle)
		{
			const bool bIsNetAuthority = IsNetAuthority();
			const UGameplayModMagnitudeCalculation* CalcClassCDO = MagnitudeCalculationClass->GetDefaultObject<UGameplayModMagnitudeCalculation>();
			const bool bRequiresDormancyFlush = CalcClassCDO ? !CalcClassCDO->ShouldAllowNonNetAuthorityDependencyRegistration() : false;

			const TSet<FActiveGameplayEffectHandle>& HandlesNeedingUpdate = ExistingDependencyHandle->ActiveEffectHandles;

			// Iterate through all effects, updating the ones that specifically respond to the external dependency being updated
			for (FActiveGameplayEffect& Effect : this)
			{
				if (HandlesNeedingUpdate.Contains(Effect.Handle))
				{
					if (bIsNetAuthority)
					{
						// By default, a dormancy flush should be required here. If a calculation class has requested that
						// non-net authorities can respond to external dependencies, the dormancy flush is skipped as a desired optimization
						AActor* OwnerActor = Owner ? Owner->GetOwnerActor() : nullptr;
						if (bRequiresDormancyFlush && OwnerActor)
						{
							OwnerActor->FlushNetDormancy();
						}

						MarkItemDirty(Effect);
					}

					Effect.Spec.CalculateModifierMagnitudes();
					UpdateAllAggregatorModMagnitudes(Effect);
				}
			}
		}
	}
}

void FActiveGameplayEffectsContainer::RestartActiveGameplayEffectDuration(FActiveGameplayEffect& ActiveGameplayEffect)
{
	ActiveGameplayEffect.StartServerWorldTime = GetServerWorldTime();
	ActiveGameplayEffect.CachedStartServerWorldTime = ActiveGameplayEffect.StartServerWorldTime;
	ActiveGameplayEffect.StartWorldTime = GetWorldTime();
	MarkItemDirty(ActiveGameplayEffect);
}

ELifetimeCondition FActiveGameplayEffectsContainer::GetReplicationCondition() const
{
	if (Owner)
	{
		const EGameplayEffectReplicationMode ReplicationMode = Owner->ReplicationMode;
		switch (ReplicationMode)
		{
			case EGameplayEffectReplicationMode::Minimal:
			{
				return COND_Never;
			}

			case EGameplayEffectReplicationMode::Mixed:
			{
				if (IsNetAuthority())
				{
					return COND_OwnerOnly;
				}
				else
				{
					return COND_ReplayOrOwner;
				}
			}

			case EGameplayEffectReplicationMode::Full:
			default:
			{
				return COND_None;
			}
		}
	}

	// If there's no owner we assume full replication is required.
	return COND_None;
}

bool FActiveGameplayEffectsContainer::NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms)
{
	// NOTE: Changes to this testing needs to be reflected in GetReplicatedCondition(), which is what is used if the container has a dynamic lifetime condition, COND_Dynamic, set.
	// These tests are only needed when sending and the instance isn't using the COND_Dynamic lifetime condition, in which case NetDeltaSerialize wouldn't be called unless it should replicate.
	if (!IsUsingReplicationCondition() && DeltaParms.Writer != nullptr && Owner != nullptr)
	{
		EGameplayEffectReplicationMode ReplicationMode = Owner->ReplicationMode;
		if (ReplicationMode == EGameplayEffectReplicationMode::Minimal)
		{
			return false;
		}
		else if (ReplicationMode == EGameplayEffectReplicationMode::Mixed)
		{
			if (UPackageMapClient* Client = Cast<UPackageMapClient>(DeltaParms.Map))
			{
				UNetConnection* Connection = Client->GetConnection();

				// Even in mixed mode, we should always replicate out to client side recorded replays so it has all information.
				if (!Connection->IsReplay() || IsNetAuthority())
				{
					// In mixed mode, we only want to replicate to the owner of this channel, minimal replication
					// data will go to everyone else.
					const AActor* ParentOwner = Owner->GetOwner();
					const UNetConnection* ParentOwnerNetConnection = ParentOwner->GetNetConnection();
					if (!ParentOwner->IsOwnedBy(Connection->OwningActor) && (ParentOwnerNetConnection != Connection))
					{
						bool bIsChildConnection = false;
						for (UChildConnection* ChildConnection : Connection->Children)
						{
							if (ParentOwner->IsOwnedBy(ChildConnection->OwningActor) || (ChildConnection == ParentOwnerNetConnection))
							{
								bIsChildConnection = true;
								break;
							}
						}
						if (!bIsChildConnection)
						{
							return false;
						}
					}
				}
			}
		}
	}

	bool RetVal = FastArrayDeltaSerialize<FActiveGameplayEffect>(GameplayEffects_Internal, DeltaParms, *this);

	// This section has been moved into new PostReplicatedReceive() method that is invoked after every call to FastArrayDeltaSerialize<> that results in data being modified
	
	return RetVal;
}

void FActiveGameplayEffectsContainer::PostReplicatedReceive(const FFastArraySerializer::FPostReplicatedReceiveParameters& Parameters)
{
	// After the array has been replicated, invoke GC events ONLY if the effect is not inhibited
	// We postpone this check because in the same net update we could receive multiple GEs that affect if one another is inhibited	
	if (Owner != nullptr)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_ActiveGameplayEffectsContainer_NetDeltaSerialize_CheckRepGameplayCues);

		if ( LIKELY(UE::GameplayEffect::bSkipUnmappedReferencesCheckForGameplayCues) )
		{
			if (Owner->IsReadyForGameplayCues())
			{
				Owner->HandleDeferredGameplayCues(this);
			}
		}
		else
		{
// Keep this until we have actually deprecated the parameter just in case.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (!Parameters.bHasMoreUnmappedReferences) // Do not invoke GCs when we have missing information (like AActor*s in EffectContext)
			{
				NumConsecutiveUnmappedReferencesDebug = 0;
				if (Owner->IsReadyForGameplayCues())
				{
					Owner->HandleDeferredGameplayCues(this);
				}
			}
			else
			{
				++NumConsecutiveUnmappedReferencesDebug;

				constexpr uint32 HighNumberOfConsecutiveUnmappedRefs = 30;
				ensureMsgf(NumConsecutiveUnmappedReferencesDebug < HighNumberOfConsecutiveUnmappedRefs, TEXT("%hs: bHasMoreUnmappedReferences is preventing GameplayCues from firing"), __func__);
				UE_CLOGF((NumConsecutiveUnmappedReferencesDebug % HighNumberOfConsecutiveUnmappedRefs) == 0, LogAbilitySystem, Error, "%s: bHasMoreUnmappedReferences is preventing GameplayCues from firing (%u consecutive misses)", __func__, NumConsecutiveUnmappedReferencesDebug);
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

void FActiveGameplayEffectsContainer::Uninitialize()
{
	UWorld* World = IsValid(Owner) ? Owner->GetWorld() : nullptr;
	for (FActiveGameplayEffect& CurEffect : this)
	{
		RemoveCustomMagnitudeExternalDependencies(CurEffect);

		// Remove any timer delegates that were scheduled to tick or end the gameplay effect
		if (World)
		{
			if (CurEffect.DurationHandle.IsValid())
			{
				World->GetTimerManager().ClearTimer(CurEffect.DurationHandle);
			}
			if (CurEffect.PeriodHandle.IsValid())
			{
				World->GetTimerManager().ClearTimer(CurEffect.PeriodHandle);
			}
		}
	}
	ensure(CustomMagnitudeClassDependencies.Num() == 0);
}

bool FActiveGameplayEffectsContainer::IsServerWorldTimeAvailable() const
{
	if (!IsValid(Owner))
	{
		return false;
	}

	UWorld* World = Owner->GetWorld();
	if (!World)
	{
		return false;
	}

	AGameStateBase* GameState = World->GetGameState();
	return (GameState != nullptr);
}

float FActiveGameplayEffectsContainer::GetServerWorldTime() const
{
	UWorld* World = Owner->GetWorld();
	AGameStateBase* GameState = World->GetGameState();
	if (GameState)
	{
		return GameState->GetServerWorldTimeSeconds();
	}

	return World->GetTimeSeconds();
}

float FActiveGameplayEffectsContainer::GetWorldTime() const
{
	UWorld *World = Owner->GetWorld();
	return World->GetTimeSeconds();
}

void FActiveGameplayEffectsContainer::CheckDuration(FActiveGameplayEffectHandle Handle)
{
	GAMEPLAYEFFECT_SCOPE_LOCK();
	// Intentionally iterating through only the internal list since we need to pass the index for removal
	// and pending effects will never need to be checked for duration expiration (They will be added to the real list first)
	for (int32 ActiveGEIdx = 0; ActiveGEIdx < GameplayEffects_Internal.Num(); ++ActiveGEIdx)
	{
		FActiveGameplayEffect& Effect = GameplayEffects_Internal[ActiveGEIdx];
		if (Effect.Handle != Handle)
		{
			continue;
		}

		if (Effect.IsPendingRemove)
		{
			// break is this effect is pending remove. 
			// (Note: don't combine this with the above if statement that is looking for the effect via handle, since we want to stop iteration if we find a matching handle but are pending remove).
			break;
		}

		// The duration may have changed since we registered this callback with the timer manager.
		// Make sure that this effect should really be destroyed now
		float Duration = Effect.GetDuration();

		// If it's an infinite Gameplay Effect, we don't want to potentially remove it (or set any timers)
		// If it was an instant Gameplay Effect, it would never make it into this container.
		if (Duration <= 0.0f)
		{
			break;
		}

		float CurrentTime = GetWorldTime();

		int32 StacksToRemove = -2;
		bool RefreshStartTime = false;
		bool RefreshDurationTimer = false;
		bool CheckForFinalPeriodicExec = false;

		if (((Effect.StartWorldTime + Duration) < CurrentTime) || FMath::IsNearlyZero(CurrentTime - Duration - Effect.StartWorldTime, KINDA_SMALL_NUMBER))
		{
			// Figure out what to do based on the expiration policy
			switch(Effect.Spec.Def->GetStackExpirationPolicy())
			{
			case EGameplayEffectStackingExpirationPolicy::ClearEntireStack:
				StacksToRemove = -1; // Remove all stacks
				CheckForFinalPeriodicExec = true;					
				break;

			case EGameplayEffectStackingExpirationPolicy::RemoveSingleStackAndRefreshDuration:
				StacksToRemove = 1;
				CheckForFinalPeriodicExec = (Effect.Spec.GetStackCount() == 1);
				RefreshStartTime = true;
				RefreshDurationTimer = true;
				break;
			case EGameplayEffectStackingExpirationPolicy::RefreshDuration:
				RefreshStartTime = true;
				RefreshDurationTimer = true;
				break;
			};					
		}
		else
		{
			// Effect isn't finished, just refresh its duration timer
			RefreshDurationTimer = true;
		}

		FTimerManager& TimerManager = Owner->GetWorld()->GetTimerManager();
		if (CheckForFinalPeriodicExec)
		{
			// This gameplay effect has hit its duration. Check if it needs to execute one last time before removing it.
			if (Effect.PeriodHandle.IsValid() && TimerManager.TimerExists(Effect.PeriodHandle))
			{
				float PeriodTimeRemaining = TimerManager.GetTimerRemaining(Effect.PeriodHandle);
				if (PeriodTimeRemaining <= KINDA_SMALL_NUMBER && !Effect.bIsInhibited)
				{
					InternalExecutePeriodicGameplayEffect(Effect);

					// The call to ExecuteActiveEffectsFrom in InternalExecutePeriodicGameplayEffect could cause this effect to be explicitly removed
					// (for example it could kill the owner and cause the effect to be wiped via death).
					// In that case, we need to early out instead of possibly continuing to the below calls to InternalRemoveActiveGameplayEffect
					if ( Effect.IsPendingRemove )
					{
						break;
					}
				}

				// Forcibly clear the periodic ticks because this effect is going to be removed
				TimerManager.ClearTimer(Effect.PeriodHandle);
			}
		}

		if (StacksToRemove >= -1)
		{
			InternalRemoveActiveGameplayEffect(ActiveGEIdx, StacksToRemove, false);
		}

		if (RefreshStartTime)
		{
			RestartActiveGameplayEffectDuration(Effect);

			// Since the GE will go through another iteration, broadcast OnDurationChange
			// allowing game code to respond to the new start and end time.
			OnDurationChange(Effect);
		}

		if (RefreshDurationTimer)
		{
			// Always reset the timer, since the duration might have been modified
			FTimerDelegate Delegate = FTimerDelegate::CreateUObject(Owner, &UAbilitySystemComponent::CheckDurationExpired, Effect.Handle);

			float NewTimerDuration = (Effect.StartWorldTime + Duration) - CurrentTime;
			TimerManager.SetTimer(Effect.DurationHandle, Delegate, NewTimerDuration, false);

			if (Effect.DurationHandle.IsValid() == false)
			{
				UE_LOGF(LogGameplayEffects, Warning, "Failed to set new timer in ::CheckDuration. Timer trying to be set for: %.2f. Removing GE instead", NewTimerDuration);
				if (!Effect.IsPendingRemove)
				{
					InternalRemoveActiveGameplayEffect(ActiveGEIdx, -1, false);
				}
				check(Effect.IsPendingRemove);
			}
		}

		break;
	}
}

bool FActiveGameplayEffectsContainer::CanApplyAttributeModifiers(const UGameplayEffect* GameplayEffect, float Level, const FGameplayEffectContextHandle& EffectContext)
{
	SCOPE_CYCLE_COUNTER(STAT_GameplayEffectsCanApplyAttributeModifiers);

	FGameplayEffectSpec	Spec(GameplayEffect, EffectContext, Level);

	Spec.CalculateModifierMagnitudes();
	
	for(int32 ModIdx = 0; ModIdx < Spec.Modifiers.Num(); ++ModIdx)
	{
		const FGameplayModifierInfo& ModDef = Spec.Def->Modifiers[ModIdx];
		const FModifierSpec& ModSpec = Spec.Modifiers[ModIdx];
	
		// It only makes sense to check additive operators
		if (ModDef.ModifierOp == EGameplayModOp::Additive)
		{
			if (!ModDef.Attribute.IsValid())
			{
				continue;
			}
			const UAttributeSet* Set = Owner->GetAttributeSubobject(ModDef.Attribute.GetAttributeSetClass());
			float CurrentValue = ModDef.Attribute.GetNumericValueChecked(Set);
			float CostValue = ModSpec.GetEvaluatedMagnitude();

			if (CurrentValue + CostValue < 0.f)
			{
				return false;
			}
		}
	}
	return true;
}

TArray<float> FActiveGameplayEffectsContainer::GetActiveEffectsTimeRemaining(const FGameplayEffectQuery& Query) const
{
	SCOPE_CYCLE_COUNTER(STAT_GameplayEffectsGetActiveEffectsTimeRemaining);

	float CurrentTime = GetWorldTime();

	TArray<float>	ReturnList;

	for (const FActiveGameplayEffect& Effect : this)
	{
		if (!Query.Matches(Effect))
		{
			continue;
		}

		float Elapsed = CurrentTime - Effect.StartWorldTime;
		float Duration = Effect.GetDuration();

		ReturnList.Add(Duration - Elapsed);
	}

	// Note: keep one return location to avoid copy operation.
	return ReturnList;
}

TArray<float> FActiveGameplayEffectsContainer::GetActiveEffectsDuration(const FGameplayEffectQuery& Query) const
{
	SCOPE_CYCLE_COUNTER(STAT_GameplayEffectsGetActiveEffectsDuration);

	TArray<float>	ReturnList;

	for (const FActiveGameplayEffect& Effect : this)
	{
		if (!Query.Matches(Effect))
		{
			continue;
		}

		ReturnList.Add(Effect.GetDuration());
	}

	// Note: keep one return location to avoid copy operation.
	return ReturnList;
}

TArray<TPair<float,float>> FActiveGameplayEffectsContainer::GetActiveEffectsTimeRemainingAndDuration(const FGameplayEffectQuery& Query) const
{
	SCOPE_CYCLE_COUNTER(STAT_GameplayEffectsGetActiveEffectsTimeRemainingAndDuration);

	TArray<TPair<float,float>> ReturnList;

	float CurrentTime = GetWorldTime();

	for (const FActiveGameplayEffect& Effect : this)
	{
		if (!Query.Matches(Effect))
		{
			continue;
		}

		float Elapsed = CurrentTime - Effect.StartWorldTime;
		float Duration = Effect.GetDuration();

		ReturnList.Emplace(Duration - Elapsed, Duration);
	}

	// Note: keep one return location to avoid copy operation.
	return ReturnList;
}

TArray<FActiveGameplayEffectHandle> FActiveGameplayEffectsContainer::GetActiveEffects(const FGameplayEffectQuery& Query) const
{
	SCOPE_CYCLE_COUNTER(STAT_GameplayEffectsGetActiveEffects);

	TArray<FActiveGameplayEffectHandle> ReturnList;

	for (const FActiveGameplayEffect& Effect : this)
	{
		if (!Query.Matches(Effect))
		{
			continue;
		}

		ReturnList.Add(Effect.Handle);
	}

	return ReturnList;
}

float FActiveGameplayEffectsContainer::GetActiveEffectsEndTime(const FGameplayEffectQuery& Query, TArray<AActor*>& Instigators) const
{
	float EndTime = 0.f;
	float Duration = 0.f;
	GetActiveEffectsEndTimeAndDuration(Query, EndTime, Duration, Instigators);
	return EndTime;
}

bool FActiveGameplayEffectsContainer::GetActiveEffectsEndTimeAndDuration(const FGameplayEffectQuery& Query, float& EndTime, float& Duration, TArray<AActor*>& Instigators) const
{
	bool FoundSomething = false;
	
	for (const FActiveGameplayEffect& Effect : this)
	{
		if (!Query.Matches(Effect))
		{
			continue;
		}
		
		FoundSomething = true;

		float ThisEndTime = Effect.GetEndTime();
		if (ThisEndTime <= UGameplayEffect::INFINITE_DURATION)
		{
			// This is an infinite duration effect, so this end time is indeterminate
			EndTime = -1.f;
			Duration = -1.f;
			return true;
		}

		if (ThisEndTime > EndTime)
		{
			EndTime = ThisEndTime;
			Duration = Effect.GetDuration();
		}

		Instigators.AddUnique(Effect.Spec.GetEffectContext().GetOriginalInstigator());
	}
	return FoundSomething;
}

TArray<FActiveGameplayEffectHandle> FActiveGameplayEffectsContainer::GetAllActiveEffectHandles() const
{
	SCOPE_CYCLE_COUNTER(STAT_GameplayEffectsGetAllActiveEffectHandles);

	TArray<FActiveGameplayEffectHandle> ReturnList;

	for (const FActiveGameplayEffect& Effect : this)
	{
		ReturnList.Add(Effect.Handle);
	}

	return ReturnList;
}

void FActiveGameplayEffectsContainer::ModifyActiveEffectStartTime(FActiveGameplayEffectHandle Handle, float StartTimeDiff)
{
	SCOPE_CYCLE_COUNTER(STAT_GameplayEffectsModifyActiveEffectStartTime);

	FActiveGameplayEffect* Effect = GetActiveGameplayEffect(Handle);

	if (Effect)
	{
		Effect->StartWorldTime += StartTimeDiff;
		Effect->StartServerWorldTime += StartTimeDiff;

		// Check if we are now expired
		CheckDuration(Handle);

		// Broadcast to anyone listening
		OnDurationChange(*Effect);

		MarkItemDirty(*Effect);
	}
}

int32 FActiveGameplayEffectsContainer::RemoveActiveEffects(const FGameplayEffectQuery& Query, int32 StacksToRemove)
{
	// Force a lock because the removals could cause other removals earlier in the array, so iterating backwards is not safe all by itself
	GAMEPLAYEFFECT_SCOPE_LOCK();
	int32 NumRemoved = 0;

	// Manually iterating through in reverse because this is a removal operation
	for (int32 idx = GetNumGameplayEffects() - 1; idx >= 0; --idx)
	{
		const FActiveGameplayEffect& Effect = *GetActiveGameplayEffect(idx);
		if (Effect.IsPendingRemove == false && Query.Matches(Effect))
		{
			InternalRemoveActiveGameplayEffect(idx, StacksToRemove, true);
			++NumRemoved;
		}
	}
	return NumRemoved;
}

int32 FActiveGameplayEffectsContainer::GetActiveEffectCount(const FGameplayEffectQuery& Query, bool bEnforceOnGoingCheck) const
{
	int32 Count = 0;

	for (const FActiveGameplayEffect& Effect : this)
	{
		if (!Effect.bIsInhibited || !bEnforceOnGoingCheck)
		{
			if (Query.Matches(Effect))
			{
				Count += Effect.Spec.GetStackCount();
			}
		}
	}

	return Count;
}

FOnGameplayAttributeChange& FActiveGameplayEffectsContainer::RegisterGameplayAttributeEvent(FGameplayAttribute Attribute)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return AttributeChangeDelegates.FindOrAdd(Attribute);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FOnGameplayAttributeValueChange& FActiveGameplayEffectsContainer::GetGameplayAttributeValueChangeDelegate(FGameplayAttribute Attribute)
{
	return AttributeValueChangeDelegates.FindOrAdd(Attribute);
}

bool FActiveGameplayEffectsContainer::HasReceivedEffectWithPredictedKey(FPredictionKey PredictionKey) const
{
	for (const FActiveGameplayEffect& Effect : this)
	{
		if (Effect.PredictionKey == PredictionKey && Effect.PredictionKey.WasReceived() == true)
		{
			return true;
		}
	}

	return false;
}

bool FActiveGameplayEffectsContainer::HasPredictedEffectWithPredictedKey(FPredictionKey PredictionKey) const
{
	for (const FActiveGameplayEffect& Effect : this)
	{
		if (Effect.PredictionKey == PredictionKey && Effect.PredictionKey.WasReceived() == false)
		{
			return true;
		}
	}

	return false;
}

void FActiveGameplayEffectsContainer::GetActiveGameplayEffectDataByAttribute(TMultiMap<FGameplayAttribute, FActiveGameplayEffectsContainer::DebugExecutedGameplayEffectData>& EffectMap) const
{
	EffectMap.Empty();

	// Add all of the active gameplay effects
	for (const FActiveGameplayEffect& Effect : this)
	{
		if (Effect.Spec.Def && Effect.Spec.Modifiers.Num() == Effect.Spec.Def->Modifiers.Num())
		{
			for (int32 Idx = 0; Idx < Effect.Spec.Modifiers.Num(); ++Idx)
			{
				FActiveGameplayEffectsContainer::DebugExecutedGameplayEffectData Data;
				Data.Attribute = Effect.Spec.Def->Modifiers[Idx].Attribute;
				Data.ActivationState = Effect.bIsInhibited ? TEXT("INHIBITED") : TEXT("ACTIVE");
				Data.GameplayEffectName = Effect.Spec.Def->GetName();
				Data.ModifierOp = Effect.Spec.Def->Modifiers[Idx].ModifierOp;
				Data.Magnitude = Effect.Spec.Modifiers[Idx].GetEvaluatedMagnitude();
				if (Effect.Spec.GetStackCount() > 1)
				{
					Data.Magnitude = GameplayEffectUtilities::ComputeStackedModifierMagnitude(Data.Magnitude, Effect.Spec.GetStackCount(), Data.ModifierOp);
				}
				Data.StackCount = Effect.Spec.GetStackCount();

				EffectMap.Add(Data.Attribute, Data);
			}
		}
	}
#if ENABLE_VISUAL_LOG
	// Add the executed gameplay effects if we recorded them
	for (FActiveGameplayEffectsContainer::DebugExecutedGameplayEffectData Data : DebugExecutedGameplayEffects)
	{
		EffectMap.Add(Data.Attribute, Data);
	}
#endif // ENABLE_VISUAL_LOG
}

#if ENABLE_VISUAL_LOG
void FActiveGameplayEffectsContainer::DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const
{
	TMultiMap<FGameplayAttribute, FActiveGameplayEffectsContainer::DebugExecutedGameplayEffectData> EffectMap;

	GetActiveGameplayEffectDataByAttribute(EffectMap);

	if (EffectMap.Num() > 0)
	{
		FVisualLogStatusCategory AttributeModStatus;
		AttributeModStatus.Category = TEXT("Attribute Mods");

		// For each attribute that was modified go through all of its modifiers and list them
		TArray<FGameplayAttribute> AttributeKeys;
		EffectMap.GetKeys(AttributeKeys);

		for (const FGameplayAttribute& Attribute : AttributeKeys)
		{
			FVisualLogStatusCategory AttributeModCategory;
			AttributeModCategory.Category = Attribute.GetName();

			TArray<FActiveGameplayEffectsContainer::DebugExecutedGameplayEffectData> AttributeEffects;
			EffectMap.MultiFind(Attribute, AttributeEffects);

			float CombinedModifierValue = 0.f;
			for (const FActiveGameplayEffectsContainer::DebugExecutedGameplayEffectData& DebugData : AttributeEffects)
			{
				AttributeModCategory.Add(DebugData.GameplayEffectName, DebugData.ActivationState);
				AttributeModCategory.Add(TEXT("Magnitude"), FString::Printf(TEXT("%f"), DebugData.Magnitude));

				if (DebugData.ActivationState != "INHIBITED")
				{
					CombinedModifierValue += DebugData.Magnitude;
				}
			}

			AttributeModCategory.Add(TEXT("Total Modification"), FString::Printf(TEXT("%f"), CombinedModifierValue));
			AttributeModStatus.AddChild(AttributeModCategory);
		}

		Snapshot->Status.Add(AttributeModStatus);
	}

	FVisualLogStatusCategory ActiveEffectsCategory;
	ActiveEffectsCategory.Category = TEXT("Active Effects");

	// Iterating through manually since this is a removal operation and we need to pass the index into InternalRemoveActiveGameplayEffect
	int32 NumGameplayEffects = GetNumGameplayEffects();
	for (int32 ActiveGEIdx = 0; ActiveGEIdx < NumGameplayEffects; ++ActiveGEIdx)
	{
		FVisualLogStatusCategory ActiveGELog;

		const FActiveGameplayEffect& Effect = *GetActiveGameplayEffect(ActiveGEIdx);
		ActiveGELog.Category = FString::Printf(TEXT("[%s] %s"), *Effect.Handle.ToString(), *Effect.Spec.ToSimpleString());

		FVisualLogStatusCategory SpecLogStatus = Effect.Spec.GrabVisLogStatus();
		ActiveGELog.AddChild(SpecLogStatus);

		if (Effect.StartWorldTime > 0.0f)
		{
			ActiveGELog.Add(TEXT("StartWorldTime"), FString::Printf(TEXT("%.3f"), Effect.StartWorldTime));
		}

		if (Effect.StartServerWorldTime > 0.0f)
		{
			ActiveGELog.Add(TEXT("ServerStartWorldTime"), FString::Printf(TEXT("%.3f"), Effect.StartServerWorldTime));
		}

		if (Effect.DurationHandle.IsValid())
		{
			ActiveGELog.Add(TEXT("Duration"), FString::Printf(TEXT("%.3f"), Effect.GetDuration()));
			ActiveGELog.Add(TEXT("TimeRemaining"), FString::Printf(TEXT("%.3f"), Effect.GetTimeRemaining(GetWorldTime())));
		}

		if (Effect.PredictionKey.IsValidKey())
		{
			ActiveGELog.Add(TEXT("PredictionKey"), Effect.PredictionKey.ToString());
		}

		if (Effect.IsPendingRemove)
		{
			ActiveGELog.Add(TEXT("IsPendingRemove"), TEXT("true"));
		}

		if (Effect.bIsInhibited)
		{
			ActiveGELog.Add(TEXT("Inhibited"), TEXT("true"));
		}

		ActiveEffectsCategory.AddChild(ActiveGELog);
	}

	Snapshot->Status.Add(ActiveEffectsCategory);
}
#endif // ENABLE_VISUAL_LOG

void FActiveGameplayEffectsContainer::DebugCyclicAggregatorBroadcasts(FAggregator* TriggeredAggregator)
{
	for (auto It = AttributeAggregatorMap.CreateIterator(); It; ++It)
	{
		FAggregatorRef AggregatorRef = It.Value();
		FGameplayAttribute Attribute = It.Key();
		if (FAggregator* Aggregator = AggregatorRef.Get())
		{
			if (Aggregator == TriggeredAggregator)
			{
				UE_LOGF(LogGameplayEffects, Warning, " Attribute %ls was the triggered aggregator (%ls)", *Attribute.GetName(), *Owner->GetPathName());
			}
			else if (Aggregator->BroadcastingDirtyCount > 0)
			{
				UE_LOGF(LogGameplayEffects, Warning, " Attribute %ls is broadcasting dirty (%ls)", *Attribute.GetName(), *Owner->GetPathName());
			}
			else
			{
				continue;
			}

			for (FActiveGameplayEffectHandle Handle : Aggregator->Dependents)
			{
				UAbilitySystemComponent* ASC = Handle.GetOwningAbilitySystemComponent();
				if (ASC)
				{
					UE_LOGF(LogGameplayEffects, Warning, "  Dependant (%ls) GE: %ls", *ASC->GetPathName(), *GetNameSafe(ASC->GetGameplayEffectDefForHandle(Handle)));
				}
			}
		}
	}
}

void FActiveGameplayEffectsContainer::CloneFrom(const FActiveGameplayEffectsContainer& Source)
{
	// Make a full copy of the source's gameplay effects
	GameplayEffects_Internal = Source.GameplayEffects_Internal;

	// Build our AttributeAggregatorMap by deep copying the source's
	AttributeAggregatorMap.Reset();

	TArray< TPair<FAggregatorRef, FAggregatorRef> >	SwappedAggregators;

	for (auto& It : Source.AttributeAggregatorMap)
	{
		const FGameplayAttribute& Attribute = It.Key;
		const FAggregatorRef& SourceAggregatorRef = It.Value;

		FAggregatorRef& NewAggregatorRef = FindOrCreateAttributeAggregator(Attribute);
		FAggregator* NewAggregator = NewAggregatorRef.Get();
		FAggregator::FOnAggregatorDirty OnDirtyDelegate = NewAggregator->OnDirty;

		// Make full copy of the source aggregator
		*NewAggregator = *SourceAggregatorRef.Get();

		// But restore the OnDirty delegate to point to our proxy ASC
		NewAggregator->OnDirty = OnDirtyDelegate;

		TPair<FAggregatorRef, FAggregatorRef> SwappedPair;
		SwappedPair.Key = SourceAggregatorRef;
		SwappedPair.Value = NewAggregatorRef;

		SwappedAggregators.Add(SwappedPair);
	}

	// Make all of our copied GEs "unique" by giving them a new handle
	TMap<FActiveGameplayEffectHandle, FActiveGameplayEffectHandle> SwappedHandles;

	for (FActiveGameplayEffect& Effect : this)
	{
		// Copy the Spec's context so we can modify it
		Effect.Spec.DuplicateEffectContext();
		Effect.Spec.SetupAttributeCaptureDefinitions();

		// For client only, capture attribute data since this data is constructed for replicated active gameplay effects by default
		Effect.Spec.RecaptureAttributeDataForClone(Source.Owner, Owner);

		FActiveGameplayEffectHandle& NewHandleRef = SwappedHandles.Add(Effect.Handle);
		Effect.Spec.CapturedRelevantAttributes.UnregisterLinkedAggregatorCallbacks(Effect.Handle);

		Effect.Handle = FActiveGameplayEffectHandle::GenerateNewHandle(Owner);
		Effect.Spec.CapturedRelevantAttributes.RegisterLinkedAggregatorCallbacks(Effect.Handle);
		NewHandleRef = Effect.Handle;

		// Update any captured attribute references to the proxy source.
		for (TPair<FAggregatorRef, FAggregatorRef>& SwapAgg : SwappedAggregators)
		{
			Effect.Spec.CapturedRelevantAttributes.SwapAggregator( SwapAgg.Key, SwapAgg.Value );
		}
	}	

	// Now go through our aggregator map and replace dependency references to the source's GEs with our GEs.
	for (auto& It : AttributeAggregatorMap)
	{
		FGameplayAttribute& Attribute = It.Key;
		FAggregatorRef& AggregatorRef = It.Value;
		FAggregator* Aggregator = AggregatorRef.Get();
		if (Aggregator)
		{
			Aggregator->OnActiveEffectDependenciesSwapped(SwappedHandles);
		}
	}

	// Broadcast dirty on everything so that the UAttributeSet properties get updated
	for (auto& It : AttributeAggregatorMap)
	{
		FAggregatorRef& AggregatorRef = It.Value;
		AggregatorRef.Get()->BroadcastOnDirty();
	}
}



// -----------------------------------------------------------------

FGameplayEffectQuery::FGameplayEffectQuery()
	: EffectSource(nullptr),
	EffectDefinition(nullptr)
{
}

FGameplayEffectQuery::FGameplayEffectQuery(const FGameplayEffectQuery& Other)
{
	*this = Other;
}


FGameplayEffectQuery::FGameplayEffectQuery(FActiveGameplayEffectQueryCustomMatch InCustomMatchDelegate)
	: CustomMatchDelegate(InCustomMatchDelegate),
	EffectSource(nullptr),
	EffectDefinition(nullptr)
{
}

FGameplayEffectQuery::FGameplayEffectQuery(FGameplayEffectQuery&& Other)
{
	*this = MoveTemp(Other);
}

FGameplayEffectQuery& FGameplayEffectQuery::operator=(FGameplayEffectQuery&& Other)
{
	CustomMatchDelegate = MoveTemp(Other.CustomMatchDelegate);
	CustomMatchDelegate_BP = MoveTemp(Other.CustomMatchDelegate_BP);
	OwningTagQuery = MoveTemp(Other.OwningTagQuery);
	EffectTagQuery = MoveTemp(Other.EffectTagQuery);
	SourceAggregateTagQuery = MoveTemp(Other.SourceAggregateTagQuery);
	SourceTagQuery = MoveTemp(Other.SourceTagQuery);
	ModifyingAttribute = MoveTemp(Other.ModifyingAttribute);
	EffectSource = Other.EffectSource;
	EffectDefinition = Other.EffectDefinition;
	IgnoreHandles = MoveTemp(Other.IgnoreHandles);
	return *this;
}

FGameplayEffectQuery& FGameplayEffectQuery::operator=(const FGameplayEffectQuery& Other)
{
	CustomMatchDelegate = Other.CustomMatchDelegate;
	CustomMatchDelegate_BP = Other.CustomMatchDelegate_BP;
	OwningTagQuery = Other.OwningTagQuery;
	EffectTagQuery = Other.EffectTagQuery;
	SourceAggregateTagQuery = Other.SourceAggregateTagQuery;
	SourceTagQuery = Other.SourceTagQuery;
	ModifyingAttribute = Other.ModifyingAttribute;
	EffectSource = Other.EffectSource;
	EffectDefinition = Other.EffectDefinition;
	IgnoreHandles = Other.IgnoreHandles;
	return *this;
}


bool FGameplayEffectQuery::Matches(const FActiveGameplayEffect& Effect) const
{
	// since all of these query conditions must be met to be considered a match, failing
	// any one of them means we can return false

	// Anything in the ignore handle list is an immediate non-match
	if (IgnoreHandles.Contains(Effect.Handle))
	{
		return false;
	}

	if (CustomMatchDelegate.IsBound())
	{
		if (CustomMatchDelegate.Execute(Effect) == false)
		{
			return false;
		}
	}

	if (CustomMatchDelegate_BP.IsBound())
	{
		bool bDelegateMatches = false;
		CustomMatchDelegate_BP.Execute(Effect, bDelegateMatches);
		if (bDelegateMatches == false)
		{
			return false;
		}
	}

	return Matches(Effect.Spec);

}

bool FGameplayEffectQuery::Matches(const FGameplayEffectSpec& Spec) const
{
	if (Spec.Def == nullptr)
	{
		return false;
	}

	if (OwningTagQuery.IsEmpty() == false)
	{
		// Combine tags from the definition and the spec into one container to match queries that may span both
		// static to avoid memory allocations every time we do a query
		check(IsInGameThread());
		static FGameplayTagContainer TargetTags;
		TargetTags.Reset();
		
		// This functionality is kept the same as prior to UE5.3:
		// I believe this is a side-effect of not understanding the previous Asset vs. Granted tags in UGameplayEffect.
		// We're keeping the functionality and actually comparing TargetTags against both Asset & Granted.
		// The one change in 5.3: We're now also comparing against DynamicAssetTags to keep it consistent.
		Spec.GetAllAssetTags(TargetTags);
		Spec.GetAllGrantedTags(TargetTags);
		
		if (OwningTagQuery.Matches(TargetTags) == false)
		{
			return false;
		}
	}

	if (EffectTagQuery.IsEmpty() == false)
	{
		// Combine tags from the definition and the spec into one container to match queries that may span both
		// static to avoid memory allocations every time we do a query
		check(IsInGameThread());
		static FGameplayTagContainer GETags;
		GETags.Reset();

		Spec.GetAllAssetTags(GETags);

		if (EffectTagQuery.Matches(GETags) == false)
		{
			return false;
		}
	}

	if (SourceAggregateTagQuery.IsEmpty() == false)
	{
		FGameplayTagContainer const& SourceAggregateTags = *Spec.CapturedSourceTags.GetAggregatedTags();
		if (SourceAggregateTagQuery.Matches(SourceAggregateTags) == false)
		{
			return false;
		}
	}

	if (SourceTagQuery.IsEmpty() == false)
	{
		FGameplayTagContainer const& SourceSpecTags = Spec.CapturedSourceTags.GetSpecTags();
		if (SourceTagQuery.Matches(SourceSpecTags) == false)
		{
			return false;
		}
	}

	// if we are looking for ModifyingAttribute go over each of the Spec Modifiers and check the Attributes
	if (ModifyingAttribute.IsValid())
	{
		bool bEffectModifiesThisAttribute = false;

		for (int32 ModIdx = 0; ModIdx < Spec.Modifiers.Num(); ++ModIdx)
		{
			const FGameplayModifierInfo& ModDef = Spec.Def->Modifiers[ModIdx];
			const FModifierSpec& ModSpec = Spec.Modifiers[ModIdx];

			if (ModDef.Attribute == ModifyingAttribute)
			{
				bEffectModifiesThisAttribute = true;
				break;
			}
		}
		if (bEffectModifiesThisAttribute == false)
		{
			// effect doesn't modify the attribute we are looking for, no match
			return false;
		}
	}

	// check source object
	if (EffectSource != nullptr)
	{
		if (Spec.GetEffectContext().GetSourceObject() != EffectSource)
		{
			return false;
		}
	}

	// check definition
	if (EffectDefinition != nullptr)
	{
		if (Spec.Def != EffectDefinition.GetDefaultObject())
		{
			return false;
		}
	}

	return true;
}

bool FGameplayEffectQuery::IsEmpty() const
{
	return 
	(
		!CustomMatchDelegate.IsBound() &&
		!CustomMatchDelegate_BP.IsBound() &&
		OwningTagQuery.IsEmpty() &&
		EffectTagQuery.IsEmpty() &&
		SourceAggregateTagQuery.IsEmpty() &&
		SourceTagQuery.IsEmpty() &&
		!ModifyingAttribute.IsValid() &&
		!EffectSource &&
		!EffectDefinition
	);
}

// static
FGameplayEffectQuery FGameplayEffectQuery::MakeQuery_MatchAnyOwningTags(const FGameplayTagContainer& InTags)
{
	SCOPE_CYCLE_COUNTER(STAT_MakeGameplayEffectQuery);
	FGameplayEffectQuery OutQuery;
	OutQuery.OwningTagQuery = FGameplayTagQuery::MakeQuery_MatchAnyTags(InTags);
	return OutQuery;
}

// static
FGameplayEffectQuery FGameplayEffectQuery::MakeQuery_MatchAllOwningTags(const FGameplayTagContainer& InTags)
{
	SCOPE_CYCLE_COUNTER(STAT_MakeGameplayEffectQuery);
	FGameplayEffectQuery OutQuery;
	OutQuery.OwningTagQuery = FGameplayTagQuery::MakeQuery_MatchAllTags(InTags);
	return OutQuery;
}

// static
FGameplayEffectQuery FGameplayEffectQuery::MakeQuery_MatchNoOwningTags(const FGameplayTagContainer& InTags)
{
	SCOPE_CYCLE_COUNTER(STAT_MakeGameplayEffectQuery);
	FGameplayEffectQuery OutQuery;
	OutQuery.OwningTagQuery = FGameplayTagQuery::MakeQuery_MatchNoTags(InTags);
	return OutQuery;
}

// static
FGameplayEffectQuery FGameplayEffectQuery::MakeQuery_MatchAnyEffectTags(const FGameplayTagContainer& InTags)
{
	SCOPE_CYCLE_COUNTER(STAT_MakeGameplayEffectQuery);
	FGameplayEffectQuery OutQuery;
	OutQuery.EffectTagQuery = FGameplayTagQuery::MakeQuery_MatchAnyTags(InTags);
	return OutQuery;
}

// static
FGameplayEffectQuery FGameplayEffectQuery::MakeQuery_MatchAllEffectTags(const FGameplayTagContainer& InTags)
{
	SCOPE_CYCLE_COUNTER(STAT_MakeGameplayEffectQuery);
	FGameplayEffectQuery OutQuery;
	OutQuery.EffectTagQuery = FGameplayTagQuery::MakeQuery_MatchAllTags(InTags);
	return OutQuery;
}

// static
FGameplayEffectQuery FGameplayEffectQuery::MakeQuery_MatchNoEffectTags(const FGameplayTagContainer& InTags)
{
	SCOPE_CYCLE_COUNTER(STAT_MakeGameplayEffectQuery);
	FGameplayEffectQuery OutQuery;
	OutQuery.EffectTagQuery = FGameplayTagQuery::MakeQuery_MatchNoTags(InTags);
	return OutQuery;
}

// static
FGameplayEffectQuery FGameplayEffectQuery::MakeQuery_MatchAnySourceSpecTags(const FGameplayTagContainer& InTags)
{
	SCOPE_CYCLE_COUNTER(STAT_MakeGameplayEffectQuery);
	FGameplayEffectQuery OutQuery;
	OutQuery.SourceTagQuery = FGameplayTagQuery::MakeQuery_MatchAnyTags(InTags);
	return OutQuery;
}

// static
FGameplayEffectQuery FGameplayEffectQuery::MakeQuery_MatchAllSourceSpecTags(const FGameplayTagContainer& InTags)
{
	SCOPE_CYCLE_COUNTER(STAT_MakeGameplayEffectQuery);
	FGameplayEffectQuery OutQuery;
	OutQuery.SourceTagQuery = FGameplayTagQuery::MakeQuery_MatchAllTags(InTags);
	return OutQuery;
}

// static
FGameplayEffectQuery FGameplayEffectQuery::MakeQuery_MatchNoSourceSpecTags(const FGameplayTagContainer& InTags)
{
	SCOPE_CYCLE_COUNTER(STAT_MakeGameplayEffectQuery);
	FGameplayEffectQuery OutQuery;
	OutQuery.SourceTagQuery = FGameplayTagQuery::MakeQuery_MatchNoTags(InTags);
	return OutQuery;
}

bool FGameplayEffectQuery::operator==(const FGameplayEffectQuery& Other) const
{
	return CustomMatchDelegate_BP == Other.CustomMatchDelegate_BP &&
	OwningTagQuery == Other.OwningTagQuery &&
	EffectTagQuery == Other.EffectTagQuery &&
	SourceAggregateTagQuery == Other.SourceAggregateTagQuery &&
	SourceTagQuery == Other.SourceTagQuery &&
	ModifyingAttribute == Other.ModifyingAttribute &&
	EffectSource == Other.EffectSource &&
	EffectDefinition == Other.EffectDefinition &&
	IgnoreHandles == Other.IgnoreHandles;
}

bool FGameplayEffectQuery::operator!=(const FGameplayEffectQuery& Other) const
{
	return !(*this == Other);
}

bool FGameplayModifierInfo::operator==(const FGameplayModifierInfo& Other) const
{
	if (Attribute != Other.Attribute)
	{
		return false;
	}

	if (ModifierOp != Other.ModifierOp)
	{
		return false;
	}

	if (ModifierMagnitude != Other.ModifierMagnitude)
	{
		return false;
	}

	if (EvaluationChannelSettings != Other.EvaluationChannelSettings)
	{
		return false;
	}

	if (SourceTags != Other.SourceTags)
	{
		return false;
	}

	if (TargetTags != Other.TargetTags)
	{
		return false;
	}

	return true;
}

bool FGameplayModifierInfo::operator!=(const FGameplayModifierInfo& Other) const
{
	return !(*this == Other);
}

void FInheritedTagContainer::UpdateInheritedTagProperties(const FInheritedTagContainer* Parent)
{
	// Make sure we've got a fresh start
	CombinedTags.Reset();

	// Re-add the Parent's tags except the one's we have removed
	if (Parent)
	{
		for (auto Itr = Parent->CombinedTags.CreateConstIterator(); Itr; ++Itr)
		{
			if (!Itr->MatchesAny(Removed))
			{
				CombinedTags.AddTag(*Itr);
			}
		}
	}

	// Add our own tags
	for (auto Itr = Added.CreateConstIterator(); Itr; ++Itr)
	{
		// Remove trumps add for explicit matches but not for parent tags.
		// This lets us remove all inherited tags starting with Foo but still add Foo.Bar
		// Keep in mind: When adding Foo.Bar, you are implicitly adding Foo (its parent tags).
		if (!Removed.HasTagExact(*Itr))
		{
			CombinedTags.AddTag(*Itr);
		}
	}

#if WITH_EDITOR
	// Notify any open editor windows that we should refresh these properties, specifically for refreshing CombinedTags.
	// TODO: This needs to call a different delegate as this the general editor refresh delegate cannot be called while loading gameplay effects
	// UGameplayTagsManager::OnEditorRefreshGameplayTagTree.Broadcast();
#endif
}

void FInheritedTagContainer::ApplyTo(FGameplayTagContainer& ApplyToContainer) const
{
	if (ApplyToContainer.IsEmpty() && !CombinedTags.IsEmpty())
	{
		// This is a fast path where our CombinedTags were already computed and
		// the container we're apply to is empty, then we simply assign the CombinedTags.
		ApplyToContainer = CombinedTags;
	}
	else
	{
		// Layer in the actual changes.  It's best understood with an example:
		// Current: Foo.Car  Remove: Foo  Add: Foo.Bar
		// Remove all tags that match the filter (e.g. Foo removes Foo.Car)
		// Then, add in all of the Adds which don't match exactly (e.g. Foo.Bar would be added, but attempting to add exactly Foo would not be allowed since Removes take precedence).
		// Keep in mind: When Foo.Bar is added, you are implicitly adding Foo (its parent tags).  Thus you can remove Foo.* and still add Foo.Bar.
		FGameplayTagContainer RemovesThatApply = Removed.Filter(ApplyToContainer);
		FGameplayTagContainer RemoveOverridesAdd = Added.FilterExact(Removed);
		RemovesThatApply.AppendTags(RemoveOverridesAdd);

		ApplyToContainer.AppendTags(Added);
		ApplyToContainer.RemoveTags(RemovesThatApply);
	}
}

void FInheritedTagContainer::AddTag(const FGameplayTag& TagToAdd)
{
	Removed.RemoveTag(TagToAdd);
	Added.AddTag(TagToAdd);
	CombinedTags.AddTag(TagToAdd);
}

void FInheritedTagContainer::RemoveTag(const FGameplayTag& TagToRemove)
{
	Added.RemoveTag(TagToRemove);
	Removed.AddTag(TagToRemove);
	CombinedTags.RemoveTag(TagToRemove);
}

bool FInheritedTagContainer::operator==(const FInheritedTagContainer& Other)  const
{
	return CombinedTags == Other.CombinedTags &&
		Added == Other.Added &&
		Removed == Other.Removed;
}

bool FInheritedTagContainer::operator!=(const FInheritedTagContainer& Other) const
{
	return !(*this == Other);
}

void FActiveGameplayEffectsContainer::IncrementLock()
{
	ScopedLockCount++;
}

void FActiveGameplayEffectsContainer::DecrementLock()
{
	if (--ScopedLockCount == 0)
	{
		// ------------------------------------------
		// Move any pending effects onto the real list
		// ------------------------------------------
		FActiveGameplayEffect* PendingGameplayEffect = PendingGameplayEffectHead;
		FActiveGameplayEffect* Stop = *PendingGameplayEffectNext;
		bool ModifiedArray = false;

		while (PendingGameplayEffect != Stop)
		{
			if (!PendingGameplayEffect->IsPendingRemove)
			{
				GameplayEffects_Internal.Add(MoveTemp(*PendingGameplayEffect));
				ModifiedArray = true;
			}
			else
			{
				PendingRemoves--;
			}
			PendingGameplayEffect = PendingGameplayEffect->PendingNext;
		}

		// Reset our pending GameplayEffect linked list
		PendingGameplayEffectNext = &PendingGameplayEffectHead;

		// -----------------------------------------
		// Delete any pending remove effects
		// -----------------------------------------
		for (int32 idx=GameplayEffects_Internal.Num()-1; idx >= 0 && PendingRemoves > 0; --idx)
		{
			FActiveGameplayEffect& Effect = GameplayEffects_Internal[idx];

			if (Effect.IsPendingRemove)
			{
				UE_LOGF(LogGameplayEffects, Verbose, "%ls: Finish PendingRemove: %ls. Auth: %d", *GetNameSafe(Owner->GetOwnerActor()), *Effect.GetDebugString(), IsNetAuthority());
				
				GameplayEffects_Internal.RemoveAtSwap(idx, EAllowShrinking::No);
				ModifiedArray = true;
				PendingRemoves--;
			}
		}

		if (!ensure(PendingRemoves == 0))
		{
			UE_LOGF(LogGameplayEffects, Error, "~FScopedActiveGameplayEffectLock has %d pending removes after a scope lock removal", PendingRemoves);
			PendingRemoves = 0;
		}

		if (ModifiedArray)
		{
			MarkArrayDirty();
		}
	}
}

FScopedActiveGameplayEffectLock::FScopedActiveGameplayEffectLock(FActiveGameplayEffectsContainer& InContainer)
	: Container(InContainer)
{
	Container.IncrementLock();
}

FScopedActiveGameplayEffectLock::~FScopedActiveGameplayEffectLock()
{
	Container.DecrementLock();
}

#undef LOCTEXT_NAMESPACE


