// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectAggregator.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"
#include "AbilitySystemStats.h"

namespace UE::AbilitySystem::Private
{
	float MultiplyMods(const TArray<FAggregatorMod>& InMods)
	{
		float Result = 1.0f;

		for (const FAggregatorMod& Mod : InMods)
		{
			if (Mod.Qualifies())
			{
				Result *= Mod.EvaluatedMagnitude;
			}
		}

		return Result;
	}
}

void FAggregatorMod::UpdateQualifies(const FAggregatorEvaluateParameters& Parameters) const
{
	static const FGameplayTagContainer EmptyTagContainer;
	const FGameplayTagContainer& SrcTags = Parameters.SourceTags ? *Parameters.SourceTags : EmptyTagContainer;
	const FGameplayTagContainer& TgtTags = Parameters.TargetTags ? *Parameters.TargetTags : EmptyTagContainer;
	bool bSourceMet = (!SourceTagReqs || SourceTagReqs->IsEmpty()) || SourceTagReqs->RequirementsMet(SrcTags);
	bool bTargetMet = (!TargetTagReqs || TargetTagReqs->IsEmpty()) || TargetTagReqs->RequirementsMet(TgtTags);

	bool bSourceFilterMet = (Parameters.AppliedSourceTagFilter.Num() == 0);
	bool bTargetFilterMet = (Parameters.AppliedTargetTagFilter.Num() == 0);

	if (Parameters.IncludePredictiveMods == false && IsPredicted)
	{
		IsQualified = false;
		return;
	}

	if (ActiveHandle.IsValid())
	{
		for (const FActiveGameplayEffectHandle& HandleToIgnore : Parameters.IgnoreHandles)
		{
			if (ActiveHandle == HandleToIgnore)
			{
				IsQualified = false;
				return;
			}
		}
	}
	
	const UAbilitySystemComponent* HandleComponent = ActiveHandle.GetOwningAbilitySystemComponent();
	if (HandleComponent)
	{
		if (!bSourceFilterMet)
		{
			const FGameplayTagContainer* SourceTags = HandleComponent->GetGameplayEffectSourceTagsFromHandle(ActiveHandle);
			bSourceFilterMet = (SourceTags && SourceTags->HasAll(Parameters.AppliedSourceTagFilter));
		}

		if (!bTargetFilterMet)
		{
			const FGameplayTagContainer* TargetTags = HandleComponent->GetGameplayEffectTargetTagsFromHandle(ActiveHandle);
			bTargetFilterMet = (TargetTags && TargetTags->HasAll(Parameters.AppliedTargetTagFilter));
		}
	}

	IsQualified = bSourceMet && bTargetMet && bSourceFilterMet && bTargetFilterMet;
}

float FAggregatorModChannel::EvaluateWithBase(float InlineBaseValue, const FAggregatorEvaluateParameters& Parameters) const
{
	for (const FAggregatorMod& Mod : Mods[EGameplayModOp::Override])
	{
		if (Mod.Qualifies())
		{
			return Mod.EvaluatedMagnitude;
		}
	}

	float Additive = SumMods(Mods[EGameplayModOp::Additive], GameplayEffectUtilities::GetModifierBiasByModifierOp(EGameplayModOp::Additive), Parameters);
	float Multiplicitive = SumMods(Mods[EGameplayModOp::Multiplicitive], GameplayEffectUtilities::GetModifierBiasByModifierOp(EGameplayModOp::Multiplicitive), Parameters);
	float Division = SumMods(Mods[EGameplayModOp::Division], GameplayEffectUtilities::GetModifierBiasByModifierOp(EGameplayModOp::Division), Parameters);
	float FinalAdd = SumMods(Mods[EGameplayModOp::AddFinal], GameplayEffectUtilities::GetModifierBiasByModifierOp(EGameplayModOp::AddFinal), Parameters);
	float CompoundMultiply = UE::AbilitySystem::Private::MultiplyMods(Mods[EGameplayModOp::MultiplyCompound]);

	if (FMath::IsNearlyZero(Division))
	{
		ABILITY_LOG(Warning, TEXT("Division summation was 0.0f in FAggregatorModChannel."));
		Division = 1.f;
	}

	return ((InlineBaseValue + Additive) * Multiplicitive / Division * CompoundMultiply) + FinalAdd;
}

bool FAggregatorModChannel::ReverseEvaluate(float FinalValue, const FAggregatorEvaluateParameters& Parameters, OUT float& ComputedValue) const
{
	for (const FAggregatorMod& Mod : Mods[EGameplayModOp::Override])
	{
		if (Mod.Qualifies())
		{
			// This is the case we can't really handle due to lack of information.
			ComputedValue = FinalValue;
			return false;
		}
	}

	float Additive = SumMods(Mods[EGameplayModOp::Additive], GameplayEffectUtilities::GetModifierBiasByModifierOp(EGameplayModOp::Additive), Parameters);
	float Multiplicitive = SumMods(Mods[EGameplayModOp::Multiplicitive], GameplayEffectUtilities::GetModifierBiasByModifierOp(EGameplayModOp::Multiplicitive), Parameters);
	float Division = SumMods(Mods[EGameplayModOp::Division], GameplayEffectUtilities::GetModifierBiasByModifierOp(EGameplayModOp::Division), Parameters);
	float FinalAdd = SumMods(Mods[EGameplayModOp::AddFinal], GameplayEffectUtilities::GetModifierBiasByModifierOp(EGameplayModOp::AddFinal), Parameters);
	float CompoundMultiply = UE::AbilitySystem::Private::MultiplyMods(Mods[EGameplayModOp::MultiplyCompound]);

	if (FMath::IsNearlyZero(Division))
	{
		ABILITY_LOG(Warning, TEXT("Division summation was 0.0f in FAggregatorModChannel."));
		Division = 1.f;
	}

	if (Multiplicitive <= SMALL_NUMBER)
	{
		ComputedValue = FinalValue;
		return false;
	}

	ComputedValue = ((FinalValue - FinalAdd) / CompoundMultiply * Division / Multiplicitive) - Additive;
	return true;
}

void FAggregatorModChannel::AddMod(float EvaluatedMagnitude, TEnumAsByte<EGameplayModOp::Type> ModOp, const FGameplayTagRequirements* SourceTagReqs, const FGameplayTagRequirements* TargetTagReqs, bool bIsPredicted, const FActiveGameplayEffectHandle& ActiveHandle)
{
	TArray<FAggregatorMod>& ModList = Mods[ModOp];

	int32 NewIdx = ModList.AddUninitialized();
	FAggregatorMod& NewMod = ModList[NewIdx];

	NewMod.SourceTagReqs = SourceTagReqs;
	NewMod.TargetTagReqs = TargetTagReqs;
	NewMod.EvaluatedMagnitude = EvaluatedMagnitude;
	NewMod.StackCount = 0;
	NewMod.ActiveHandle = ActiveHandle;
	NewMod.IsPredicted = bIsPredicted;
}

void FAggregatorModChannel::RemoveModsWithActiveHandle(const FActiveGameplayEffectHandle& Handle)
{
	check(Handle.IsValid());

	for (int32 ModOpIdx = 0; ModOpIdx < UE_ARRAY_COUNT(Mods); ++ModOpIdx)
	{
		Mods[ModOpIdx].RemoveAllSwap([&Handle](const FAggregatorMod& Element)
		{
			return (Element.ActiveHandle == Handle);
		}, 
		EAllowShrinking::No);
	}
}

void FAggregatorModChannel::AddModsFrom(const FAggregatorModChannel& Other)
{
	for (int32 ModOpIdx = 0; ModOpIdx < UE_ARRAY_COUNT(Mods); ++ModOpIdx)
	{
		Mods[ModOpIdx].Append(Other.Mods[ModOpIdx]);
	}
}

void FAggregatorModChannel::UpdateQualifiesOnAllMods(const FAggregatorEvaluateParameters& Parameters) const
{
	for (int32 ModOpIdx = 0; ModOpIdx < UE_ARRAY_COUNT(Mods); ++ModOpIdx)
	{
		for (const FAggregatorMod& Mod : Mods[ModOpIdx])
		{
			Mod.UpdateQualifies(Parameters);
		}
	}
}

void FAggregatorModChannel::ForEachMod(FAggregatorModInfo& Info, TFunction<void (const FAggregatorModInfo&) > Func) const
{
	for (int32 ModOpIdx = 0; ModOpIdx < UE_ARRAY_COUNT(Mods); ++ModOpIdx)
	{
		Info.Op = (EGameplayModOp::Type)ModOpIdx;
		for (const FAggregatorMod& Mod : Mods[ModOpIdx])
		{
			Info.Mod = &Mod;
			Func(Info);
		}
	}
}

void FAggregatorModChannel::GetAllAggregatorMods(EGameplayModEvaluationChannel Channel, OUT TMap<EGameplayModEvaluationChannel, const TArray<FAggregatorMod>* >& OutMods) const
{
	OutMods.Add(Channel, Mods);
}

void FAggregatorModChannel::OnActiveEffectDependenciesSwapped(const TMap<FActiveGameplayEffectHandle, FActiveGameplayEffectHandle>& SwappedDependencies)
{
	for (int32 ModOpIdx = 0; ModOpIdx < UE_ARRAY_COUNT(Mods); ++ModOpIdx)
	{
		for (FAggregatorMod& Mod : Mods[ModOpIdx])
		{
			const FActiveGameplayEffectHandle* NewHandle = SwappedDependencies.Find(Mod.ActiveHandle);
			if (NewHandle)
			{
				Mod.ActiveHandle = *NewHandle;
			}
		}
	}
}

float FAggregatorModChannel::SumMods(const TArray<FAggregatorMod>& InMods, float Bias, const FAggregatorEvaluateParameters& Parameters)
{
	float Sum = Bias;

	for (const FAggregatorMod& Mod : InMods)
	{
		if (Mod.Qualifies())
		{
			Sum += (Mod.EvaluatedMagnitude - Bias);
		}
	}

	return Sum;
}

FAggregatorModChannel& FAggregatorModChannelContainer::FindOrAddModChannel(EGameplayModEvaluationChannel Channel)
{
	FAggregatorModChannel* FoundChannel = ModChannelsMap.Find(Channel);
	if (!FoundChannel)
	{
		// Adding a new channel, need to resort the map to preserve key order for evaluation
		ModChannelsMap.Add(Channel);
		ModChannelsMap.KeySort(TLess<EGameplayModEvaluationChannel>());
		FoundChannel = ModChannelsMap.Find(Channel);
	}
	check(FoundChannel);
	return *FoundChannel;
}

int32 FAggregatorModChannelContainer::GetNumChannels() const
{
	return ModChannelsMap.Num();
}

float FAggregatorModChannelContainer::EvaluateWithBase(float InlineBaseValue, const FAggregatorEvaluateParameters& Parameters) const
{
	float ComputedValue = InlineBaseValue;

	for (auto& ChannelEntry : ModChannelsMap)
	{
		const FAggregatorModChannel& CurChannel = ChannelEntry.Value;
		ComputedValue = CurChannel.EvaluateWithBase(ComputedValue, Parameters);
	}

	return ComputedValue;
}

float FAggregatorModChannelContainer::EvaluateWithBaseToChannel(float InlineBaseValue, const FAggregatorEvaluateParameters& Parameters, EGameplayModEvaluationChannel FinalChannel) const
{
	float ComputedValue = InlineBaseValue;

	const int32 FinalChannelIntVal = static_cast<int32>(FinalChannel);
	for (auto& ChannelEntry : ModChannelsMap)
	{
		const int32 CurChannelIntVal = static_cast<int32>(ChannelEntry.Key);
		if (CurChannelIntVal <= FinalChannelIntVal)
		{
			const FAggregatorModChannel& CurChannel = ChannelEntry.Value;
			ComputedValue = CurChannel.EvaluateWithBase(ComputedValue, Parameters);
		}
		else
		{
			break;
		}
	}

	return ComputedValue;
}

float FAggregatorModChannelContainer::ReverseEvaluate(float FinalValue, const FAggregatorEvaluateParameters& Parameters) const
{
	float ComputedValue = FinalValue;

	// TMap API doesn't allow reverse iteration, so need to request the key array and then
	// traverse it in reverse instead
	TArray<EGameplayModEvaluationChannel> ChannelArray;
	ModChannelsMap.GenerateKeyArray(ChannelArray);

	for (int32 ModChannelIdx = ChannelArray.Num() - 1; ModChannelIdx >= 0; --ModChannelIdx)
	{
		const FAggregatorModChannel& Channel = ModChannelsMap.FindRef(ChannelArray[ModChannelIdx]);
		 if (!Channel.ReverseEvaluate(ComputedValue, Parameters, ComputedValue))
		 {
			 ComputedValue = FinalValue;
			 break;
		 }
	}

	return ComputedValue;
}

void FAggregatorModChannelContainer::EvaluateQualificationForAllMods(const FAggregatorEvaluateParameters& Parameters) const
{
	// First run our "Default" qualifies function
	for (auto& MapIt : ModChannelsMap)
	{
		const FAggregatorModChannel& Channel = MapIt.Value;
		Channel.UpdateQualifiesOnAllMods(Parameters);
	}
}


void FAggregatorModChannelContainer::RemoveAggregatorMod(const FActiveGameplayEffectHandle& ActiveHandle)
{
	if (ActiveHandle.IsValid())
	{
		for (auto& ChannelEntry : ModChannelsMap)
		{
			FAggregatorModChannel& CurChannel = ChannelEntry.Value;
			CurChannel.RemoveModsWithActiveHandle(ActiveHandle);
		}
	}
}

void FAggregatorModChannelContainer::AddModsFrom(const FAggregatorModChannelContainer& Other)
{
	for (const auto& SourceChannelEntry : Other.ModChannelsMap)
	{
		EGameplayModEvaluationChannel SourceChannelEnum = SourceChannelEntry.Key;
		const FAggregatorModChannel& SourceChannel = SourceChannelEntry.Value;

		FAggregatorModChannel& TargetChannel = FindOrAddModChannel(SourceChannelEnum);
		TargetChannel.AddModsFrom(SourceChannel);
	}
}

void FAggregatorModChannelContainer::GetAllAggregatorMods(OUT TMap<EGameplayModEvaluationChannel, const TArray<FAggregatorMod>*>& OutMods) const
{
	for (const auto& ChannelEntry : ModChannelsMap)
	{
		EGameplayModEvaluationChannel CurChannelEnum = ChannelEntry.Key;
		const FAggregatorModChannel& CurChannel = ChannelEntry.Value;
		
		CurChannel.GetAllAggregatorMods(CurChannelEnum, OutMods);
	}
}

void FAggregatorModChannelContainer::OnActiveEffectDependenciesSwapped(const TMap<FActiveGameplayEffectHandle, FActiveGameplayEffectHandle>& SwappedDependencies)
{
	for (auto& ChannelEntry : ModChannelsMap)
	{
		FAggregatorModChannel& CurChannel = ChannelEntry.Value;
		CurChannel.OnActiveEffectDependenciesSwapped(SwappedDependencies);
	}
}

void FAggregatorModChannelContainer::ForEachMod( TFunction<void (const FAggregatorModInfo&) > Func) const
{
	FAggregatorModInfo Info;
	for (const auto& MapIt : ModChannelsMap)
	{
		Info.Channel = MapIt.Key;
		const FAggregatorModChannel& Channel = MapIt.Value;
		Channel.ForEachMod(Info, Func);
	}
}

FAggregator::~FAggregator()
{
	int32 NumRemoved = FScopedAggregatorOnDirtyBatch::DirtyAggregators.Remove(this);
	ensure(NumRemoved == 0);
}

float FAggregator::Evaluate(const FAggregatorEvaluateParameters& Parameters) const
{
	EvaluateQualificationForAllMods(Parameters);
	return ModChannels.EvaluateWithBase(BaseValue, Parameters);
}

float FAggregator::EvaluateToChannel(const FAggregatorEvaluateParameters& Parameters, EGameplayModEvaluationChannel FinalChannel) const
{
	EvaluateQualificationForAllMods(Parameters);
	return ModChannels.EvaluateWithBaseToChannel(BaseValue, Parameters, FinalChannel);
}

float FAggregator::EvaluateWithBase(float InlineBaseValue, const FAggregatorEvaluateParameters& Parameters) const
{
	EvaluateQualificationForAllMods(Parameters);
	return ModChannels.EvaluateWithBase(InlineBaseValue, Parameters);
}

float FAggregator::ReverseEvaluate(float FinalValue, const FAggregatorEvaluateParameters& Parameters) const
{
	EvaluateQualificationForAllMods(Parameters);
	return ModChannels.ReverseEvaluate(FinalValue, Parameters);
}

float FAggregator::EvaluateBonus(const FAggregatorEvaluateParameters& Parameters) const
{
	return (Evaluate(Parameters) - GetBaseValue());
}

float FAggregator::EvaluateContribution(const FAggregatorEvaluateParameters& Parameters, FActiveGameplayEffectHandle ActiveHandle) const
{
	if (ActiveHandle.IsValid())
	{
		FAggregatorEvaluateParameters ParamsExcludingHandle(Parameters);
		ParamsExcludingHandle.IgnoreHandles.Add(ActiveHandle);

		return Evaluate(Parameters) - Evaluate(ParamsExcludingHandle);
	}

	return 0.f;
}

void FAggregator::EvaluateQualificationForAllMods(const FAggregatorEvaluateParameters& Parameters) const
{
	// First run our "Default" qualifies function
	ModChannels.EvaluateQualificationForAllMods(Parameters);

	// Then run custom func
	if (EvaluationMetaData && EvaluationMetaData->CustomQualifiesFunc)
	{
		EvaluationMetaData->CustomQualifiesFunc(Parameters, this);
	}
}

float FAggregator::GetBaseValue() const
{
	return BaseValue;
}

void FAggregator::SetBaseValue(float NewBaseValue, bool BroadcastDirtyEvent)
{
	BaseValue = NewBaseValue;
	if (BroadcastDirtyEvent)
	{
		BroadcastOnDirty();
	}
}

float FAggregator::StaticExecModOnBaseValue(float BaseValue, TEnumAsByte<EGameplayModOp::Type> ModifierOp, float EvaluatedMagnitude)
{
	switch (ModifierOp)
	{
		case EGameplayModOp::Override:
		{
			BaseValue = EvaluatedMagnitude;
			break;
		}
		case EGameplayModOp::AddBase:
		case EGameplayModOp::AddFinal:
		{
			BaseValue += EvaluatedMagnitude;
			break;
		}
		case EGameplayModOp::MultiplyAdditive:
		case EGameplayModOp::MultiplyCompound:
		{
			BaseValue *= EvaluatedMagnitude;
			break;
		}
		case EGameplayModOp::DivideAdditive:
		{
			if (FMath::IsNearlyZero(EvaluatedMagnitude) == false)
			{
				BaseValue /= EvaluatedMagnitude;
			}
			break;
		}
	}

	return BaseValue;
}

void FAggregator::ExecModOnBaseValue(TEnumAsByte<EGameplayModOp::Type> ModifierOp, float EvaluatedMagnitude)
{
	BaseValue = StaticExecModOnBaseValue(BaseValue, ModifierOp, EvaluatedMagnitude);
	BroadcastOnDirty();
}

void FAggregator::AddAggregatorMod(float EvaluatedMagnitude, TEnumAsByte<EGameplayModOp::Type> ModifierOp, EGameplayModEvaluationChannel ModifierChannel, const FGameplayTagRequirements* SourceTagReqs, const FGameplayTagRequirements* TargetTagReqs, bool IsPredicted, FActiveGameplayEffectHandle ActiveHandle)
{
	FAggregatorModChannel& ModChannelToAddTo = ModChannels.FindOrAddModChannel(ModifierChannel);
	ModChannelToAddTo.AddMod(EvaluatedMagnitude, ModifierOp, SourceTagReqs, TargetTagReqs, IsPredicted, ActiveHandle);

	BroadcastOnDirty();
}

void FAggregator::RemoveAggregatorMod(FActiveGameplayEffectHandle ActiveHandle)
{
	ModChannels.RemoveAggregatorMod(ActiveHandle);

	// mark it as dirty so that all the stats get updated
	BroadcastOnDirty();
}

void FAggregator::UpdateAggregatorMod(FActiveGameplayEffectHandle ActiveHandle, const FGameplayAttribute& Attribute, const FGameplayEffectSpec& Spec, bool bWasLocallyGenerated, FActiveGameplayEffectHandle InHandle)
{
	// remove the mods but don't mark it as dirty until we re-add the aggregators, we are doing this so the UAttributeSets stats only know about the delta change.
	ModChannels.RemoveAggregatorMod(ActiveHandle);

	// Now re-add ALL of our mods
	for (int32 ModIdx = 0; ModIdx < Spec.Modifiers.Num(); ++ModIdx)
	{
		const FGameplayModifierInfo& ModDef = Spec.Def->Modifiers[ModIdx];
		if (ModDef.Attribute == Attribute)
		{
			FAggregatorModChannel& ModChannel = ModChannels.FindOrAddModChannel(ModDef.EvaluationChannelSettings.GetEvaluationChannel());
			ModChannel.AddMod(Spec.GetModifierMagnitude(ModIdx), ModDef.ModifierOp, &ModDef.SourceTags, &ModDef.TargetTags, bWasLocallyGenerated, InHandle);
		}
	}

	// mark it as dirty so that all the stats get updated
	BroadcastOnDirty();
}

void FAggregator::AddModsFrom(const FAggregator& SourceAggregator)
{
	// @todo: should this broadcast dirty?
	ModChannels.AddModsFrom(SourceAggregator.ModChannels);
}

void FAggregator::AddDependent(FActiveGameplayEffectHandle Handle)
{
	Dependents.Add(Handle);
}

void FAggregator::RemoveDependent(FActiveGameplayEffectHandle Handle)
{
	Dependents.Remove(Handle);
}

void FAggregator::GetAllAggregatorMods(OUT TMap<EGameplayModEvaluationChannel, const TArray<FAggregatorMod>*>& OutMods) const
{
	ModChannels.GetAllAggregatorMods(OutMods);
}

void FAggregator::OnActiveEffectDependenciesSwapped(const TMap<FActiveGameplayEffectHandle, FActiveGameplayEffectHandle>& SwappedDependencies)
{
	for (int32 DependentIdx = Dependents.Num() - 1; DependentIdx >= 0; DependentIdx--)
	{
		FActiveGameplayEffectHandle& DependentHandle = Dependents[DependentIdx];

		bool bStillValidDependent = false;

		// Check to see if the dependent handle was an old handle that has been replaced; If so, it must be updated
		const FActiveGameplayEffectHandle* NewHandle = SwappedDependencies.Find(DependentHandle);
		if (NewHandle)
		{
			bStillValidDependent = true;
			DependentHandle = *NewHandle;
		}
		// Check to see if the dependent handle is a new handle, in which case it is still valid, but no update is required
		else if (SwappedDependencies.FindKey(DependentHandle))
		{
			bStillValidDependent = true;
		}

		if (!bStillValidDependent)
		{
			Dependents.RemoveAtSwap(DependentIdx, EAllowShrinking::No);
		}

		ModChannels.OnActiveEffectDependenciesSwapped(SwappedDependencies);
	}
}

void FAggregator::ForEachMod( TFunction<void (const FAggregatorModInfo&) > Func ) const
{
	ModChannels.ForEachMod(Func);
}

void FAggregator::TakeSnapshotOf(const FAggregator& AggToSnapshot)
{
	BaseValue = AggToSnapshot.BaseValue;
	ModChannels = AggToSnapshot.ModChannels;
}

void FAggregator::BroadcastOnDirty()
{
	// --------------------------------------------------
	// If we are batching all on Dirty calls (and we actually have dependents registered with us) then early out.
	// --------------------------------------------------
	if (FScopedAggregatorOnDirtyBatch::GlobalBatchCount > 0 && (Dependents.Num() > 0 || OnDirty.IsBound()))
	{
		FScopedAggregatorOnDirtyBatch::DirtyAggregators.Add(this);
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_AggregatorBroadcastOnDirty);

	// --------------------------------------------------
	//	The code below attempts to avoid recursion issues: an aggregator is dirty and while it is broadcasting this out, someone dirties it again.
	//	The degenerate case is cyclic attribute dependencies: MaxHealth -> MaxMana -> MaxHealth. This prob can't be fixed. We should instead detect earlier.
	//
	//	The less serious case is while doing your broadcast, someone applies a GE that dirties the attribute again. As long as this isn't an infinite loop... it should be ok.
	//
	//	This code now allows MAX_BROADCAST_DIRTY recursion calls. This is pretty arbitrary. We are trying to provide a solution that doesn't crash (infinite loop) from bad data
	//	while not breaking/causing bugs when this is triggered.
	//
	// --------------------------------------------------

	const int32 MAX_BROADCAST_DIRTY = 10;

	if (BroadcastingDirtyCount > MAX_BROADCAST_DIRTY)
	{
		// This call will at least update the backing uproperty values so that they don't get stale. We will still skip dependant attribute magnitudes and potential game code listening for attribute changes!
		OnDirtyRecursive.Broadcast(this);
		
		ABILITY_LOG(Warning, TEXT("FAggregator detected cyclic attribute dependencies. We are skipping a recursive dirty call. Its possible the resulting attribute values are not what you expect!"));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Additional, slow, debugging that will print all aggregator/attributes that are currently dirty
		for (TObjectIterator<UAbilitySystemComponent> It; It; ++It)
		{
			It->DebugCyclicAggregatorBroadcasts(this);
		}
#endif
		return;
	}

	BroadcastingDirtyCount++;
	OnDirty.Broadcast(this);

	// ----------------------------------------------------------
	//	Lets Dependant GEs know about this too.
	// ----------------------------------------------------------

	// Copy Dependants here to avoid recursive issues if any more Dependants are added while we are broadcasting out
	TArray<FActiveGameplayEffectHandle> DependantsLocalCopy = Dependents;
	Dependents.Empty(); // We will add valid handles back as we process the local list

	for (FActiveGameplayEffectHandle& Handle : DependantsLocalCopy)
	{
		UAbilitySystemComponent* ASC = Handle.GetOwningAbilitySystemComponent();
		if (ASC)
		{
			ASC->OnMagnitudeDependencyChange(Handle, this);
			Dependents.Add(Handle);
		}
	}

	BroadcastingDirtyCount--;

}

void FAggregatorRef::TakeSnapshotOf(const FAggregatorRef& RefToSnapshot)
{
	if (RefToSnapshot.Data.IsValid())
	{
		FAggregator* SrcData = RefToSnapshot.Data.Get();

		Data = MakeShared<FAggregator>();
		Data->TakeSnapshotOf(*SrcData);
	}
	else
	{
		Data.Reset();
	}
}

int32 FScopedAggregatorOnDirtyBatch::GlobalBatchCount = 0;
TSet<FAggregator*> FScopedAggregatorOnDirtyBatch::DirtyAggregators;
bool FScopedAggregatorOnDirtyBatch::GlobalFromNetworkUpdate = false;
int32 FScopedAggregatorOnDirtyBatch::NetUpdateID = 1;

FScopedAggregatorOnDirtyBatch::FScopedAggregatorOnDirtyBatch()
{
	BeginLock();
}

FScopedAggregatorOnDirtyBatch::~FScopedAggregatorOnDirtyBatch()
{
	EndLock();
}

void FScopedAggregatorOnDirtyBatch::BeginLock()
{
	GlobalBatchCount++;
}
void FScopedAggregatorOnDirtyBatch::EndLock()
{
	GlobalBatchCount--;
	if (GlobalBatchCount == 0)
	{
		TSet<FAggregator*> LocalSet(MoveTemp(DirtyAggregators));
		for (FAggregator* Agg : LocalSet)
		{
			Agg->BroadcastOnDirty();
		}
		LocalSet.Empty();
	}
}


void FScopedAggregatorOnDirtyBatch::BeginNetReceiveLock()
{
	BeginLock();
}
void FScopedAggregatorOnDirtyBatch::EndNetReceiveLock()
{
	// The network lock must end the first time it is called.
	// Subsequent calls to EndNetReceiveLock() should not trigger a full EndLock, only the first one.
	if (GlobalBatchCount > 0)
	{
		GlobalBatchCount = 1;
		NetUpdateID++;
		GlobalFromNetworkUpdate = true;
		EndLock();
		GlobalFromNetworkUpdate = false;
	}
}
