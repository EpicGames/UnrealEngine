// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectAggregatorLibrary.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "AbilitySystemComponent.h"

/** Custom functions. The idea here is that we may want to mix and match these (if FAggregatorEvaluateMetaData starts to hold more than just the qualifier functions) */
void QualifierFunc_MostNegativeMod_AllPositiveMods(const FAggregatorEvaluateParameters& EvalParameters, const FAggregator* Aggregator)
{
	// We want to inhibit all qualified negative effects except for the most severe. We want to leave positive modifiers alone
	const FAggregatorMod* MostNegativeMod = nullptr;
	float CurrentMostNegativeDelta = 0.f;
	float BaseValue = Aggregator->GetBaseValue();

	Aggregator->ForEachMod( [&](const FAggregatorModInfo& ModInfo)
	{
		if (!ModInfo.Mod->Qualifies())
		{
			// Mod doesn't qualify (for other reasons) so ignore
			return;
		}

		float ExpectedDelta = 0.f;
		switch( ModInfo.Op )
		{
		case EGameplayModOp::AddBase:
		case EGameplayModOp::AddFinal:
			ExpectedDelta = ModInfo.Mod->EvaluatedMagnitude;
			break;
		case EGameplayModOp::MultiplyAdditive:
		case EGameplayModOp::MultiplyCompound:
			ExpectedDelta = (BaseValue * ModInfo.Mod->EvaluatedMagnitude) - BaseValue;
			break;
		case EGameplayModOp::DivideAdditive:
			ExpectedDelta = ModInfo.Mod->EvaluatedMagnitude > 0.f ? ((BaseValue / ModInfo.Mod->EvaluatedMagnitude) - BaseValue) : 0.f;
			break;
		case EGameplayModOp::Override:
			ExpectedDelta = ModInfo.Mod->EvaluatedMagnitude - BaseValue;
			break;
		};

		// If its a negative mod
		if (ExpectedDelta < 0.f)
		{
			// Turn it off no matter what (we will only enable to most negative at the end)
			ModInfo.Mod->SetExplicitQualifies(false);

			// If its the most negative, safe a pointer to it so we can enable it once we finish
			if (ExpectedDelta < CurrentMostNegativeDelta)
			{
				CurrentMostNegativeDelta = ExpectedDelta;
				MostNegativeMod = ModInfo.Mod;
			}
		}
	});

	if (MostNegativeMod)
	{
		MostNegativeMod->SetExplicitQualifies(true);
	}
}

/** static FAggregatorEvaluateMetaDatas that use the above functions */
FAggregatorEvaluateMetaData FAggregatorEvaluateMetaDataLibrary::MostNegativeMod_AllPositiveMods(QualifierFunc_MostNegativeMod_AllPositiveMods);