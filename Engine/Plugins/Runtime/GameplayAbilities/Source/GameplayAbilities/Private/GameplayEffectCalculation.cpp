// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectCalculation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayEffectCalculation)

UGameplayEffectCalculation::UGameplayEffectCalculation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

const TArray<FGameplayEffectAttributeCaptureDefinition>& UGameplayEffectCalculation::GetAttributeCaptureDefinitions() const
{
	return RelevantAttributesToCapture;
}

