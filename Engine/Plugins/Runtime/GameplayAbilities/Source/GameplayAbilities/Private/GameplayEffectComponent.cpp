// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectComponent.h"

#include "AbilitySystemLog.h"
#include "GameplayEffect.h"
#include "Misc/DataValidation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayEffectComponent)

#define LOCTEXT_NAMESPACE "GameplayEffectComponent"

UGameplayEffectComponent::UGameplayEffectComponent()
{
#if WITH_EDITORONLY_DATA
	EditorFriendlyName = FName::NameToDisplayString(GetNameSafe(GetClass()), false);
	EditorFriendlyName.ReplaceInline(TEXT("Gameplay Effect Component"), TEXT(""));
#endif
}

UGameplayEffect* UGameplayEffectComponent::GetOwner() const
{
	return GetTypedOuter<UGameplayEffect>();
}

#if WITH_EDITOR
EDataValidationResult UGameplayEffectComponent::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	TSubclassOf<UGameplayEffectComponent> ThisClass = GetClass();
	
	const UGameplayEffect* Owner = GetOwner();
	if (Owner)
	{
		const UGameplayEffectComponent* FirstComponentOfThisType = Owner->FindComponent(ThisClass);
		if (!FirstComponentOfThisType)
		{
			Context.AddError(LOCTEXT("ComponentNotOnGE", "Component does not exist in its Owner's GEComponents Array"));
			Result = EDataValidationResult::Invalid;
		}
		else if (FirstComponentOfThisType != this)
		{
			Context.AddError(FText::FormatOrdered(LOCTEXT("MultipleComponentsOfSameTypeError", "Two or more types of {0} exist on GE"), FText::FromString(ThisClass->GetName())));
			Result = EDataValidationResult::Invalid;
		}
	}
	else
	{
		FText ErrorText = FText::FormatOrdered(LOCTEXT("NoGEOwner", "{0} has invalid Outer: {1}"), FText::FromString(GetNameSafe(this)), FText::FromString(GetNameSafe(GetOuter())));
		Context.AddError(ErrorText);

		UE_LOGF(LogGameplayEffects, Error, "%ls", *ErrorText.ToString());

		Result = EDataValidationResult::Invalid;
	}

	// Let's just say we've done enough to say it's valid and it's up to the child classes to invalidate
	if (Result == EDataValidationResult::NotValidated)
	{
		Result = EDataValidationResult::Valid;
	}

	return Result;
}
#endif

#undef LOCTEXT_NAMESPACE
