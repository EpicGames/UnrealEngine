// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectComponents/AbilitiesGameplayEffectComponent.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"
#include "Misc/DataValidation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilitiesGameplayEffectComponent)

#define LOCTEXT_NAMESPACE "AbilitiesGameplayEffectComponent"

bool operator==(const FGameplayAbilitySpecConfig& Lhs, const FGameplayAbilitySpecConfig& Rhs)
{
	return Lhs.Ability == Rhs.Ability &&
		Lhs.InputID == Rhs.InputID &&
		Lhs.LevelScalableFloat == Rhs.LevelScalableFloat &&
		Lhs.RemovalPolicy == Rhs.RemovalPolicy;
}

UAbilitiesGameplayEffectComponent::UAbilitiesGameplayEffectComponent()
{
#if WITH_EDITORONLY_DATA
	EditorFriendlyName = TEXT("Grant Abilities While Active");
#endif
}

void UAbilitiesGameplayEffectComponent::AddGrantedAbilityConfig(const FGameplayAbilitySpecConfig& Config)
{
	GrantAbilityConfigs.AddUnique(Config);
}

// Reminder: this is happening on the authority only
void UAbilitiesGameplayEffectComponent::OnInhibitionChanged(FActiveGameplayEffectHandle ActiveGEHandle, bool bIsInhibited) const
{
	UAbilitySystemComponent* ASC = ActiveGEHandle.GetOwningAbilitySystemComponent();
	if (!ensure(ASC))
	{
		UE_LOGF(LogGameplayEffects, Error, "%s was passed an ActiveGEHandle %ls which did not have a valid associated AbilitySystemComponent (is it PendingRemove?)", __FUNCTION__, *ActiveGEHandle.ToString());
		return;
	}

	const FActiveGameplayEffect* ActiveGE = ASC->GetActiveGameplayEffect(ActiveGEHandle);
	if (!ensure(ActiveGE))
	{
		UE_LOGF(LogGameplayEffects, Error, "ActiveGEHandle %ls did not corresponds to Active Gameplay Effect on %ls. This is unexpected since GetOwningAbilitySystemComponent succeeded.", *ActiveGEHandle.ToString(), *ASC->GetName());
		return;
	}

	if (bIsInhibited)
	{
		RemoveAbilities(*ASC, *ActiveGE);
	}
	else
	{
		GrantAbilities(*ASC, *ActiveGE);
	}
}

// Reminder: this is happening on the authority only and any call to an ASC member function can invalidate ActiveGE.
void UAbilitiesGameplayEffectComponent::GrantAbilities(UAbilitySystemComponent& ASC, const FActiveGameplayEffect& ActiveGE) const
{
	if (ASC.bSuppressGrantAbility)
	{
		UE_LOGF(LogGameplayEffects, Log, "%ls suppressed by %ls bSuppressGrantAbility", *GetName(), *ASC.GetName());
		return;
	}

	// Make sure these are copies because ASC.GiveAbility could invalidate ActiveGE
	const FGameplayEffectSpec ActiveGESpec = ActiveGE.Spec;
	const FActiveGameplayEffectHandle ActiveGEHandle = ActiveGE.Handle;

	const TArray<FGameplayAbilitySpec>& AllAbilities = ASC.GetActivatableAbilities();
	for (const FGameplayAbilitySpecConfig& AbilityConfig : GrantAbilityConfigs)
	{
		// Check that we're configured
		const UGameplayAbility* AbilityCDO = AbilityConfig.Ability.GetDefaultObject();
		if (!AbilityCDO)
		{
			continue;
		}

		// Only do this if we haven't assigned the ability yet! This prevents cases where stacking GEs
		// would regrant the ability every time the stack was applied
		const bool bAlreadyGrantedAbility = AllAbilities.ContainsByPredicate([AbilityCDO, &ActiveGEHandle](FGameplayAbilitySpec& Spec) { return Spec.Ability == AbilityCDO && Spec.GameplayEffectHandle == ActiveGEHandle; });
		if (bAlreadyGrantedAbility)
		{
			continue;
		}

		const FString ContextString = FString::Printf(TEXT("%hs for %s from %s"), __FUNCTION__, *AbilityCDO->GetName(), *GetNameSafe(ActiveGESpec.Def));
		const int32 Level = static_cast<int32>(AbilityConfig.LevelScalableFloat.GetValueAtLevel(ActiveGESpec.GetLevel(), &ContextString));

		// Now grant that ability to the owning actor
		FGameplayAbilitySpec AbilitySpec{ AbilityConfig.Ability, Level, AbilityConfig.InputID, ActiveGESpec.GetEffectContext().GetSourceObject() };
		AbilitySpec.SetByCallerTagMagnitudes = ActiveGESpec.SetByCallerTagMagnitudes;
		AbilitySpec.GameplayEffectHandle = ActiveGEHandle;

		ASC.GiveAbility(AbilitySpec);
	}
}

// Reminder: this is happening on the authority only and the ActiveGE can be considered 'inactive' (pending remove) by this time.
// Once you do anything on the ASC, ActiveGE can become invalid -- so cache any values you need before any ASC calls.
void UAbilitiesGameplayEffectComponent::RemoveAbilities(UAbilitySystemComponent& ASC, const FActiveGameplayEffect& ActiveGE) const
{
	FScopedAbilityListLock ScopedAbilityListLock(ASC);
	const TArray<const FGameplayAbilitySpec*> GrantedAbilities = ASC.FindAbilitySpecsFromGEHandle(ScopedAbilityListLock, ActiveGE.Handle, EConsiderPending::All);
	for (const FGameplayAbilitySpecConfig& AbilityConfig : GrantAbilityConfigs)
	{
		// Check that we're configured
		const UGameplayAbility* AbilityCDO = AbilityConfig.Ability.GetDefaultObject();
		if (!AbilityCDO)
		{
			continue;
		}

		// See if we were granted, and if so we can remove it
		const FGameplayAbilitySpec* const* AbilitySpecItem = GrantedAbilities.FindByPredicate([AbilityCDO](const FGameplayAbilitySpec* Spec) { return Spec->Ability == AbilityCDO; });
		if (!AbilitySpecItem || !(*AbilitySpecItem))
		{
			continue;
		}
		const FGameplayAbilitySpec& AbilitySpecDef = (**AbilitySpecItem);

		switch (AbilityConfig.RemovalPolicy)
		{
			case EGameplayEffectGrantedAbilityRemovePolicy::CancelAbilityImmediately:
			{
				ASC.ClearAbility(AbilitySpecDef.Handle);
				break;
			}
			case EGameplayEffectGrantedAbilityRemovePolicy::RemoveAbilityOnEnd:
			{
				ASC.SetRemoveAbilityOnEnd(AbilitySpecDef.Handle);
				break;
			}
			default:
			{
				// Do nothing to granted ability
				break;
			}
		}
	}
}

bool UAbilitiesGameplayEffectComponent::OnActiveGameplayEffectAdded(FActiveGameplayEffectsContainer& ActiveGEContainer, FActiveGameplayEffect& ActiveGE) const
{
	if (ActiveGEContainer.IsNetAuthority())
	{
		ActiveGE.EventSet.OnEffectRemoved.AddUObject(this, &UAbilitiesGameplayEffectComponent::OnActiveGameplayEffectRemoved);
		ActiveGE.EventSet.OnInhibitionChanged.AddUObject(this, &UAbilitiesGameplayEffectComponent::OnInhibitionChanged);
	}

	return true;
}

void UAbilitiesGameplayEffectComponent::OnActiveGameplayEffectRemoved(const FGameplayEffectRemovalInfo& RemovalInfo) const
{
	const FActiveGameplayEffect* ActiveGE = RemovalInfo.ActiveEffect;
	if (!ensure(ActiveGE))
	{
		UE_LOGF(LogGameplayEffects, Error, "FGameplayEffectRemovalInfo::ActiveEffect was not populated in %s", __FUNCTION__);
		return;
	}

	UAbilitySystemComponent* OwningASC = RemovalInfo.OwningASC;
	if (ensure(OwningASC))
	{
		RemoveAbilities(*OwningASC, *ActiveGE);
	}
}

#if WITH_EDITOR
EDataValidationResult UAbilitiesGameplayEffectComponent::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	if (GetOwner()->DurationPolicy == EGameplayEffectDurationType::Instant)
	{
		if (GrantAbilityConfigs.Num() > 0)
		{
			Context.AddError(FText::FormatOrdered(LOCTEXT("InstantDoesNotWorkWithGrantAbilities", "GrantAbilityConfigs does not work with Instant Effects: {0}."), FText::FromString(GetClass()->GetName())));
			Result = EDataValidationResult::Invalid;
		}
	}

	for (int Index = 0; Index < GrantAbilityConfigs.Num(); ++Index)
	{
		const TSubclassOf<UGameplayAbility> AbilityClass = GrantAbilityConfigs[Index].Ability;
		for (int CheckIndex = Index + 1; CheckIndex < GrantAbilityConfigs.Num(); ++CheckIndex)
		{
			if (AbilityClass == GrantAbilityConfigs[CheckIndex].Ability)
			{
				Context.AddError(FText::FormatOrdered(LOCTEXT("GrantAbilitiesMustBeUnique", "Multiple Abilities of the same type cannot be granted by {0}."), FText::FromString(GetClass()->GetName())));
				Result = EDataValidationResult::Invalid;
			}
		}
	}

	return Result;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
