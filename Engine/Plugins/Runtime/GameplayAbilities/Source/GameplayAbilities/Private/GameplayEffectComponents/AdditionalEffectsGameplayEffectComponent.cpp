// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectComponents/AdditionalEffectsGameplayEffectComponent.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"
#include "Misc/DataValidation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AdditionalEffectsGameplayEffectComponent)

#define LOCTEXT_NAMESPACE "AdditionalEffectsGameplayEffectComponent"

bool UAdditionalEffectsGameplayEffectComponent::OnActiveGameplayEffectAdded(FActiveGameplayEffectsContainer& ActiveGEContainer, FActiveGameplayEffect& ActiveGE) const
{
	// We don't allow prediction of expiration (on removed) effects
	if (ActiveGEContainer.IsNetAuthority())
	{
		// When this ActiveGE gets removed, so will our events so no need to unbind this.
		ActiveGE.EventSet.OnEffectRemoved.AddUObject(this, &UAdditionalEffectsGameplayEffectComponent::OnActiveGameplayEffectRemoved, &ActiveGEContainer);
	}

	return true;
}

void UAdditionalEffectsGameplayEffectComponent::OnGameplayEffectApplied(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const
{
	if (!ensureMsgf(ActiveGEContainer.Owner, TEXT("OnGameplayEffectApplied is passed an ActiveGEContainer which lives within an ASC but that ASC was somehow null")))
	{
		return;
	}

	const float GELevel = GESpec.GetLevel();
	const FGameplayEffectContextHandle& GEContextHandle = GESpec.GetEffectContext();

	/** other effects that need to be applied to the target if this effect is successful */
	TArray<FGameplayEffectSpecHandle> TargetEffectSpecs;
	for (const FConditionalGameplayEffect& ConditionalEffect : OnApplicationGameplayEffects)
	{
		const UGameplayEffect* GameplayEffectDef = ConditionalEffect.EffectClass.GetDefaultObject();
		if (!GameplayEffectDef)
		{
			continue;
		}

		if (ConditionalEffect.CanApply(GESpec.CapturedSourceTags.GetActorTags(), GELevel))
		{
			FGameplayEffectSpecHandle SpecHandle;
			if (bOnApplicationCopyDataFromOriginalSpec)
			{
				SpecHandle = FGameplayEffectSpecHandle(new FGameplayEffectSpec());
				SpecHandle.Data->InitializeFromLinkedSpec(GameplayEffectDef, GESpec);
			}
			else
			{
				SpecHandle = ConditionalEffect.CreateSpec(GEContextHandle, GELevel);
			}

			if (ensure(SpecHandle.IsValid()))
			{
				TargetEffectSpecs.Add(SpecHandle);
			}
		}
	}

	// Add all of the dynamically linked ones that don't have any conditions applied to them
	// Note: I believe this functionality should be removed as its usage is dynamic and thus not 
	// synchronized between client/server.  I've deprecated the variable to help enforce this.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TargetEffectSpecs.Append(GESpec.TargetEffectSpecs);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UAbilitySystemComponent& AppliedToASC = *ActiveGEContainer.Owner;
	for (const FGameplayEffectSpecHandle& TargetSpec : TargetEffectSpecs)
	{
		if (TargetSpec.IsValid())
		{
			AppliedToASC.ApplyGameplayEffectSpecToSelf(*TargetSpec.Data.Get(), PredictionKey);
		}
	}
}

void UAdditionalEffectsGameplayEffectComponent::OnActiveGameplayEffectRemoved(const FGameplayEffectRemovalInfo& RemovalInfo, FActiveGameplayEffectsContainer* ActiveGEContainer) const
{
	FScopedActiveGameplayEffectLock ActiveScopeLock(*ActiveGEContainer);

	const FActiveGameplayEffect* ActiveGE = RemovalInfo.ActiveEffect;
	if (!ensure(ActiveGE))
	{
		UE_LOGF(LogGameplayEffects, Error, "FGameplayEffectRemovalInfo::ActiveEffect was not populated in OnActiveGameplayEffectRemoved");
		return;
	}

	UAbilitySystemComponent* ASC = ActiveGEContainer->Owner;
	if (!ensure(IsValid(ASC)))
	{
		UE_LOGF(LogGameplayEffects, Error, "ActiveGEContainer was invalid in OnActiveGameplayEffectRemoved");
		return;
	}

	// Determine the appropriate type of effect to apply depending on whether the effect is being prematurely removed or not
	const TArray<TSubclassOf<UGameplayEffect>>& ExpiryEffects = (RemovalInfo.bPrematureRemoval ? OnCompletePrematurely : OnCompleteNormal);

	// Mix-in the always-executing GameplayEffects
	TArray<TSubclassOf<UGameplayEffect>> AllGameplayEffects{ ExpiryEffects };
	AllGameplayEffects.Append(OnCompleteAlways);

	for (const TSubclassOf<UGameplayEffect>& CurExpiryEffect : AllGameplayEffects)
	{
		if (const UGameplayEffect* CurExpiryCDO = CurExpiryEffect.GetDefaultObject())
		{
			FGameplayEffectSpec NewSpec;
			NewSpec.InitializeFromLinkedSpec(CurExpiryCDO, ActiveGE->Spec);

			ASC->ApplyGameplayEffectSpecToSelf(NewSpec);
		}
	}

	// Remove any on-application effects that were configured with RemoveGrantedEffectOnEnd
	for (const FConditionalGameplayEffect& ConditionalEffect : OnApplicationGameplayEffects)
	{
		if (ConditionalEffect.RemovalPolicy == EGameplayEffectGrantedEffectRemovalPolicy::RemoveGrantedEffectOnEnd
			&& ConditionalEffect.EffectClass != nullptr)
		{
			FGameplayEffectQuery RemoveQuery;
			RemoveQuery.EffectDefinition = ConditionalEffect.EffectClass;
			ActiveGEContainer->RemoveActiveEffects(RemoveQuery, ConditionalEffect.StackCountToRemove);
		}
	}
}

#if WITH_EDITOR
EDataValidationResult UAdditionalEffectsGameplayEffectComponent::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	const bool bOwnerIsInstant = (GetOwner()->DurationPolicy == EGameplayEffectDurationType::Instant);

	if (bOwnerIsInstant)
	{
		const bool bHasOnCompleteEffects = (OnCompleteAlways.Num() + OnCompleteNormal.Num() + OnCompletePrematurely.Num() > 0);
		if (bHasOnCompleteEffects)
		{
			Context.AddError(FText::FormatOrdered(LOCTEXT("InstantDoesNotWorkWithOnComplete", "Instant GE will never receive OnComplete for {0}."), FText::FromString(GetClass()->GetName())));
			Result = EDataValidationResult::Invalid;
		}
	}
	else if (GetOwner()->Period.Value > 0.0f)
	{
		if (OnApplicationGameplayEffects.Num() > 0)
		{
			Context.AddWarning(LOCTEXT("IsPeriodicAndHasOnApplication", "Periodic GE has OnApplicationGameplayEffects. Those GE's will only be applied once."));
		}
	}

	for (const FConditionalGameplayEffect& ConditionalEffect : OnApplicationGameplayEffects)
	{
		if (ConditionalEffect.RemovalPolicy != EGameplayEffectGrantedEffectRemovalPolicy::RemoveGrantedEffectOnEnd)
		{
			continue;
		}

		if (bOwnerIsInstant)
		{
			Context.AddError(LOCTEXT("RemoveGrantedEffectOnEndWithInstantOwner",
				"RemoveGrantedEffectOnEnd cannot be used on an Instant owning GE. Instant effects are never added to the ActiveGE container, so the removal callback will never fire."));
			Result = EDataValidationResult::Invalid;
		}

		// The granted GE must be non-Instant: instant effects are applied and immediately gone, there is nothing to remove later
		if (ConditionalEffect.EffectClass != nullptr)
		{
			if (const UGameplayEffect* GrantedCDO = ConditionalEffect.EffectClass.GetDefaultObject())
			{
				if (GrantedCDO->DurationPolicy == EGameplayEffectDurationType::Instant)
				{
					Context.AddError(LOCTEXT("RemoveGrantedEffectOnEndWithInstantChild",
							"RemoveGrantedEffectOnEnd cannot be used with Instant granted effect. Instant effects are applied and immediately removed; there is nothing to clean up when the owning GE ends."));
					Result = EDataValidationResult::Invalid;
				}
			}
		}
	}

	return Result;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
