// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectComponents/AssetTagsGameplayEffectComponent.h"
#include "AbilitySystemLog.h"
#include "Logging/StructuredLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetTagsGameplayEffectComponent)

#define LOCTEXT_NAMESPACE "AssetTagsGameplayEffectComponent"

void UAssetTagsGameplayEffectComponent::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	EditorFriendlyName = LOCTEXT("AssetTagsOnGE", "Asset Tags (on Gameplay Effect)").ToString();
#endif

	// This code is meant to catch the case of newly constructed Components (not loaded Components).
	// With asynchronous loading, we are not guaranteed the Parent object is loaded at this point.
	// During loading, we'll call OnGameplayEffectChanged from UGameplayEffect::PostLoad().
	if (IsInAsyncLoadingThread() || IsInParallelLoadingThread())
	{
		return;
	}

	// Try to find the parent and update the inherited tags.  We do this 'early' because this is the only function
	// we can use for creation of the object (and this runs post-constructor).
	const UAssetTagsGameplayEffectComponent* Parent = FindParentComponent(*this);
	if (Parent == this)
	{
		UE_LOGFMT(LogAbilitySystem, Warning, "UAssetTagsGameplayEffectComponent got self for parent component when trying to update inherited tag container. {Component}", FSoftObjectPath(this));
	}
	else 
	{
		InheritableAssetTags.UpdateInheritedTagProperties(Parent ? &Parent->InheritableAssetTags : nullptr);
	}
}

void UAssetTagsGameplayEffectComponent::OnGameplayEffectChanged()
{
	Super::OnGameplayEffectChanged();

#if WITH_EDITORONLY_DATA
	if (GetOwner())
	{
		EditorFriendlyName = FText::FormatOrdered(LOCTEXT("AssetTagsOnObj", "Asset Tags (on {0})"), GetOwner()->GetClass()->GetDisplayNameText()).ToString();
	}
	else
	{
		EditorFriendlyName = LOCTEXT("AssetTagsOnGE", "Asset Tags (on Gameplay Effect)").ToString();
	}
#endif // WITH_EDITORONLY_DATA

	SetAndApplyAssetTagChanges(InheritableAssetTags);
}

#if WITH_EDITOR
void UAssetTagsGameplayEffectComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName()  == GetInheritableAssetTagsName())
	{
		// Tell the GE it needs to reconfigure itself based on these updated properties (this will reaggregate the tags)
		UGameplayEffect* Owner = GetOwner();
		Owner->OnGameplayEffectChanged();
	}
}
#endif // WITH_EDITOR

void UAssetTagsGameplayEffectComponent::SetAndApplyAssetTagChanges(const FInheritedTagContainer& TagContainerMods)
{
	InheritableAssetTags = TagContainerMods;

	// Try to find the parent and update the inherited tags
	const UAssetTagsGameplayEffectComponent* Parent = FindParentComponent(*this);
	InheritableAssetTags.UpdateInheritedTagProperties(Parent ? &Parent->InheritableAssetTags : nullptr);

	ApplyAssetTagChanges();
}

void UAssetTagsGameplayEffectComponent::ApplyAssetTagChanges() const
{
	// Apply to the owning Gameplay Effect Component
	UGameplayEffect* Owner = GetOwner();
	InheritableAssetTags.ApplyTo(Owner->CachedAssetTags);
}

#undef LOCTEXT_NAMESPACE
