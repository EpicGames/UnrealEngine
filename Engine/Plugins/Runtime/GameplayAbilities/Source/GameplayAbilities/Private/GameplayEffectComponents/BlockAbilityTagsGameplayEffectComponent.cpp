// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectComponents/BlockAbilityTagsGameplayEffectComponent.h"
#include "Misc/DataValidation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlockAbilityTagsGameplayEffectComponent)

#define LOCTEXT_NAMESPACE "BlockAbilityTagsGameplayEffectComponent"

void UBlockAbilityTagsGameplayEffectComponent::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	EditorFriendlyName = TEXT("Block Abilities w/ Tags");
#endif

	// This code is meant to catch the case of newly constructed Components (not loaded Components).
	// With asynchronous loading, we are not guaranteed the Parent object is loaded at this point.
	// During loading, we'll call OnGameplayEffectChanged from UGameplayEffect::PostLoad().
	if (IsInAsyncLoadingThread() || IsInParallelLoadingThread())
	{
		return;
	}

	// Try to find the parent and update the inherited tags.  We do this 'early' because this is the only function
	// we can use for creation of the object (and this runs post-constructor).
	const UBlockAbilityTagsGameplayEffectComponent* Parent = FindParentComponent(*this);
	InheritableBlockedAbilityTagsContainer.UpdateInheritedTagProperties(Parent ? &Parent->InheritableBlockedAbilityTagsContainer : nullptr);
}

void UBlockAbilityTagsGameplayEffectComponent::OnGameplayEffectChanged()
{
	Super::OnGameplayEffectChanged();
	SetAndApplyBlockedAbilityTagChanges(InheritableBlockedAbilityTagsContainer);
}

#if WITH_EDITOR
void UBlockAbilityTagsGameplayEffectComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GetInheritableBlockedAbilityTagsContainerPropertyName())
	{
		// Tell the GE it needs to reconfigure itself based on these updated properties (this will reaggregate the tags)
		UGameplayEffect* Owner = GetOwner();
		Owner->OnGameplayEffectChanged();
	}
}

EDataValidationResult UBlockAbilityTagsGameplayEffectComponent::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	const bool bInstantEffect = (GetOwner()->DurationPolicy == EGameplayEffectDurationType::Instant);
	if (bInstantEffect && !InheritableBlockedAbilityTagsContainer.CombinedTags.IsEmpty())
	{
		Context.AddError(FText::FormatOrdered(LOCTEXT("GEInstantAndBlockAbilityTags", "GE {0} is set to Instant so {1} will not be able to function as expected."), FText::FromString(GetNameSafe(GetOwner())), FText::FromString(EditorFriendlyName)));
		Result = EDataValidationResult::Invalid;
	}

	return Result;
}

#endif // WITH_EDITOR

void UBlockAbilityTagsGameplayEffectComponent::SetAndApplyBlockedAbilityTagChanges(const FInheritedTagContainer& TagContainerMods)
{
	InheritableBlockedAbilityTagsContainer = TagContainerMods;

	// Try to find the parent and update the inherited tags
	const UBlockAbilityTagsGameplayEffectComponent* Parent = FindParentComponent(*this);
	InheritableBlockedAbilityTagsContainer.UpdateInheritedTagProperties(Parent ? &Parent->InheritableBlockedAbilityTagsContainer : nullptr);

	ApplyBlockedAbilityTagChanges();
}

void UBlockAbilityTagsGameplayEffectComponent::ApplyBlockedAbilityTagChanges() const
{
	// Apply to the owning Gameplay Effect Component
	UGameplayEffect* Owner = GetOwner();
	InheritableBlockedAbilityTagsContainer.ApplyTo(Owner->CachedBlockedAbilityTags);
}

#undef LOCTEXT_NAMESPACE
