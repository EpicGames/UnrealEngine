// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectComponents/CancelAbilityTagsGameplayEffectComponent.h"

#include "AbilitySystemComponent.h"
#include "Misc/DataValidation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CancelAbilityTagsGameplayEffectComponent)

#define LOCTEXT_NAMESPACE "CancelAbilityTagsGameplayEffectComponent"

void UCancelAbilityTagsGameplayEffectComponent::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	EditorFriendlyName = TEXT("Cancel Abilities With Tags");
#endif

	// This code is meant to catch the case of newly constructed Components (not loaded Components).
	// With asynchronous loading, we are not guaranteed the Parent object is loaded at this point.
	// During loading, we'll call OnGameplayEffectChanged from UGameplayEffect::PostLoad().
	if (IsInAsyncLoadingThread() || IsInParallelLoadingThread())
	{
		return;
	}

	// Try to find the parent and update the inherited tags.  We do this 'early' because this is the only function
	// we can use for creation of the object (and this runs post-constructor).
	const UCancelAbilityTagsGameplayEffectComponent* Parent = FindParentComponent(*this);
	InheritableCancelAbilitiesWithTagsContainer.UpdateInheritedTagProperties(Parent ? &Parent->InheritableCancelAbilitiesWithTagsContainer : nullptr);
	InheritableCancelAbilitiesWithoutTagsContainer.UpdateInheritedTagProperties(Parent ? &Parent->InheritableCancelAbilitiesWithoutTagsContainer : nullptr);
}

void UCancelAbilityTagsGameplayEffectComponent::OnGameplayEffectApplied(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const
{
	Super::OnGameplayEffectApplied(ActiveGEContainer, GESpec, PredictionKey);

	if (ComponentMode != ECancelAbilityTagsGameplayEffectComponentMode::OnApplication)
	{
		return;
	}

	CancelOwnerAbilities(ActiveGEContainer);
}

void UCancelAbilityTagsGameplayEffectComponent::OnGameplayEffectExecuted(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const
{
	Super::OnGameplayEffectExecuted(ActiveGEContainer, GESpec, PredictionKey);

	if (ComponentMode != ECancelAbilityTagsGameplayEffectComponentMode::OnExecution)
	{
		return;
	}
	
	CancelOwnerAbilities(ActiveGEContainer);
}

void UCancelAbilityTagsGameplayEffectComponent::OnGameplayEffectChanged()
{
	Super::OnGameplayEffectChanged();
	SetAndApplyCanceledAbilityTagChanges(InheritableCancelAbilitiesWithTagsContainer, InheritableCancelAbilitiesWithoutTagsContainer);
}

void UCancelAbilityTagsGameplayEffectComponent::SetAndApplyCanceledAbilityTagChanges(const FInheritedTagContainer& CanceledAbilityWithTagsMods, const FInheritedTagContainer& CanceledAbilityWithoutTagsMods)
{
	InheritableCancelAbilitiesWithTagsContainer = CanceledAbilityWithTagsMods;
	InheritableCancelAbilitiesWithoutTagsContainer = CanceledAbilityWithoutTagsMods;
	
	// Try to find the parent and update the inherited tags
	const UCancelAbilityTagsGameplayEffectComponent* Parent = FindParentComponent(*this);
	InheritableCancelAbilitiesWithTagsContainer.UpdateInheritedTagProperties(Parent ? &Parent->InheritableCancelAbilitiesWithTagsContainer : nullptr);
	InheritableCancelAbilitiesWithoutTagsContainer.UpdateInheritedTagProperties(Parent ? &Parent->InheritableCancelAbilitiesWithoutTagsContainer : nullptr);
}

void UCancelAbilityTagsGameplayEffectComponent::CancelOwnerAbilities(const FActiveGameplayEffectsContainer& ActiveGEContainer) const
{
	if (!ActiveGEContainer.OwnerIsNetAuthority)
	{
		return;
	}
	
	UAbilitySystemComponent* ASC = ActiveGEContainer.Owner;
	if (!ensure(ASC))
	{
		return;
	}

	// Empty container will fail the HasAny check, so pass nullptr instead
	const FGameplayTagContainer* WithTags = !InheritableCancelAbilitiesWithTagsContainer.CombinedTags.IsEmpty() ? &InheritableCancelAbilitiesWithTagsContainer.CombinedTags : nullptr;
	const FGameplayTagContainer* WithoutTags = !InheritableCancelAbilitiesWithoutTagsContainer.CombinedTags.IsEmpty() ? &InheritableCancelAbilitiesWithoutTagsContainer.CombinedTags : nullptr;
	ASC->CancelAbilities(WithTags, WithoutTags);
}

#if WITH_EDITOR
void UCancelAbilityTagsGameplayEffectComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GetInheritableCancelAbilitiesWithTagsContainerPropertyName() ||
		PropertyChangedEvent.GetMemberPropertyName() == GetInheritableCancelAbilitiesWithoutTagsContainerPropertyName())
	{
		// Tell the GE it needs to reconfigure itself based on these updated properties (this will reaggregate the tags)
		UGameplayEffect* Owner = GetOwner();
		Owner->OnGameplayEffectChanged();
	}
}

EDataValidationResult UCancelAbilityTagsGameplayEffectComponent::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	const bool bInstantEffect = (GetOwner()->DurationPolicy == EGameplayEffectDurationType::Instant);
	const bool bHasPeriod = GetOwner()->Period.Value > 0.0f;
	if (!bInstantEffect && !bHasPeriod && ComponentMode == ECancelAbilityTagsGameplayEffectComponentMode::OnExecution && !InheritableCancelAbilitiesWithTagsContainer.CombinedTags.IsEmpty())
	{
		Context.AddError(FText::FormatOrdered(
			LOCTEXT("GEInstantAndBlockAbilityTags", "GE {0} has a duration and no period while {1} has mode: OnExecution. {1} will not be able to function as expected, change mode to 'OnApplication'."),
			FText::FromString(GetNameSafe(GetOwner())),
			FText::FromString(EditorFriendlyName)));
		
		Result = EDataValidationResult::Invalid;
	}

	return Result;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE