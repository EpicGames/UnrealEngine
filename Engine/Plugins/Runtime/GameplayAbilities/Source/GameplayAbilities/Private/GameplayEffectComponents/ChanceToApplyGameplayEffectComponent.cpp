// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectComponents/ChanceToApplyGameplayEffectComponent.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChanceToApplyGameplayEffectComponent)

#if WITH_EDITOR
#define GETCURVE_REPORTERROR_WITHPOSTLOAD(Handle) \
	if (Handle.CurveTable) const_cast<UCurveTable*>(ToRawPtr(Handle.CurveTable))->ConditionalPostLoad(); \
	GETCURVE_REPORTERROR(Handle);

#endif // WITH_EDITOR

void UChanceToApplyGameplayEffectComponent::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	GETCURVE_REPORTERROR_WITHPOSTLOAD(ChanceToApplyToTarget.Curve);
#endif // WITH_EDITOR
}

bool UChanceToApplyGameplayEffectComponent::CanGameplayEffectApply(const FActiveGameplayEffectsContainer& ActiveGEContainer, const FGameplayEffectSpec& GESpec) const
{
	const FString ContextString = GESpec.Def->GetName();
	const float CalculatedChanceToApplyToTarget = ChanceToApplyToTarget.GetValueAtLevel(GESpec.GetLevel(), &ContextString);

	// check probability to apply
	if ((CalculatedChanceToApplyToTarget < 1.f - SMALL_NUMBER) && (FMath::FRand() > CalculatedChanceToApplyToTarget))
	{
		return false;
	}

	return true;
}

void UChanceToApplyGameplayEffectComponent::SetChanceToApplyToTarget(const FScalableFloat& InChanceToApplyToTarget)
{
	ChanceToApplyToTarget = InChanceToApplyToTarget;
}
