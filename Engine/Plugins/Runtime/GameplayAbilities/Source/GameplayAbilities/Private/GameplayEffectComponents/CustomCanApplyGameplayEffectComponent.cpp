// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectComponents/CustomCanApplyGameplayEffectComponent.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomCanApplyGameplayEffectComponent)

UCustomCanApplyGameplayEffectComponent::UCustomCanApplyGameplayEffectComponent()
{
#if WITH_EDITORONLY_DATA
	EditorFriendlyName = TEXT("Custom Can Apply Function (in Blueprint)");
#endif
}

bool UCustomCanApplyGameplayEffectComponent::CanGameplayEffectApply(const FActiveGameplayEffectsContainer& ActiveGEContainer, const FGameplayEffectSpec& GESpec) const
{
	// do custom application checks
	for (const TSubclassOf<UGameplayEffectCustomApplicationRequirement>& AppReq : ApplicationRequirements)
	{
		if (*AppReq && AppReq->GetDefaultObject<UGameplayEffectCustomApplicationRequirement>()->CanApplyGameplayEffect(GESpec.Def, GESpec, ActiveGEContainer.Owner) == false)
		{
			return false;
		}
	}

	return true;
}
