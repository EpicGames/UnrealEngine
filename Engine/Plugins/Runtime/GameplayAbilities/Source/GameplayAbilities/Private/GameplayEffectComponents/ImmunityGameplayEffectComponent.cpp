// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectComponents/ImmunityGameplayEffectComponent.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"
#include "AbilitySystemStats.h"
#include "Misc/DataValidation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImmunityGameplayEffectComponent)

#define LOCTEXT_NAMESPACE "ImmunityGameplayEffectComponent"

UImmunityGameplayEffectComponent::UImmunityGameplayEffectComponent()
{
#if WITH_EDITORONLY_DATA
	EditorFriendlyName = TEXT("Immunity (Prevent Other GEs)");
#endif // WITH_EDITORONLY_DATA
}

bool UImmunityGameplayEffectComponent::OnActiveGameplayEffectAdded(FActiveGameplayEffectsContainer& ActiveGEContainer, FActiveGameplayEffect& ActiveGE) const
{
	FActiveGameplayEffectHandle& ActiveGEHandle = ActiveGE.Handle;
	UAbilitySystemComponent* OwnerASC = ActiveGEContainer.Owner;

	// Register our immunity query to potentially block applications of any Gameplay Effects
	FGameplayEffectApplicationQuery& BoundQuery = OwnerASC->GameplayEffectApplicationQueries.AddDefaulted_GetRef();
	BoundQuery.BindUObject(this, &UImmunityGameplayEffectComponent::AllowGameplayEffectApplication, ActiveGEHandle);

	// Now that we've bound that function, let's unbind it when we're removed.  This is safe because once we're removed, EventSet is gone.
	ActiveGE.EventSet.OnEffectRemoved.AddLambda([OwnerASC, QueryToRemove = BoundQuery.GetHandle()](const FGameplayEffectRemovalInfo& RemovalInfo)
		{
			if (ensure(IsValid(OwnerASC)))
			{
				TArray<FGameplayEffectApplicationQuery>& GEAppQueries = OwnerASC->GameplayEffectApplicationQueries;
				for (auto It = GEAppQueries.CreateIterator(); It; ++It)
				{
					if (It->GetHandle() == QueryToRemove)
					{
						It.RemoveCurrentSwap();
						break;
					}
				}
			}
		});
	return true;
}

bool UImmunityGameplayEffectComponent::AllowGameplayEffectApplication(const FActiveGameplayEffectsContainer& ActiveGEContainer, const FGameplayEffectSpec& GESpecToConsider, FActiveGameplayEffectHandle ImmunityActiveGEHandle) const
{
	SCOPE_CYCLE_COUNTER(STAT_HasApplicationImmunityToSpec)

	const UAbilitySystemComponent* ASC = ActiveGEContainer.Owner;

	// ActiveGE can be null (IsPendingRemove) but we could have not yet received OnEffectRemoved. This can happen when applying more effects during its removal.
	const FActiveGameplayEffect* ActiveGE = ASC->GetActiveGameplayEffect(ImmunityActiveGEHandle);
	if (!ActiveGE || ActiveGE->bIsInhibited)
	{
		return true;
	}

	ensureMsgf(ASC == ImmunityActiveGEHandle.GetOwningAbilitySystemComponent(), TEXT("%hs: FActiveGameplayEffectHandle lost its Owning ASC"), __FUNCTION__);

	for (const FGameplayEffectQuery& ImmunityQuery : ImmunityQueries)
	{
		if (!ImmunityQuery.IsEmpty() && ImmunityQuery.Matches(GESpecToConsider))
		{
			ASC->OnImmunityBlockGameplayEffectDelegate.Broadcast(GESpecToConsider, ActiveGE);
			return false;
		}
	}

	return true;
}

#if WITH_EDITOR
EDataValidationResult UImmunityGameplayEffectComponent::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	if (GetOwner()->DurationPolicy == EGameplayEffectDurationType::Instant)
	{
		Context.AddError(FText::FormatOrdered(LOCTEXT("IsInstantEffectError", "GE is an Instant Effect and incompatible with {0}"), FText::FromString(GetClass()->GetName())));
		Result = EDataValidationResult::Invalid;
	}

	return Result;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
