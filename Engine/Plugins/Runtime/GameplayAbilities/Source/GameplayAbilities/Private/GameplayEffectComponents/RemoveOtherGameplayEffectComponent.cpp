// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectComponents/RemoveOtherGameplayEffectComponent.h"
#include "AbilitySystemComponent.h"
#include "Misc/DataValidation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RemoveOtherGameplayEffectComponent)

#define LOCTEXT_NAMESPACE "RemoveOtherGameplayEffectComponent"

URemoveOtherGameplayEffectComponent::URemoveOtherGameplayEffectComponent()
{
#if WITH_EDITORONLY_DATA
	EditorFriendlyName = TEXT("Remove Other Gameplay Effects");
#endif
}

void URemoveOtherGameplayEffectComponent::OnGameplayEffectApplied(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const
{
	if (!ActiveGEContainer.OwnerIsNetAuthority)
	{
		return;
	}

	FGameplayEffectQuery FindOwnerQuery;
	FindOwnerQuery.EffectDefinition = GetOwner() ? GetOwner()->GetClass() : nullptr;

	// We need to keep track to ensure we never remove ourselves
	TArray<FActiveGameplayEffectHandle> ActiveGEHandles = ActiveGEContainer.GetActiveEffects(FindOwnerQuery);

	constexpr int32 RemoveAllStacks = -1;
	for (const FGameplayEffectQuery& RemoveQuery : RemoveGameplayEffectQueries)
	{
		if (!RemoveQuery.IsEmpty())
		{
			// If we have an ActiveGEHandle, make sure we never remove ourselves.
			// If we don't, there's no need to make a copy.
			if (ActiveGEHandles.IsEmpty())
			{
				// Faster path: No copy needed
				ActiveGEContainer.RemoveActiveEffects(RemoveQuery, RemoveAllStacks);
			}
			else
			{
				FGameplayEffectQuery MutableRemoveQuery = RemoveQuery;
				MutableRemoveQuery.IgnoreHandles = MoveTemp(ActiveGEHandles);

				ActiveGEContainer.RemoveActiveEffects(MutableRemoveQuery, RemoveAllStacks);
			}
		}
	}
}

#if WITH_EDITOR
EDataValidationResult URemoveOtherGameplayEffectComponent::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	if (GetOwner()->DurationPolicy != EGameplayEffectDurationType::Instant)
	{
		if (GetOwner()->Period.Value > 0.0f)
		{
			Context.AddError(FText::FormatOrdered(LOCTEXT("PeriodicEffectError", "GE is Periodic. Remove {0} and use TagRequirements (Ongoing) instead."), FText::FromString(GetClass()->GetName())));
			Result = EDataValidationResult::Invalid;
		}
	}

	return Result;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
