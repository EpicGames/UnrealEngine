// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectComponents/TargetTagRequirementsGameplayEffectComponent.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"
#include "Algo/Find.h"
#include "Misc/DataValidation.h"
#include "AbilitySystemPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetTagRequirementsGameplayEffectComponent)

#define LOCTEXT_NAMESPACE "TargetTagRequirementsGameplayEffectComponent"

UTargetTagRequirementsGameplayEffectComponent::UTargetTagRequirementsGameplayEffectComponent()
{
#if WITH_EDITORONLY_DATA
	EditorFriendlyName = TEXT("Target Tag Reqs (While GE is Active)");
#endif // WITH_EDITORONLY_DATA
}

bool UTargetTagRequirementsGameplayEffectComponent::CanGameplayEffectApply(const FActiveGameplayEffectsContainer& ActiveGEContainer, const FGameplayEffectSpec& GESpec) const
{
	FGameplayTagContainer Tags;
	ActiveGEContainer.Owner->GetOwnedGameplayTags(Tags);
	
	if (ApplicationTagRequirements.RequirementsMet(Tags) == false)
	{
		return false;
	}

	if (HaveRemovalRequirementsBeenMet(Tags, ActiveGEContainer.IsNetAuthority()))
	{
		return false;
	}

	return true;
}

// UTargetTagRequirementsGameplayEffectComponent lives on an asset.  This doesn't get instanced at runtime, so this is NOT A UNIQUE INSTANCE (it is a shared instance for any GEContainer/ActiveGE that wants to use it).
bool UTargetTagRequirementsGameplayEffectComponent::OnActiveGameplayEffectAdded(FActiveGameplayEffectsContainer& GEContainer, FActiveGameplayEffect& ActiveGE) const
{
	UAbilitySystemComponent* ASC = GEContainer.Owner;
	if (!ensure(ASC))
	{
		return false;
	}

	using namespace UE::AbilitySystem::Private;
	const bool bAllowClientSideRemoval = EnumHasAnyFlags(static_cast<EAllowPredictiveGEFlags>(CVarAllowPredictiveGEFlagsValue), EAllowPredictiveGEFlags::AllowRemovalByTagRequirements);
	const bool bAuthority = GEContainer.IsNetAuthority();

	FActiveGameplayEffectHandle ActiveGEHandle = ActiveGE.Handle;
	if (FActiveGameplayEffectEvents* EventSet = ASC->GetActiveEffectEventSet(ActiveGEHandle))
	{
		// Quick method of appending a TArray to another TArray with no duplicates.
		auto AppendUnique = [](TArray<FGameplayTag>& Destination, const TArray<FGameplayTag>& Source)
		{
			// Make sure the array won't allocate during the loop
			if (Destination.GetSlack() < Source.Num())
			{
				Destination.Reserve(Destination.Num() + Source.Num());
			}
			const TConstArrayView<FGameplayTag> PreModifiedDestinationView{ Destination.GetData(), Destination.Num() };

			for (const FGameplayTag& Tag : Source)
			{
				if (!Algo::Find(PreModifiedDestinationView, Tag))
				{
					Destination.Emplace(Tag);
				}
			}
		};

		// We should gather a list of tags to listen on events for
		TArray<FGameplayTag> GameplayTagsToBind;
		AppendUnique(GameplayTagsToBind, OngoingTagRequirements.IgnoreTags.GetGameplayTagArray());
		AppendUnique(GameplayTagsToBind, OngoingTagRequirements.RequireTags.GetGameplayTagArray());
		AppendUnique(GameplayTagsToBind, OngoingTagRequirements.TagQuery.GetGameplayTagArray());

		if (bAuthority || bAllowClientSideRemoval)
		{
			AppendUnique(GameplayTagsToBind, RemovalTagRequirements.IgnoreTags.GetGameplayTagArray());
			AppendUnique(GameplayTagsToBind, RemovalTagRequirements.RequireTags.GetGameplayTagArray());
			AppendUnique(GameplayTagsToBind, RemovalTagRequirements.TagQuery.GetGameplayTagArray());
		}

		// Add our tag requirements to the ASC's Callbacks map. This helps filter down the amount of callbacks we'll get due to tag changes
		// (rather than registering for the one callback whenever any tag changes).  We also need to keep track to remove those registered delegates in OnEffectRemoved.
		TArray<TTuple<FGameplayTag, FDelegateHandle>> AllBoundEvents;
		for (const FGameplayTag& Tag : GameplayTagsToBind)
		{
			FOnGameplayEffectTagCountChanged& OnTagEvent = ASC->RegisterGameplayTagEvent(Tag, EGameplayTagEventType::NewOrRemoved);
			FDelegateHandle Handle = OnTagEvent.AddUObject(this, &UTargetTagRequirementsGameplayEffectComponent::OnTagChanged, ActiveGEHandle);
			AllBoundEvents.Emplace(Tag, Handle);
		}

		// Now when this Effect is removed, we should remove all of our registered callbacks.
		EventSet->OnEffectRemoved.AddUObject(this, &UTargetTagRequirementsGameplayEffectComponent::OnGameplayEffectRemoved, ASC, MoveTemp(AllBoundEvents));
	}
	else
	{
		UE_LOGF(LogGameplayEffects, Error, "UTargetTagRequirementsGameplayEffectComponent::OnGameplayEffectAdded called with ActiveGE: %ls which had an invalid FActiveGameplayEffectHandle.", *ActiveGE.GetDebugString());
	}

	FGameplayTagContainer TagContainer;
	ASC->GetOwnedGameplayTags(TagContainer);

	return OngoingTagRequirements.RequirementsMet(TagContainer);
}

void UTargetTagRequirementsGameplayEffectComponent::OnGameplayEffectRemoved(const FGameplayEffectRemovalInfo& GERemovalInfo, UAbilitySystemComponent* ASC, TArray<TTuple<FGameplayTag, FDelegateHandle>> AllBoundEvents) const
{
	for (TTuple<FGameplayTag, FDelegateHandle>& Pair : AllBoundEvents)
	{
		const bool bSuccess = ASC->UnregisterGameplayTagEvent(Pair.Value, Pair.Key, EGameplayTagEventType::NewOrRemoved);
		UE_CLOGF(!bSuccess, LogGameplayEffects, Error, "%ls tried to unregister GameplayTagEvent '%ls' on GameplayEffect '%ls' but failed.", *GetName(), *Pair.Key.ToString(), *GetNameSafe(GetOwner()));
	}
}

void UTargetTagRequirementsGameplayEffectComponent::OnTagChanged(const FGameplayTag GameplayTag, int32 NewCount, FActiveGameplayEffectHandle ActiveGEHandle) const
{
	// Note: This function can remove us (RemoveActiveGameplayEffect eventually calling OnGameplayEffectRemoved),
	// but we could be in the middle of a stack of OnTagChanged callbacks, wo we could get a stale OnTagChanged.
	UAbilitySystemComponent* Owner = ActiveGEHandle.GetOwningAbilitySystemComponent();
	if (!Owner)
	{
		return;
	}

	// It's possible for this to return nullptr if it was in the process of being removed (IsPendingRemove)
	const FActiveGameplayEffect* ActiveGE = Owner->GetActiveGameplayEffect(ActiveGEHandle);
	if (ActiveGE)
	{
		FGameplayTagContainer OwnedTags;
		Owner->GetOwnedGameplayTags(OwnedTags);

		const bool bRemovalRequirementsMet = HaveRemovalRequirementsBeenMet(OwnedTags, Owner->IsOwnerActorAuthoritative());
		if (bRemovalRequirementsMet)
		{
			// This is slightly different functionality from pre-UE5.3, we're calling RemoveActiveGameplayEffect rather than InternalRemoveActiveGameplayEffect.
			// The result is we set the calculated magnitudes back to zero.  In UE5.3 this used to also run on the client (incorrect).
			Owner->RemoveActiveGameplayEffect(ActiveGEHandle);
		}
		else
		{
			// See if we should be inhibiting the execution
			constexpr bool bInvokeCuesIfStateChanged = true;
			const bool bOngoingRequirementsMet = OngoingTagRequirements.IsEmpty() || OngoingTagRequirements.RequirementsMet(OwnedTags);
			Owner->SetActiveGameplayEffectInhibit(MoveTemp(ActiveGEHandle), !bOngoingRequirementsMet, bInvokeCuesIfStateChanged);
		}
	}
}

bool UTargetTagRequirementsGameplayEffectComponent::HaveRemovalRequirementsBeenMet(const FGameplayTagContainer& TargetOwnedTags, bool bNetAuthority) const
{
	using namespace UE::AbilitySystem::Private;
	const bool bAllowClientSideRemoval = EnumHasAnyFlags(static_cast<EAllowPredictiveGEFlags>(CVarAllowPredictiveGEFlagsValue), EAllowPredictiveGEFlags::AllowRemovalByTagRequirements);
	const bool bRemovalRequirementsMet = (bAllowClientSideRemoval || bNetAuthority) && !RemovalTagRequirements.IsEmpty() && RemovalTagRequirements.RequirementsMet(TargetOwnedTags);

	return bRemovalRequirementsMet;
}

#if WITH_EDITOR
/**
 * Validate incompatable configurations
 */
EDataValidationResult UTargetTagRequirementsGameplayEffectComponent::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	const bool bInstantEffect = (GetOwner()->DurationPolicy == EGameplayEffectDurationType::Instant);
	if (bInstantEffect && !OngoingTagRequirements.IsEmpty())
	{
		Context.AddError(LOCTEXT("GEInstantAndOngoing", "GE is instant but has OngoingTagRequirements."));
		Result = EDataValidationResult::Invalid;
	}

	return Result;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
