// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectComponents/TargetTagsGameplayEffectComponent.h"
#include "Misc/DataValidation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetTagsGameplayEffectComponent)

#define LOCTEXT_NAMESPACE "TargetTagsGameplayEffectComponent"

void UTargetTagsGameplayEffectComponent::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	EditorFriendlyName = TEXT("Target Tags (Granted to Actor)");
#endif // WITH_EDITORONLY_DATA

	// This code is meant to catch the case of newly constructed Components (not loaded Components).
	// With asynchronous loading, we are not guaranteed the Parent object is loaded at this point.
	// During loading, we'll call OnGameplayEffectChanged from UGameplayEffect::PostLoad().
	if (IsInAsyncLoadingThread() || IsInParallelLoadingThread())
	{
		return;
	}

	// Try to find the parent and update the inherited tags.  We do this 'early' because this is the only function
	// we can use for creation of the object (and this runs post-constructor).
	const UTargetTagsGameplayEffectComponent* Parent = FindParentComponent(*this);
	InheritableGrantedTagsContainer.UpdateInheritedTagProperties(Parent ? &Parent->InheritableGrantedTagsContainer : nullptr);
}

void UTargetTagsGameplayEffectComponent::OnGameplayEffectChanged()
{
	Super::OnGameplayEffectChanged();
	SetAndApplyTargetTagChanges(InheritableGrantedTagsContainer);
}

#if WITH_EDITOR

void UTargetTagsGameplayEffectComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GetInheritableGrantedTagsContainerName())
	{
		// Tell the GE it needs to reconfigure itself based on these updated properties (this will reaggregate the tags)
		UGameplayEffect* Owner = GetOwner();
		Owner->OnGameplayEffectChanged();
	}
}

EDataValidationResult UTargetTagsGameplayEffectComponent::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	const bool bInstantEffect = (GetOwner()->DurationPolicy == EGameplayEffectDurationType::Instant);
	if (bInstantEffect && !InheritableGrantedTagsContainer.CombinedTags.IsEmpty())
	{
		Context.AddError(FText::FormatOrdered(LOCTEXT("GEInstantAndTargetTags", "GE {0} is set to Instant so TargetTagsGameplayEffectComponent will not be able to apply its tags."), FText::FromString(GetNameSafe(GetOwner()))));
		Result = EDataValidationResult::Invalid;
	}

	return Result;
}

#endif // WITH_EDITOR

void UTargetTagsGameplayEffectComponent::SetAndApplyTargetTagChanges(const FInheritedTagContainer& TagContainerMods)
{
	InheritableGrantedTagsContainer = TagContainerMods;

	// Try to find the parent and update the inherited tags
	const UTargetTagsGameplayEffectComponent* Parent = FindParentComponent(*this);
	InheritableGrantedTagsContainer.UpdateInheritedTagProperties(Parent ? &Parent->InheritableGrantedTagsContainer : nullptr);

	// Apply to the owning Gameplay Effect Component
	ApplyTargetTagChanges();
}

void UTargetTagsGameplayEffectComponent::ApplyTargetTagChanges() const
{
	UGameplayEffect* Owner = GetOwner();
	InheritableGrantedTagsContainer.ApplyTo(Owner->CachedGrantedTags);
}

#undef LOCTEXT_NAMESPACE
