// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectCustomApplicationRequirement.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayEffectCustomApplicationRequirement)

bool UGameplayEffectCustomApplicationRequirement::CanApplyGameplayEffect_Implementation(const UGameplayEffect* GameplayEffect, const FGameplayEffectSpec& Spec, UAbilitySystemComponent* ASC) const
{
	return true;
}

