// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectExecutionCalculation.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayEffectExecutionCalculation)

FGameplayEffectCustomExecutionParameters::FGameplayEffectCustomExecutionParameters()
	: OwningSpec(nullptr)
	, TargetAbilitySystemComponent(nullptr)
	, PassedInTags(FGameplayTagContainer())
{
}

FGameplayEffectCustomExecutionParameters::FGameplayEffectCustomExecutionParameters(FGameplayEffectSpec& InOwningSpec, const TArray<FGameplayEffectExecutionScopedModifierInfo>& InScopedMods, UAbilitySystemComponent* InTargetAbilityComponent, const FGameplayTagContainer& InPassedInTags, const FPredictionKey& InPredictionKey)
	: OwningSpec(&InOwningSpec)
	, TargetAbilitySystemComponent(InTargetAbilityComponent)
	, PassedInTags(InPassedInTags)
	, PredictionKey(InPredictionKey)
{
	check(InOwningSpec.Def);

	FActiveGameplayEffectHandle ModifierHandle = FActiveGameplayEffectHandle::GenerateNewHandle(InTargetAbilityComponent);

	for (const FGameplayEffectExecutionScopedModifierInfo& CurScopedMod : InScopedMods)
	{
		FAggregator* ScopedAggregator = nullptr;

		// Scoped mod is a traditional, attribute-backed aggregator; Try to find an existing one or fetch one from a capture spec
		if (CurScopedMod.AggregatorType == EGameplayEffectScopedModifierAggregatorType::CapturedAttributeBacked)
		{
			ScopedAggregator = ScopedModifierAggregators.Find(CurScopedMod.CapturedAttribute);
			if (!ScopedAggregator)
			{
				const FGameplayEffectAttributeCaptureSpec* CaptureSpec = InOwningSpec.CapturedRelevantAttributes.FindCaptureSpecByDefinition(CurScopedMod.CapturedAttribute, true);

				FAggregator SnapshotAgg;
				if (CaptureSpec && CaptureSpec->AttemptGetAttributeAggregatorSnapshot(SnapshotAgg))
				{
					ScopedAggregator = &(ScopedModifierAggregators.Add(CurScopedMod.CapturedAttribute, SnapshotAgg));
				}
			}
		}
		// Scoped mod is acting on a "temporary variable," find/create the appropriate transient aggregator
		else
		{
			ScopedAggregator = &ScopedTransientAggregators.FindOrAdd(CurScopedMod.TransientAggregatorIdentifier);
		}

		float ModEvalValue = 0.f;
		if (ScopedAggregator && CurScopedMod.ModifierMagnitude.AttemptCalculateMagnitude(InOwningSpec, ModEvalValue))
		{
			ScopedAggregator->AddAggregatorMod(ModEvalValue, CurScopedMod.ModifierOp, CurScopedMod.EvaluationChannelSettings.GetEvaluationChannel(), &CurScopedMod.SourceTags, &CurScopedMod.TargetTags, false, ModifierHandle);
		}
		else
		{
			ABILITY_LOG(Warning, TEXT("Attempted to apply a scoped modifier from %s's %s magnitude calculation that could not be properly calculated. Some attributes necessary for the calculation were missing."), *InOwningSpec.Def->GetName(), *CurScopedMod.CapturedAttribute.ToSimpleString());
		}
	}
}

FGameplayEffectCustomExecutionParameters::FGameplayEffectCustomExecutionParameters(FGameplayEffectSpec& InOwningSpec, const TArray<FGameplayEffectExecutionScopedModifierInfo>& InScopedMods, UAbilitySystemComponent* InTargetAbilityComponent, const FGameplayTagContainer& InPassedInTags, const FPredictionKey& InPredictionKey, const TArray<FActiveGameplayEffectHandle>& InIgnoreHandles)
	: FGameplayEffectCustomExecutionParameters(InOwningSpec, InScopedMods, InTargetAbilityComponent, InPassedInTags, InPredictionKey)
{
	IgnoreHandles = InIgnoreHandles;
}

const FGameplayEffectSpec& FGameplayEffectCustomExecutionParameters::GetOwningSpec() const
{
	check(OwningSpec);
	return *OwningSpec;
}

FGameplayEffectSpec* FGameplayEffectCustomExecutionParameters::GetOwningSpecForPreExecuteMod() const
{
	check(OwningSpec);
	return OwningSpec;
}

UAbilitySystemComponent* FGameplayEffectCustomExecutionParameters::GetTargetAbilitySystemComponent() const
{
	return TargetAbilitySystemComponent.Get();
}

UAbilitySystemComponent* FGameplayEffectCustomExecutionParameters::GetSourceAbilitySystemComponent() const
{
	check(OwningSpec);
	return OwningSpec->GetContext().GetInstigatorAbilitySystemComponent();
}

const FGameplayTagContainer& FGameplayEffectCustomExecutionParameters::GetPassedInTags() const
{
	return PassedInTags;
}

TArray<FActiveGameplayEffectHandle> FGameplayEffectCustomExecutionParameters::GetIgnoreHandles() const
{
	return IgnoreHandles;
}
	
FPredictionKey FGameplayEffectCustomExecutionParameters::GetPredictionKey() const
{
	return PredictionKey;
}

bool FGameplayEffectCustomExecutionParameters::AttemptCalculateCapturedAttributeMagnitude(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, const FAggregatorEvaluateParameters& InEvalParams, OUT float& OutMagnitude) const
{
	check(OwningSpec);

	const FAggregator* CalcAgg = ScopedModifierAggregators.Find(InCaptureDef);
	if (CalcAgg)
	{
		OutMagnitude = CalcAgg->Evaluate(InEvalParams);
		return true;
	}
	else
	{
		const FGameplayEffectAttributeCaptureSpec* CaptureSpec = OwningSpec->CapturedRelevantAttributes.FindCaptureSpecByDefinition(InCaptureDef, true);
		if (CaptureSpec)
		{
			return CaptureSpec->AttemptCalculateAttributeMagnitude(InEvalParams, OutMagnitude);
		}
	}

	return false;
}

bool FGameplayEffectCustomExecutionParameters::AttemptCalculateCapturedAttributeMagnitudeWithBase(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, const FAggregatorEvaluateParameters& InEvalParams, float InBaseValue, OUT float& OutMagnitude) const
{
	check(OwningSpec);

	const FAggregator* CalcAgg = ScopedModifierAggregators.Find(InCaptureDef);
	if (CalcAgg)
	{
		OutMagnitude = CalcAgg->EvaluateWithBase(InBaseValue, InEvalParams);
		return true;
	}
	else
	{
		const FGameplayEffectAttributeCaptureSpec* CaptureSpec = OwningSpec->CapturedRelevantAttributes.FindCaptureSpecByDefinition(InCaptureDef, true);
		if (CaptureSpec)
		{
			return CaptureSpec->AttemptCalculateAttributeMagnitudeWithBase(InEvalParams, InBaseValue, OutMagnitude);
		}
	}

	return false;
}

bool FGameplayEffectCustomExecutionParameters::AttemptCalculateCapturedAttributeBaseValue(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, OUT float& OutBaseValue) const
{
	check(OwningSpec);

	const FAggregator* CalcAgg = ScopedModifierAggregators.Find(InCaptureDef);
	if (CalcAgg)
	{
		OutBaseValue = CalcAgg->GetBaseValue();
		return true;
	}
	else
	{
		const FGameplayEffectAttributeCaptureSpec* CaptureSpec = OwningSpec->CapturedRelevantAttributes.FindCaptureSpecByDefinition(InCaptureDef, true);
		if (CaptureSpec)
		{
			return CaptureSpec->AttemptCalculateAttributeBaseValue(OutBaseValue);
		}
	}

	return false;
}

bool FGameplayEffectCustomExecutionParameters::AttemptCalculateCapturedAttributeBonusMagnitude(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, const FAggregatorEvaluateParameters& InEvalParams, OUT float& OutBonusMagnitude) const
{
	check(OwningSpec);

	const FAggregator* CalcAgg = ScopedModifierAggregators.Find(InCaptureDef);
	if (CalcAgg)
	{
		OutBonusMagnitude = CalcAgg->EvaluateBonus(InEvalParams);
		return true;
	}
	else
	{
		const FGameplayEffectAttributeCaptureSpec* CaptureSpec = OwningSpec->CapturedRelevantAttributes.FindCaptureSpecByDefinition(InCaptureDef, true);
		if (CaptureSpec)
		{
			return CaptureSpec->AttemptCalculateAttributeBonusMagnitude(InEvalParams, OutBonusMagnitude);
		}
	}

	return false;
}

bool FGameplayEffectCustomExecutionParameters::AttemptGetCapturedAttributeAggregatorSnapshot(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, OUT FAggregator& OutSnapshottedAggregator) const
{
	check(OwningSpec);

	const FAggregator* CalcAgg = ScopedModifierAggregators.Find(InCaptureDef);
	if (CalcAgg)
	{
		OutSnapshottedAggregator.TakeSnapshotOf(*CalcAgg);
		return true;
	}
	else
	{
		const FGameplayEffectAttributeCaptureSpec* CaptureSpec = OwningSpec->CapturedRelevantAttributes.FindCaptureSpecByDefinition(InCaptureDef, true);
		if (CaptureSpec)
		{
			return CaptureSpec->AttemptGetAttributeAggregatorSnapshot(OutSnapshottedAggregator);
		}
	}

	return false;
}

bool FGameplayEffectCustomExecutionParameters::AttemptGatherAttributeMods(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, const FAggregatorEvaluateParameters& InEvalParams, OUT TMap<EGameplayModEvaluationChannel, const TArray<FAggregatorMod>*>& OutModMap) const
{
	check(OwningSpec);

	const FAggregator* CalcAgg = ScopedModifierAggregators.Find(InCaptureDef);
	if (CalcAgg)
	{
		CalcAgg->GetAllAggregatorMods(OutModMap);
	}
	else
	{
		const FGameplayEffectAttributeCaptureSpec* CaptureSpec = OwningSpec->CapturedRelevantAttributes.FindCaptureSpecByDefinition(InCaptureDef, true);
		if (CaptureSpec)
		{
			return CaptureSpec->AttemptGatherAttributeMods(InEvalParams, OutModMap);
		}
	}

	return false;
}

bool FGameplayEffectCustomExecutionParameters::ForEachQualifiedAttributeMod(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, const FAggregatorEvaluateParameters& InEvalParams, TFunction< void(EGameplayModEvaluationChannel, EGameplayModOp::Type, const FAggregatorMod&) > Func) const
{
	TMap<EGameplayModEvaluationChannel, const TArray<FAggregatorMod>*> ModMap;
	if (AttemptGatherAttributeMods(InCaptureDef, InEvalParams, ModMap))
	{
		for (auto It : ModMap)
		{
			const TArray<FAggregatorMod>* ModList = It.Value;
			for (int32 ModOpIdx = 0; ModOpIdx < EGameplayModOp::Max; ++ModOpIdx)
			{
				const TArray<FAggregatorMod>& CurModArray = ModList[ModOpIdx];
				for (const FAggregatorMod& AggMod : CurModArray)
				{
					if (AggMod.Qualifies())
					{
						Func(It.Key, (EGameplayModOp::Type)ModOpIdx, AggMod);
					}
				}
			}
		}
		return true;
	}

	return false;
}

bool FGameplayEffectCustomExecutionParameters::AttemptCalculateTransientAggregatorMagnitude(const FGameplayTag& InAggregatorIdentifier, const FAggregatorEvaluateParameters& InEvalParams, OUT float& OutMagnitude) const
{
	const FAggregator* CalcAgg = ScopedTransientAggregators.Find(InAggregatorIdentifier);
	if (CalcAgg)
	{
		OutMagnitude = CalcAgg->Evaluate(InEvalParams);
		return true;
	}
	
	return false;
}

bool FGameplayEffectCustomExecutionParameters::AttemptCalculateTransientAggregatorMagnitudeWithBase(const FGameplayTag& InAggregatorIdentifier, const FAggregatorEvaluateParameters& InEvalParams, float InBaseValue, OUT float& OutMagnitude) const
{
	const FAggregator* CalcAgg = ScopedTransientAggregators.Find(InAggregatorIdentifier);
	if (CalcAgg)
	{
		OutMagnitude = CalcAgg->EvaluateWithBase(InBaseValue, InEvalParams);
		return true;
	}

	return false;
}

bool FGameplayEffectCustomExecutionParameters::AttemptCalculateTransientAggregatorBaseValue(const FGameplayTag& InAggregatorIdentifier, OUT float& OutBaseValue) const
{
	const FAggregator* CalcAgg = ScopedTransientAggregators.Find(InAggregatorIdentifier);
	if (CalcAgg)
	{
		OutBaseValue = CalcAgg->GetBaseValue();
		return true;
	}

	return false;
}

bool FGameplayEffectCustomExecutionParameters::AttemptCalculateTransientAggregatorBonusMagnitude(const FGameplayTag& InAggregatorIdentifier, const FAggregatorEvaluateParameters& InEvalParams, OUT float& OutBonusMagnitude) const
{
	const FAggregator* CalcAgg = ScopedTransientAggregators.Find(InAggregatorIdentifier);
	if (CalcAgg)
	{
		OutBonusMagnitude = CalcAgg->EvaluateBonus(InEvalParams);
		return true;
	}

	return false;
}

bool FGameplayEffectCustomExecutionParameters::AttemptGetCapturedAttributeAggregatorSnapshot(const FGameplayTag& InAggregatorIdentifier, OUT FAggregator& OutSnapshottedAggregator) const
{
	const FAggregator* CalcAgg = ScopedTransientAggregators.Find(InAggregatorIdentifier);
	if (CalcAgg)
	{
		OutSnapshottedAggregator.TakeSnapshotOf(*CalcAgg);
		return true;
	}

	return false;
}

FGameplayEffectCustomExecutionOutput::FGameplayEffectCustomExecutionOutput()
	: bTriggerConditionalGameplayEffects(false)
	, bHandledStackCountManually(false)
	, bHandledGameplayCuesManually(false)
{
}

void FGameplayEffectCustomExecutionOutput::MarkStackCountHandledManually()
{
	bHandledStackCountManually = true;
}

bool FGameplayEffectCustomExecutionOutput::IsStackCountHandledManually() const
{
	return bHandledStackCountManually;
}

bool FGameplayEffectCustomExecutionOutput::AreGameplayCuesHandledManually() const
{
	return bHandledGameplayCuesManually;
}

void FGameplayEffectCustomExecutionOutput::MarkConditionalGameplayEffectsToTrigger()
{
	bTriggerConditionalGameplayEffects = true;
}

void FGameplayEffectCustomExecutionOutput::MarkGameplayCuesHandledManually()
{
	bHandledGameplayCuesManually = true;
}

bool FGameplayEffectCustomExecutionOutput::ShouldTriggerConditionalGameplayEffects() const
{
	return bTriggerConditionalGameplayEffects;
}

void FGameplayEffectCustomExecutionOutput::AddOutputModifier(const FGameplayModifierEvaluatedData& InOutputMod)
{
	OutputModifiers.Add(InOutputMod);
}

const TArray<FGameplayModifierEvaluatedData>& FGameplayEffectCustomExecutionOutput::GetOutputModifiers() const
{
	return OutputModifiers;
}

void FGameplayEffectCustomExecutionOutput::GetOutputModifiers(OUT TArray<FGameplayModifierEvaluatedData>& OutOutputModifiers) const
{
	OutOutputModifiers.Append(OutputModifiers);
}

UGameplayEffectExecutionCalculation::UGameplayEffectExecutionCalculation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bRequiresPassedInTags = false;
}

#if WITH_EDITORONLY_DATA
void UGameplayEffectExecutionCalculation::GetValidScopedModifierAttributeCaptureDefinitions(OUT TArray<FGameplayEffectAttributeCaptureDefinition>& OutScopableModifiers) const
{
	OutScopableModifiers.Empty();

	const TArray<FGameplayEffectAttributeCaptureDefinition>& DefaultCaptureDefs = GetAttributeCaptureDefinitions();
	for (const FGameplayEffectAttributeCaptureDefinition& CurDef : DefaultCaptureDefs)
	{
		if (!InvalidScopedModifierAttributes.Contains(CurDef))
		{
			OutScopableModifiers.Add(CurDef);
		}
	}
}

const FGameplayTagContainer& UGameplayEffectExecutionCalculation::GetValidTransientAggregatorIdentifiers() const
{
	return ValidTransientAggregatorIdentifiers;
}

bool UGameplayEffectExecutionCalculation::DoesRequirePassedInTags() const
{
	return bRequiresPassedInTags;
}
#endif // #if WITH_EDITORONLY_DATA

void UGameplayEffectExecutionCalculation::Execute_Implementation(const FGameplayEffectCustomExecutionParameters& ExecutionParams, OUT FGameplayEffectCustomExecutionOutput& OutExecutionOutput) const
{
}

