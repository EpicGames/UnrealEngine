// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectTypes.h"
#include "AbilitySystemLog.h"
#include "GameFramework/Pawn.h"
#include "GameplayTagAssetInterface.h"
#include "GameplayEffect.h"
#include "GameFramework/Controller.h"
#include "Misc/ConfigCacheIni.h"
#include "AbilitySystemGlobals.h"
#include "AbilitySystemComponent.h"
#include "Engine/NetConnection.h"
#include "Engine/PackageMapClient.h"
#include "GameplayAbilitiesDeveloperSettings.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/PolymorphicNetSerializerImpl.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayEffectTypes)


#define LOCTEXT_NAMESPACE "GameplayEffectTypes"


#if WITH_EDITORONLY_DATA
const FName FGameplayModEvaluationChannelSettings::ForceHideMetadataKey(TEXT("ForceHideEvaluationChannel"));
const FString FGameplayModEvaluationChannelSettings::ForceHideMetadataEnabledValue(TEXT("True"));
#endif // #if WITH_EDITORONLY_DATA

#if !UE_BUILD_SHIPPING
namespace UE::Private
{
static bool bWarnIfTryingToReplicateNotSupportedActorReference = false;
static FAutoConsoleVariableRef CVarWarnIfTryingToReplicateNotSupportedActorReference(
	TEXT("GameplayEffectContext.WarnIfTryingToReplicateNotSupportedActorReference"),
	bWarnIfTryingToReplicateNotSupportedActorReference,
	TEXT("If set to true a warning will be issued if we are trying to replicate a reference to a not supported actor as part of a GameplayEffectContext."),
	ECVF_Default);
}
#endif

TAutoConsoleVariable<bool> CVarReplicateTagCountContainerWithIris(
		TEXT("AbilitySystem.Fix.ReplicateTagCountContainerWithIris"),
		false,
		TEXT("Default: False.  When true will use Iris serialization for Gameplay Tag Count Container enabling tag rep state promotion with iris."),
		ECVF_Default);

TAutoConsoleVariable<bool> CVarGESimOnlyTags(
		TEXT("AbilitySystem.Fix.GESimOnlyTags"),
		true,
		TEXT("Default: True.  When active GE's will use the SimOnly tag replication state so that owning client lets the GEs manage their own tags"),
		ECVF_Default);

FGameplayModEvaluationChannelSettings::FGameplayModEvaluationChannelSettings()
{
	static const UEnum* EvalChannelEnum = nullptr;
	static EGameplayModEvaluationChannel DefaultChannel = EGameplayModEvaluationChannel::Channel0;

	// The default value for this struct is actually dictated by a config value, so a degree of trickery is involved.
	// The first time through, try to find the enum and the default value, if any, and then use that to set the
	// static default channel used to initialize this struct
	if (!EvalChannelEnum)
	{
		EvalChannelEnum = StaticEnum<EGameplayModEvaluationChannel>();
		if (ensure(EvalChannelEnum) && ensure(GConfig))
		{
			const FString INISection(TEXT("/Script/GameplayAbilities.AbilitySystemGlobals"));
			const FString INIKey(TEXT("DefaultGameplayModEvaluationChannel"));
			
			FString DefaultEnumString;
			if (GConfig->GetString(*INISection, *INIKey, DefaultEnumString, GGameIni))
			{
				if (!DefaultEnumString.IsEmpty())
				{
					const int32 EnumVal = EvalChannelEnum->GetValueByName(FName(*DefaultEnumString));
					if (EnumVal != INDEX_NONE)
					{
						DefaultChannel = static_cast<EGameplayModEvaluationChannel>(EnumVal);
					}
				}
			}
		}
	}

	Channel = DefaultChannel;
}

EGameplayModEvaluationChannel FGameplayModEvaluationChannelSettings::GetEvaluationChannel() const
{
	if (ensure(UAbilitySystemGlobals::Get().IsGameplayModEvaluationChannelValid(Channel)))
	{
		return Channel;
	}

	return EGameplayModEvaluationChannel::Channel0;
}

void FGameplayModEvaluationChannelSettings::SetEvaluationChannel(EGameplayModEvaluationChannel NewChannel)
{
	if (ensure(UAbilitySystemGlobals::Get().IsGameplayModEvaluationChannelValid(NewChannel)))
	{
		Channel = NewChannel;
	}
}

bool FGameplayModEvaluationChannelSettings::operator==(const FGameplayModEvaluationChannelSettings& Other) const
{
	return GetEvaluationChannel() == Other.GetEvaluationChannel();
}

bool FGameplayModEvaluationChannelSettings::operator!=(const FGameplayModEvaluationChannelSettings& Other) const
{
	return !(*this == Other);
}

float GameplayEffectUtilities::GetModifierBiasByModifierOp(EGameplayModOp::Type ModOp)
{
	static const float ModifierOpBiases[EGameplayModOp::Max] = {0.f, 1.f, 1.f, 0.f, 0.f, 0.f};
	check(ModOp >= 0 && ModOp < EGameplayModOp::Max);

	return ModifierOpBiases[ModOp];
}

float GameplayEffectUtilities::ComputeStackedModifierMagnitude(float BaseComputedMagnitude, int32 StackCount, EGameplayModOp::Type ModOp)
{
	const float OperationBias = GameplayEffectUtilities::GetModifierBiasByModifierOp(ModOp);

	StackCount = FMath::Clamp<int32>(StackCount, 0, StackCount);

	float StackMag = BaseComputedMagnitude;
	
	// Override modifiers don't care about stack count at all. All other modifier ops need to subtract out their bias value in order to handle
	// stacking correctly
	if (ModOp != EGameplayModOp::Override)
	{
		StackMag -= OperationBias;
		if (ModOp == EGameplayModOp::MultiplyCompound)
		{
			StackMag = FMath::Pow(StackMag, StackCount);
		}
		else
		{
			StackMag *= StackCount;
		}
		StackMag += OperationBias;
	}

	return StackMag;
}

bool FGameplayEffectAttributeCaptureDefinition::operator==(const FGameplayEffectAttributeCaptureDefinition& Other) const
{
	return ((AttributeToCapture == Other.AttributeToCapture) && (AttributeSource == Other.AttributeSource) && (bSnapshot == Other.bSnapshot));
}

bool FGameplayEffectAttributeCaptureDefinition::operator!=(const FGameplayEffectAttributeCaptureDefinition& Other) const
{
	return ((AttributeToCapture != Other.AttributeToCapture) || (AttributeSource != Other.AttributeSource) || (bSnapshot != Other.bSnapshot));
}

FString FGameplayEffectAttributeCaptureDefinition::ToSimpleString() const
{
	return FString::Printf(TEXT("Attribute: %s, Capture: %s, Snapshot: %d"), *AttributeToCapture.GetName(), AttributeSource == EGameplayEffectAttributeCaptureSource::Source ? TEXT("Source") : TEXT("Target"), bSnapshot);
}

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	FGameplayEffectContext
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

bool FGameplayEffectContext::CanActorReferenceBeReplicated(const AActor* Actor)
{
	// We always support replication of null references and stably named actors
	if (!Actor || Actor->IsFullNameStableForNetworking())
	{
		return true;
	}

	// If we get here this is a dynamic object and we only want to replicate the reference if the actor is set to replicate, otherwise the resolve on the client will constantly fail
	const bool bIsSupportedForNetWorking = Actor->IsSupportedForNetworking();
	const bool bCanDynamicReferenceBeReplicated = bIsSupportedForNetWorking && Actor->GetIsReplicated();

#if !UE_BUILD_SHIPPING
	// Optionally trigger warning if we are trying to replicate a reference to an object that never will be resolvable on receiving end
	if (UE::Private::bWarnIfTryingToReplicateNotSupportedActorReference && (!bCanDynamicReferenceBeReplicated && bIsSupportedForNetWorking))
	{
		ABILITY_LOG(Warning, TEXT("Attempted to replicate a reference to dynamically spawned object that is set to not replicate %s."), *(Actor->GetName()));
	}
#endif

	return bCanDynamicReferenceBeReplicated;
}

void FGameplayEffectContext::AddInstigator(class AActor *InInstigator, class AActor *InEffectCauser)
{
	Instigator = InInstigator;
	bReplicateInstigator = CanActorReferenceBeReplicated(InInstigator);

	SetEffectCauser(InEffectCauser);

	InstigatorAbilitySystemComponent = NULL;

	// Cache off the AbilitySystemComponent.
	InstigatorAbilitySystemComponent = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Instigator.Get());
}

void FGameplayEffectContext::SetAbility(const UGameplayAbility* InGameplayAbility)
{
	if (InGameplayAbility)
	{
		AbilityInstanceNotReplicated = MakeWeakObjectPtr(const_cast<UGameplayAbility*>(InGameplayAbility));
		AbilityCDO = InGameplayAbility->GetClass()->GetDefaultObject<UGameplayAbility>();
		AbilityLevel = InGameplayAbility->GetAbilityLevel();
	}
}

const UGameplayAbility* FGameplayEffectContext::GetAbility() const
{
	return AbilityCDO.Get();
}

const UGameplayAbility* FGameplayEffectContext::GetAbilityInstance_NotReplicated() const
{
	return AbilityInstanceNotReplicated.Get();
}


void FGameplayEffectContext::AddActors(const TArray<TWeakObjectPtr<AActor>>& InActors, bool bReset)
{
	if (bReset && Actors.Num())
	{
		Actors.Reset();
	}

	Actors.Append(InActors);
}

void FGameplayEffectContext::AddHitResult(const FHitResult& InHitResult, bool bReset)
{
	if (bReset && HitResult.IsValid())
	{
		HitResult.Reset();
		bHasWorldOrigin = false;
	}

	check(!HitResult.IsValid());
	HitResult = TSharedPtr<FHitResult>(new FHitResult(InHitResult));
	if (bHasWorldOrigin == false)
	{
		AddOrigin(InHitResult.TraceStart);
	}
}

bool FGameplayEffectContext::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	//!!!!!!!!!!!!!!!! Warning !!!!!!!!!!!!!!!
	// Any changes to this function also need to be done to FGameplayEffectContextNetSerializer to support Iris replication
	//!!!!!!!!!!!!!!!! Warning !!!!!!!!!!!!!!!

	uint8 RepBits = 0;
	if (Ar.IsSaving())
	{
		if (bReplicateInstigator && Instigator.IsValid())
		{
			RepBits |= 1 << 0;
		}
		if (bReplicateEffectCauser && EffectCauser.IsValid() )
		{
			RepBits |= 1 << 1;
		}
		if (AbilityCDO.IsValid())
		{
			RepBits |= 1 << 2;
		}
		if (bReplicateSourceObject && SourceObject.IsValid())
		{
			RepBits |= 1 << 3;
		}
		if (Actors.Num() > 0)
		{
			RepBits |= 1 << 4;
		}
		if (HitResult.IsValid())
		{
			RepBits |= 1 << 5;
		}
		if (bHasWorldOrigin)
		{
			RepBits |= 1 << 6;
		}
	}

	Ar.SerializeBits(&RepBits, 7);

	if (RepBits & (1 << 0))
	{
		Ar << Instigator;
	}
	if (RepBits & (1 << 1))
	{
		Ar << EffectCauser;
	}
	if (RepBits & (1 << 2))
	{
		Ar << AbilityCDO;
	}
	if (RepBits & (1 << 3))
	{
		Ar << SourceObject;
	}
	if (RepBits & (1 << 4))
	{
		SafeNetSerializeTArray_Default<31>(Ar, Actors);
	}
	if (RepBits & (1 << 5))
	{
		if (Ar.IsLoading())
		{
			if (!HitResult.IsValid())
			{
				HitResult = TSharedPtr<FHitResult>(new FHitResult());
			}
		}
		HitResult->NetSerialize(Ar, Map, bOutSuccess);
	}
	if (RepBits & (1 << 6))
	{
		Ar << WorldOrigin;
		bHasWorldOrigin = true;
	}
	else
	{
		bHasWorldOrigin = false;
	}

	if (Ar.IsLoading())
	{
		AddInstigator(Instigator.Get(), EffectCauser.Get()); // Just to initialize InstigatorAbilitySystemComponent
	}	
	
	bOutSuccess = true;
	return true;
}

FString FGameplayEffectContext::ToString() const
{
	const AActor* InstigatorPtr = Instigator.Get();
	return (InstigatorPtr ? InstigatorPtr->GetName() : FString(TEXT("NONE")));
}

bool FGameplayEffectContext::IsLocallyControlled() const
{
	APawn* Pawn = Cast<APawn>(Instigator.Get());
	if (!Pawn)
	{
		Pawn = Cast<APawn>(EffectCauser.Get());
	}
	if (Pawn)
	{
		return Pawn->IsLocallyControlled();
	}
	return false;
}

bool FGameplayEffectContext::IsLocallyControlledPlayer() const
{
	APawn* Pawn = Cast<APawn>(Instigator.Get());
	if (!Pawn)
	{
		Pawn = Cast<APawn>(EffectCauser.Get());
	}
	if (Pawn && Pawn->GetController())
	{
		return Pawn->GetController()->IsLocalPlayerController();
	}
	return false;
}

void FGameplayEffectContext::AddOrigin(FVector InOrigin)
{
	bHasWorldOrigin = true;
	WorldOrigin = InOrigin;
}

void FGameplayEffectContext::GetOwnedGameplayTags(OUT FGameplayTagContainer& ActorTagContainer, OUT FGameplayTagContainer& SpecTagContainer) const
{
	IGameplayTagAssetInterface* TagInterface = Cast<IGameplayTagAssetInterface>(Instigator.Get());
	if (TagInterface)
	{
		TagInterface->GetOwnedGameplayTags(ActorTagContainer);
	}
	else if (UAbilitySystemComponent* ASC = InstigatorAbilitySystemComponent.Get())
	{
		ASC->GetOwnedGameplayTags(ActorTagContainer);
	}
}

struct FGameplayEffectContextDeleter
{
	FORCEINLINE void operator()(FGameplayEffectContext* Object) const
	{
		check(Object);
		UScriptStruct* ScriptStruct = Object->GetScriptStruct();
		check(ScriptStruct);
		ScriptStruct->DestroyStruct(Object);
		FMemory::Free(Object);
	}
};

bool FGameplayEffectContextHandle::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	bool ValidData = Data.IsValid();
	Ar.SerializeBits(&ValidData,1);

	if (ValidData)
	{
		TCheckedObjPtr<UScriptStruct> ScriptStruct = Data.IsValid() ? Data->GetScriptStruct() : nullptr;
		
		UAbilitySystemGlobals::Get().EffectContextStructCache.NetSerialize(Ar, ScriptStruct.Get());

		if (ScriptStruct.IsValid())
		{
			if (Ar.IsLoading())
			{
				// If data is invalid, or a different type, allocate
				if (!Data.IsValid() || (Data->GetScriptStruct() != ScriptStruct.Get()))
				{
					FGameplayEffectContext* NewData = (FGameplayEffectContext*)FMemory::Malloc(ScriptStruct->GetStructureSize());
					ScriptStruct->InitializeStruct(NewData);

					Data = TSharedPtr<FGameplayEffectContext>(NewData, FGameplayEffectContextDeleter());
				}
			}

			check(Data.IsValid());
			if (ScriptStruct->StructFlags & STRUCT_NetSerializeNative)
			{
				ScriptStruct->GetCppStructOps()->NetSerialize(Ar, Map, bOutSuccess, Data.Get());
			}
			else
			{
				// This won't work since FStructProperty::NetSerializeItem is deprecrated.
				//	1) we have to manually crawl through the topmost struct's fields since we don't have a FStructProperty for it (just the UScriptProperty)
				//	2) if there are any UStructProperties in the topmost struct's fields, we will assert in FStructProperty::NetSerializeItem.

				ABILITY_LOG(Fatal, TEXT("FGameplayEffectContextHandle::NetSerialize called on data struct %s without a native NetSerialize"), *ScriptStruct->GetName());
			}
		}
		else if (ScriptStruct.IsError())
		{
			ABILITY_LOG(Error, TEXT("FGameplayEffectContextHandle::NetSerialize: Bad ScriptStruct serialized, can't recover."));
			Ar.SetError();
			Data.Reset();
			bOutSuccess = false;
			return false;
		}
	}
	else
	{
		Data.Reset();
	}

	bOutSuccess = true;
	return true;
}


// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	Misc
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

FString EGameplayModOpToString(int32 Type)
{
	static UEnum *e = StaticEnum<EGameplayModOp::Type>();
	return e->GetNameStringByValue(Type);
}

FString EGameplayModToString(int32 Type)
{
	// Enum no longer exists.
	return FString();
}

FString EGameplayModEffectToString(int32 Type)
{
	// Enum no longer exists.
	return FString();
}

FString EGameplayCueEventToString(int32 Type)
{
	static UEnum *e = StaticEnum<EGameplayCueEvent::Type>();
	return e->GetNameStringByValue(Type);
}

class FNetGameplayTagCountContainerState : public INetDeltaBaseState
{
public:
	struct FGameplayTagCountStateItem
	{
		FGameplayTag Tag;
		int32 Count = -1;
		EGameplayTagReplicationState ReplicationState = EGameplayTagReplicationState::None;

		bool operator== (const FGameplayTagCountStateItem& Other) const
		{
			return Tag == Other.Tag && Count == Other.Count && ReplicationState == Other.ReplicationState;
		}
	};

	virtual bool IsStateEqual(INetDeltaBaseState* OtherState) override
	{
		FNetGameplayTagCountContainerState* Other = static_cast<FNetGameplayTagCountContainerState*>(OtherState);
		return (TagStates == Other->TagStates);
	}

	TArray<FGameplayTagCountStateItem> TagStates;
};

bool FGameplayTagCountContainer::NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParams)
{
	bool bPackCount = false;

	// Determine if we are replicating as a NetOwner
	bool bNetOwner = false;
	if (DeltaParams.Connection)
	{
		AActor* OwnerActor = nullptr;
		if (UActorComponent* ParentComponent = Cast<UActorComponent>(DeltaParams.Object))
		{
			OwnerActor = ParentComponent->GetOwner();
		}
		else
		{
			OwnerActor = Cast<AActor>(DeltaParams.Object);
		}

		bNetOwner = OwnerActor && (OwnerActor->GetNetConnection() == DeltaParams.Connection);
	}

	if (DeltaParams.Writer)
	{		
		FBitWriter& Writer = *DeltaParams.Writer;

		FNetGameplayTagCountContainerState* OldState = static_cast<FNetGameplayTagCountContainerState*>(DeltaParams.OldState);
		FNetGameplayTagCountContainerState* NewState = new FNetGameplayTagCountContainerState();

		// Keep track of the indices for changed data in the Items list so we can iterate just those after calculating the new state.
		TArray<int32> ChangeIdx;

		check(DeltaParams.NewState);
		*DeltaParams.NewState = TSharedPtr<INetDeltaBaseState>(NewState);
		NewState->TagStates.Reserve(Items.Num());
		ChangeIdx.Reserve(Items.Num());

		// We set ChangelistHistory and LastAckedHistory to 0, as that's the default RepLayout expects.
		// Setting them to anything else may cause issues with initial sends.
		uint32 OldChangelistHistory = 0;
		uint32 OldLastAckedHistory = 0;
		
		TArray<FNetGameplayTagCountContainerState::FGameplayTagCountStateItem> Removals = OldState ? OldState->TagStates : TArray<FNetGameplayTagCountContainerState::FGameplayTagCountStateItem>{};

		// Iterate over our current items and build our new state data while removing valid entries from the Removals list so we know what's missing
		for (const FGameplayTagCountItem& CountItem : Items)
		{
			// Find this Item in the OldState
			FNetGameplayTagCountContainerState::FGameplayTagCountStateItem* OldItem = Removals.FindByPredicate(
				[&CountItem](FNetGameplayTagCountContainerState::FGameplayTagCountStateItem& Item)
				{
					return Item.Tag == CountItem.Tag;
				});

			// Warn if the ReplicationState ever changed; it's an indication that the code that uses it has conflicting tag count requirements
			UE_CLOGF(OldItem && OldItem->ReplicationState > CountItem.ReplicationState, LogAbilitySystem, Warning,
				"Existing tag: %ls with \"%ls\" replication state now serializing with state \"%ls\"", 
				*CountItem.Tag.ToString(), *UEnum::GetDisplayValueAsText(OldItem->ReplicationState).ToString(), *UEnum::GetDisplayValueAsText(CountItem.ReplicationState).ToString());

			if (CountItem.ReplicationState == EGameplayTagReplicationState::None)
			{
				// We don't replicate this tag, so don't save its state
				continue;
			}
			
			// if the state is SimulatedOnly we want to skip sending this tag to the owner
			if (CVarGESimOnlyTags.GetValueOnAnyThread() && CountItem.ReplicationState == EGameplayTagReplicationState::SimulatedTagOnly && bNetOwner)
			{
				continue;
			}

			// Record the NewState for the CountItem
			FNetGameplayTagCountContainerState::FGameplayTagCountStateItem& NewItem = NewState->TagStates.AddDefaulted_GetRef();
			NewItem.Tag = CountItem.Tag;
			NewItem.ReplicationState = CountItem.ReplicationState;

			bPackCount = CountItem.ReplicationState >= (bNetOwner ? EGameplayTagReplicationState::CountToOwner : EGameplayTagReplicationState::TagAndCountToAll);
			NewItem.Count = bPackCount ? CountItem.Count : 1;
			
			if (!OldItem)
			{
				// This was a tag added
				ChangeIdx.Add(NewState->TagStates.Num() - 1);
				continue;
			}

			if (bPackCount && OldItem->Count != NewItem.Count)
			{
				// This was a tag count change
				ChangeIdx.Add(NewState->TagStates.Num() - 1);
			}

			Removals.RemoveSingleSwap(*OldItem);
		}
		
		for (int idx = 0; idx < Removals.Num(); idx++)
		{
			// if the state is SimulatedOnly we want to skip sending this tag removal to the owner
			if (CVarGESimOnlyTags.GetValueOnAnyThread() && Removals[idx].ReplicationState == EGameplayTagReplicationState::SimulatedTagOnly && bNetOwner)
			{
				Removals.RemoveAtSwap(idx, EAllowShrinking::No);
				--idx;
			}
		}
		uint32 NumRemovals = Removals.Num();

		// If there's no changes between the states we can just leave
		if (ChangeIdx.Num() == 0 && NumRemovals == 0)
		{
			return false;
		}

		if (OldState)
		{
			OldChangelistHistory = DeltaParams.OldState->GetChangelistHistory();
			OldLastAckedHistory = DeltaParams.OldState->GetLastAckedHistory();
		}

		NewState->SetChangelistHistory(OldChangelistHistory);
		NewState->SetLastAckedHistory(OldLastAckedHistory);

		// Pack any tags left in the Removals array to be removed on the client side
		Writer.SerializeIntPacked(NumRemovals);
		bool bTagSuccess = true;
		if (NumRemovals > 0)
		{
			for (FNetGameplayTagCountContainerState::FGameplayTagCountStateItem& Item : Removals)
			{
				Item.Tag.NetSerialize(Writer, DeltaParams.Map, bTagSuccess);
			}
		}

		// Finish packing up any remaining changes to new tags or count changes.
		uint32 NumChanges = ChangeIdx.Num();
		Writer.SerializeIntPacked(NumChanges);

		uint32 TagCount = 0;
		for (int32& Idx : ChangeIdx)
		{
			FNetGameplayTagCountContainerState::FGameplayTagCountStateItem& NewItem = NewState->TagStates[Idx];
			// only pack count if we're sending to the correct client for the replication state.
			bPackCount = NewItem.ReplicationState >= (bNetOwner ? EGameplayTagReplicationState::CountToOwner : EGameplayTagReplicationState::TagAndCountToAll);
			NewItem.Tag.NetSerialize(Writer, DeltaParams.Map, bTagSuccess);
			Writer.SerializeBits(&bPackCount, 1);
			if (bPackCount)
			{
				TagCount = (uint32)NewItem.Count;
				Writer.SerializeIntPacked(TagCount);
			}
		}
	}
	
	if (DeltaParams.Reader)
	{
		FBitReader& Reader = *DeltaParams.Reader;
		bool bTagSuccess = true;

		uint32 NumRemovals = 0;
		Reader.SerializeIntPacked(NumRemovals);
		if (NumRemovals > 0)
		{
			FNetGameplayTagCountContainerState::FGameplayTagCountStateItem RemovedItem{};
			for (; NumRemovals > 0; NumRemovals--)
			{
				RemovedItem.Tag.NetSerialize(Reader, DeltaParams.Map, bTagSuccess);
				int32* Count = GameplayTagCountMap.Find(RemovedItem.Tag);
				if (Count && *Count > 0)
				{
					UpdateTagMap_Internal(RemovedItem.Tag, -(*Count), EGameplayTagReplicationState::None);
				}
			}
		}
		
		uint32 NumChanges = 0;
		Reader.SerializeIntPacked(NumChanges);
		if (NumChanges > 0)
		{
			FNetGameplayTagCountContainerState::FGameplayTagCountStateItem UpdatedItem{};
			FGameplayTag UpdatedTag;
			uint32 NewCount = 1;
			for (; NumChanges > 0; NumChanges--)
			{
				NewCount = 1;
				UpdatedItem.Tag.NetSerialize(Reader, DeltaParams.Map, bTagSuccess);

				// We need to reconstruct the ReplicationState because of the MultiServerProxy which needs to pass this
				// data along (and thus it needs to know what to Serialize). We start by assuming it's TagOnly.
				EGameplayTagReplicationState InferredReplicationState = EGameplayTagReplicationState::TagOnly;

				Reader.SerializeBits(&bPackCount, 1);
				if (bPackCount)
				{
					FGameplayTagCountItem* ExistingItem = Items.FindByPredicate(
						[&UpdatedItem](const FGameplayTagCountItem& ExistingItem)
						{
							return ExistingItem.Tag == UpdatedItem.Tag;
						});

					// If we're receiving a count, then we are at least CountToOwner, and possibly TagAndCountToAll if we ever receive this on a non-owning connection.
					InferredReplicationState = bNetOwner ? EGameplayTagReplicationState::CountToOwner : EGameplayTagReplicationState::TagAndCountToAll;
					InferredReplicationState = ExistingItem ? FMath::Max(ExistingItem->ReplicationState, InferredReplicationState) : InferredReplicationState;

					Reader.SerializeIntPacked(NewCount);
				}

				SetTagCount(UpdatedItem.Tag, NewCount, InferredReplicationState);
			}
		}
	}
	return true;
}

void FGameplayTagCountContainer::SetOwner(UAbilitySystemComponent* ASC)
{
	Owner = ASC;
}

void FGameplayTagCountContainer::Notify_StackCountChange(const FGameplayTag& Tag)
{	
	// The purpose of this function is to let anyone listening on the EGameplayTagEventType::AnyCountChange event know that the 
	// stack count of a GE that was backing this GE has changed. We do not update our internal map/count with this info, since that
	// map only counts the number of GE/sources that are giving that tag.
	FGameplayTagContainer TagAndParentsContainer = Tag.GetGameplayTagParents();
	for (auto CompleteTagIt = TagAndParentsContainer.CreateConstIterator(); CompleteTagIt; ++CompleteTagIt)
	{
		const FGameplayTag& CurTag = *CompleteTagIt;
		FDelegateInfo* DelegateInfo = GameplayTagEventMap.Find(CurTag);
		if (DelegateInfo)
		{
			int32 TagCount = GameplayTagCountMap.FindOrAdd(CurTag);
			DelegateInfo->OnAnyChange.Broadcast(CurTag, TagCount);
		}
	}
}

FOnGameplayEffectTagCountChanged& FGameplayTagCountContainer::RegisterGameplayTagEvent(const FGameplayTag& Tag, EGameplayTagEventType::Type EventType)
{
	FDelegateInfo& Info = GameplayTagEventMap.FindOrAdd(Tag);

	if (EventType == EGameplayTagEventType::NewOrRemoved)
	{
		return Info.OnNewOrRemove;
	}

	return Info.OnAnyChange;
}

void FGameplayTagCountContainer::Reset(bool bResetCallbacks)
{
	GameplayTagCountMap.Reset();

	if (bResetCallbacks)
	{
		GameplayTagEventMap.Reset();
		OnAnyTagChangeDelegate.Clear();
	}
}

bool FGameplayTagCountContainer::UpdateExplicitTags(const FGameplayTag& Tag, const int32 CountDelta, const bool bDeferParentTagsOnRemove, EGameplayTagReplicationState TagRepState)
{
	int32 TagCountIndex = Items.Find(Tag);

	if (TagCountIndex != INDEX_NONE)
	{
		FGameplayTagCountItem& TagCountItem = Items[TagCountIndex];
		const bool RepTypeMatch = TagCountItem.ReplicationState == TagRepState;
		if (!RepTypeMatch && CountDelta > 0)
		{
			// We're attempting to predict a replicated tag on a client but they already have potentially non-replicated tags
			if (Owner && Owner->GetOwnerRole() != ROLE_Authority && TagCountItem.ReplicationState == EGameplayTagReplicationState::None)
			{
				ABILITY_LOG(Warning, TEXT("Attempting to predict %s tag addition, but potentially non-replicated tag already exists"), *Tag.ToString());
			}
			// We're changing the rep state of an existing tag on the server
			else if (Owner && Owner->GetOwnerRole() == ROLE_Authority && TagCountItem.ReplicationState > EGameplayTagReplicationState::None)
			{
				ABILITY_LOG(Warning, TEXT("Trying to change existing tag \"%s\" replication state from \"%s\" to \"%s\""), *Tag.ToString(), *UEnum::GetDisplayValueAsText(TagCountItem.ReplicationState).ToString(), *UEnum::GetDisplayValueAsText(TagRepState).ToString());
			}

			// Promote our rep state to the higher level
			TagCountItem.ReplicationState = TagRepState > TagCountItem.ReplicationState ? TagRepState : TagCountItem.ReplicationState;
			TagRepState = TagCountItem.ReplicationState;
		}
		if (!UE::Net::ShouldUseIrisReplication() || (UE::Net::ShouldUseIrisReplication() && CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread()))
		{
			// make sure clients only have the 1 tag if it's tag Only.
			if (Owner && Owner->GetOwnerRole() != ROLE_Authority && CountDelta > 0  && TagCountItem.ReplicationState == EGameplayTagReplicationState::TagOnly)
			{
				TagCountItem.Count = 1;
			}
			else
			{
				TagCountItem.Count += CountDelta;
			}
		}
		else
		{
			TagCountItem.Count += CountDelta;
		}
		
		if (TagCountItem.Count <= 0)
		{
			Items.RemoveAtSwap(TagCountIndex);
			ExplicitTags.RemoveTag(Tag, bDeferParentTagsOnRemove);
		}
	}
	else if (CountDelta > 0)
	{
		Items.Add(FGameplayTagCountItem(Tag, CountDelta, TagRepState));
		ExplicitTags.AddTag(Tag);
	}
	else
	{
		ABILITY_LOG(Verbose, TEXT("Attempted to remove tag: %s from tag count container, but it is not explicitly in the container!"), *Tag.ToString());
		return false;
	}

	return true;
}

bool FGameplayTagCountContainer::GatherTagChangeDelegates(const FGameplayTag& Tag, const int32 CountDelta, TArray<FDeferredTagChangeDelegate>& TagChangeDelegates, EGameplayTagReplicationState TagRepState)
{
	// Check if change delegates are required to fire for the tag or any of its parents based on the count change
	FGameplayTagContainer TagAndParentsContainer = Tag.GetGameplayTagParents();
	bool CreatedSignificantChange = false;
	for (auto CompleteTagIt = TagAndParentsContainer.CreateConstIterator(); CompleteTagIt; ++CompleteTagIt)
	{
		const FGameplayTag& CurTag = *CompleteTagIt;

		// Get the current count of the specified tag. NOTE: Stored as a reference, so subsequent changes propagate to the map.
		int32& TagCountRef = GameplayTagCountMap.FindOrAdd(CurTag);

		const int32 OldCount = TagCountRef;
		
		const bool UsingCountContainerRep = !UE::Net::ShouldUseIrisReplication() || (UE::Net::ShouldUseIrisReplication() && CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread());
		const bool ClampTagOnlyCount = UsingCountContainerRep && Owner && Owner->GetOwnerRole() != ROLE_Authority && CountDelta > 0  && TagRepState == EGameplayTagReplicationState::TagOnly;
		
		// Apply the delta to the count in the map
		int32 NewTagCount = FMath::Max(OldCount + CountDelta, 0);
		
		TagCountRef = ClampTagOnlyCount ? FMath::Min(NewTagCount, 1) : NewTagCount;

		// If a significant change (new addition or total removal) occurred, trigger related delegates
		const bool SignificantChange = (OldCount == 0 || NewTagCount == 0);
		CreatedSignificantChange |= SignificantChange;
		if (SignificantChange)
		{
			TagChangeDelegates.AddDefaulted();
			TagChangeDelegates.Last().BindLambda([Delegate = OnAnyTagChangeDelegate, CurTag, TagCountRef]()
			{
				Delegate.Broadcast(CurTag, TagCountRef);
			});
		}

		FDelegateInfo* DelegateInfo = GameplayTagEventMap.Find(CurTag);
		if (DelegateInfo)
		{
			TagChangeDelegates.AddDefaulted();
			TagChangeDelegates.Last().BindLambda([Delegate = DelegateInfo->OnAnyChange, CurTag, TagCountRef]()
			{
				Delegate.Broadcast(CurTag, TagCountRef);
			});

			if (SignificantChange)
			{
				TagChangeDelegates.AddDefaulted();
				TagChangeDelegates.Last().BindLambda([Delegate = DelegateInfo->OnNewOrRemove, CurTag, TagCountRef]()
				{
					Delegate.Broadcast(CurTag, TagCountRef);
				});
			}
		}
	}

	return CreatedSignificantChange;
}

bool FGameplayTagCountContainer::UpdateTagMap_Internal(const FGameplayTag& Tag, int32 CountDelta, EGameplayTagReplicationState TagRepState)
{
	if (!ensureMsgf(Tag.IsValid(), TEXT("FGameplayTagCountContainer attempted to update tag map with an invalid Tag (None)!")))
	{
		return false;
	}
	
	if (!ensureMsgf(CountDelta != 0, TEXT("FGameplayTagCountContainer Modifying tag count by 0 for tag %s"), *Tag.ToString()))
	{
		return false;
	}

	if (!UpdateExplicitTags(Tag, CountDelta, false, TagRepState))
	{
		return false;
	}

	TArray<FDeferredTagChangeDelegate> DeferredTagChangeDelegates;
	bool bSignificantChange = GatherTagChangeDelegates(Tag, CountDelta, DeferredTagChangeDelegates, TagRepState);
	for (FDeferredTagChangeDelegate& Delegate : DeferredTagChangeDelegates)
	{
		Delegate.Execute();
	}

	return bSignificantChange;
}

bool FGameplayTagCountContainer::UpdateTagMapDeferredParentRemoval_Internal(const FGameplayTag& Tag, int32 CountDelta, TArray<FDeferredTagChangeDelegate>& DeferredTagChangeDelegates, EGameplayTagReplicationState TagRepState)
{
	if (!UpdateExplicitTags(Tag, CountDelta, true, TagRepState))
	{
		return false;
	}

	return GatherTagChangeDelegates(Tag, CountDelta, DeferredTagChangeDelegates, TagRepState);
}

FGameplayTagBlueprintPropertyMap::FGameplayTagBlueprintPropertyMap()
{
}

FGameplayTagBlueprintPropertyMap::FGameplayTagBlueprintPropertyMap(const FGameplayTagBlueprintPropertyMap& Other)
{
	ensureMsgf(Other.CachedOwner.IsExplicitlyNull(), TEXT("FGameplayTagBlueprintPropertyMap cannot be used inside an array or other container that is copied after register!"));
	PropertyMappings = Other.PropertyMappings;
}

FGameplayTagBlueprintPropertyMap::~FGameplayTagBlueprintPropertyMap()
{
	Unregister();
}

#if WITH_EDITOR
EDataValidationResult FGameplayTagBlueprintPropertyMap::IsDataValid(const UObject* Owner, FDataValidationContext& Context) const
{
	UClass* OwnerClass = ((Owner != nullptr) ? Owner->GetClass() : nullptr);
	if (!OwnerClass)
	{
		ABILITY_LOG(Error, TEXT("FGameplayTagBlueprintPropertyMap: IsDataValid() called with an invalid Owner."));
		return EDataValidationResult::Invalid;
	}

	for (const FGameplayTagBlueprintPropertyMapping& Mapping : PropertyMappings)
	{
		if (!Mapping.TagToMap.IsValid())
		{
			Context.AddError(FText::Format(LOCTEXT("GameplayTagBlueprintPropertyMap_BadTag", "The gameplay tag [{0}] for property [{1}] is empty or invalid."),
				FText::AsCultureInvariant(Mapping.TagToMap.ToString()),
				FText::FromName(Mapping.PropertyName)));
		}

		if (FProperty* Property = OwnerClass->FindPropertyByName(Mapping.PropertyName))
		{
			if (!IsPropertyTypeValid(Property))
			{
				Context.AddError(FText::Format(LOCTEXT("GameplayTagBlueprintPropertyMap_BadType", "The property [{0}] for gameplay tag [{1}] is not a supported type.  Supported types are: integer, float, and boolean."),
					FText::FromName(Mapping.PropertyName),
					FText::AsCultureInvariant(Mapping.TagToMap.ToString())));
			}
		}
		else
		{
			Context.AddError(FText::Format(LOCTEXT("GameplayTagBlueprintPropertyMap_MissingProperty", "The property [{0}] for gameplay tag [{1}] could not be found."),
				FText::FromName(Mapping.PropertyName),
				FText::AsCultureInvariant(Mapping.TagToMap.ToString())));
		}
	}

	return ((Context.GetNumErrors() > 0) ? EDataValidationResult::Invalid : EDataValidationResult::Valid);
}
#endif // #if WITH_EDITOR

void FGameplayTagBlueprintPropertyMap::Initialize(UObject* Owner, UAbilitySystemComponent* ASC)
{
	UClass* OwnerClass = (Owner ? Owner->GetClass() : nullptr);
	if (!OwnerClass)
	{
		ABILITY_LOG(Error, TEXT("FGameplayTagBlueprintPropertyMap: Initialize() called with an invalid Owner."));
		return;
	}

	if (!ASC)
	{
		ABILITY_LOG(Error, TEXT("FGameplayTagBlueprintPropertyMap: Initialize() called with an invalid AbilitySystemComponent."));
		return;
	}

	if ((CachedOwner == Owner) && (CachedASC == ASC))
	{
		// Already initialized.
		return;
	}

	if (CachedOwner.IsValid())
	{
		Unregister();
	}

	CachedOwner = Owner;
	CachedASC = ASC;

	FOnGameplayEffectTagCountChanged::FDelegate Delegate = FOnGameplayEffectTagCountChanged::FDelegate::CreateRaw(this, &FGameplayTagBlueprintPropertyMap::GameplayTagEventCallback, CachedOwner);

	// Process array starting at the end so we can remove invalid entries.
	for (int32 MappingIndex = (PropertyMappings.Num() - 1); MappingIndex >= 0; --MappingIndex)
	{
		FGameplayTagBlueprintPropertyMapping& Mapping = PropertyMappings[MappingIndex];

		if (Mapping.TagToMap.IsValid())
		{
			FProperty* Property = OwnerClass->FindPropertyByName(Mapping.PropertyName);
			if (Property && IsPropertyTypeValid(Property))
			{
				Mapping.PropertyToEdit = Property;
				Mapping.DelegateHandle = ASC->RegisterAndCallGameplayTagEvent(Mapping.TagToMap, Delegate, GetGameplayTagEventType(Property));
				continue;
			}
		}

		// Entry was invalid.  Remove it from the array.
		ABILITY_LOG(Error, TEXT("FGameplayTagBlueprintPropertyMap: Removing invalid GameplayTagBlueprintPropertyMapping [Index: %d, Tag:%s, Property:%s] for [%s]."),
			MappingIndex, *Mapping.TagToMap.ToString(), *Mapping.PropertyName.ToString(), *GetNameSafe(Owner));

		PropertyMappings.RemoveAtSwap(MappingIndex, EAllowShrinking::No);
	}
}

void FGameplayTagBlueprintPropertyMap::Unregister()
{
	if (UAbilitySystemComponent* ASC = CachedASC.Get())
	{
		for (FGameplayTagBlueprintPropertyMapping& Mapping : PropertyMappings)
		{
			if (Mapping.PropertyToEdit.Get() && Mapping.TagToMap.IsValid())
			{
				ASC->UnregisterGameplayTagEvent(Mapping.DelegateHandle, Mapping.TagToMap, GetGameplayTagEventType(Mapping.PropertyToEdit.Get()));
			}

			Mapping.PropertyToEdit = nullptr;
			Mapping.DelegateHandle.Reset();
		}
	}

	CachedOwner = nullptr;
	CachedASC = nullptr;
}

void FGameplayTagBlueprintPropertyMap::GameplayTagEventCallback(const FGameplayTag Tag, int32 NewCount, TWeakObjectPtr<UObject> RegisteredOwner)
{
	if (!RegisteredOwner.IsValid() || !CachedOwner.IsValid())
	{
		ABILITY_LOG(Warning, TEXT("FGameplayTagBlueprintPropertyMap::GameplayTagEventCallback called with corrupted RegisteredOwner or corrupted CachedOwner!"));
		return;
	}
	
	// If the index and serial don't match with registered owner, the memory might be trashed so abort
	if (!ensure(RegisteredOwner.HasSameIndexAndSerialNumber(CachedOwner)))
	{
		ABILITY_LOG(Error, TEXT("FGameplayTagBlueprintPropertyMap::GameplayTagEventCallback called with corrupted Owner!"));
		return;
	}

	UObject* Owner = CachedOwner.Get();
	if (!Owner)
	{
		ABILITY_LOG(Warning, TEXT("FGameplayTagBlueprintPropertyMap::GameplayTagEventCallback has an invalid Owner."));
		return;
	}

	FGameplayTagBlueprintPropertyMapping* Mapping = PropertyMappings.FindByPredicate([Tag](const FGameplayTagBlueprintPropertyMapping& Test)
	{
		return (Tag == Test.TagToMap);
	});

	if (Mapping && Mapping->PropertyToEdit.Get())
	{
		if (const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(Mapping->PropertyToEdit.Get()))
		{
			BoolProperty->SetPropertyValue_InContainer(Owner, NewCount > 0);
		}
		else if (const FIntProperty* IntProperty = CastField<const FIntProperty>(Mapping->PropertyToEdit.Get()))
		{
			IntProperty->SetPropertyValue_InContainer(Owner, NewCount);
		}
		else if (const FFloatProperty* FloatProperty = CastField<const FFloatProperty>(Mapping->PropertyToEdit.Get()))
		{
			FloatProperty->SetPropertyValue_InContainer(Owner, (float)NewCount);
		}
	}
}

void FGameplayTagBlueprintPropertyMap::ApplyCurrentTags()
{
	UObject* Owner = CachedOwner.Get();
	if (!Owner)
	{
		ABILITY_LOG(Warning, TEXT("FGameplayTagBlueprintPropertyMap::ApplyCurrentTags called with an invalid Owner."));
		return;
	}

	UAbilitySystemComponent* ASC = CachedASC.Get();
	if (!ASC)
	{
		ABILITY_LOG(Warning, TEXT("FGameplayTagBlueprintPropertyMap::ApplyCurrentTags called with an invalid AbilitySystemComponent."));
		return;
	}

	for (FGameplayTagBlueprintPropertyMapping& Mapping : PropertyMappings)
	{
		if (Mapping.PropertyToEdit.Get() && Mapping.TagToMap.IsValid())
		{
			int32 NewCount = ASC->GetTagCount(Mapping.TagToMap);
			
			if (const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(Mapping.PropertyToEdit.Get()))
			{
				BoolProperty->SetPropertyValue_InContainer(Owner, NewCount > 0);
			}
			else if (const FIntProperty* IntProperty = CastField<const FIntProperty>(Mapping.PropertyToEdit.Get()))
			{
				IntProperty->SetPropertyValue_InContainer(Owner, NewCount);
			}
			else if (const FFloatProperty* FloatProperty = CastField<const FFloatProperty>(Mapping.PropertyToEdit.Get()))
			{
				FloatProperty->SetPropertyValue_InContainer(Owner, (float)NewCount);
			}
		}
	}
}

bool FGameplayTagBlueprintPropertyMap::IsPropertyTypeValid(const FProperty* Property) const
{
	check(Property);
	return (Property->IsA<FBoolProperty>() || Property->IsA<FIntProperty>() || Property->IsA<FFloatProperty>());
}

EGameplayTagEventType::Type FGameplayTagBlueprintPropertyMap::GetGameplayTagEventType(const FProperty* Property) const
{
	check(Property);
	return (Property->IsA(FBoolProperty::StaticClass()) ? EGameplayTagEventType::NewOrRemoved : EGameplayTagEventType::AnyCountChange);
}

bool FGameplayTagRequirements::RequirementsMet(const FGameplayTagContainer& Container) const
{
	const bool bHasRequired = Container.HasAll(RequireTags);
	const bool bHasIgnored = Container.HasAny(IgnoreTags);
	const bool bMatchQuery = TagQuery.IsEmpty() || TagQuery.Matches(Container);

	return bHasRequired && !bHasIgnored && bMatchQuery;
}

bool FGameplayTagRequirements::IsEmpty() const
{
	return (RequireTags.Num() == 0 && IgnoreTags.Num() == 0 && TagQuery.IsEmpty());
}

FString FGameplayTagRequirements::ToString() const
{
	FString Str;

	if (RequireTags.Num() > 0)
	{
		Str += FString::Printf(TEXT("require: %s "), *RequireTags.ToStringSimple());
	}
	if (IgnoreTags.Num() >0)
	{
		Str += FString::Printf(TEXT("ignore: %s "), *IgnoreTags.ToStringSimple());
	}
	if (!TagQuery.IsEmpty())
	{
		Str += TagQuery.GetDescription();
	}

	return Str;
}

bool FGameplayTagRequirements::operator==(const FGameplayTagRequirements& Other) const
{
	return RequireTags == Other.RequireTags && IgnoreTags == Other.IgnoreTags && TagQuery == Other.TagQuery;
}

bool FGameplayTagRequirements::operator!=(const FGameplayTagRequirements& Other) const
{
	return !(*this == Other);
}

FGameplayTagQuery FGameplayTagRequirements::ConvertTagFieldsToTagQuery() const
{
	const bool bHasRequireTags = !RequireTags.IsEmpty();
	const bool bHasIgnoreTags = !IgnoreTags.IsEmpty();

	if (!bHasIgnoreTags && !bHasRequireTags)
	{
		return FGameplayTagQuery{};
	}

	// FGameplayTagContainer::RequirementsMet is HasAll(RequireTags) && !HasAny(IgnoreTags);
	FGameplayTagQueryExpression RequiredTagsQueryExpression = FGameplayTagQueryExpression().AllTagsMatch().AddTags(RequireTags);
	FGameplayTagQueryExpression IgnoreTagsQueryExpression = FGameplayTagQueryExpression().NoTagsMatch().AddTags(IgnoreTags);

	FGameplayTagQueryExpression RootQueryExpression;
	if (bHasRequireTags && bHasIgnoreTags)
	{
		RootQueryExpression = FGameplayTagQueryExpression().AllExprMatch().AddExpr(RequiredTagsQueryExpression).AddExpr(IgnoreTagsQueryExpression);
	}
	else if (bHasRequireTags)
	{
		RootQueryExpression = RequiredTagsQueryExpression;
	}
	else // bHasIgnoreTags
	{
		RootQueryExpression = IgnoreTagsQueryExpression;
	}

	// Build the expression
	return FGameplayTagQuery::BuildQuery(RootQueryExpression);
}

void FActiveGameplayEffectsContainer::PrintAllGameplayEffects() const
{
	for (const FActiveGameplayEffect& Effect : this)
	{
		Effect.PrintAll();
	}
}

void FActiveGameplayEffect::PrintAll() const
{
	ABILITY_LOG(Log, TEXT("Handle: %s"), *Handle.ToString());
	ABILITY_LOG(Log, TEXT("StartWorldTime: %.2f"), StartWorldTime);
	Spec.PrintAll();
}

void FGameplayEffectSpec::PrintAll() const
{
	ABILITY_LOG(Log, TEXT("Def: %s"), *Def->GetName());
	ABILITY_LOG(Log, TEXT("Duration: %.2f"), GetDuration());
	ABILITY_LOG(Log, TEXT("Period: %.2f"), GetPeriod());
	ABILITY_LOG(Log, TEXT("Modifiers:"));
}

FString FGameplayEffectSpec::ToSimpleString() const
{
	return GetNameSafe(Def);
}

const FGameplayTagContainer* FTagContainerAggregator::GetAggregatedTags() const
{
	if (CacheIsValid == false)
	{
		CacheIsValid = true;
		CachedAggregator.Reset(CapturedActorTags.Num() + CapturedSpecTags.Num());
		CachedAggregator.AppendTags(CapturedActorTags);
		CachedAggregator.AppendTags(CapturedSpecTags);
	}

	return &CachedAggregator;
}

FGameplayTagContainer& FTagContainerAggregator::GetActorTags()
{
	CacheIsValid = false;
	return CapturedActorTags;
}

const FGameplayTagContainer& FTagContainerAggregator::GetActorTags() const
{
	return CapturedActorTags;
}

FGameplayTagContainer& FTagContainerAggregator::GetSpecTags()
{
	CacheIsValid = false;
	return CapturedSpecTags;
}

const FGameplayTagContainer& FTagContainerAggregator::GetSpecTags() const
{
	CacheIsValid = false;
	return CapturedSpecTags;
}


FGameplayEffectSpecHandle::FGameplayEffectSpecHandle()
{

}

FGameplayEffectSpecHandle::FGameplayEffectSpecHandle(FGameplayEffectSpec* DataPtr)
	: Data(DataPtr)
{

}

bool FGameplayEffectSpecHandle::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	ABILITY_LOG(Fatal, TEXT("FGameplayEffectSpecHandle should not be NetSerialized"));
	return false;
}

FGameplayCueParameters::FGameplayCueParameters(const FGameplayEffectSpecForRPC& Spec)
: NormalizedMagnitude(0.0f)
, RawMagnitude(0.0f)
, Location(ForceInitToZero)
, Normal(ForceInitToZero)
, GameplayEffectLevel(1)
, AbilityLevel(1)
{
	UAbilitySystemGlobals::Get().InitGameplayCueParameters(*this, Spec);
}

FGameplayCueParameters::FGameplayCueParameters(const struct FGameplayEffectContextHandle& InEffectContext)
: NormalizedMagnitude(0.0f)
, RawMagnitude(0.0f)
, Location(ForceInitToZero)
, Normal(ForceInitToZero)
, GameplayEffectLevel(1)
, AbilityLevel(1)
{
	UAbilitySystemGlobals::Get().InitGameplayCueParameters(*this, InEffectContext);
}

bool FGameplayCueParameters::operator==(const FGameplayCueParameters& Other) const
{
	return ((NormalizedMagnitude == Other.NormalizedMagnitude) &&
		    (RawMagnitude == Other.RawMagnitude) &&
		    (Location == Other.Location) &&
		    (Normal == Other.Normal) &&
		    (GameplayEffectLevel == Other.GameplayEffectLevel) &&
		    (AbilityLevel == Other.AbilityLevel) &&
		    (EffectContext == Other.EffectContext) &&
			(MatchedTagName == Other.MatchedTagName) &&
			(OriginalTag == Other.OriginalTag) &&
			(AggregatedSourceTags == Other.AggregatedSourceTags) &&
			(AggregatedTargetTags == Other.AggregatedTargetTags) &&
			(Instigator == Other.Instigator) &&
			(EffectCauser == Other.EffectCauser) &&
			(SourceObject == Other.SourceObject) &&
			(PhysicalMaterial == Other.PhysicalMaterial) &&
			(TargetAttachComponent == Other.TargetAttachComponent) &&
			(bReplicateLocationWhenUsingMinimalRepProxy == Other.bReplicateLocationWhenUsingMinimalRepProxy));
}

bool FGameplayCueParameters::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	static const uint8 NUM_LEVEL_BITS = 5; // need to bump this up to support 20 levels for AbilityLevel
	static const uint8 MAX_LEVEL = (1 << NUM_LEVEL_BITS) - 1;

	enum RepFlag
	{
		REP_NormalizedMagnitude = 0,
		REP_RawMagnitude,
		REP_EffectContext,
		REP_Location,
		REP_Normal,
		REP_Instigator,
		REP_EffectCauser,
		REP_SourceObject,
		REP_TargetAttachComponent,
		REP_PhysMaterial,
		REP_GELevel,
		REP_AbilityLevel,

		REP_MAX
	};

	uint16 RepBits = 0;
	if (Ar.IsSaving())
	{
		if (NormalizedMagnitude != 0.f)
		{
			RepBits |= (1 << REP_NormalizedMagnitude);
		}
		if (RawMagnitude != 0.f)
		{
			RepBits |= (1 << REP_RawMagnitude);
		}
		if (EffectContext.IsValid())
		{
			RepBits |= (1 << REP_EffectContext);
		}
		if (Location.IsNearlyZero() == false)
		{
			RepBits |= (1 << REP_Location);
		}
		if (Normal.IsNearlyZero() == false)
		{
			RepBits |= (1 << REP_Normal);
		}
		if (Instigator.IsValid())
		{
			RepBits |= (1 << REP_Instigator);
		}
		if (EffectCauser.IsValid())
		{
			RepBits |= (1 << REP_EffectCauser);
		}
		if (SourceObject.IsValid())
		{
			RepBits |= (1 << REP_SourceObject);
		}
		if (TargetAttachComponent.IsValid())
		{
			RepBits |= (1 << REP_TargetAttachComponent);
		}
		if (PhysicalMaterial.IsValid())
		{
			RepBits |= (1 << REP_PhysMaterial);
		}
		if (GameplayEffectLevel != 1)
		{
			RepBits |= (1 << REP_GELevel);
		}
		if (AbilityLevel != 1)
		{
			RepBits |= (1 << REP_AbilityLevel);
		}
	}

	Ar.SerializeBits(&RepBits, REP_MAX);

	// Tag containers serialize empty containers with 1 bit, so no need to serialize this in the RepBits field.
	AggregatedSourceTags.NetSerialize(Ar, Map, bOutSuccess);
	AggregatedTargetTags.NetSerialize(Ar, Map, bOutSuccess);

	if (RepBits & (1 << REP_NormalizedMagnitude))
	{
		Ar << NormalizedMagnitude;
	}
	if (RepBits & (1 << REP_RawMagnitude))
	{
		Ar << RawMagnitude;
	}
	if (RepBits & (1 << REP_EffectContext))
	{
		EffectContext.NetSerialize(Ar, Map, bOutSuccess);
	}
	if (RepBits & (1 << REP_Location))
	{
		Location.NetSerialize(Ar, Map, bOutSuccess);
	}
	if (RepBits & (1 << REP_Normal))
	{
		Normal.NetSerialize(Ar, Map, bOutSuccess);
	}
	if (RepBits & (1 << REP_Instigator))
	{
		Ar << Instigator;
	}
	if (RepBits & (1 << REP_EffectCauser))
	{
		Ar << EffectCauser;
	}
	if (RepBits & (1 << REP_SourceObject))
	{
		Ar << SourceObject;
	}
	if (RepBits & (1 << REP_TargetAttachComponent))
	{
		Ar << TargetAttachComponent;
	}
	if (RepBits & (1 << REP_PhysMaterial))
	{
		Ar << PhysicalMaterial;
	}
	if (RepBits & (1 << REP_GELevel))
	{
		ensureMsgf(GameplayEffectLevel <= MAX_LEVEL, TEXT("FGameplayCueParameters::NetSerialize trying to serialize GC parameters with a GameplayEffectLevel of %d"), GameplayEffectLevel);
		if (Ar.IsLoading())
		{
			GameplayEffectLevel = 0;
		}

		Ar.SerializeBits(&GameplayEffectLevel, NUM_LEVEL_BITS);
	}
	if (RepBits & (1 << REP_AbilityLevel))
	{
		ensureMsgf(AbilityLevel <= MAX_LEVEL, TEXT("FGameplayCueParameters::NetSerialize trying to serialize GC parameters with an AbilityLevel of %d"), AbilityLevel);
		if (Ar.IsLoading())
		{
			AbilityLevel = 0;
		}

		Ar.SerializeBits(&AbilityLevel, NUM_LEVEL_BITS);
	}

	bOutSuccess = true;
	return true;
}

bool FGameplayCueParameters::IsInstigatorLocallyControlled(AActor* FallbackActor) const
{
	if (EffectContext.IsValid())
	{
		return EffectContext.IsLocallyControlled();
	}

	APawn* Pawn = Cast<APawn>(Instigator.Get());
	if (!Pawn)
	{
		Pawn = Cast<APawn>(EffectCauser.Get());
		if (!Pawn && FallbackActor != nullptr)
		{
			// Fallback to passed in actor
			Pawn = Cast<APawn>(FallbackActor);
			if (!Pawn)
			{
				Pawn = FallbackActor->GetInstigator<APawn>();
			}
		}
	}
	if (Pawn)
	{
		return Pawn->IsLocallyControlled();
	}
	return false;
}

bool FGameplayCueParameters::IsInstigatorLocallyControlledPlayer(AActor* FallbackActor) const
{
	// If there is an effect context, just ask it
	if (EffectContext.IsValid())
	{
		return EffectContext.IsLocallyControlledPlayer();
	}
	
	// Look for a pawn and use its controller
	{
		APawn* Pawn = Cast<APawn>(Instigator.Get());
		if (!Pawn)
		{
			// If no instigator, look at effect causer
			Pawn = Cast<APawn>(EffectCauser.Get());
			if (!Pawn && FallbackActor != nullptr)
			{
				// Fallback to passed in actor
				Pawn = Cast<APawn>(FallbackActor);
				if (!Pawn)
				{
					Pawn = FallbackActor->GetInstigator<APawn>();
				}
			}
		}

		if (Pawn && Pawn->GetController())
		{
			return Pawn->GetController()->IsLocalPlayerController();
		}
	}

	return false;
}

AActor* FGameplayCueParameters::GetInstigator() const
{
	if (Instigator.IsValid())
	{
		return Instigator.Get();
	}

	// Fallback to effect context if the explicit data on gameplaycue parameters is not there.
	return EffectContext.GetInstigator();
}

AActor* FGameplayCueParameters::GetEffectCauser() const
{
	if (EffectCauser.IsValid())
	{
		return EffectCauser.Get();
	}

	// Fallback to effect context if the explicit data on gameplaycue parameters is not there.
	return EffectContext.GetEffectCauser();
}

const UObject* FGameplayCueParameters::GetSourceObject() const
{
	if (SourceObject.IsValid())
	{
		return SourceObject.Get();
	}

	// Fallback to effect context if the explicit data on gameplaycue parameters is not there.
	return EffectContext.GetSourceObject();
}

void FMinimalReplicationTagCountMap::RemoveTag(const FGameplayTag& Tag)
{
	MapID++;
	if (int32* CountPtr = TagMap.Find(Tag))
	{
		int32& Count = *CountPtr;
		Count--;
		if (Count <= 0)
		{
			// Remove from map so that we do not replicate
			TagMap.Remove(Tag);
		}
	}
	else
	{
		ABILITY_LOG(Error, TEXT("FMinimalReplicationTagCountMap::RemoveTag called on Tag %s that wasn't in the tag map."), *Tag.ToString());
	}
}

// WARNING: Changes to this implementation REQUIRES making sure FMinimalReplicationTagCountMapNetSerializer and FMinimalReplicationTagCountMapReplicationFragment remains compatible.
bool FMinimalReplicationTagCountMap::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	const UGameplayAbilitiesDeveloperSettings* DeveloperSettings = GetDefault<UGameplayAbilitiesDeveloperSettings>();
	const int32 CountBits = DeveloperSettings->MinimalReplicationTagCountBits;
	const int32 MaxCount = ((1 << CountBits)-1);

	if (Ar.IsSaving())
	{
		// if Count is too high then print a warning and clamp Count
		int32 Count = TagMap.Num();
		if (Count > MaxCount)
		{
#if UE_BUILD_SHIPPING
			ABILITY_LOG(Error, TEXT("FMinimalReplicationTagCountMap has too many tags (%d) when the limit is %d. This will cause tags to not replicate. See FMinimalReplicationTagCountMap::NetSerialize"), TagMap.Num(), MaxCount);
#else
			TArray<FGameplayTag> TagKeys;
			TagMap.GetKeys(TagKeys);

			const int32 SpaceToReservePerEntry = 40;
			FString TagsString;
			// reserve lots of space for our tags string
			TagsString.Reserve(Count * SpaceToReservePerEntry);

			TagsString = TEXT("");
			TagKeys[0].GetTagName().AppendString(TagsString);
			for (int32 TagIndex = 1; TagIndex < Count; ++TagIndex)
			{
				// appends ", %tag_name" to the string
				TagsString.Append(TEXT(", "));
				TagKeys[TagIndex].GetTagName().AppendString(TagsString);
			}

			ABILITY_LOG(Error, TEXT("FMinimalReplicationTagCountMap has too many tags (%d) when the limit is %d. This will cause tags to not replicate. See FMinimalReplicationTagCountMap::NetSerialize\nTagMap tags: %s"), TagMap.Num(), MaxCount, *TagsString);
#endif	

			//clamp the count
			Count = MaxCount;
		}

		Ar.SerializeBits(&Count, CountBits);
		for(auto& It : TagMap)
		{
			FGameplayTag& Tag = It.Key;
			Tag.NetSerialize(Ar, Map, bOutSuccess);
			if (--Count <= 0)
			{
				break;
			}
		}
	}
	else
	{
		// Update MapID even when loading so that when the property is compared for replication,
		// it will be different, ensuring the data will be recorded in client replays.
		MapID++;

		int32 Count = TagMap.Num();
		Ar.SerializeBits(&Count, CountBits);

		// Reset our local map
		for(auto& It : TagMap)
		{
			It.Value = 0;
		}

		// See what we have
		while(Count-- > 0)
		{
			FGameplayTag Tag;
			Tag.NetSerialize(Ar, Map, bOutSuccess);
			TagMap.FindOrAdd(Tag) = 1;
		}

		UPackageMapClient* PackageMap = CastChecked<UPackageMapClient>(Map);
		LastConnection = PackageMap ? PackageMap->GetConnection() : nullptr;

		if (Owner)
		{
			UpdateOwnerTagMap();
		}
	}


	bOutSuccess = true;
	return true;
}

void FMinimalReplicationTagCountMap::SetOwner(UAbilitySystemComponent* ASC)
{
	Owner = ASC;
	if (Owner && TagMap.Num() > 0)
	{
		// Invoke events in case we skipped them during ::NetSerialize
		UpdateOwnerTagMap();
	}
}

void FMinimalReplicationTagCountMap::RemoveAllTags()
{
	if (Owner)
	{
		for(auto It = TagMap.CreateIterator(); It; ++It)
		{
			Owner->SetTagMapCount(It->Key, 0);
		}

		TagMap.Reset();
	}
}

void FMinimalReplicationTagCountMap::UpdateOwnerTagMap()
{
	bool bUpdateOwnerTagMap = true;
	if (bRequireNonOwningNetConnection)
	{
		if (Owner)
		{
			if (AActor* OwningActor = Owner->GetOwner())
			{
				// Note we deliberately only want to do this if the NetConnection is not null
				if (UNetConnection* OwnerNetConnection = OwningActor->GetNetConnection())
				{
					if (OwnerNetConnection == LastConnection.Get())
					{
						bUpdateOwnerTagMap = false;
					}
				}
			}
		}
	}

	if (!bUpdateOwnerTagMap)
	{
		return;
	}

	if (Owner)
	{
		// We need to copy the TagMap since changing a tag count can cause side effects such as adding or removing other tags
		TMap<FGameplayTag,int32> TagMapCopy = TagMap;
		for (auto It = TagMapCopy.CreateConstIterator(); It; ++It)
		{
			const FGameplayTag Tag = It->Key;
			const int32 NewValue = It->Value;

			Owner->SetTagMapCount(Tag, NewValue);
			UE_CLOGF((NewValue != It->Value), LogAbilitySystem, Warning, "Callback for SetTagMapCount(%ls, %d) caused its own TagCount to change to %d (likely an unwanted recursion in your registered TagChanged Listeners)", *Tag.ToString(), NewValue, It->Value);
		}

		for (auto It = TagMap.CreateIterator(); It; ++It)
		{
			// Remove tags with a count of zero from the map. This prevents them
			// from being replicated incorrectly when recording client replays.
			if (It->Value == 0)
			{
				It.RemoveCurrent();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
