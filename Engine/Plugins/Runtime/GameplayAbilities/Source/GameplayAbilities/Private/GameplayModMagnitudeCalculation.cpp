// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayModMagnitudeCalculation.h"
#include "AbilitySystemLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayModMagnitudeCalculation)

UGameplayModMagnitudeCalculation::UGameplayModMagnitudeCalculation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAllowNonNetAuthorityDependencyRegistration(false)
{
}

float UGameplayModMagnitudeCalculation::CalculateBaseMagnitude_Implementation(const FGameplayEffectSpec& Spec) const
{
	return 0.f;
}

FOnExternalGameplayModifierDependencyChange* UGameplayModMagnitudeCalculation::GetExternalModifierDependencyMulticast(const FGameplayEffectSpec& Spec, UWorld* World) const
{
	return nullptr;
}

bool UGameplayModMagnitudeCalculation::ShouldAllowNonNetAuthorityDependencyRegistration() const
{
	ensureMsgf(!bAllowNonNetAuthorityDependencyRegistration || RelevantAttributesToCapture.Num() == 0, TEXT("Cannot have a client-side based custom mod calculation that relies on attribute capture!"));
	return bAllowNonNetAuthorityDependencyRegistration;
}

bool UGameplayModMagnitudeCalculation::GetCapturedAttributeMagnitude(const FGameplayEffectAttributeCaptureDefinition& Def, const FGameplayEffectSpec& Spec, const FAggregatorEvaluateParameters& EvaluationParameters, OUT float& Magnitude) const
{
	const FGameplayEffectAttributeCaptureSpec* CaptureSpec = Spec.CapturedRelevantAttributes.FindCaptureSpecByDefinition(Def, true);
	if (CaptureSpec == nullptr)
	{
		ABILITY_LOG(Error, TEXT("GetCapturedAttributeMagnitude unable to get capture spec."));
		return false;
	}
	if (CaptureSpec->AttemptCalculateAttributeMagnitude(EvaluationParameters, Magnitude) == false)
	{
		ABILITY_LOG(Error, TEXT("GetCapturedAttributeMagnitude unable to calculate attribute magnitude."));
		return false;
	}

	return true;
}

float UGameplayModMagnitudeCalculation::K2_GetCapturedAttributeMagnitude(const FGameplayEffectSpec& EffectSpec, FGameplayAttribute Attribute, const FGameplayTagContainer& SourceTags, const FGameplayTagContainer& TargetTags) const
{
	float Magnitude = 0.0f;

	// look for the attribute in the capture list
	for (const FGameplayEffectAttributeCaptureDefinition& CurrentCapture : RelevantAttributesToCapture)
	{
		if (CurrentCapture.AttributeToCapture == Attribute)
		{
			// configure the aggregator evaluation parameters
			// TODO: apply filters?
			FAggregatorEvaluateParameters EvaluationParameters;

			EvaluationParameters.SourceTags = &SourceTags;
			EvaluationParameters.TargetTags = &TargetTags;

			// get the attribute magnitude
			GetCapturedAttributeMagnitude(CurrentCapture, EffectSpec, EvaluationParameters, Magnitude);

			// capture found. Stop the search
			break;
		}
	}

	return Magnitude;
}

float UGameplayModMagnitudeCalculation::GetSetByCallerMagnitudeByTag(const FGameplayEffectSpec& EffectSpec, const FGameplayTag& Tag) const
{
	return EffectSpec.GetSetByCallerMagnitude(Tag, true, 0.0f);
}

float UGameplayModMagnitudeCalculation::GetSetByCallerMagnitudeByName(const FGameplayEffectSpec& EffectSpec, const FName& MagnitudeName) const
{
	return EffectSpec.GetSetByCallerMagnitude(MagnitudeName, true, 0.0f);
}

FGameplayTagContainer UGameplayModMagnitudeCalculation::GetSourceAggregatedTags(const FGameplayEffectSpec& EffectSpec) const
{
	const FGameplayTagContainer* Tags = EffectSpec.CapturedSourceTags.GetAggregatedTags();

	if (Tags)
	{
		return *Tags;
	}

	return FGameplayTagContainer();
}

const FGameplayTagContainer& UGameplayModMagnitudeCalculation::GetSourceActorTags(const FGameplayEffectSpec& EffectSpec) const
{
	return EffectSpec.CapturedSourceTags.GetActorTags();
}

const FGameplayTagContainer& UGameplayModMagnitudeCalculation::GetSourceSpecTags(const FGameplayEffectSpec& EffectSpec) const
{
	return EffectSpec.CapturedSourceTags.GetSpecTags();
}

FGameplayTagContainer UGameplayModMagnitudeCalculation::GetTargetAggregatedTags(const FGameplayEffectSpec& EffectSpec) const
{
	const FGameplayTagContainer* Tags = EffectSpec.CapturedTargetTags.GetAggregatedTags();

	if (Tags)
	{
		return *Tags;
	}

	return FGameplayTagContainer();
}

const FGameplayTagContainer& UGameplayModMagnitudeCalculation::GetTargetActorTags(const FGameplayEffectSpec& EffectSpec) const
{
	return EffectSpec.CapturedTargetTags.GetActorTags();
}

const FGameplayTagContainer& UGameplayModMagnitudeCalculation::GetTargetSpecTags(const FGameplayEffectSpec& EffectSpec) const
{
	return EffectSpec.CapturedTargetTags.GetSpecTags();
}

