// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayPrediction.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemLog.h"
#include "Engine/World.h"
#include "Engine/NetConnection.h"
#include "Engine/NetDriver.h"
#include "Engine/PackageMapClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayPrediction)

DEFINE_LOG_CATEGORY_STATIC(LogPredictionKey, Display, All);

namespace UE::AbilitySystem::Private
{
	// This controls the client-side code that catches up missed/unacknowledged keys.  In theory this can happen in bad network conditions where
	// a server can fail to acknowledge a key due to packet loss, but it also happens due to bugs where we create & register prediction keys that
	// are never sent to the server.  The science behind this number isn't exact: we default to enough that all local players can exhaust their ring buffer.
	// Set it lower to catch bugs related to not sending prediction keys sooner in your logs, or higher if you have a massive local player count.
	constexpr int32 NumExpectedLocalPlayers = 4;
	int32 CVarMaxStaleKeysBeforeAckValue = FReplicatedPredictionKeyMap::KeyRingBufferSize * NumExpectedLocalPlayers;
	FAutoConsoleVariableRef CVarMaxStaleKeysBeforeAck(TEXT("AbilitySystem.PredictionKey.MaxStaleKeysBeforeAck"), CVarMaxStaleKeysBeforeAckValue,
		TEXT("How many prediction keys can be dropped before StaleKeyBehavior is run."));

	// What should we do with these old stale FPredictionKeys?  Prior to UE5.5, we always CaughtUp.  I believe it's actually safer to drop.
	int32 CVarStaleKeyBehaviorValue = 2;
	FAutoConsoleVariableRef CVarStaleKeyBehavior(TEXT("AbilitySystem.PredictionKey.StaleKeyBehavior"), CVarStaleKeyBehaviorValue,
		TEXT("How do we handle stale keys? 0 = CaughtUp. 1 = Reject. 2 = Drop"));

	// How should we deal with dependent keys (in a chain)?  Prior to UE5.5, old keys implied new keys.  We introduced some new functionality (explained in the help text).
	// 0 (no bitmask) is legacy behavior.  Logically, (0x1 | 0x2) = 3 is the correct value. We default to 1 to move towards a long-term fix (of value 3).
	int32 CVarDependentChainBehaviorValue = 1;
	FAutoConsoleVariableRef CVarDependentChainBehavior(TEXT("AbilitySystem.PredictionKey.DepChainBehavior"), CVarDependentChainBehaviorValue,
		TEXT("How do we handle dependency key chains? Bitmask: 0 = Old Accept/Rejected Implies Newer Accepted/Rejected. 0x1 = Newer Accepted also implies Older Accepted. 0x2 = Old Accepted Does NOT imply Newer Accepted"));

	// Prior to UE5.4.2, we used to allow Server Initiated Replication Keys to be sent to the client as 'acknowledged' but that hardly makes sense.
	// Unfortunately, it also causes key hash collisions since there is limited space based on the key value.
	int32 CVarReplicateServerKeysAsAcknowledgedValue = 0;
	FAutoConsoleVariableRef CVarReplicateServerKeysAsAcknowledged(TEXT("AbilitySystem.PredictionKey.RepServerKeysAsAcknowledged"), CVarReplicateServerKeysAsAcknowledgedValue,
		TEXT("Do we send server initiated keys as acknowledged to the client? Default false. Was true prior to UE5.4.2."));

	/**
	 * Given an FProperty Link (such as a UFunction's Properties, or a UStruct's Properties), find and return any linked FPredictionKeys.
	 * This is useful for determining if we're sending FPredictionKeys via RPC.
	 */
	TArray<FPredictionKey> FindPredictionKeysInPropertyLink(const FProperty* InPropertyLink, uint8_t* StartDataOffset)
	{
		TArray<FPredictionKey> PredictionKeys;

		// Go through the PropertyLinks and look for an FPredictionKey or FPredictionKey nested in a FStruct...
		for (const FProperty* Property = InPropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				if (FPredictionKey::StaticStruct() == StructProperty->Struct)
				{
					// Now if this FPredictionKey argument, get it from memory and prefer this value.
					FPredictionKey& PassedInArg = PredictionKeys.AddDefaulted_GetRef();
					Property->CopyCompleteValue(&PassedInArg, StartDataOffset + Property->GetOffset_ForUFunction());
				}
				else if (StructProperty->Struct)
				{
					// Find a nested prediction key, but don't return it (prefer one at the top-level)
					TArray<FPredictionKey> InnerKeys = FindPredictionKeysInPropertyLink(StructProperty->Struct->PropertyLink, StartDataOffset + StructProperty->GetOffset_ForUFunction());
					PredictionKeys.Append( MoveTempIfPossible(InnerKeys) );
				}
			}
		}

		return PredictionKeys;
	}
}

FReplicatedPredictionKeyItem::FReplicatedPredictionKeyItem()
{
};

FReplicatedPredictionKeyItem::FReplicatedPredictionKeyItem(const FReplicatedPredictionKeyItem& Other)
{
	*this = Other;
}

FReplicatedPredictionKeyItem& FReplicatedPredictionKeyItem::operator=(const FReplicatedPredictionKeyItem& Other)
{
	if (&Other != this)
	{
		ReplicationID = Other.ReplicationID;
		ReplicationKey = Other.ReplicationKey;
		MostRecentArrayReplicationKey = Other.MostRecentArrayReplicationKey;
		PredictionKey = Other.PredictionKey;
	}
	return *this;
}

FReplicatedPredictionKeyItem::FReplicatedPredictionKeyItem(FReplicatedPredictionKeyItem&& Other)
: PredictionKey(MoveTemp(Other.PredictionKey))
{
	ReplicationID = Other.ReplicationID;
	ReplicationKey = Other.ReplicationKey;
	MostRecentArrayReplicationKey = Other.MostRecentArrayReplicationKey;
}

FReplicatedPredictionKeyItem& FReplicatedPredictionKeyItem::operator=(FReplicatedPredictionKeyItem&& Other)
{
	ReplicationID = Other.ReplicationID;
	ReplicationKey = Other.ReplicationKey;
	MostRecentArrayReplicationKey = Other.MostRecentArrayReplicationKey;
	PredictionKey = MoveTemp(Other.PredictionKey);

	return *this;
}

/** The key to understanding this function is that when a key is received by the server, we note which connection gave it to us. We only serialize the key back to that client.  */
bool FPredictionKey::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);
	const bool bReplicateDeprecatedBaseForDemoPurposes = (Ar.EngineNetVer() < FEngineNetworkCustomVersion::PredictionKeyBaseNotReplicated);

	// First bit for valid key for this connection or not. (most keys are not valid)
	uint8 ValidKeyForConnection = 0;
	if (Ar.IsSaving())
	{
		/**
		 *	Only serialize the payload if we have no owning connection (Client sending to server)
		 *	or if the owning connection is this connection (Server only sends the prediction key to the client who gave it to us)
		 *  or if this is a server initiated key (valid on all connections)
		 */		
		ValidKeyForConnection = (Current > 0) && (bIsServerInitiated || (PredictiveConnectionObjectKey == FObjectKey()) || (PredictiveConnectionObjectKey == FObjectKey(Map)));

#if UE_WITH_REMOTE_OBJECT_HANDLE
		// If we are a server communicating with a proxy (or vice versa) we are always valid
		if (GIsServer && !ValidKeyForConnection)
		{
			if (UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Map))
			{
				UNetConnection* NetConnection = PackageMapClient->GetConnection();
				const bool bIsServerSendingToProxy = NetConnection && NetConnection->IsProxyConnection();
				const bool bIsProxySendingToServer = NetConnection && (NetConnection->GetDriver()->GetProxyType() == UE::Net::EProxyType::Backend);
				if (bIsServerSendingToProxy || bIsProxySendingToServer)
				{
					ValidKeyForConnection = true;
				}
			}
		}
#endif
	}
	Ar.SerializeBits(&ValidKeyForConnection, 1);

	// Second bit for the now-deprecated base key (only if valid connection)
	uint8 HasBaseKey = 0;
	if (bReplicateDeprecatedBaseForDemoPurposes && ValidKeyForConnection)
	{
		if (Ar.IsSaving())
		{
			HasBaseKey = Base > 0;
		}
		Ar.SerializeBits(&HasBaseKey, 1);
	}

	// Third bit for server initiated
	uint8 ServerInitiatedByte = bIsServerInitiated;
	Ar.SerializeBits(&ServerInitiatedByte, 1);
	bIsServerInitiated = ServerInitiatedByte & 1;

	// Conditionally Serialize the Current and Base keys
	if (ValidKeyForConnection)
	{
		Ar << Current;
		if (HasBaseKey)
		{
			ensureMsgf(bReplicateDeprecatedBaseForDemoPurposes, TEXT("We should only ever be replicating a Base Key if we're loading an old demo, see bReplicateDeprecatedBaseForDemoPurposes"));
			Ar << Base;
		}
	}	
	if (Ar.IsLoading())
	{
		// We are reading this key: the connection that gave us this key is the predictive connection, and we will only serialize this key back to it.
		if (!bIsServerInitiated)
		{
			PredictiveConnectionObjectKey = FObjectKey(Map);
		}
	}

	bOutSuccess = true;
	return true;
}

void FPredictionKey::GenerateNewPredictionKey()
{
	static KeyType GKey = 1;
	Current = GKey++;
	if (GKey <= 0)
	{
		GKey = 1;
	}
}

void FPredictionKey::GenerateDependentPredictionKey()
{
	if (bIsServerInitiated)
	{
		// Can't have dependent keys on server keys, use same key
		return;
	}

	KeyType Previous = Current;
	if (Base == 0)
	{
		Base = Current;
	}

	GenerateNewPredictionKey();

	ensureAlwaysMsgf((Base == 0) || (Current - Base < 20), TEXT("Deep PredictionKey Chain Detected.  It's likely there's circular logic that could stack overflow."));

	if (Previous > 0)
	{
		FPredictionKeyDelegates::AddDependency(Current, Previous);
	}
}

FPredictionKey FPredictionKey::CreateNewPredictionKey(const UAbilitySystemComponent* OwningComponent)
{
	FPredictionKey NewKey;
	
	// We should never generate prediction keys on the authority
	if(OwningComponent->GetOwnerRole() != ROLE_Authority)
	{
		NewKey.GenerateNewPredictionKey();
	}
	return NewKey;
}

FPredictionKey FPredictionKey::CreateNewServerInitiatedKey(const UAbilitySystemComponent* OwningComponent)
{
	FPredictionKey NewKey;

	// Only valid on the server
	if (OwningComponent->GetOwnerRole() == ROLE_Authority)
	{
		// Make sure the Server and Client aren't synchronized in terms of key generation or it can hide bugs.
		static KeyType GServerKey = 1;
		NewKey.bIsServerInitiated = true;
		NewKey.Current = GServerKey++;
		if (GServerKey <= 0)
		{
			GServerKey = 1;
		}
	}
	return NewKey;
}


FPredictionKeyEvent& FPredictionKey::NewRejectedDelegate()
{
	return FPredictionKeyDelegates::NewRejectedDelegate(Current);
}

FPredictionKeyEvent& FPredictionKey::NewCaughtUpDelegate()
{
	return FPredictionKeyDelegates::NewCaughtUpDelegate(Current);
}

void FPredictionKey::NewRejectOrCaughtUpDelegate(FPredictionKeyEvent Event)
{
	FPredictionKeyDelegates::NewRejectOrCaughtUpDelegate(Current, Event);
}

// -------------------------------------

FPredictionKeyDelegates& FPredictionKeyDelegates::Get()
{
	static FPredictionKeyDelegates StaticMap;
	return StaticMap;
}

FPredictionKeyEvent& FPredictionKeyDelegates::NewRejectedDelegate(FPredictionKey::KeyType Key)
{
	TArray<FPredictionKeyEvent>& DelegateList = Get().DelegateMap.FindOrAdd(Key).RejectedDelegates;
	DelegateList.Add(FPredictionKeyEvent());
	return DelegateList.Top();
}

FPredictionKeyEvent& FPredictionKeyDelegates::NewCaughtUpDelegate(FPredictionKey::KeyType Key)
{
	TArray<FPredictionKeyEvent>& DelegateList = Get().DelegateMap.FindOrAdd(Key).CaughtUpDelegates;
	DelegateList.Add(FPredictionKeyEvent());
	return DelegateList.Top();
}

void FPredictionKeyDelegates::NewRejectOrCaughtUpDelegate(FPredictionKey::KeyType Key, FPredictionKeyEvent NewEvent)
{
	FDelegates& Delegates = Get().DelegateMap.FindOrAdd(Key);
	Delegates.CaughtUpDelegates.Add(NewEvent);
	Delegates.RejectedDelegates.Add(NewEvent);
}

void FPredictionKeyDelegates::BroadcastRejectedDelegate(FPredictionKey::KeyType Key)
{
	// Intentionally making a copy of the delegate list since it may change when firing one of the delegates
	static TArray<FPredictionKeyEvent> DelegateList;
	DelegateList.Reset();
	DelegateList = Get().DelegateMap.FindOrAdd(Key).RejectedDelegates;
	for (auto& Delegate : DelegateList)
	{
		Delegate.ExecuteIfBound();
	}
}

void FPredictionKeyDelegates::BroadcastCaughtUpDelegate(FPredictionKey::KeyType Key)
{
	// Intentionally making a copy of the delegate list since it may change when firing one of the delegates
	static TArray<FPredictionKeyEvent> DelegateList;
	DelegateList.Reset();
	DelegateList = Get().DelegateMap.FindOrAdd(Key).CaughtUpDelegates;
	for (auto& Delegate : DelegateList)
	{
		Delegate.ExecuteIfBound();
	}
}

void FPredictionKeyDelegates::Reject(FPredictionKey::KeyType Key)
{
	FDelegates* DelPtr = Get().DelegateMap.Find(Key);
	if (DelPtr)
	{
		// Copy & Remove first, because during the delegate, we're likely to call this again, 
		// thereby invalidating DelPtr.
		TArray<FPredictionKeyEvent> RejectedDelegates = MoveTemp(DelPtr->RejectedDelegates);
		Get().DelegateMap.Remove(Key);

		for (auto& Delegate : RejectedDelegates)
		{
			Delegate.ExecuteIfBound();
		}
	}
}

void FPredictionKeyDelegates::CatchUpTo(FPredictionKey::KeyType Key)
{
	FDelegates* DelPtr = Get().DelegateMap.Find(Key);
	if (DelPtr)
	{
		// Copy & Remove first, because during the delegate, we're likely to call this again, 
		// thereby invalidating DelPtr.
		TArray<FPredictionKeyEvent> CaughtUpDelegates = MoveTemp(DelPtr->CaughtUpDelegates);
		Get().DelegateMap.Remove(Key);

		for (auto& Delegate : CaughtUpDelegates)
		{
			Delegate.ExecuteIfBound();
		}
	}
}

void FPredictionKeyDelegates::AddDependency(FPredictionKey::KeyType ThisKey, FPredictionKey::KeyType DependsOn)
{
	// If we Reject the BaseKey, then notify ThisKey it has also been Rejected.
	NewRejectedDelegate(DependsOn).BindStatic(&FPredictionKeyDelegates::Reject, ThisKey);

	// 1. If we receive confirmation that a newer key happened, notify the previous one it also happened.
	//	This allows a FPredictionKey to _not_ be sent to the server, yet be acknowledged as long as a newer dependent key was sent & acknowledged.
	//	This is logically correct and newly introduced behavior.
	if ((UE::AbilitySystem::Private::CVarDependentChainBehaviorValue & 1) != 0)
	{
		NewCaughtUpDelegate(ThisKey).BindStatic(&FPredictionKeyDelegates::CatchUpTo, DependsOn);
	}

	// 2. If we receive an earlier key in the chain of dependencies, then the later ones should also be acknowledged. 
	//	This is unintuitive, but it's possible to create FPredictionKeys that are never sent to the server.
	//	This is the case with FScopedServerAbilityRPCBatcher.  It sends only the BaseKey but needs to notify dependents.
	//	This isn't logically correct, so we're introducing a way to disable this legacy functionality once Engine fixes are finished.
	if ((UE::AbilitySystem::Private::CVarDependentChainBehaviorValue & 2) == 0)
	{
		NewCaughtUpDelegate(DependsOn).BindStatic(&FPredictionKeyDelegates::CatchUpTo, ThisKey);
	}
	
}

// -------------------------------------

/** 
 * This is the Server version of FScopedPredictionWindow constructor.
 * This exists for legacy reasons and I'm not convinced this needs to exist.  Instead we should manually accept/reject the FPredictionKey (currently done in the destructor).
 */
FScopedPredictionWindow::FScopedPredictionWindow(UAbilitySystemComponent* AbilitySystemComponent, FPredictionKey InPredictionKey, bool InSetReplicatedPredictionKey /*=true*/)
{
	if (AbilitySystemComponent == nullptr)
	{
		return;
	}

	// This is used to set an already generated prediction key as the current scoped prediction key.
	// Should be called on the server for logical scopes where a given key is valid. E.g, "client gave me this key, we both are going to run Foo()".
	
	if (AbilitySystemComponent->IsNetSimulating() == false)
	{
		Owner = AbilitySystemComponent;
		check(Owner.IsValid());
		RestoreKey = AbilitySystemComponent->ScopedPredictionKey;
		AbilitySystemComponent->ScopedPredictionKey = InPredictionKey;
		ClearScopedPredictionKey = true;
		SetReplicatedPredictionKey = InSetReplicatedPredictionKey;
	}
}

FScopedPredictionWindow::FScopedPredictionWindow(UAbilitySystemComponent* InAbilitySystemComponent, bool bCanGenerateNewKey)
{
	// On the server, this will do nothing since it is authoritative and doesn't need a prediction key for anything.
	// On the client, this will generate a new prediction key if bCanGenerateNewKey is true, and we have a invalid prediction key.

	ClearScopedPredictionKey = false;
	SetReplicatedPredictionKey = false;

	// Owners that are mid destruction will not be valid and will trigger the ensure below (ie. when they stop their anim montages)
	// Original ensure has been left in to catch other cases of invalid Owner ASCs
	if ((!InAbilitySystemComponent) || (InAbilitySystemComponent->IsBeingDestroyed()) || (!IsValidChecked(InAbilitySystemComponent) || InAbilitySystemComponent->IsUnreachable()))
	{
		ABILITY_LOG(Verbose, TEXT("FScopedPredictionWindow() aborting due to Owner (ASC) being null, destroyed or pending kill / unreachable"));
		return;
	}

	Owner = InAbilitySystemComponent;
	if (!ensure(Owner.IsValid()) || InAbilitySystemComponent->IsNetSimulating() == false)
	{
		return;
	}

	// Because of the check above, this is expected to only run on the client.
	if (bCanGenerateNewKey)
	{
		check(InAbilitySystemComponent != NULL); // Should have bailed above with ensure(Owner.IsValid())
		ClearScopedPredictionKey = true;
		RestoreKey = InAbilitySystemComponent->ScopedPredictionKey;
		InAbilitySystemComponent->ScopedPredictionKey.GenerateDependentPredictionKey();
	}

#if !UE_BUILD_SHIPPING
	// Add some debugging functionality if we're the first scoped prediction window (will become a BaseKey value)
	if (bCanGenerateNewKey && !RestoreKey.IsValidKey())
	{
		// Intercept any RPC's of FPredictionKeys so that we can mark them as sent to the server.
		// For RPC's not marked as sent-to-server, we can log them and track them down with breakpoints.
		const UWorld* NetWorldPtr = Owner->GetWorld();
		UNetDriver* NetDriver = NetWorldPtr ? NetWorldPtr->GetNetDriver() : nullptr;
		if (NetDriver && NetDriver->GetNetMode() == NM_Client)
		{
			DebugSavedNetDriver = NetDriver;
			DebugSavedOnSendRPC = NetDriver->SendRPCDel;
			DebugBaseKeyOfChain = InAbilitySystemComponent->ScopedPredictionKey.Current;

			NetDriver->SendRPCDel.BindWeakLambda(InAbilitySystemComponent,
				[this, SendRPCDel = DebugSavedOnSendRPC](AActor* Actor, UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack, UObject* SubObject, bool& bBlockSendRPC)
				{
					// Chain the previous RPC callback (e.g. packet loss simulation)
					SendRPCDel.ExecuteIfBound(Actor, Function, Parameters, OutParms, Stack, SubObject, bBlockSendRPC);

					// If we've already reset this, it's an indication this PredictionKey Chain has already been sent.
					if (!DebugBaseKeyOfChain.IsSet())
					{
						return;
					}

					const FPredictionKey::KeyType BaseKey = DebugBaseKeyOfChain.GetValue();

					// See if we are communicating a key based on this scope, if so, reset our debug value so as not to warn that it was never sent on Scope destruct.
					TArray<FPredictionKey> PassedInKeys = UE::AbilitySystem::Private::FindPredictionKeysInPropertyLink(Function->PropertyLink, static_cast<uint8_t*>(Parameters));
					for (const FPredictionKey& PassedInKey : PassedInKeys)
					{
						if (PassedInKey.Current == BaseKey || PassedInKey.Base == BaseKey)
						{
							UE_LOGF(LogPredictionKey, Verbose, "Sent key %ls with %ls to confirm BaseKey %d", *PassedInKey.ToString(), *Function->GetName(), BaseKey);
							DebugBaseKeyOfChain.Reset();
							break;
						}
					}
				});
		}
	}
#endif
}

FScopedPredictionWindow::~FScopedPredictionWindow()
{
#if !UE_BUILD_SHIPPING
	// Stop intercepting RPC calls
	if (UNetDriver* NetDriver = DebugSavedNetDriver.Get())
	{
		if (DebugBaseKeyOfChain.IsSet())
		{
			const FPredictionKey::KeyType BaseKey = DebugBaseKeyOfChain.GetValue();
			if (const FPredictionKeyDelegates::FDelegates* BoundDelegates = FPredictionKeyDelegates::Get().DelegateMap.Find(BaseKey))
			{
				const bool bIsBound = BoundDelegates->RejectedDelegates.Num() > 0 || BoundDelegates->CaughtUpDelegates.Num() > 0;

				// If you get this log, you can set a breakpoint here and inspect the Delegates to see what it's bound to and therefore what code is at fault (the binding code, or the code that never sends the key to the server). 
				UE_CLOGF(bIsBound, LogPredictionKey, Warning, "No key based off PredictionKey %d was communicated to the server during ScopedPredictionWindow. The PredictionKey has bound callbacks that can never be called (leak).", BaseKey);

				// If you get this log, you can check the call stack to see where this FScopedPredictionWindow originated from, and why it would be creating an entry in the FPredictionKeyDelegates without any bindings to it.
				UE_CLOGF(!bIsBound, LogPredictionKey, Warning, "No key based off PredictionKey %d was communicated to the server during ScopedPredictionWindow. The PredictionKey is not bound, but has leaked an entry in FPredictionKeyDelegates.", BaseKey);
			}
		}
		
		NetDriver->SendRPCDel = DebugSavedOnSendRPC;
		DebugSavedOnSendRPC.Unbind();
		DebugSavedNetDriver.Reset();
		DebugBaseKeyOfChain.Reset();
	}
#endif

	if (UAbilitySystemComponent* OwnerPtr = Owner.Get())
	{
		if (SetReplicatedPredictionKey)
		{
			// It is important to not set the ReplicatedPredictionKey unless it is valid (>0).
			// If we werent given a new prediction key for this scope from the client, then setting the
			// replicated prediction key back to 0 could cause OnReps to be missed on the client during high PL.
			// (for example, predict w/ key 100 -> prediction key replication dropped -> predict w/ invalid key -> next rep of prediction key is 0).
			if (OwnerPtr->ScopedPredictionKey.IsValidKey())
			{
				const bool bServerInitiatedKey = OwnerPtr->ScopedPredictionKey.IsServerInitiatedKey();
				const bool bAllowAckServerInitiatedKey = UE::AbilitySystem::Private::CVarReplicateServerKeysAsAcknowledgedValue > 0;
				if (!bServerInitiatedKey || bAllowAckServerInitiatedKey)
				{
					UE_CLOGF(bServerInitiatedKey, LogAbilitySystem, Warning, "Replicating Server Initiated PredictionKey %ls (this may stomp a client key leaving it unack'd). See CVarReplicateServerKeysAsAcknowledged", *OwnerPtr->ScopedPredictionKey.ToString());
					UE_VLOG_UELOG(OwnerPtr->GetOwnerActor(), LogPredictionKey, Verbose, TEXT("Server: ReplicatePredictionKey %s"), *OwnerPtr->ScopedPredictionKey.ToString());
					OwnerPtr->ReplicatedPredictionKeyMap.ReplicatePredictionKey(OwnerPtr->ScopedPredictionKey);
				}
				else
				{
					UE_LOGF(LogAbilitySystem, Verbose, "Server: NOT replicating PredictionKey %ls due to being server initiated. See CVarReplicateServerKeysAsAcknowledged.", *OwnerPtr->ScopedPredictionKey.ToString());
				}
			}
		}
		if (ClearScopedPredictionKey)
		{
			OwnerPtr->ScopedPredictionKey = RestoreKey;
		}
	}
}

// -----------------------------------

FScopedDiscardPredictions::FScopedDiscardPredictions(UAbilitySystemComponent* AbilitySystemComponent, EGasPredictionKeyResult HowToHandlePredictions)
	: PredictionKeyChainResult(HowToHandlePredictions)
{
	if (AbilitySystemComponent && !AbilitySystemComponent->IsOwnerActorAuthoritative())
	{
		Owner = AbilitySystemComponent;
		KeyToRestoreOnOwner = AbilitySystemComponent->ScopedPredictionKey;

		if (HowToHandlePredictions == EGasPredictionKeyResult::SilentlyDrop)
		{
			AbilitySystemComponent->ScopedPredictionKey = FPredictionKey{};
		}
		else
		{
			// Warn here because we're using a behavior that explicitly says Accept or Reject this key, and we could be using a CVar that doesn't behave as the user expects (hopefully this will be removed when fix the behavior and remove this CVar).
			UE_CLOGF(!GIsAutomationTesting && (UE::AbilitySystem::Private::CVarDependentChainBehaviorValue != 3), LogPredictionKey, Warning, "%s using EGasPredictionKeyResult = %d (which is != EGasPredicitonKeyResult::SilentlyDrop) with CVarDependentChainBehavior not set to proper value.", __func__, static_cast<int>(HowToHandlePredictions));
			AbilitySystemComponent->ScopedPredictionKey.GenerateDependentPredictionKey();
			BaseKeyToAck = AbilitySystemComponent->ScopedPredictionKey;
		}
	}
}

FScopedDiscardPredictions::~FScopedDiscardPredictions()
{
	if (UAbilitySystemComponent* ASC = Owner.Get())
	{
		ASC->ScopedPredictionKey = KeyToRestoreOnOwner;
		if (BaseKeyToAck.IsValidKey())
		{
			if (PredictionKeyChainResult == EGasPredictionKeyResult::Accept)
			{
				UE_LOGF(LogPredictionKey, Verbose, "%s Accepting chain ending with %ls", __func__, *BaseKeyToAck.ToString());
				FPredictionKeyDelegates::CatchUpTo(BaseKeyToAck.Current);
			}
			else if (PredictionKeyChainResult == EGasPredictionKeyResult::Reject)
			{
				UE_LOGF(LogPredictionKey, Verbose, "%s Rejecting chain starting with %ls", __func__, *BaseKeyToAck.ToString());
				FPredictionKeyDelegates::Reject(BaseKeyToAck.Current);
			}
			else
			{
				UE_LOGF(LogPredictionKey, Error, "%s had a valid BaseKeyToAck (%ls) but we were configured to Drop the Key (it should have been invalid)", __func__, *BaseKeyToAck.ToString());
			}
		}
	}
}

// -----------------------------------

void FReplicatedPredictionKeyItem::OnRep(const FReplicatedPredictionKeyMap& InArray)
{
	using namespace UE::AbilitySystem::Private;

	// If either of these logs are set to Verbose, log on one (and only one) of them.
	ABILITY_LOG(Verbose, TEXT("FReplicatedPredictionKeyItem::OnRep %s"), *PredictionKey.ToString());
	if (!UE_LOG_ACTIVE(LogAbilitySystem, Verbose))
	{
		UE_LOGF(LogPredictionKey, Verbose, "FReplicatedPredictionKeyItem::OnRep %ls", *PredictionKey.ToString());
	}

	// We should only run catch-up logic to locally predicted keys.  Otherwise, there will eventually be a case where the server
	// initiates a key that matches a local value.  Then we catch-up to a local value, even though the server was not specifically acknowledging it.
	if (PredictionKey.bIsServerInitiated)
	{
		UE_LOGF(LogPredictionKey, Warning, "FReplicatedPredictionKeyItem::OnRep received Server Initiated Key %ls (which likely stomped a local key due to keymap hash collisions). This shouldn't happen if CVarReplicateServerKeysAsAcknowledged is 0.", *PredictionKey.ToString());
		return;
	}

	// Every predictive action we've done in the current chain of dependencies (including the current value) of ReplicatedPredictionKey needs to be acknowledged
	FPredictionKeyDelegates::CatchUpTo(PredictionKey.Current);

	// Remove Stale Prediction Keys
	//
	// Let's check for older keys that should have been cleaned up.  This is an indication that these FPredictionKeys were never acknowledged
	// and one way that can happen, is if no FPredictionKeys in a dependency chain are sent to the server.  In such a case, you've locally predicted
	// a bunch of events that the server has no way of accepting/rejecting.  A decision was made long ago to auto-accept these, so let's keep that functionality.
	const int MaxKeyLagValue = FMath::Max(CVarMaxStaleKeysBeforeAckValue, FReplicatedPredictionKeyMap::KeyRingBufferSize);

	// We now define a range from [Min,Max] but since it's a circular buffer, it can also be [0,Min] [Max,KeyMax]
	constexpr int32 MaxKeyValue = std::numeric_limits<FPredictionKey::KeyType>::max();
	int32 RangeMin = (PredictionKey.Current - MaxKeyLagValue) > 0 ? (PredictionKey.Current - MaxKeyLagValue) : MaxKeyValue + (PredictionKey.Current - MaxKeyLagValue);
	int32 RangeMax = (PredictionKey.Current + MaxKeyLagValue) % MaxKeyValue;

	// If Max > Min, then we're not wrapped and the safe range is [Min,Max]
	const bool bRangeIsSafeZone = (RangeMax > RangeMin);
	if (!bRangeIsSafeZone)
	{
		// Otherwise, let's swap so [Min, Max] is the unsafe zone and we'll pair it with !bRangeIsSafeZone.
		Swap(RangeMin, RangeMax);
	}

	// Go through the unordered delegates and look for old entries
	TArray<FPredictionKey::KeyType> StalePredictionKeys;
	for (auto MapIt = FPredictionKeyDelegates::Get().DelegateMap.CreateIterator(); MapIt; ++MapIt)
	{
		const FPredictionKey::KeyType ExistingKey = MapIt.Key();

		// Now check if we're within the range specified.  If we are, and the range is not the safe zone (or vice versa) catch-up.
		const bool bWithinRange = (ExistingKey >= RangeMin) && (ExistingKey <= RangeMax);
		if (bWithinRange ^ bRangeIsSafeZone)
		{
			StalePredictionKeys.Add(ExistingKey);
		}
	}

	// If there were any unacknowledged keys who have since timed out
	if (StalePredictionKeys.Num() > 0)
	{
		// Sort so we do the acks in the correct order
		StalePredictionKeys.Sort();
		for (FPredictionKey::KeyType Key : StalePredictionKeys)
		{
			const int32 KeyIndex = Key % InArray.KeyRingBufferSize;
			if (!ensure(InArray.PredictionKeys.IsValidIndex(KeyIndex)))
			{
				continue;
			}

			// We don't want to be creating these FPredictionKeys that aren't sent to the server.  We should warn the user that they need to accept/reject them based on their own logic.
			const FPredictionKey& ExistingKeyInSlot = InArray.PredictionKeys[KeyIndex].PredictionKey;
			UE_LOGF(LogPredictionKey, Warning, "UnAck'd PredictionKey %d in DelegateMap while OnRep %ls. This indicates keychains are created w/o being sent to & ack'd by the server. Last ack'd key in that slot was %ls.", Key, *PredictionKey.ToString(), *ExistingKeyInSlot.ToString());
			
			if (CVarStaleKeyBehaviorValue == 0)
			{
				// This is legacy functionality.
				FPredictionKeyDelegates::CatchUpTo(Key);
			}
			else if (CVarStaleKeyBehaviorValue == 1)
			{
				// Alternatively, this may be more appropriate as the server didn't specifically acknowledge us.
				FPredictionKeyDelegates::Reject(Key);
			}
			else
			{
				// This is the safest.  It could leave Abilities waiting in async tasks forever, but should never crash.
				FPredictionKeyDelegates::Get().DelegateMap.Remove(Key);
			}
		}
	}
}

const int32 FReplicatedPredictionKeyMap::KeyRingBufferSize = 32;

FReplicatedPredictionKeyMap::FReplicatedPredictionKeyMap()
{
	PredictionKeys.SetNum(KeyRingBufferSize);
	for (FReplicatedPredictionKeyItem& Item : PredictionKeys)
	{
		MarkItemDirty(Item);
	}
}

bool FReplicatedPredictionKeyMap::NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms)
{
	return FastArrayDeltaSerialize<FReplicatedPredictionKeyItem>(PredictionKeys, DeltaParms, *this);
}

void FReplicatedPredictionKeyMap::ReplicatePredictionKey(FPredictionKey Key)
{	
	int32 Index = (Key.Current % KeyRingBufferSize);
	PredictionKeys[Index].PredictionKey = Key;
	MarkItemDirty(PredictionKeys[Index]);
}

FString FReplicatedPredictionKeyMap::GetDebugString() const
{
	FPredictionKey HighKey;
	for (const FReplicatedPredictionKeyItem& Item : PredictionKeys)
	{
		if (Item.PredictionKey.Current > HighKey.Current)
		{
			HighKey = Item.PredictionKey;
		}
	}

	return HighKey.ToString();
}

