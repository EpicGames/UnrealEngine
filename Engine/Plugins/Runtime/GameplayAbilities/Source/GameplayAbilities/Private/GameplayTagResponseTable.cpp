// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagResponseTable.h"
#include "AbilitySystemComponent.h"
#include "Misc/TimeGuard.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagResponseTable)

// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	UGameplayTagReponseTable
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------


UGameplayTagReponseTable::UGameplayTagReponseTable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Make an empty tag query. We will actually fill the tag out prior to evaluating the query with ::MakeQuery

	Query.OwningTagQuery = FGameplayTagQuery::BuildQuery(
		FGameplayTagQueryExpression()
		.AllTagsMatch()
		.AddTag(FGameplayTag())
	);
	LastASCPurgeTime = 0;
}

void UGameplayTagReponseTable::PostLoad()
{
	Super::PostLoad();

	// Fixup
	for (FGameplayTagResponseTableEntry& Entry :  Entries)
	{
		if (*Entry.Positive.ResponseGameplayEffect)
		{
			Entry.Positive.ResponseGameplayEffects.Add(Entry.Positive.ResponseGameplayEffect);
			Entry.Positive.ResponseGameplayEffect = nullptr;
		}
		if (*Entry.Negative.ResponseGameplayEffect)
		{
			Entry.Negative.ResponseGameplayEffects.Add(Entry.Negative.ResponseGameplayEffect);
			Entry.Negative.ResponseGameplayEffect = nullptr;
		}
	}
}

void UGameplayTagReponseTable::RegisterResponseForEvents(UAbilitySystemComponent* ASC)
{
	if (RegisteredASCs.Contains(ASC))
	{
		return;
	}

	TArray<FGameplayTagResponseAppliedInfo> AppliedInfoList;
	AppliedInfoList.SetNum(Entries.Num());

	RegisteredASCs.Add(ASC, AppliedInfoList );

	for (int32 idx=0; idx < Entries.Num(); ++idx)
	{
		const FGameplayTagResponseTableEntry& Entry = Entries[idx];
		if (Entry.Positive.Tag.IsValid())
		{
			ASC->RegisterGameplayTagEvent( Entry.Positive.Tag, EGameplayTagEventType::AnyCountChange ).AddUObject(this, &UGameplayTagReponseTable::TagResponseEvent, ASC, idx);
		}
		if (Entry.Negative.Tag.IsValid())
		{
			ASC->RegisterGameplayTagEvent( Entry.Negative.Tag, EGameplayTagEventType::AnyCountChange ).AddUObject(this, &UGameplayTagReponseTable::TagResponseEvent, ASC, idx);
		}
	}

	// Need to periodically cull null entries. We can do this very infrequently as the memory overhead is not great
	if (FPlatformTime::Seconds() - LastASCPurgeTime >= 300.0f)
	{
		SCOPE_TIME_GUARD_MS(TEXT("GameplayTagReponseTableCleanup"), 1);

		bool Removed = false;
		for (auto It = RegisteredASCs.CreateIterator(); It; ++It)
		{
			if (It.Key().IsValid() == false)
			{
				Removed = true;
				It.RemoveCurrent();
			}
		}

		if (Removed)
		{
			RegisteredASCs.Compact();
		}

		LastASCPurgeTime = FPlatformTime::Seconds();
	}
}

void UGameplayTagReponseTable::TagResponseEvent(const FGameplayTag Tag, int32 NewCount, UAbilitySystemComponent* ASC, int32 idx)
{
	if (!ensure(Entries.IsValidIndex(idx)))
	{
		return;
	}

	const FGameplayTagResponseTableEntry& Entry = Entries[idx];
	int32 TotalCount = 0;

	{
		QUICK_SCOPE_CYCLE_COUNTER(ABILITY_TRT_CALC_COUNT);

		int32 Positive = GetCount(Entry.Positive, ASC);
		int32 Negative = GetCount(Entry.Negative, ASC);

		TotalCount = Positive - Negative;
	}

	TArray<FGameplayTagResponseAppliedInfo>& InfoList = RegisteredASCs.FindChecked(ASC);
	FGameplayTagResponseAppliedInfo& Info = InfoList[idx];

	if (TotalCount < 0)
	{
		Remove(ASC, Info.PositiveHandles);
		AddOrUpdate(ASC, Entry.Negative.ResponseGameplayEffects, TotalCount, Info.NegativeHandles);
	}
	else if (TotalCount > 0)
	{
		Remove(ASC, Info.NegativeHandles);
		AddOrUpdate(ASC, Entry.Positive.ResponseGameplayEffects, TotalCount, Info.PositiveHandles);
	}
	else
	{
		Remove(ASC, Info.PositiveHandles);
		Remove(ASC, Info.NegativeHandles);
	}
}

int32 UGameplayTagReponseTable::GetCount(const FGameplayTagReponsePair& Pair, UAbilitySystemComponent* ASC) const
{
	int32 Count=0;
	if (Pair.Tag.IsValid())
	{
		Count = ASC->GetAggregatedStackCount(MakeQuery(Pair.Tag));
		if (Pair.SoftCountCap > 0)
		{
			Count = FMath::Min<int32>(Pair.SoftCountCap, Count);
		}
	}

	return Count;
}

void UGameplayTagReponseTable::Remove(UAbilitySystemComponent* ASC, TArray<FActiveGameplayEffectHandle>& Handles)
{
	for (FActiveGameplayEffectHandle& Handle : Handles)
	{
		if (Handle.IsValid())
		{
			ASC->RemoveActiveGameplayEffect(Handle);
			Handle.Invalidate();
		}
	}
	Handles.Reset();
}

void UGameplayTagReponseTable::AddOrUpdate(UAbilitySystemComponent* ASC, const TArray<TSubclassOf<UGameplayEffect> >& ResponseGameplayEffects, int32 TotalCount, TArray<FActiveGameplayEffectHandle>& Handles)
{
	if (ResponseGameplayEffects.Num() > 0)
	{
		if (Handles.Num() > 0)
		{
			// Already been applied
			for (FActiveGameplayEffectHandle& Handle : Handles)
			{
				ASC->SetActiveGameplayEffectLevel(Handle, TotalCount);
			}
		}
		else
		{
			for (const TSubclassOf<UGameplayEffect>& ResponseGameplayEffect : ResponseGameplayEffects)
			{
				FActiveGameplayEffectHandle NewHandle = ASC->ApplyGameplayEffectToSelf(Cast<UGameplayEffect>(ResponseGameplayEffect->GetDefaultObject(false)), TotalCount, ASC->MakeEffectContext());
				if (NewHandle.IsValid())
				{
					Handles.Add(NewHandle);
				}
			}
		}
	}
}

