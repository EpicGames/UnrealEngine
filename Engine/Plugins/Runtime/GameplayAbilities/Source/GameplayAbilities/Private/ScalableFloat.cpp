// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScalableFloat.h"
#include "Stats/StatsMisc.h"
#include "EngineDefines.h"
#include "Engine/Blueprint.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/ObjectLibrary.h"
#include "AbilitySystemLog.h"
#include "UObject/UObjectIterator.h"
#include "DataRegistrySubsystem.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScalableFloat)

#define LOCTEXT_NAMESPACE "ScalableFloat"

#if WITH_EDITOR
#include "EditorReimportHandler.h"
#include "Misc/DataValidation.h"

EDataValidationResult FScalableFloat::IsDataValid(FDataValidationContext& Context, const FString& PathName) const
{
	if (Curve.CurveTable)
	{
		if (Curve.RowName.IsNone())
		{
			Context.AddError(FText::Format(LOCTEXT("RowNameIsNone", "{0}: CurveTable Row is None"), FText::FromString(PathName)));
			return EDataValidationResult::Invalid;
		}

		if (!Curve.CurveTable->FindCurve(Curve.RowName, PathName, false))
		{
			Context.AddError(FText::Format(LOCTEXT("RowNameDoesNotExist", "{0}: RowName {1} does not exist in CurveTable {2}"),
				FText::FromString(PathName), FText::FromString(Curve.RowName.ToString()), FText::FromString(Curve.CurveTable.GetName())));
			return EDataValidationResult::Invalid;
		}
	}
	else if (RegistryType.IsValid())
	{
		if (Curve.RowName.IsNone())
		{
			Context.AddError(FText::Format(LOCTEXT("RegistryRowNameIsNone", "{0}: Registry CurveTable Row is None"), FText::FromString(PathName)));
			return EDataValidationResult::Invalid;
		}

		const UDataRegistrySubsystem* SubSystem = UDataRegistrySubsystem::Get();
		if (!SubSystem->IsConfigEnabled(false) || !SubSystem->AreRegistriesInitialized())
		{
			// If we're given a use case, it's more likely to be a more thorough IsDataValid, so let them know why this failed to validate.
			if (Context.GetValidationUsecase() != EDataValidationUsecase::None)
			{
				Context.AddWarning(FText::Format(LOCTEXT("DataRegistriesNotLoaded", "{0}: DataRegistries were not loaded in time for validation"),
					FText::FromString(PathName)));
			}

			// We are not ready to evaluate this until after load
			return EDataValidationResult::NotValidated;
		}

		// This case is a bit tough, we are not 100% sure all DataRegistries are loaded.  We should give some indication an error can be occurring,
		// but we also don't want to fail any automated checks that skip DataRegistry loading.
		float OutValue = 0.0f;
		const FRealCurve* OutCurve = nullptr;
		const FDataRegistryId DataRegistryId(RegistryType, Curve.RowName);
		const FDataRegistryCacheGetResult GetCurveResult = SubSystem->EvaluateCachedCurve(OutValue, OutCurve, DataRegistryId, OutValue);
		if (!GetCurveResult)
		{
			FText WarningText = FText::Format(LOCTEXT("DataRegistryCurveDoesNotExist", "{0}: Specified DataRegistry Curve '{1}' does not exist (or was not loaded - you may need to load it to fix this message)"),
				FText::FromString(PathName), DataRegistryId.ToText());

			switch (Context.GetValidationUsecase())
			{
				// If we're manually testing, we can enforce the user loads the DataRegistry to confirm it's right 
				case EDataValidationUsecase::Manual:
					Context.AddWarning(WarningText);
					return EDataValidationResult::NotValidated;

				// For all other cases, let's downgrade to an info message so we can inform a user without tripping any automation errors
				default:
					Context.AddMessage(EMessageSeverity::Info, WarningText);
					return EDataValidationResult::NotValidated;
			}
		}
	}

	return EDataValidationResult::Valid;
}
#endif

bool FScalableFloat::EvaluateCurveAtLevel(float& OutValue, const FRealCurve*& OutCurve, float Level, const FString& ContextString, bool bWarnIfInvalid) const
{
	if (!Curve.RowName.IsNone())
	{
		// This is a simple mechanism for invalidating our cached curve. If someone calls FScalableFloat::InvalidateAllCachedCurves (static method)
		// all cached curve tables are invalidated and will be updated the next time they are accessed
		const int32 GlobalCachedCurveID = UCurveTable::GetGlobalCachedCurveID();
		if (LocalCachedCurveID != GlobalCachedCurveID)
		{
			ClearCachedCurve();
		}

		OutCurve = CachedCurve;
		if (OutCurve == nullptr)
		{
			// Cache not valid, look at sources
			if (Curve.CurveTable)
			{
				CachedCurve = OutCurve = Curve.GetCurve(ContextString, bWarnIfInvalid);
				LocalCachedCurveID = GlobalCachedCurveID;
			}
			// Don't lookup during shutdown as the subsystem may have been deleted
			else if (RegistryType.IsValid() && !IsEngineExitRequested())
			{
				UDataRegistrySubsystem* SubSystem = UDataRegistrySubsystem::Get();
				if (ensure(SubSystem))
				{
					// Always evaluate
					float OutTempValue = 0.0f;
					FDataRegistryCacheGetResult GetResult = SubSystem->EvaluateCachedCurve(OutTempValue, OutCurve, FDataRegistryId(RegistryType, Curve.RowName), Level);

					if (GetResult)
					{
						if (GetResult.IsPersistent() && GetResult.GetVersionSource() == EDataRegistryCacheVersionSource::CurveTable)
						{
							// Only cache if safe
							CachedCurve = OutCurve;
							LocalCachedCurveID = GetResult.GetCacheVersion();
						}

						OutValue = Value * OutTempValue;
						return true;
					}
				}
			}
		}

		// Use cached curve to evaluate
		if (OutCurve != nullptr)
		{
			OutValue = Value * OutCurve->Eval(Level);
			return true;
		}
		else
		{
			// Has row name but no curve, this is an error but fallback to raw value
			if (bWarnIfInvalid && !Curve.CurveTable)
			{
				// CurveTable case is handled in GetCurve above
				if (RegistryType.IsValid())
				{
					// This can happen if the data registry hasn't loaded yet so don't warn by default
					ABILITY_LOG(Verbose, TEXT("FScalableFloat could not find curve for DataRegistryId %s:%s (%s)."), *RegistryType.ToString(), *Curve.RowName.ToString(), *ContextString);
				}
				else
				{
					ABILITY_LOG(Warning, TEXT("FScalableFloat has no CurveTable or DataRegistry for row %s (%s)."), *Curve.RowName.ToString(), *ContextString);
				}
			}

			OutValue = Value;
			return false;
		}
	}
	else if (bWarnIfInvalid)
	{
		// These cases are errors, but fallback to no curve case for compatibility
		if (RegistryType.IsValid())
		{
			ABILITY_LOG(Warning, TEXT("FScalableFloat has no row for DataRegistry %s (%s)."), *RegistryType.ToString(), *ContextString);
		}
		else if (Curve.CurveTable)
		{
			ABILITY_LOG(Warning, TEXT("FScalableFloat has no row for CurveTable %s (%s)."), *Curve.CurveTable->GetPathName(), *ContextString);
		}
	}

	// No curve, use raw value
	OutCurve = nullptr;
	OutValue = Value;
	return true;
}

float FScalableFloat::GetValueAtLevel(float Level, const FString* ContextString) const
{
	float OutFloat;
	const FRealCurve* FoundCurve;
	static const FString DefaultContextString = TEXT("FScalableFloat::GetValueAtLevel");

	// Only print warnings if we have a useful context string
	EvaluateCurveAtLevel(OutFloat, FoundCurve, Level, ContextString != nullptr ? *ContextString : DefaultContextString, ContextString != nullptr);

	return OutFloat;
}

float FScalableFloat::GetValue(const FString* ContextString /*= nullptr*/) const
{
	return GetValueAtLevel(0, ContextString);
}

bool FScalableFloat::AsBool(float Level, const FString* ContextString) const
{
	return GetValueAtLevel(Level, ContextString) > 0.0f;
}

int32 FScalableFloat::AsInteger(float Level, const FString* ContextString) const
{
	return (int32)GetValueAtLevel(Level, ContextString);
}

void FScalableFloat::SetValue(float NewValue)
{
	Value = NewValue;
	Curve.CurveTable = nullptr;
	Curve.RowName = NAME_None;
	ClearCachedCurve();
}

void FScalableFloat::SetScalingValue(float InCoeffecient, FName InRowName, UCurveTable * InTable)
{
	Value = InCoeffecient;
	Curve.RowName = InRowName;
	Curve.CurveTable = InTable;
	ClearCachedCurve();
}

FString FScalableFloat::ToSimpleString() const
{
	if (Curve.RowName != NAME_None)
	{
		if (Curve.CurveTable)
		{
			return FString::Printf(TEXT("%.2f - %s@%s"), Value, *Curve.RowName.ToString(), *Curve.CurveTable->GetName());
		}
		return FString::Printf(TEXT("%.2f - %s:%s"), Value, *RegistryType.ToString(), *Curve.RowName.ToString());
	}
	return FString::Printf(TEXT("%.2f"), Value);
}

bool FScalableFloat::IsValid() const
{
	float OutFloat;
	const FRealCurve* FoundCurve;
	static const FString DefaultContextString = TEXT("FScalableFloat::IsValid");

	return EvaluateCurveAtLevel(OutFloat, FoundCurve, 0.f, DefaultContextString, false);
}

bool FScalableFloat::IsValidWithWarnings(const FString& ContextString) const
{
	return true;
}

bool FScalableFloat::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_FloatProperty)
	{
		float OldValue;
		Slot << OldValue;
		*this = FScalableFloat(OldValue);

		return true;
	}
	else if (Tag.Type == NAME_IntProperty)
	{
		int32 OldValue;
		Slot << OldValue;
		*this = FScalableFloat((float)OldValue);

		return true;
	}
	else if (Tag.Type == NAME_Int8Property)
	{
		int8 OldValue;
		Slot << OldValue;
		*this = FScalableFloat((float)OldValue);

		return true;
	}
	else if (Tag.Type == NAME_Int16Property)
	{
		int16 OldValue;
		Slot << OldValue;
		*this = FScalableFloat((float)OldValue);

		return true;
	}
	else if (Tag.Type == NAME_BoolProperty)
	{
		*this = FScalableFloat(Tag.BoolVal ? 1.f : 0.f);
		return true; 
	}
	return false;
}

void FScalableFloat::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		ClearCachedCurve();
	}
}

bool FScalableFloat::operator==(const FScalableFloat& Other) const
{
	return ((Other.Curve == Curve) && (Other.RegistryType == RegistryType) && (Other.Value == Value));
}

bool FScalableFloat::operator!=(const FScalableFloat& Other) const
{
	return ((Other.Curve != Curve) || (Other.RegistryType != RegistryType) || (Other.Value != Value));
}

FScalableFloat& FScalableFloat::operator=(const FScalableFloat& Src)
{
	Value = Src.Value;
	Curve = Src.Curve;
	RegistryType = Src.RegistryType;
	
	// Clear cache here instead of copying it over to ensure we never have out of date cached values
	ClearCachedCurve();

	return *this;
}


// --------------------------------------------------------------------------------

#if WITH_EDITOR

struct FBadScalableFloat
{
	UObject* Asset;
	FProperty* Property;

	FString String;
};

static FBadScalableFloat GCurrentBadScalableFloat;
static TArray<FBadScalableFloat> GCurrentBadScalableFloatList;
static TArray<FBadScalableFloat> GCurrentNaughtyScalableFloatList;


static bool CheckForBadScalableFloats_r(void* Data, UStruct* Struct, UClass* Class);

static bool CheckForBadScalableFloats_Prop_r(void* Data, FProperty* Prop, UClass* Class)
{
	void* InnerData = Prop->ContainerPtrToValuePtr<void>(Data);

	FStructProperty* StructProperty = CastField<FStructProperty>(Prop);
	if (StructProperty)
	{
		if (StructProperty->Struct == FScalableFloat::StaticStruct())
		{
			FScalableFloat* ThisScalableFloat = static_cast<FScalableFloat*>(InnerData);
			if (ThisScalableFloat)
			{
				if (ThisScalableFloat->IsValid() == false)
				{
					if (ThisScalableFloat->Curve.RowName == NAME_None)
					{
						// Just fix this case up here
						ThisScalableFloat->Curve.CurveTable = nullptr;
						ThisScalableFloat->RegistryType = NAME_None;
						GCurrentBadScalableFloat.Asset->MarkPackageDirty();
					}
					else if (ThisScalableFloat->Curve.CurveTable == nullptr && !ThisScalableFloat->RegistryType.IsValid())
					{
						// Just fix this case up here
						ThisScalableFloat->Curve.RowName = NAME_None;
						GCurrentBadScalableFloat.Asset->MarkPackageDirty();
					}
					else
					{
						GCurrentBadScalableFloat.Property = Prop;
						GCurrentBadScalableFloat.String = ThisScalableFloat->ToSimpleString();

						GCurrentBadScalableFloatList.Add(GCurrentBadScalableFloat);
					}
				}
				else 
				{
					if (ThisScalableFloat->Curve.RowName != NAME_None && ThisScalableFloat->Value != 1.f)
					{
						GCurrentBadScalableFloat.Property = Prop;
						GCurrentBadScalableFloat.String = ThisScalableFloat->ToSimpleString();

						GCurrentNaughtyScalableFloatList.Add(GCurrentBadScalableFloat);
					}
				}
			}
		}
		else
		{
			CheckForBadScalableFloats_r(InnerData, StructProperty->Struct, Class);
		}
	}

	FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Prop);
	if (ArrayProperty)
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, InnerData);
		int32 n = ArrayHelper.Num();
		for (int32 i=0; i < n; ++i)
		{
			void* ArrayData = ArrayHelper.GetRawPtr(i);			
			CheckForBadScalableFloats_Prop_r(ArrayData, ArrayProperty->Inner, Class);
		}
	}

	return false;
}

static bool	CheckForBadScalableFloats_r(void* Data, UStruct* Struct, UClass* Class)
{
	for (TFieldIterator<FProperty> FieldIt(Struct, EFieldIteratorFlags::IncludeSuper); FieldIt; ++FieldIt)
	{
		FProperty* Prop = *FieldIt;
		CheckForBadScalableFloats_Prop_r(Data, Prop, Class);
		
	}

	return false;
}

// -------------

static bool FindClassesWithScalableFloat_r(const TArray<FString>& Args, UStruct* Struct, UClass* Class);

static bool FindClassesWithScalableFloat_Prop_r(const TArray<FString>& Args, FProperty* Prop, UClass* Class)
{
	FStructProperty* StructProperty = CastField<FStructProperty>(Prop);
	if (StructProperty)
	{
		if (StructProperty->Struct == FScalableFloat::StaticStruct())
		{
			return true;
				
		}
		else
		{
			return FindClassesWithScalableFloat_r(Args, StructProperty->Struct, Class);
		}
	}

	FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Prop);
	if (ArrayProperty)
	{
		return FindClassesWithScalableFloat_Prop_r(Args, ArrayProperty->Inner, Class);
	}

	return false;
}

static bool	FindClassesWithScalableFloat_r(const TArray<FString>& Args, UStruct* Struct, UClass* Class)
{
	for (TFieldIterator<FProperty> FieldIt(Struct, EFieldIteratorFlags::ExcludeSuper); FieldIt; ++FieldIt)
	{
		FProperty* Prop = *FieldIt;
		if (FindClassesWithScalableFloat_Prop_r(Args, Prop, Class))
		{
			return true;
		}
	}

	return false;
}

static void	FindInvalidScalableFloats(const TArray<FString>& Args, bool ShowCoeffecients)
{
	GCurrentBadScalableFloatList.Empty();

	TArray<UClass*>	ClassesWithScalableFloats;
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* ThisClass = *ClassIt;
		if (FindClassesWithScalableFloat_r(Args, ThisClass, ThisClass))
		{
			ClassesWithScalableFloats.Add(ThisClass);
			ABILITY_LOG(Warning, TEXT("Class has scalable float: %s"), *ThisClass->GetName());
		}
	}

	for (UClass* ThisClass : ClassesWithScalableFloats)
	{
		UObjectLibrary* ObjLibrary = nullptr;
		TArray<FAssetData> AssetDataList;
		TArray<FString> Paths;
		Paths.Add(TEXT("/Game/"));

		{
			FString PerfMessage = FString::Printf(TEXT("Loading %s via ObjectLibrary"), *ThisClass->GetName() );
			SCOPE_LOG_TIME_IN_SECONDS(*PerfMessage, nullptr)
			ObjLibrary = UObjectLibrary::CreateLibrary(ThisClass, true, true);

			ObjLibrary->LoadBlueprintAssetDataFromPaths(Paths, true);
			ObjLibrary->LoadAssetsFromAssetData();
			ObjLibrary->GetAssetDataList(AssetDataList);

			ABILITY_LOG( Warning, TEXT("Found: %d %s assets."), AssetDataList.Num(), *ThisClass->GetName());
		}


		for (FAssetData Data: AssetDataList)
		{
			UPackage* ThisPackage = Data.GetPackage();
			UBlueprint* ThisBlueprint =  CastChecked<UBlueprint>(Data.GetAsset());
			UClass* AssetClass = ThisBlueprint->GeneratedClass;
			UObject* ThisCDO = AssetClass->GetDefaultObject();		
		
			FString PathName = ThisCDO->GetName();
			PathName.RemoveFromStart(TEXT("Default__"));

			GCurrentBadScalableFloat.Asset = ThisCDO;
			
						
			//ABILITY_LOG( Warning, TEXT("Asset: %s "), *PathName	);
			CheckForBadScalableFloats_r(ThisCDO, AssetClass, AssetClass);
		}
	}


	ABILITY_LOG( Error, TEXT(""));
	ABILITY_LOG( Error, TEXT(""));

	if (ShowCoeffecients == false)
	{

		for ( FBadScalableFloat& BadFoo : GCurrentBadScalableFloatList)
		{
			ABILITY_LOG( Error, TEXT(", %s, %s, %s,"), *BadFoo.Asset->GetFullName(), *BadFoo.Property->GetFullName(), *BadFoo.String );

		}

		ABILITY_LOG( Error, TEXT(""));
		ABILITY_LOG( Error, TEXT("%d Errors total"), GCurrentBadScalableFloatList.Num() );
	}
	else
	{
		ABILITY_LOG( Error, TEXT("Non 1 coefficients: "));

		for ( FBadScalableFloat& BadFoo : GCurrentNaughtyScalableFloatList)
		{
			ABILITY_LOG( Error, TEXT(", %s, %s, %s"), *BadFoo.Asset->GetFullName(), *BadFoo.Property->GetFullName(), *BadFoo.String );

		}
	}
}

FAutoConsoleCommand FindInvalidScalableFloatsCommand(
	TEXT("FindInvalidScalableFloats"), 
	TEXT( "Searches for invalid scalable floats in all assets. Warning this is slow!" ), 
	FConsoleCommandWithArgsDelegate::CreateStatic(FindInvalidScalableFloats, false)
);

FAutoConsoleCommand FindCoefficientScalableFloatsCommand(
	TEXT("FindCoefficientScalableFloats"), 
	TEXT( "Searches for scalable floats with a non 1 coeffecient. Warning this is slow!" ), 
	FConsoleCommandWithArgsDelegate::CreateStatic(FindInvalidScalableFloats, true)
);

#endif

#undef LOCTEXT_NAMESPACE
