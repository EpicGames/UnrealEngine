// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneGameplayCueSections.h"

#include "AbilitySystemGlobals.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEvaluationHookSystem.h"
#include "Evaluation/MovieSceneEvaluationState.h"
#include "GameplayCueManager.h"
#include "Sequencer/MovieSceneGameplayCueTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneGameplayCueSections)

namespace UE
{
namespace MovieScene
{

AActor* ActorFromResolvedObject(UObject* BoundObject)
{
	AActor* Actor = Cast<AActor>(BoundObject);
	if (Actor)
	{
		return Actor;
	}

	if (UActorComponent* ActorComponent = Cast<UActorComponent>(BoundObject))
	{
		return ActorComponent->GetOwner();
	}

	return nullptr;
}


void ExecuteGameplayCueEvent(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params, const FMovieSceneGameplayCueKey& GameplayCueKey, EGameplayCueEvent::Type Event)
{
	auto OverrideHandler = [](AActor* InActor, FGameplayTag InTag, const FGameplayCueParameters& InParams, EGameplayCueEvent::Type InEvent)
	{
		UMovieSceneGameplayCueTrack::OnHandleCueEvent.Execute(InActor, InTag, InParams, InEvent);
	};
	auto DefaultHandler = [](AActor* InActor, FGameplayTag InTag, const FGameplayCueParameters& InParams, EGameplayCueEvent::Type InEvent)
	{
		UAbilitySystemGlobals::Get().GetGameplayCueManager()->HandleGameplayCue(InActor, InTag, InEvent, InParams);
	};

	void (*Handler)(AActor*, FGameplayTag, const FGameplayCueParameters&, EGameplayCueEvent::Type) = nullptr;
	if (UMovieSceneGameplayCueTrack::OnHandleCueEvent.IsBound())
	{
		Handler = OverrideHandler;
	}
	else
	{
		Handler = DefaultHandler;
	}

	FGameplayCueParameters CueParameters;
	CueParameters.NormalizedMagnitude  = GameplayCueKey.NormalizedMagnitude;
	CueParameters.RawMagnitude         = GameplayCueKey.NormalizedMagnitude;
	CueParameters.MatchedTagName       = GameplayCueKey.Cue.GameplayCueTag;
	CueParameters.OriginalTag          = GameplayCueKey.Cue.GameplayCueTag;
	CueParameters.Location             = GameplayCueKey.Location;
	CueParameters.Normal               = GameplayCueKey.Normal;
	CueParameters.PhysicalMaterial     = GameplayCueKey.PhysicalMaterial;
	CueParameters.GameplayEffectLevel  = GameplayCueKey.GameplayEffectLevel;
	CueParameters.AbilityLevel         = GameplayCueKey.AbilityLevel;

	if (GameplayCueKey.Instigator.IsValid())
	{
		for (TWeakObjectPtr<> WeakObject : GameplayCueKey.Instigator.ResolveBoundObjects(Params.SequenceID, SharedPlaybackState))
		{
			if (AActor* Actor = ActorFromResolvedObject(WeakObject.Get()))
			{
				CueParameters.Instigator = Actor;
				break;
			}
		}
	}

	if (GameplayCueKey.EffectCauser.IsValid())
	{
		for (TWeakObjectPtr<> WeakObject : GameplayCueKey.EffectCauser.ResolveBoundObjects(Params.SequenceID, SharedPlaybackState))
		{
			if (AActor* Actor = ActorFromResolvedObject(WeakObject.Get()))
			{
				CueParameters.EffectCauser = Actor;
				break;
			}
		}
	}


	if (Params.ObjectBindingID.IsValid())
	{
		TArrayView<TWeakObjectPtr<>> BoundObjects = SharedPlaybackState->FindBoundObjects(Params.ObjectBindingID, Params.SequenceID);
		for (TWeakObjectPtr<> WeakObject : BoundObjects)
		{
			UObject* Object = WeakObject.Get();
			if (!Object)
			{
				continue;
			}

			if (!GameplayCueKey.bAttachToBinding)
			{
				CueParameters.TargetAttachComponent = nullptr;
			}

			AActor* Actor = Cast<AActor>(Object);
			if (!Actor)
			{
				if (UActorComponent* ActorComponent = Cast<UActorComponent>(Object))
				{
					Actor = ActorComponent->GetOwner();

					USceneComponent* SceneComponent = Cast<USceneComponent>(Object);
					if (SceneComponent && GameplayCueKey.bAttachToBinding)
					{
						CueParameters.TargetAttachComponent = SceneComponent;
					}
				}
			}
			else if (GameplayCueKey.bAttachToBinding)
			{
				CueParameters.TargetAttachComponent = Actor->GetRootComponent();
			}

			if (!Actor)
			{
				continue;
			}

			if (GameplayCueKey.bAttachToBinding)
			{
				if (CueParameters.TargetAttachComponent.Get() != nullptr)
				{
					FTransform Transform = FTransform::Identity;

					if (GameplayCueKey.AttachSocketName != NAME_None)
					{
						Transform = CueParameters.TargetAttachComponent->GetSocketTransform(GameplayCueKey.AttachSocketName, RTS_World);
					}
					else
					{
						Transform = CueParameters.TargetAttachComponent->GetComponentTransform();
					}

					CueParameters.Location = Transform.TransformPositionNoScale(GameplayCueKey.Location);
					CueParameters.Normal   = Transform.TransformVectorNoScale(GameplayCueKey.Normal);
				}
				else
				{
					CueParameters.Location = GameplayCueKey.Location;
					CueParameters.Normal   = GameplayCueKey.Normal;
				}
			}

			Handler(Actor, GameplayCueKey.Cue.GameplayCueTag, CueParameters, Event);
			if (Event == EGameplayCueEvent::OnActive)
			{
				Handler(Actor, GameplayCueKey.Cue.GameplayCueTag, CueParameters, EGameplayCueEvent::WhileActive);
			}
		}
	}
	else
	{
		Handler(nullptr, GameplayCueKey.Cue.GameplayCueTag, CueParameters, Event);
		if (Event == EGameplayCueEvent::OnActive)
		{
			Handler(nullptr, GameplayCueKey.Cue.GameplayCueTag, CueParameters, EGameplayCueEvent::WhileActive);
		}
	}
}

} // namespace MovieScene
} // namespace UE

void FMovieSceneGameplayCueChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneGameplayCueChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneGameplayCueChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneGameplayCueChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneGameplayCueChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneGameplayCueChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

void FMovieSceneGameplayCueChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}

TRange<FFrameNumber> FMovieSceneGameplayCueChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneGameplayCueChannel::GetNumKeys() const
{
	return KeyTimes.Num();
}

void FMovieSceneGameplayCueChannel::Reset()
{
	KeyTimes.Reset();
	KeyValues.Reset();
	KeyHandles.Reset();
}

void FMovieSceneGameplayCueChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}



UMovieSceneGameplayCueTriggerSection::UMovieSceneGameplayCueTriggerSection(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	SetRange(TRange<FFrameNumber>::All());

	bRequiresTriggerHooks = true;
}

EMovieSceneChannelProxyType UMovieSceneGameplayCueTriggerSection::CacheChannelProxy()
{
#if WITH_EDITOR

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(Channel, FMovieSceneChannelMetaData());

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(Channel);

#endif
	return EMovieSceneChannelProxyType::Static;
}

void UMovieSceneGameplayCueTriggerSection::Trigger(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	TMovieSceneChannelData<const FMovieSceneGameplayCueKey> ChannelData = Channel.GetData();
	if (!ensureMsgf(ChannelData.GetValues().IsValidIndex(Params.TriggerIndex), TEXT("Invalid trigger index specified: %d (Num triggers in channel = %d)"), Params.TriggerIndex, ChannelData.GetValues().Num()))
	{
		return;
	}

	if (Params.Context.GetStatus() == EMovieScenePlayerStatus::Playing && Params.Context.GetDirection() == EPlayDirection::Forwards && !Params.Context.IsSilent())
	{
		UE::MovieScene::ExecuteGameplayCueEvent(SharedPlaybackState, Params, ChannelData.GetValues()[Params.TriggerIndex], EGameplayCueEvent::Executed);
	}
}


UMovieSceneGameplayCueSection::UMovieSceneGameplayCueSection(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	bRequiresRangedHook = true;
}

void UMovieSceneGameplayCueSection::Begin(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	if (Params.Context.GetStatus() == EMovieScenePlayerStatus::Playing && Params.Context.GetDirection() == EPlayDirection::Forwards && !Params.Context.IsSilent())
	{
		UE::MovieScene::ExecuteGameplayCueEvent(SharedPlaybackState, Params, Cue, EGameplayCueEvent::OnActive);
	}
}

void UMovieSceneGameplayCueSection::Update(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	if (Params.Context.GetStatus() == EMovieScenePlayerStatus::Playing && Params.Context.GetDirection() == EPlayDirection::Forwards && !Params.Context.IsSilent())
	{
		UE::MovieScene::ExecuteGameplayCueEvent(SharedPlaybackState, Params, Cue, EGameplayCueEvent::WhileActive);
	}
}

void UMovieSceneGameplayCueSection::End(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	if (Params.Context.GetStatus() == EMovieScenePlayerStatus::Playing && Params.Context.GetDirection() == EPlayDirection::Forwards && !Params.Context.IsSilent())
	{
		UE::MovieScene::ExecuteGameplayCueEvent(SharedPlaybackState, Params, Cue, EGameplayCueEvent::Removed);
	}
}
