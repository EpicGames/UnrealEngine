// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneGameplayCueTrack.h"
#include "Sequencer/MovieSceneGameplayCueSections.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneGameplayCueTrack)


FMovieSceneGameplayCueEvent UMovieSceneGameplayCueTrack::OnHandleCueEvent;

void UMovieSceneGameplayCueTrack::SetSequencerTrackHandler(FMovieSceneGameplayCueEvent InGameplayCueTrackHandler)
{
	OnHandleCueEvent = InGameplayCueTrackHandler;
}

void UMovieSceneGameplayCueTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}

bool UMovieSceneGameplayCueTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneGameplayCueSection::StaticClass();
}

UMovieSceneSection* UMovieSceneGameplayCueTrack::CreateNewSection()
{
	return NewObject<UMovieSceneGameplayCueSection>(this, NAME_None, RF_Transactional);
}

const TArray<UMovieSceneSection*>& UMovieSceneGameplayCueTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneGameplayCueTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

bool UMovieSceneGameplayCueTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

void UMovieSceneGameplayCueTrack::RemoveAllAnimationData()
{
	// ?
}

void UMovieSceneGameplayCueTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneGameplayCueTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex, 1);
}


#if WITH_EDITORONLY_DATA

FText UMovieSceneGameplayCueTrack::GetDefaultDisplayName() const
{
	return NSLOCTEXT("GameplayAbilities", "DefaultSequencerTrackName", "Gameplay Cues");
}

#endif

