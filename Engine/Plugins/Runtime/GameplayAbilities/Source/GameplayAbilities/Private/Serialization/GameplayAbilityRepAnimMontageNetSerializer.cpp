// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/GameplayAbilityRepAnimMontageNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityRepAnimMontageNetSerializer)


#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimMontage.h"
#include "Abilities/GameplayAbilityTypes.h"
#include "Abilities/GameplayAbilityRepAnimMontage.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Net/Core/Trace/NetTrace.h"

namespace UE::Net
{

// Custom NetSerializer required for GameplayAbilityRepAnimMontage
struct FGameplayAbilityRepAnimMontageNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing
	static constexpr bool bHasConnectionSpecificSerialization = true; // One of our members requires connection specific serialization
	static constexpr bool bHasCustomNetReference = true; // We have object references that are not directly accessible
	static constexpr bool bUseSerializerIsEqual = true; // Since FGameplayAbilityRepAnimMontageNetSerializer conditionally replicates position and SectionId we need to use the IsEqual function provided by the NetSerializer when comparing the property
	static constexpr bool bHasDynamicState = true;

	// Types
	enum EReplicationFlags : uint8
	{
		ReplicatePosition = 1U,
		IsAnimMontage = ReplicatePosition << 1U,
	};

	struct FQuantizedType
	{
		alignas(16) uint8 GameplayAbilityRepAnimMontage[84];

		float Position;
		float BlendOutTime;

		uint8 ReplicationFlags;
		uint8 SectionIdToPlay;
	};

	typedef FGameplayAbilityRepAnimMontage SourceType;
	typedef FQuantizedType QuantizedType;
	typedef FGameplayAbilityRepAnimMontageNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
		virtual void OnPostFreezeNetSerializerRegistry() override;
	};

	static FGameplayAbilityRepAnimMontageNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	static FStructNetSerializerConfig StructNetSerializerConfigForBase;
	static const FNetSerializer* StructNetSerializer;
};

UE_NET_IMPLEMENT_SERIALIZER(FGameplayAbilityRepAnimMontageNetSerializer);

const FGameplayAbilityRepAnimMontageNetSerializer::ConfigType FGameplayAbilityRepAnimMontageNetSerializer::DefaultConfig;
FGameplayAbilityRepAnimMontageNetSerializer::FNetSerializerRegistryDelegates FGameplayAbilityRepAnimMontageNetSerializer::NetSerializerRegistryDelegates;
FStructNetSerializerConfig FGameplayAbilityRepAnimMontageNetSerializer::StructNetSerializerConfigForBase;
const FNetSerializer* FGameplayAbilityRepAnimMontageNetSerializer::StructNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);

void FGameplayAbilityRepAnimMontageNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	const uint8 ReplicationFlags = Value.ReplicationFlags;

	// We do not really need to serialize this as it is part of the basestate, but it makes it easier to access from quantized state and as long as it is a single bit it does not matter much
	Writer->WriteBits(ReplicationFlags, 2U);

	// Forward to normal StructNetSerializer
	FNetSerializeArgs NetSerializeArgs = {};
	NetSerializeArgs.NetSerializerConfig = &StructNetSerializerConfigForBase;
	NetSerializeArgs.Source = NetSerializerValuePointer(&Value.GameplayAbilityRepAnimMontage);
	StructNetSerializer->Serialize(Context, NetSerializeArgs);

	// Conditionally serialize Position and SectionIdToPlay
	if (ReplicationFlags & EReplicationFlags::ReplicatePosition)
	{
		UE_NET_TRACE_SCOPE(Position, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer& FloatSerializer = UE_NET_GET_SERIALIZER(FFloatNetSerializer);

		FNetSerializeArgs FloatNetSerializeArgs = {};
		FloatNetSerializeArgs.Version = FloatSerializer.Version;
		FloatNetSerializeArgs.Source = NetSerializerValuePointer(&Value.Position);
		FloatNetSerializeArgs.NetSerializerConfig = NetSerializerConfigParam(FloatSerializer.DefaultConfig);
		FloatSerializer.Serialize(Context, FloatNetSerializeArgs);
	}
	else
	{
		UE_NET_TRACE_SCOPE(SectionIdToPlay, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		if (ReplicationFlags & EReplicationFlags::IsAnimMontage)
		{
			Writer->WriteBits(Value.SectionIdToPlay, 7U);
		}
	}

	if ((ReplicationFlags & EReplicationFlags::IsAnimMontage) == 0U)
	{
		const FNetSerializer& FloatSerializer = UE_NET_GET_SERIALIZER(FFloatNetSerializer);

		FNetSerializeArgs FloatNetSerializeArgs = {};
		FloatNetSerializeArgs.Version = FloatSerializer.Version;
		FloatNetSerializeArgs.Source = NetSerializerValuePointer(&Value.BlendOutTime);
		FloatNetSerializeArgs.NetSerializerConfig = NetSerializerConfigParam(FloatSerializer.DefaultConfig);
		FloatSerializer.Serialize(Context, FloatNetSerializeArgs);
	}
}

void FGameplayAbilityRepAnimMontageNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const uint8 ReplicationFlags = Reader->ReadBits(2U);
	Target.ReplicationFlags = ReplicationFlags;

	// Forward to normal StructNetSerializer
	FNetDeserializeArgs NetDeserializeArgs = {};
	NetDeserializeArgs.NetSerializerConfig = &StructNetSerializerConfigForBase;
	NetDeserializeArgs.Target = NetSerializerValuePointer(&Target.GameplayAbilityRepAnimMontage);
	StructNetSerializer->Deserialize(Context, NetDeserializeArgs);

	// Conditionally deserialize Position and SectionIdToPlay
	if (ReplicationFlags & EReplicationFlags::ReplicatePosition)
	{
		UE_NET_TRACE_SCOPE(Position, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

		const FNetSerializer& FloatSerializer = UE_NET_GET_SERIALIZER(FFloatNetSerializer);

		FNetDeserializeArgs FloatNetDeserializeArgs = {};
		FloatNetDeserializeArgs.Version = FloatSerializer.Version;
		FloatNetDeserializeArgs.Target = NetSerializerValuePointer(&Target.Position);
		FloatNetDeserializeArgs.NetSerializerConfig = NetSerializerConfigParam(FloatSerializer.DefaultConfig);
		FloatSerializer.Deserialize(Context, FloatNetDeserializeArgs);
		Target.SectionIdToPlay = 0;
	}
	else
	{
		UE_NET_TRACE_SCOPE(SectionIdToPlay, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

		Target.Position = 0.f;
		if (ReplicationFlags & EReplicationFlags::IsAnimMontage)
		{
			Target.SectionIdToPlay = Reader->ReadBits(7U);
		}
		else
		{
			Target.SectionIdToPlay = 0U;
		}
	}

	if ((ReplicationFlags & EReplicationFlags::IsAnimMontage) == 0U)
	{
		const FNetSerializer& FloatSerializer = UE_NET_GET_SERIALIZER(FFloatNetSerializer);

		FNetDeserializeArgs FloatNetDeserializeArgs = {};
		FloatNetDeserializeArgs.Version = FloatSerializer.Version;
		FloatNetDeserializeArgs.Target = NetSerializerValuePointer(&Target.BlendOutTime);
		FloatNetDeserializeArgs.NetSerializerConfig = NetSerializerConfigParam(FloatSerializer.DefaultConfig);
		FloatSerializer.Deserialize(Context, FloatNetDeserializeArgs);		
	}
}

void FGameplayAbilityRepAnimMontageNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	const QuantizedType& PrevValue = *reinterpret_cast<QuantizedType*>(Args.Prev);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// We do waste a bit here that also is included in the base state
	const uint32 ReplicationFlags = Value.ReplicationFlags;

	// We do not really need to serialize this as it is part of the basestate, but it makes it easier to access and as long as it is a single bit it does not matter much
	Writer->WriteBits(ReplicationFlags, 2U);

	// Forward to normal StructNetSerializer
	FNetSerializeDeltaArgs NetSerializeDeltaArgs = {};
	NetSerializeDeltaArgs.Version = 0;
	NetSerializeDeltaArgs.NetSerializerConfig = &StructNetSerializerConfigForBase;
	NetSerializeDeltaArgs.Source = NetSerializerValuePointer(&Value.GameplayAbilityRepAnimMontage);
	NetSerializeDeltaArgs.Prev = NetSerializerValuePointer(&PrevValue.GameplayAbilityRepAnimMontage);

	StructNetSerializer->SerializeDelta(Context, NetSerializeDeltaArgs);

	// Conditionally serialize Position and SectionIdToPlay
	if (ReplicationFlags & EReplicationFlags::ReplicatePosition)
	{
		UE_NET_TRACE_SCOPE(Position, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

		const FNetSerializer& FloatSerializer = UE_NET_GET_SERIALIZER(FFloatNetSerializer);

		FNetSerializeDeltaArgs FloatNetSerializeDeltaArgs = {};
		FloatNetSerializeDeltaArgs.Version = FloatSerializer.Version;
		FloatNetSerializeDeltaArgs.Source = NetSerializerValuePointer(&Value.Position);
		FloatNetSerializeDeltaArgs.Prev = NetSerializerValuePointer(&PrevValue.Position);
		FloatNetSerializeDeltaArgs.NetSerializerConfig = NetSerializerConfigParam(FloatSerializer.DefaultConfig);
		FloatSerializer.SerializeDelta(Context, FloatNetSerializeDeltaArgs);
	}
	else
	{
		UE_NET_TRACE_SCOPE(SectionIdToPlay, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		if (ReplicationFlags & EReplicationFlags::IsAnimMontage)
		{
			if (Writer->WriteBool(Value.SectionIdToPlay != PrevValue.SectionIdToPlay))
			{
				Writer->WriteBits(Value.SectionIdToPlay, 7U);
			}
		}
	}

	if ((ReplicationFlags & EReplicationFlags::IsAnimMontage) == 0U)
	{
		const FNetSerializer& FloatSerializer = UE_NET_GET_SERIALIZER(FFloatNetSerializer);

		FNetSerializeDeltaArgs FloatNetSerializeDeltaArgs = {};
		FloatNetSerializeDeltaArgs.Version = FloatSerializer.Version;
		FloatNetSerializeDeltaArgs.Source = NetSerializerValuePointer(&Value.BlendOutTime);
		FloatNetSerializeDeltaArgs.Prev = NetSerializerValuePointer(&PrevValue.BlendOutTime);
		FloatNetSerializeDeltaArgs.NetSerializerConfig = NetSerializerConfigParam(FloatSerializer.DefaultConfig);
		FloatSerializer.SerializeDelta(Context, FloatNetSerializeDeltaArgs);
	}
}

void FGameplayAbilityRepAnimMontageNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	QuantizedType& Prev = *reinterpret_cast<QuantizedType*>(Args.Prev);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const uint32 ReplicationFlags = Reader->ReadBits(2U);
	Target.ReplicationFlags = ReplicationFlags;

	FNetDeserializeDeltaArgs NetDeserializeDeltaArgs = {};
	NetDeserializeDeltaArgs.Version = 0;
	NetDeserializeDeltaArgs.NetSerializerConfig = &StructNetSerializerConfigForBase;
	NetDeserializeDeltaArgs.Target = NetSerializerValuePointer(&Target.GameplayAbilityRepAnimMontage);
	NetDeserializeDeltaArgs.Prev = NetSerializerValuePointer(&Prev.GameplayAbilityRepAnimMontage);
	StructNetSerializer->DeserializeDelta(Context, NetDeserializeDeltaArgs);

	// Conditionally deserialize Position and SectionIdToPlay
	if (ReplicationFlags & EReplicationFlags::ReplicatePosition)
	{
		UE_NET_TRACE_SCOPE(Position, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

		const FNetSerializer& FloatSerializer = UE_NET_GET_SERIALIZER(FFloatNetSerializer);

		FNetDeserializeDeltaArgs FloatNetDeserializeDeltaArgs = {};
		FloatNetDeserializeDeltaArgs.Version = FloatSerializer.Version;
		FloatNetDeserializeDeltaArgs.Target = NetSerializerValuePointer(&Target.Position);
		FloatNetDeserializeDeltaArgs.Prev = NetSerializerValuePointer(&Prev.Position);
		FloatNetDeserializeDeltaArgs.NetSerializerConfig = NetSerializerConfigParam(FloatSerializer.DefaultConfig);
		FloatSerializer.DeserializeDelta(Context, FloatNetDeserializeDeltaArgs);
		Target.SectionIdToPlay = 0;
	}
	else
	{
		UE_NET_TRACE_SCOPE(SectionIdToPlay, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		Target.Position = 0.f;

		if (ReplicationFlags & EReplicationFlags::IsAnimMontage)
		{
			if (Reader->ReadBool())
			{
				Target.SectionIdToPlay = Reader->ReadBits(7U);
			}
			else
			{
				Target.SectionIdToPlay = Prev.SectionIdToPlay;
			}
		}
		else
		{
			Target.SectionIdToPlay = 0;
		}
	}

	if ((ReplicationFlags & EReplicationFlags::IsAnimMontage) == 0U)
	{
		const FNetSerializer& FloatSerializer = UE_NET_GET_SERIALIZER(FFloatNetSerializer);

		FNetDeserializeDeltaArgs FloatNetDeserializeDeltaArgs = {};
		FloatNetDeserializeDeltaArgs.Version = FloatSerializer.Version;
		FloatNetDeserializeDeltaArgs.Target = NetSerializerValuePointer(&Target.BlendOutTime);
		FloatNetDeserializeDeltaArgs.Prev = NetSerializerValuePointer(&Prev.BlendOutTime);
		FloatNetDeserializeDeltaArgs.NetSerializerConfig = NetSerializerConfigParam(FloatSerializer.DefaultConfig);
		FloatSerializer.DeserializeDelta(Context, FloatNetDeserializeDeltaArgs);
	}
}

void FGameplayAbilityRepAnimMontageNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	
	uint32 ReplicationFlags = 0;

	ReplicationFlags |= SourceValue.bRepPosition == 1 ? EReplicationFlags::ReplicatePosition : 0U;
	ReplicationFlags |= (SourceValue.Animation && SourceValue.Animation->IsA<UAnimMontage>()) ? EReplicationFlags::IsAnimMontage : 0U;

	TargetValue.ReplicationFlags = ReplicationFlags;

	// Forward to normal StructNetSerializer
	FNetQuantizeArgs QuantizeArgs = {};
	QuantizeArgs.NetSerializerConfig = &StructNetSerializerConfigForBase;
	QuantizeArgs.Source = Args.Source;
	QuantizeArgs.Target = NetSerializerValuePointer(&TargetValue.GameplayAbilityRepAnimMontage);
	StructNetSerializer->Quantize(Context, QuantizeArgs);

	if (SourceValue.bRepPosition)
	{
		TargetValue.Position = SourceValue.Position;
		TargetValue.SectionIdToPlay = 0;
	}
	else
	{
		TargetValue.Position = 0.f;
		if (ReplicationFlags & IsAnimMontage)
		{

			TargetValue.SectionIdToPlay = SourceValue.SectionIdToPlay;
		}
		else
		{
			// Always set to zero if not included to be deterministic
			TargetValue.SectionIdToPlay	= 0;
		}
	}

	if ((ReplicationFlags & IsAnimMontage) == 0U)
	{
		TargetValue.BlendOutTime = SourceValue.BlendOutTime;
	}
	else
	{
		// Always set to zero if not included to be deterministic
		TargetValue.BlendOutTime = 0.f;
	}
}

void FGameplayAbilityRepAnimMontageNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);
	
	const uint32 ReplicationFlags = SourceValue.ReplicationFlags;

	// Forward to normal StructNetSerializer
	FNetDequantizeArgs DequantizeArgs = {};
	DequantizeArgs.NetSerializerConfig = &StructNetSerializerConfigForBase;
	DequantizeArgs.Source = NetSerializerValuePointer(&SourceValue.GameplayAbilityRepAnimMontage);
	DequantizeArgs.Target = Args.Target;
	StructNetSerializer->Dequantize(Context, DequantizeArgs);
	
	if (ReplicationFlags & EReplicationFlags::ReplicatePosition)
	{
		TargetValue.Position = SourceValue.Position;
		TargetValue.SectionIdToPlay = 0;
		TargetValue.SkipPositionCorrection = 0;
	}
	else
	{
		TargetValue.Position = 0.f;
		if (ReplicationFlags & IsAnimMontage)
		{
			TargetValue.SectionIdToPlay = SourceValue.SectionIdToPlay;
		}
		else
		{
			TargetValue.SectionIdToPlay = 0;
		}
		TargetValue.SkipPositionCorrection = 1;
	}

	if (ReplicationFlags & IsAnimMontage)
	{
		TargetValue.BlendOutTime = 0.f;
	}
	else
	{
		TargetValue.BlendOutTime = SourceValue.BlendOutTime;
	}
}

bool FGameplayAbilityRepAnimMontageNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		if (Value0.ReplicationFlags != Value1.ReplicationFlags)
		{
			return false;
		}

		// Forward to normal StructNetSerializer
		FNetIsEqualArgs IsEqualArgs = Args;
		IsEqualArgs.NetSerializerConfig = &StructNetSerializerConfigForBase;
		IsEqualArgs.Source0 = NetSerializerValuePointer(&Value0.GameplayAbilityRepAnimMontage);
		IsEqualArgs.Source1 = NetSerializerValuePointer(&Value1.GameplayAbilityRepAnimMontage);

		if (!StructNetSerializer->IsEqual(Context, IsEqualArgs))
		{
			return false;
		}

		if (Value0.Position != Value1.Position || Value0.SectionIdToPlay != Value1.SectionIdToPlay || Value0.BlendOutTime != Value1.BlendOutTime)
		{
			return false;
		}
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		// Forward to normal StructNetSerializer
		FNetIsEqualArgs IsEqualArgs = Args;
		IsEqualArgs.NetSerializerConfig = &StructNetSerializerConfigForBase;
		IsEqualArgs.Source0 = Args.Source0;
		IsEqualArgs.Source1 = Args.Source1;

		if (!StructNetSerializer->IsEqual(Context, IsEqualArgs))
		{
			return false;
		}

		// As bRepPosition is included in the base state we can assume that it is the same for Value0 and Value1
		if (SourceValue0.bRepPosition && SourceValue0.Position != SourceValue1.Position)
		{
			return false;
		}
		else if (SourceValue0.SectionIdToPlay != SourceValue1.SectionIdToPlay)
		{
			return false;
		}

		if (SourceValue0.BlendOutTime != SourceValue1.BlendOutTime)
		{
			return false;
		}
	}

	return true;
}

bool FGameplayAbilityRepAnimMontageNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	FNetValidateArgs ValidateArgs = {};
	ValidateArgs.NetSerializerConfig = &StructNetSerializerConfigForBase;
	ValidateArgs.Source = NetSerializerValuePointer(Args.Source);
	if (!StructNetSerializer->Validate(Context, ValidateArgs))
	{
		return false;
	}

	return true;
}

void FGameplayAbilityRepAnimMontageNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	
	FNetCollectReferencesArgs CollectReferencesArgs = Args;
	CollectReferencesArgs.NetSerializerConfig = &StructNetSerializerConfigForBase;
	CollectReferencesArgs.Source = NetSerializerValuePointer(&Value.GameplayAbilityRepAnimMontage);
	StructNetSerializer->CollectNetReferences(Context, CollectReferencesArgs);
}

void FGameplayAbilityRepAnimMontageNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	
	FNetCloneDynamicStateArgs CloneDynamicStateArgs = {};
	CloneDynamicStateArgs.NetSerializerConfig = &StructNetSerializerConfigForBase;
	CloneDynamicStateArgs.Source = NetSerializerValuePointer(&SourceValue.GameplayAbilityRepAnimMontage);
	CloneDynamicStateArgs.Target = NetSerializerValuePointer(&TargetValue.GameplayAbilityRepAnimMontage);
	StructNetSerializer->CloneDynamicState(Context, CloneDynamicStateArgs);
}

void FGameplayAbilityRepAnimMontageNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	
	FNetFreeDynamicStateArgs FreeDynamicStateArgs = {};
	FreeDynamicStateArgs.NetSerializerConfig = &StructNetSerializerConfigForBase;
	FreeDynamicStateArgs.Source = NetSerializerValuePointer(&SourceValue.GameplayAbilityRepAnimMontage);
	StructNetSerializer->FreeDynamicState(Context, FreeDynamicStateArgs);
}

static const FName PropertyNetSerializerRegistry_NAME_GameplayAbilityRepAnimMontage("GameplayAbilityRepAnimMontage");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayAbilityRepAnimMontage, FGameplayAbilityRepAnimMontageNetSerializer);

FGameplayAbilityRepAnimMontageNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayAbilityRepAnimMontage);
}

void FGameplayAbilityRepAnimMontageNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayAbilityRepAnimMontage);
}

void FGameplayAbilityRepAnimMontageNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	/*
#if PLATFORM_WINDOWS && UE_BUILD_DEVELOPMENT && !UE_EDITOR
	constexpr SIZE_T ExpectedSizeOfFGameplayAbilityRepAnimMontage = 64;
	constexpr SIZE_T ExpectedAlignOfFGameplayAbilityRepAnimMontage = 8;

	// Do our best to detect changes to FGameplayAbilityRepAnimMontage
	// If this assert triggers, this implementation must be verified against FGameplayAbilityRepAnimMontage::NetSerializee before updating the size and alignment
	static_assert(sizeof(FGameplayAbilityRepAnimMontage) == ExpectedSizeOfFGameplayAbilityRepAnimMontage && alignof(FGameplayAbilityRepAnimMontage) == ExpectedAlignOfFGameplayAbilityRepAnimMontage, "FGameplayAbilityRepAnimMontage layout has changed. Might need to update FGameplayAbilityRepAnimMontageNetSerializer to include new data or update the size.");
#endif
	*/

	// Use helper to avoid getting hold of the FGameplayAbilityRepAnimMontageNetSerializer that we are setting up and validating.
	const UStruct* BaseStruct = FGameplayAbilityRepAnimMontage::StaticStruct();
	FReplicationStateDescriptorBuilder::FParameters Params;
	Params.SkipCheckForCustomNetSerializerForStruct = 1U;
	StructNetSerializerConfigForBase.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(BaseStruct, Params);

	const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfigForBase.StateDescriptor.GetReference();
	check(Descriptor != nullptr);

	// Verify traits
	ValidateForwardingNetSerializerTraits(&UE_NET_GET_SERIALIZER(FGameplayAbilityRepAnimMontageNetSerializer), Descriptor->Traits);
	
	// Validate our assumptions regarding quantized state size and alignment.
	constexpr SIZE_T OffsetOfGameplayAbilityRepAnimMontage = offsetof(FQuantizedType, GameplayAbilityRepAnimMontage);
	if ((sizeof(FQuantizedType::GameplayAbilityRepAnimMontage) < Descriptor->InternalSize) || (((OffsetOfGameplayAbilityRepAnimMontage/Descriptor->InternalAlignment)*Descriptor->InternalAlignment) != OffsetOfGameplayAbilityRepAnimMontage))
	{
		LowLevelFatalError(TEXT("FQuantizedType::GameplayAbilityRepAnimMontage has size %u but requires size %u and alignment %u."), uint32(sizeof(FQuantizedType::GameplayAbilityRepAnimMontage)), uint32(Descriptor->InternalSize), uint32(Descriptor->InternalAlignment));
	}
}

}


