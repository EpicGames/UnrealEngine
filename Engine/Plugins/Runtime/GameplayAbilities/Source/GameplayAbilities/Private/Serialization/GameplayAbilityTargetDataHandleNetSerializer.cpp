// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/GameplayAbilityTargetDataHandleNetSerializer.h"


#include "Abilities/GameplayAbilityTargetTypes.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/PolymorphicNetSerializerImpl.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"

namespace UE::Net
{

// Since the GameplayAbilityTargetDataHandle carries additional information in addition to the "polymorphic"-struct data  we wrap the TPolymorphicArrayStructNetSerializer
// and manually serialize the extra data and then forward the polymorphic part.
struct FGameplayAbilityTargetDataHandleNetSerializer
{
private:
	static TArrayView<TSharedPtr<FGameplayAbilityTargetData>> GetArray(FGameplayAbilityTargetDataHandle& ArrayContainer);
	static void SetArrayNum(FGameplayAbilityTargetDataHandle& ArrayContainer, SIZE_T Num);

public:	
	typedef FGameplayAbilityTargetDataHandle SourceType;
	typedef FGameplayAbilityTargetData SourceArrayItemType;
	typedef TPolymorphicArrayStructNetSerializerImpl<SourceType, SourceArrayItemType, GetArray, SetArrayNum> InternalSerializerType;
	typedef InternalSerializerType::ConfigType ConfigType;

	struct FQuantizedType
	{
		InternalSerializerType::QuantizedType InternalQuantizedType;
		uint8 UniqueId;
	};

	typedef FQuantizedType QuantizedType;

	// Traits
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing
	static constexpr bool bHasCustomNetReference = true; // We must support this as we do not know the type

	static const uint32 Version = 0;	
	static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

	static void InitTypeCache();

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		FNetSerializerRegistryDelegates();
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
		virtual void OnPostFreezeNetSerializerRegistry() override;
		virtual void OnLoadedModulesUpdated() override;
	};

	static FGameplayAbilityTargetDataHandleNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	static bool bIsPostFreezeCalled;
};

UE_NET_IMPLEMENT_SERIALIZER(FGameplayAbilityTargetDataHandleNetSerializer);

const FGameplayAbilityTargetDataHandleNetSerializer::ConfigType FGameplayAbilityTargetDataHandleNetSerializer::DefaultConfig;
FGameplayAbilityTargetDataHandleNetSerializer::FNetSerializerRegistryDelegates FGameplayAbilityTargetDataHandleNetSerializer::NetSerializerRegistryDelegates;
bool FGameplayAbilityTargetDataHandleNetSerializer::bIsPostFreezeCalled = false;

void FGameplayAbilityTargetDataHandleNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();
	const FQuantizedType& SourceValue = *reinterpret_cast<const FQuantizedType*>(Args.Source);

	Writer.WriteBits(SourceValue.UniqueId, 8U);

	// Forward
	FNetSerializeArgs InternalArgs = Args;
	InternalArgs.Source = NetSerializerValuePointer(&SourceValue.InternalQuantizedType);
	InternalSerializerType::Serialize(Context, InternalArgs);
}

void FGameplayAbilityTargetDataHandleNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();
	FQuantizedType& TargetValue = *reinterpret_cast<FQuantizedType*>(Args.Target);

	TargetValue.UniqueId = Reader.ReadBits(8U);

	// Forward
	FNetDeserializeArgs InternalArgs = Args;
	InternalArgs.Target = NetSerializerValuePointer(&TargetValue.InternalQuantizedType);
	InternalSerializerType::Deserialize(Context, InternalArgs);
}

void FGameplayAbilityTargetDataHandleNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	// Not implemented: Just forward to Serialize
	Serialize(Context, Args);
}

void FGameplayAbilityTargetDataHandleNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	// Not implemented: Just forward to Deserialize
	Deserialize(Context, Args);
}

void FGameplayAbilityTargetDataHandleNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const FGameplayAbilityTargetDataHandleNetSerializer::SourceType& SourceValue = *reinterpret_cast<const FGameplayAbilityTargetDataHandleNetSerializer::SourceType*>(Args.Source);
	FQuantizedType& TargetValue = *reinterpret_cast<FQuantizedType*>(Args.Target);

	TargetValue.UniqueId = SourceValue.UniqueId;

	// Forward
	FNetQuantizeArgs InternalArgs = Args;
	InternalArgs.Target = NetSerializerValuePointer(&TargetValue.InternalQuantizedType);
	InternalSerializerType::Quantize(Context, InternalArgs);
}

void FGameplayAbilityTargetDataHandleNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	FGameplayAbilityTargetDataHandleNetSerializer::SourceType& TargetValue = *reinterpret_cast<FGameplayAbilityTargetDataHandleNetSerializer::SourceType*>(Args.Target);
	const FQuantizedType& SourceValue = *reinterpret_cast<const FQuantizedType*>(Args.Source);

	TargetValue.UniqueId = SourceValue.UniqueId;

	// Forward
	FNetDequantizeArgs InternalArgs = Args;
	InternalArgs.Source = NetSerializerValuePointer(&SourceValue.InternalQuantizedType);
	InternalSerializerType::Dequantize(Context, InternalArgs);
}

bool FGameplayAbilityTargetDataHandleNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const FQuantizedType& ValueA = *reinterpret_cast<const FQuantizedType*>(Args.Source0);
		const FQuantizedType& ValueB = *reinterpret_cast<const FQuantizedType*>(Args.Source1);

		if (ValueA.UniqueId != ValueB.UniqueId)
		{
			return false;
		}

		// Forward	
		FNetIsEqualArgs InternalArgs = Args;
		InternalArgs.Source0 = NetSerializerValuePointer(&ValueA.InternalQuantizedType);
		InternalArgs.Source1 = NetSerializerValuePointer(&ValueB.InternalQuantizedType);
		return  InternalSerializerType::IsEqual(Context, InternalArgs);
	}
	else
	{
		const SourceType& ValueA = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& ValueB = *reinterpret_cast<const SourceType*>(Args.Source1);
		return ValueA == ValueB;
	}
}

bool FGameplayAbilityTargetDataHandleNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	return InternalSerializerType::Validate(Context, Args);
}

void FGameplayAbilityTargetDataHandleNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const FQuantizedType& SourceValue = *reinterpret_cast<const FQuantizedType*>(Args.Source);
	FQuantizedType& TargetValue = *reinterpret_cast<FQuantizedType*>(Args.Target);

	// Forward
	FNetCloneDynamicStateArgs InternalArgs = Args;
	InternalArgs.Source = NetSerializerValuePointer(&SourceValue.InternalQuantizedType);
	InternalArgs.Target = NetSerializerValuePointer(&TargetValue.InternalQuantizedType);
	InternalSerializerType::CloneDynamicState(Context, InternalArgs);
}

void FGameplayAbilityTargetDataHandleNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	FQuantizedType& SourceValue = *reinterpret_cast<FQuantizedType*>(Args.Source);

	// Forward
	FNetFreeDynamicStateArgs InternalArgs = Args;
	InternalArgs.Source = NetSerializerValuePointer(&SourceValue.InternalQuantizedType);
	InternalSerializerType::FreeDynamicState(Context, InternalArgs);
}

void FGameplayAbilityTargetDataHandleNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	FQuantizedType& SourceValue = *reinterpret_cast<FQuantizedType*>(Args.Source);

	// Forward
	FNetCollectReferencesArgs InternalArgs = Args;
	InternalArgs.Source = NetSerializerValuePointer(&SourceValue.InternalQuantizedType);
	InternalSerializerType::CollectNetReferences(Context, InternalArgs);
}

void FGameplayAbilityTargetDataHandleNetSerializer::InitTypeCache()
{
	// When post freeze is called we expect all custom serializers to have been registered
	// so that the type cache will get the appropriate serializer when creating the ReplicationStateDescriptor.
	if (bIsPostFreezeCalled)
	{
		InternalSerializerType::InitTypeCache<FGameplayAbilityTargetDataHandleNetSerializer>();
	}
}

TArrayView<TSharedPtr<FGameplayAbilityTargetData>> FGameplayAbilityTargetDataHandleNetSerializer::GetArray(FGameplayAbilityTargetDataHandle& ArrayContainer)
{
	return MakeArrayView(ArrayContainer.Data);
}

void FGameplayAbilityTargetDataHandleNetSerializer::SetArrayNum(FGameplayAbilityTargetDataHandle& ArrayContainer, SIZE_T Num)
{
	ArrayContainer.Data.SetNum(static_cast<SSIZE_T>(Num));
}

static const FName PropertyNetSerializerRegistry_NAME_GameplayAbilityTargetDataHandle(TEXT("GameplayAbilityTargetDataHandle"));
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayAbilityTargetDataHandle, FGameplayAbilityTargetDataHandleNetSerializer);

FGameplayAbilityTargetDataHandleNetSerializer::FNetSerializerRegistryDelegates::FNetSerializerRegistryDelegates()
: UE::Net::FNetSerializerRegistryDelegates(EFlags::ShouldBindLoadedModulesUpdatedDelegate)
{
}

FGameplayAbilityTargetDataHandleNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayAbilityTargetDataHandle);
}

void FGameplayAbilityTargetDataHandleNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayAbilityTargetDataHandle);
}

void FGameplayAbilityTargetDataHandleNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	bIsPostFreezeCalled = true;
}

void FGameplayAbilityTargetDataHandleNetSerializer::FNetSerializerRegistryDelegates::OnLoadedModulesUpdated()
{
	InitGameplayAbilityTargetDataHandleNetSerializerTypeCache();
}

}

// This can be called multiple times due to plugins calling AbilitySystemGlobals.InitTargetDataScriptStructCache().
void InitGameplayAbilityTargetDataHandleNetSerializerTypeCache()
{
	UE::Net::FGameplayAbilityTargetDataHandleNetSerializer::InitTypeCache();
}
