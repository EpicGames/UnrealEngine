// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/GameplayAbilityTargetingLocationInfoNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityTargetingLocationInfoNetSerializer)


#include "Abilities/GameplayAbilityTargetTypes.h"
#include "Iris/Core/BitTwiddling.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Net/Core/NetBitArray.h"
#include "Net/Core/Trace/NetTrace.h"

namespace UE::Net
{

struct FGameplayAbilityTargetingLocationInfoNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing
	static constexpr bool bHasCustomNetReference = true;
	static constexpr bool bHasDynamicState = true;

	// Types
	struct FQuantizedData
	{
		// Keep TargetingLocationInfo first. It simplifies things.
		alignas(16) uint8 TargetingLocationInfo[208];

		uint8 LocationType;
	};

	typedef FGameplayAbilityTargetingLocationInfo SourceType;
	typedef FQuantizedData QuantizedType;
	typedef FGameplayAbilityTargetingLocationInfoNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	static void CopyReplicatedFields(SourceType& Target, const SourceType& Source, EGameplayAbilityTargetingLocationType::Type Type);

	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
		virtual void OnPostFreezeNetSerializerRegistry() override;

		inline static const FName PropertyNetSerializerRegistry_NAME_GameplayAbilityTargetingLocationInfo = FName("GameplayAbilityTargetingLocationInfo");
		UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayAbilityTargetingLocationInfo, FGameplayAbilityTargetingLocationInfoNetSerializer);
	};

	static constexpr uint32 LocationTypeBitCount = 2U;

	static const FNetSerializer* StructNetSerializer;
	inline static uint32 LocationTypeChangemasks[EGameplayAbilityTargetingLocationType::SocketTransform + 1];

	inline static FStructNetSerializerConfig StructNetSerializerConfig;
	inline static FGameplayAbilityTargetingLocationInfoNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};

UE_NET_IMPLEMENT_SERIALIZER(FGameplayAbilityTargetingLocationInfoNetSerializer);

const FNetSerializer* FGameplayAbilityTargetingLocationInfoNetSerializer::StructNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);

void FGameplayAbilityTargetingLocationInfoNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	const uint32 LocationType = Value.LocationType;
	{
		UE_NET_TRACE_SCOPE(LocationType, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		Writer->WriteBits(LocationType, LocationTypeBitCount);
	}

	{
		const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfig.StateDescriptor.GetReference();
		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
		const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;

		uint32 MemberMaskStorage = LocationTypeChangemasks[LocationType];
		const FNetBitArrayView MemberMask(&MemberMaskStorage, sizeof(MemberMaskStorage)*8U, FNetBitArrayView::NoResetNoValidate);
		const uint32 RelevantMemberCount = GetBitsNeeded(MemberMaskStorage);
		for (uint32 MemberIt = 0, MemberEndIt = RelevantMemberCount; MemberIt != MemberEndIt; ++MemberIt)
		{
			if (!MemberMask.GetBit(MemberIt))
			{
				continue;
			}

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
			const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];

			FNetSerializeArgs SerializeArgs;
			SerializeArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			SerializeArgs.Source = reinterpret_cast<NetSerializerValuePointer>(&Value.TargetingLocationInfo) + MemberDescriptor.InternalMemberOffset;
			MemberSerializerDescriptor.Serializer->Serialize(Context, SerializeArgs);
		}
	}
}

void FGameplayAbilityTargetingLocationInfoNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	// Deserialize into a cleared buffer to make sure we don't store stale data in case there was a location type change.
	QuantizedType TempValue = {};

	uint8 LocationType;
	{
		UE_NET_TRACE_SCOPE(LocationType, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		LocationType = Reader->ReadBits(LocationTypeBitCount);
		if (LocationType > EGameplayAbilityTargetingLocationType::SocketTransform)
		{
			Context.SetError(GNetError_InvalidValue);
			return;
		}

		TempValue.LocationType = static_cast<uint8>(LocationType);
	}

	{
		const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfig.StateDescriptor.GetReference();
		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
		const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;

		uint32 MemberMaskStorage = LocationTypeChangemasks[LocationType];
		const FNetBitArrayView MemberMask(&MemberMaskStorage, sizeof(MemberMaskStorage)*8U, FNetBitArrayView::NoResetNoValidate);
		const uint32 RelevantMemberCount = GetBitsNeeded(MemberMaskStorage);
		for (uint32 MemberIt = 0, MemberEndIt = RelevantMemberCount; MemberIt != MemberEndIt; ++MemberIt)
		{
			if (!MemberMask.GetBit(MemberIt))
			{
				continue;
			}

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
			const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];

			FNetDeserializeArgs DeserializeArgs;
			DeserializeArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			DeserializeArgs.Target = reinterpret_cast<NetSerializerValuePointer>(&TempValue.TargetingLocationInfo) + MemberDescriptor.InternalMemberOffset;
			MemberSerializerDescriptor.Serializer->Deserialize(Context, DeserializeArgs);

			if (Context.HasErrorOrOverflow())
			{
				return;
			}
		}
	}

	// Free potential dynamic state in previous value
	{
		FNetFreeDynamicStateArgs FreeArgs;
		FreeArgs.NetSerializerConfig = &StructNetSerializerConfig;
		FreeArgs.Source = NetSerializerValuePointer(&Target);
		StructNetSerializer->FreeDynamicState(Context, FreeArgs);
	}

	Target = TempValue;
}

void FGameplayAbilityTargetingLocationInfoNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	NetSerializeDeltaDefault<Serialize>(Context, Args);
}

void FGameplayAbilityTargetingLocationInfoNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	NetDeserializeDeltaDefault<Deserialize>(Context, Args);
}

void FGameplayAbilityTargetingLocationInfoNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	
	if (SourceValue.LocationType > EGameplayAbilityTargetingLocationType::SocketTransform)
	{
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	// We store the relevant data in the TempValue. Note that do not set the LocationType as that would make the quantized state different on sending and receiving side as we serialize that member separately.
	SourceType TempValue;
	CopyReplicatedFields(TempValue, SourceValue, SourceValue.LocationType);

	TargetValue.LocationType = SourceValue.LocationType;

	// Avoid complex code by quantizing the entire temp value.
	{
		FNetQuantizeArgs QuantizeArgs = Args;
		QuantizeArgs.NetSerializerConfig = &StructNetSerializerConfig;
		QuantizeArgs.Source = NetSerializerValuePointer(&TempValue);
		QuantizeArgs.Target = NetSerializerValuePointer(&TargetValue.TargetingLocationInfo);
		StructNetSerializer->Quantize(Context, QuantizeArgs);
	}
}

void FGameplayAbilityTargetingLocationInfoNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	// Avoid complex code and dequantize the entire struct.
	{
		FNetDequantizeArgs DequantizeArgs = Args;
		DequantizeArgs.NetSerializerConfig = &StructNetSerializerConfig;
		StructNetSerializer->Dequantize(Context, DequantizeArgs);
	}

	// Need to store LocationType last as it would otherwise be overwritten by the struct dequantization.
	{
		const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
		SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);

		TargetValue.LocationType = TEnumAsByte<EGameplayAbilityTargetingLocationType::Type>(SourceValue.LocationType);
	}
}

bool FGameplayAbilityTargetingLocationInfoNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		if (Value0.LocationType != Value1.LocationType)
		{
			return false;
		}

		{
			FNetIsEqualArgs IsEqualArgs = Args;
			IsEqualArgs.NetSerializerConfig = &StructNetSerializerConfig;
			if (!StructNetSerializer->IsEqual(Context, IsEqualArgs))
			{
				return false;
			}
		}
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		if (Value0.LocationType != Value1.LocationType)
		{
			return false;
		}

		switch (Value0.LocationType)
		{
		case EGameplayAbilityTargetingLocationType::LiteralTransform:
		{
			return Value0.LiteralTransform.Equals(Value1.LiteralTransform, 0 /* zero tolerance */);
		}

		case EGameplayAbilityTargetingLocationType::ActorTransform:
		{
			return Value0.SourceActor == Value1.SourceActor;
		}

		case EGameplayAbilityTargetingLocationType::SocketTransform:
		{
			return Value0.SourceSocketName == Value1.SourceSocketName && Value0.SourceComponent == Value1.SourceComponent;
		}

		default:
		{
			break;
		}
		}
	}

	return true;
}

bool FGameplayAbilityTargetingLocationInfoNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);

	if (Source.LocationType > EGameplayAbilityTargetingLocationType::SocketTransform)
	{
		return false;
	}

	{
		SourceType TempValue;	
		CopyReplicatedFields(TempValue, Source, Source.LocationType);

		FNetValidateArgs ValidateArgs = Args;
		ValidateArgs.NetSerializerConfig = &StructNetSerializerConfig;
		ValidateArgs.Source = NetSerializerValuePointer(&TempValue);
		if (!StructNetSerializer->Validate(Context, ValidateArgs))
		{
			return false;
		}
	}

	return true;
}

void FGameplayAbilityTargetingLocationInfoNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	
	if (Value.LocationType != EGameplayAbilityTargetingLocationType::LiteralTransform)
	{
		FNetCollectReferencesArgs CollectReferencesArgs = Args;
		CollectReferencesArgs.NetSerializerConfig = &StructNetSerializerConfig;
		StructNetSerializer->CollectNetReferences(Context, CollectReferencesArgs);
	}
}

void FGameplayAbilityTargetingLocationInfoNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	FNetCloneDynamicStateArgs CloneArgs = Args;
	CloneArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->CloneDynamicState(Context, CloneArgs);
}

void FGameplayAbilityTargetingLocationInfoNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	FNetFreeDynamicStateArgs FreeArgs = Args;
	FreeArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->FreeDynamicState(Context, FreeArgs);
}

void FGameplayAbilityTargetingLocationInfoNetSerializer::CopyReplicatedFields(SourceType& Target, const SourceType& Source, EGameplayAbilityTargetingLocationType::Type Type)
{
	switch (Type)
	{
	case EGameplayAbilityTargetingLocationType::LiteralTransform:
	{
		Target.LiteralTransform = Source.LiteralTransform;
		break;
	}

	case EGameplayAbilityTargetingLocationType::ActorTransform:
	{
		Target.SourceActor = Source.SourceActor;
		break;
	}

	case EGameplayAbilityTargetingLocationType::SocketTransform:
	{
		Target.SourceComponent = Source.SourceComponent;
		Target.SourceSocketName = Source.SourceSocketName;
		break;
	}
	}
}

FGameplayAbilityTargetingLocationInfoNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayAbilityTargetingLocationInfo);
}

void FGameplayAbilityTargetingLocationInfoNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayAbilityTargetingLocationInfo);
}

void FGameplayAbilityTargetingLocationInfoNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	const UEnum* TargetingLocationTypeEnum = StaticEnum<EGameplayAbilityTargetingLocationType::Type>();
	ensureAlwaysMsgf(TargetingLocationTypeEnum != nullptr && TargetingLocationTypeEnum->GetMaxEnumValue() == UE_ARRAY_COUNT(LocationTypeChangemasks), TEXT("%hs"), "EGameplayAbilityTargetingLocationType::Type has added location types that are not supported by FGameplayAbilityTargetingLocationInfoNetSerializer.");

	FReplicationStateDescriptorBuilder::FParameters Params;
	Params.SkipCheckForCustomNetSerializerForStruct = true;
	const UStruct* Struct = FGameplayAbilityTargetingLocationInfo::StaticStruct();
	StructNetSerializerConfig.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(Struct, Params);

	const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfig.StateDescriptor.GetReference();
	ValidateForwardingNetSerializerTraits(&UE_NET_GET_SERIALIZER(FGameplayAbilityTargetingLocationInfoNetSerializer), Descriptor->Traits);

	// Validate our assumptions regarding quantized state size and alignment.
	static_assert(offsetof(FQuantizedData, TargetingLocationInfo) == 0, "");
	if ((sizeof(FQuantizedData::TargetingLocationInfo) < Descriptor->InternalSize) || alignof(FQuantizedData) < Descriptor->InternalAlignment)
	{
		LowLevelFatalError(TEXT("FQuantizedData::TargetingLocationInfo has size %u and alignment %u but requires size %u and alignment %u."), uint32(sizeof(FQuantizedData::TargetingLocationInfo)), uint32(alignof(FQuantizedData)), uint32(Descriptor->InternalSize), uint32(Descriptor->InternalAlignment));
	}

	// Find the properties we're interested in and build change masks for the different location types.
	{
		const ANSICHAR* const PropertyNames[] =
		{
			"SourceActor", "SourceComponent", "SourceSocketName", "LiteralTransform",
		};

		uint32 PropertyToMemberIndex[UE_ARRAY_COUNT(PropertyNames)] = {};
		uint32 FoundPropertyMask = 0;

		// Find all replicated properties of interest.
		const FProperty* const* MemberProperties = Descriptor->MemberProperties;
		for (SIZE_T PropertyIndex = 0; PropertyIndex != UE_ARRAY_COUNT(PropertyNames); ++PropertyIndex)
		{
			for (const FProperty* const& MemberProperty : MakeArrayView(MemberProperties, Descriptor->MemberCount))
			{
				const SIZE_T MemberIndex = &MemberProperty - MemberProperties;
				if (MemberProperty->GetFName() == PropertyNames[PropertyIndex])
				{
					if (ensureAlwaysMsgf(MemberIndex < 32U, TEXT("Property %hs has a member index %zu > 31U. FGameplayAbilityTargetingLocationInfoNetSerializer needs to be adjusted."), PropertyNames[PropertyIndex], MemberIndex))
					{
						FoundPropertyMask |= 1U << PropertyIndex;
						PropertyToMemberIndex[PropertyIndex] = MemberIndex;
					}
					break;
				}
			}
		}

		// The ensure will trigger also if any member index is higher than expected, but that's ok.
		ensureAlwaysMsgf(FoundPropertyMask == ((1U << UE_ARRAY_COUNT(PropertyToMemberIndex)) - 1U), TEXT("%hs"), "Couldn't find expected replicated members in FGameplayAbilityTargetingLocationInfo. It will not be replicated properly.");

		// Do our best effort to build the appropriate change masks for the various location types, disregarding any ensures that have fired.
		// LiteralTransform
		{
			uint32 Changemask = 0;
			Changemask |= 1U << PropertyToMemberIndex[3];
			LocationTypeChangemasks[EGameplayAbilityTargetingLocationType::LiteralTransform] = Changemask;
		}

		// ActorTransform
		{
			uint32 Changemask = 0;
			Changemask |= 1U << PropertyToMemberIndex[0];
			LocationTypeChangemasks[EGameplayAbilityTargetingLocationType::ActorTransform] = Changemask;
		}

		// SocketTransform
		{
			uint32 Changemask = 0;
			Changemask |= 1U << PropertyToMemberIndex[1];
			Changemask |= 1U << PropertyToMemberIndex[2];
			LocationTypeChangemasks[EGameplayAbilityTargetingLocationType::SocketTransform] = Changemask;
		}
	}
}

}

