// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/GameplayEffectContextHandleNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayEffectContextHandleNetSerializer)


#include "GameplayEffectTypes.h"
#include "Iris/Serialization/PolymorphicNetSerializerImpl.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"

namespace UE::Net
{

struct FGameplayEffectContextHandleAccessorForNetSerializer
{
	static TSharedPtr<FGameplayEffectContext>& GetItem(FGameplayEffectContextHandle& Source)
	{
		return Source.Data;
	}
};

struct FGameplayEffectContextHandleNetSerializer : TPolymorphicStructNetSerializerImpl<FGameplayEffectContextHandle, FGameplayEffectContext, FGameplayEffectContextHandleAccessorForNetSerializer::GetItem>
{
	typedef TPolymorphicStructNetSerializerImpl<FGameplayEffectContextHandle, FGameplayEffectContext, FGameplayEffectContextHandleAccessorForNetSerializer::GetItem> InternalNetSerializerType;
	typedef FGameplayEffectContextHandleNetSerializerConfig ConfigType;

	static const uint32 Version = 0;	
	static const ConfigType DefaultConfig;

	static void InitTypeCache();

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		FNetSerializerRegistryDelegates();
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
		virtual void OnPostFreezeNetSerializerRegistry() override;
		virtual void OnLoadedModulesUpdated() override;
	};

	static FGameplayEffectContextHandleNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	static bool bIsPostFreezeCalled;
};

UE_NET_IMPLEMENT_SERIALIZER(FGameplayEffectContextHandleNetSerializer);

const FGameplayEffectContextHandleNetSerializer::ConfigType FGameplayEffectContextHandleNetSerializer::DefaultConfig;
FGameplayEffectContextHandleNetSerializer::FNetSerializerRegistryDelegates FGameplayEffectContextHandleNetSerializer::NetSerializerRegistryDelegates;
bool FGameplayEffectContextHandleNetSerializer::bIsPostFreezeCalled = false;

void FGameplayEffectContextHandleNetSerializer::InitTypeCache()
{
	// When post freeze is called we expect all custom serializers to have been registered
	// so that the type cache will get the appropriate serializer when creating the ReplicationStateDescriptor.
	if (bIsPostFreezeCalled)
	{
		InternalNetSerializerType::InitTypeCache<FGameplayEffectContextHandleNetSerializer>();
	}
}

static const FName PropertyNetSerializerRegistry_NAME_GameplayEffectContextHandle("GameplayEffectContextHandle");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayEffectContextHandle, FGameplayEffectContextHandleNetSerializer);

FGameplayEffectContextHandleNetSerializer::FNetSerializerRegistryDelegates::FNetSerializerRegistryDelegates()
: UE::Net::FNetSerializerRegistryDelegates(EFlags::ShouldBindLoadedModulesUpdatedDelegate)
{
}

FGameplayEffectContextHandleNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayEffectContextHandle);
}

void FGameplayEffectContextHandleNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayEffectContextHandle);
}

void FGameplayEffectContextHandleNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	bIsPostFreezeCalled = true;
}

void FGameplayEffectContextHandleNetSerializer::FNetSerializerRegistryDelegates::OnLoadedModulesUpdated()
{
	InitGameplayEffectContextHandleNetSerializerTypeCache();
}

}

void InitGameplayEffectContextHandleNetSerializerTypeCache()
{
	UE::Net::FGameplayEffectContextHandleNetSerializer::InitTypeCache();
}

