// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/GameplayEffectContextNetSerializer.h"
#include "Serialization/InternalGameplayEffectContextNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayEffectContextNetSerializer)


#include "Engine/HitResult.h"
#include "GameplayEffectTypes.h"
#include "Engine/HitResult.h"
#include "Net/Core/NetBitArray.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"

// Populated after generating a ReplicationStateDescriptor for this struct
uint16 FGameplayEffectContextAccessorForNetSerializer::PropertyToMemberIndex[FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_ReplicatedPropertyCount];

namespace UE::Net
{

struct FGameplayEffectContextNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing
	static constexpr bool bHasCustomNetReference = true;
	static constexpr bool bHasDynamicState = true;

	// Types
	enum EReplicationFlags : uint32
	{
		ReplicateWorldOrigin = 1U,
		ReplicateSourceObject = ReplicateWorldOrigin << 1U,
		ReplicateInstigator = ReplicateSourceObject << 1U,
		ReplicateEffectCauser = ReplicateInstigator << 1U,
		ReplicateHitResult =  ReplicateEffectCauser << 1U,
	};

	struct FQuantizedType
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE || (UE_NET_IRIS_NETREFHANDLE_SERIAL_SIZE == 63)
		alignas(16) uint8 EffectContext[192];
#else
		alignas(16) uint8 EffectContext[160];
#endif
		// 320 should be large enough and makes struct a multiple of 16 bytes
		alignas(16) uint8 HitResult[320];
		uint32 ReplicationFlags;
	};
	// This is a bit backwards but we don't want to expose the quantized type in public headers.
	static_assert(GetGameplayEffectContextNetSerializerSafeQuantizedSize() >= sizeof(FQuantizedType), "GetGameplayEffectContextNetSerializerSafeQuantizedSize() returns too small a size.");

	typedef FGameplayEffectContextAccessorForNetSerializer SourceType;
	typedef FQuantizedType QuantizedType;
	typedef FGameplayEffectContextNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
		virtual void OnPostFreezeNetSerializerRegistry() override;
	};

	static FGameplayEffectContextNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	static FStructNetSerializerConfig StructNetSerializerConfigForGE;
	static FStructNetSerializerConfig StructNetSerializerConfigForHitResult;
	static const FNetSerializer* StructNetSerializer;
	static UE::Net::EReplicationStateTraits GEStateTraits;
	static UE::Net::EReplicationStateTraits HitResultStateTraits;
};

UE_NET_IMPLEMENT_SERIALIZER(FGameplayEffectContextNetSerializer);

const FGameplayEffectContextNetSerializer::ConfigType FGameplayEffectContextNetSerializer::DefaultConfig;
FGameplayEffectContextNetSerializer::FNetSerializerRegistryDelegates FGameplayEffectContextNetSerializer::NetSerializerRegistryDelegates;
FStructNetSerializerConfig FGameplayEffectContextNetSerializer::StructNetSerializerConfigForGE;
FStructNetSerializerConfig FGameplayEffectContextNetSerializer::StructNetSerializerConfigForHitResult;
 const FNetSerializer* FGameplayEffectContextNetSerializer::StructNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);

EReplicationStateTraits FGameplayEffectContextNetSerializer::GEStateTraits;
EReplicationStateTraits FGameplayEffectContextNetSerializer::HitResultStateTraits;

void FGameplayEffectContextNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	//!!!!!!!!!!!!!!!! Warning !!!!!!!!!!!!!!!
	// Any changes to this function also need to be done to FGameplayEffectContext::NetSerialize to support generic replication
	//!!!!!!!!!!!!!!!! Warning !!!!!!!!!!!!!!!

	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	const uint32 ReplicationFlags = Value.ReplicationFlags;
	Writer->WriteBits(ReplicationFlags, 5U);

	// We need to manually serialize the properties as there are some replicated properties that don't need to be replicated,
	// as is the case with some of the bools that are covered by the ReplicationFlags.
	{
		const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfigForGE.StateDescriptor.GetReference();
		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
		const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
		
		uint32 MemberMaskStorage = ~0U;
		FNetBitArrayView MemberMask(&MemberMaskStorage, FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_ReplicatedPropertyCount, FNetBitArrayView::NoResetNoValidate);
		MemberMask.SetBitValue(FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_Instigator, (ReplicationFlags & EReplicationFlags::ReplicateInstigator) != 0);
		MemberMask.SetBitValue(FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_EffectCauser, (ReplicationFlags & EReplicationFlags::ReplicateEffectCauser) != 0);
		MemberMask.SetBitValue(FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_SourceObject, (ReplicationFlags & EReplicationFlags::ReplicateSourceObject) != 0);
		MemberMask.SetBitValue(FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_WorldOrigin, (ReplicationFlags & EReplicationFlags::ReplicateWorldOrigin) != 0);
		for (uint32 PropertyIt = 0, PropertyEndIt = FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_ReplicatedPropertyCount; PropertyIt != PropertyEndIt; ++PropertyIt)
		{
			if (!MemberMask.GetBit(PropertyIt))
			{
				continue;
			}

			const uint32 MemberIndex = FGameplayEffectContextAccessorForNetSerializer::PropertyToMemberIndex[PropertyIt];
			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIndex];
			const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIndex];

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIndex].DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberSerializerDescriptor.Serializer->Name, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

			FNetSerializeArgs MemberSerializeArgs;
			MemberSerializeArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			MemberSerializeArgs.Source = reinterpret_cast<NetSerializerValuePointer>(&Value.EffectContext) + MemberDescriptor.InternalMemberOffset;
			MemberSerializerDescriptor.Serializer->Serialize(Context, MemberSerializeArgs);
		}
	}

	if (ReplicationFlags & EReplicationFlags::ReplicateHitResult)
	{
		FNetSerializeArgs HitResultNetSerializeArgs = {};
		HitResultNetSerializeArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
		HitResultNetSerializeArgs.Source = NetSerializerValuePointer(&Value.HitResult);
		StructNetSerializer->Serialize(Context, HitResultNetSerializeArgs);
	}
}

void FGameplayEffectContextNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const uint32 ReplicationFlags = Reader->ReadBits(5U);
	Target.ReplicationFlags = ReplicationFlags;

	// We need to manually deserialize the properties as there are some replicated properties that don't need to be replicated,
	// as is the case with some of the bools that are covered by the ReplicationFlags.
	{
		const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfigForGE.StateDescriptor.GetReference();
		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
		const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
		
		uint32 MemberMaskStorage = ~0U;
		FNetBitArrayView MemberMask(&MemberMaskStorage, FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_ReplicatedPropertyCount, FNetBitArrayView::NoResetNoValidate);
		MemberMask.SetBitValue(FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_Instigator, (ReplicationFlags & EReplicationFlags::ReplicateInstigator) != 0);
		MemberMask.SetBitValue(FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_EffectCauser, (ReplicationFlags & EReplicationFlags::ReplicateEffectCauser) != 0);
		MemberMask.SetBitValue(FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_SourceObject, (ReplicationFlags & EReplicationFlags::ReplicateSourceObject) != 0);
		MemberMask.SetBitValue(FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_WorldOrigin, (ReplicationFlags & EReplicationFlags::ReplicateWorldOrigin) != 0);
		for (uint32 PropertyIt = 0, PropertyEndIt = FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_ReplicatedPropertyCount; PropertyIt != PropertyEndIt; ++PropertyIt)
		{
			if (!MemberMask.GetBit(PropertyIt))
			{
				continue;
			}

			const uint32 MemberIndex = FGameplayEffectContextAccessorForNetSerializer::PropertyToMemberIndex[PropertyIt];
			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIndex];
			const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIndex];

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIndex].DebugName, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberSerializerDescriptor.Serializer->Name, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

			FNetDeserializeArgs MemberDeserializeArgs;
			MemberDeserializeArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			MemberDeserializeArgs.Target = reinterpret_cast<NetSerializerValuePointer>(&Target.EffectContext) + MemberDescriptor.InternalMemberOffset;
			MemberSerializerDescriptor.Serializer->Deserialize(Context, MemberDeserializeArgs);
		}
	}

	if (ReplicationFlags & EReplicationFlags::ReplicateHitResult)
	{
		FNetDeserializeArgs HitResultNetDeserializeArgs = {};
		HitResultNetDeserializeArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
		HitResultNetDeserializeArgs.Target = NetSerializerValuePointer(&Target.HitResult);
		StructNetSerializer->Deserialize(Context, HitResultNetDeserializeArgs);
	}
}

void FGameplayEffectContextNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	NetSerializeDeltaDefault<Serialize>(Context, Args);
}

void FGameplayEffectContextNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	NetDeserializeDeltaDefault<Deserialize>(Context, Args);
}

void FGameplayEffectContextNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	
	uint32 ReplicationFlags = 0;

	SourceType TempGE;
	TempGE.CopyReplicatedFieldsFrom(SourceValue);

	const FHitResult* HitResult = SourceValue.GetHitResult();
	
	ReplicationFlags |= (TempGE.ShouldReplicateInstigator() ? EReplicationFlags::ReplicateInstigator : 0);
	ReplicationFlags |= (TempGE.ShouldReplicateEffectCauser() ? EReplicationFlags::ReplicateEffectCauser : 0);
	ReplicationFlags |= (TempGE.ShouldReplicateWorldOrigin() ? EReplicationFlags::ReplicateWorldOrigin : 0);
	ReplicationFlags |= (TempGE.ShouldReplicateSourceObject() ? EReplicationFlags::ReplicateSourceObject : 0);
	ReplicationFlags |= (HitResult != nullptr ? EReplicationFlags::ReplicateHitResult : 0);

	TargetValue.ReplicationFlags = ReplicationFlags;

	// To keep code simple we're not checking conditions to avoid quantizing conditionally replicated members.
	{
		FNetQuantizeArgs GEQuantizeArgs = {};
		GEQuantizeArgs.NetSerializerConfig = &StructNetSerializerConfigForGE;
		GEQuantizeArgs.Source = NetSerializerValuePointer(&TempGE);
		GEQuantizeArgs.Target = NetSerializerValuePointer(&TargetValue.EffectContext);
		StructNetSerializer->Quantize(Context, GEQuantizeArgs);
	}

	// Can't quantize a null pointer so need to check that the hit result is valid.
	// In case the quantized contains dynamic state we could optionally free that memory
	// if the HitResult isn't valid. At the moment we expect little memory gains in doing that.
	if (ReplicationFlags & EReplicationFlags::ReplicateHitResult)
	{
		FNetQuantizeArgs HitResultQuantizeArgs = {};
		HitResultQuantizeArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
		HitResultQuantizeArgs.Source = NetSerializerValuePointer(HitResult);
		HitResultQuantizeArgs.Target = NetSerializerValuePointer(&TargetValue.HitResult);
		StructNetSerializer->Quantize(Context, HitResultQuantizeArgs);
	}
}

void FGameplayEffectContextNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);
	
	const uint32 ReplicationFlags = SourceValue.ReplicationFlags;

	FGameplayEffectContextAccessorForNetSerializer TempGE;
	{
		FNetDequantizeArgs GEDequantizeArgs = {};
		GEDequantizeArgs.NetSerializerConfig = &StructNetSerializerConfigForGE;
		GEDequantizeArgs.Source = NetSerializerValuePointer(&SourceValue.EffectContext);
		GEDequantizeArgs.Target = NetSerializerValuePointer(&TempGE);
		StructNetSerializer->Dequantize(Context, GEDequantizeArgs);
		TempGE.SetShouldReplicateInstigator((ReplicationFlags & EReplicationFlags::ReplicateInstigator) != 0);
		TempGE.SetShouldReplicateEffectCauser((ReplicationFlags & EReplicationFlags::ReplicateEffectCauser) != 0);
		TempGE.SetShouldReplicateWorldOrigin((ReplicationFlags & EReplicationFlags::ReplicateWorldOrigin) != 0);
		TempGE.SetShouldReplicateSourceObject((ReplicationFlags & EReplicationFlags::ReplicateSourceObject) != 0);
	}

	// Can't quantize a null pointer so need to check that the hit result is valid.
	// In case the quantized contains dynamic state we could optionally free that memory
	// if the HitResult isn't valid. At the moment we expect little memory gains in doing that.
	if (ReplicationFlags & EReplicationFlags::ReplicateHitResult)
	{
		FHitResult* HitResult = new FHitResult();
		FNetDequantizeArgs HitResultQuantizeArgs = {};
		HitResultQuantizeArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
		HitResultQuantizeArgs.Source = NetSerializerValuePointer(&SourceValue.HitResult);
		HitResultQuantizeArgs.Target = NetSerializerValuePointer(HitResult);
		StructNetSerializer->Dequantize(Context, HitResultQuantizeArgs);
		TempGE.SetHitResult(MakeShareable(HitResult));
	}

	TempGE.AssignReplicatedFieldsTo(TargetValue);
}

bool FGameplayEffectContextNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		if (Value0.ReplicationFlags != Value1.ReplicationFlags)
		{
			return false;
		}

		{
			FNetIsEqualArgs GEIsEqualArgs = Args;
			GEIsEqualArgs.NetSerializerConfig = &StructNetSerializerConfigForGE;
			GEIsEqualArgs.Source0 = NetSerializerValuePointer(&Value0.EffectContext);
			GEIsEqualArgs.Source1 = NetSerializerValuePointer(&Value1.EffectContext);

			if (!StructNetSerializer->IsEqual(Context, GEIsEqualArgs))
			{
				return false;
			}
		}

		if (Value0.ReplicationFlags & EReplicationFlags::ReplicateHitResult)
		{
			FNetIsEqualArgs HitResultIsEqualArgs = Args;
			HitResultIsEqualArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
			HitResultIsEqualArgs.Source0 = NetSerializerValuePointer(&Value0.HitResult);
			HitResultIsEqualArgs.Source1 = NetSerializerValuePointer(&Value1.HitResult);

			if (!StructNetSerializer->IsEqual(Context, HitResultIsEqualArgs))
			{
				return false;
			}
		}
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		if (SourceValue0.ShouldReplicateSourceObject() != SourceValue1.ShouldReplicateSourceObject())
		{
			return false;
		}

		if (SourceValue0.ShouldReplicateWorldOrigin() != SourceValue1.ShouldReplicateWorldOrigin())
		{
			return false;
		}

		const FHitResult* HitResult0 = SourceValue0.GetHitResult();
		const FHitResult* HitResult1 = SourceValue1.GetHitResult();
		if ((HitResult0 != nullptr) && (HitResult1 != nullptr))
		{
			FNetIsEqualArgs HitEffectIsEqualArgs = Args;
			HitEffectIsEqualArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
			HitEffectIsEqualArgs.Source0 = NetSerializerValuePointer(HitResult0);
			HitEffectIsEqualArgs.Source1 = NetSerializerValuePointer(HitResult1);

			if (!StructNetSerializer->IsEqual(Context, HitEffectIsEqualArgs))
			{
				return false;
			}
		}
		else if (HitResult0 != HitResult1)
		{
			// Only one of the pointers is null
			return false;
		}

		// Because of conditionals we need to create a temporary copy of the source data. 
		{
			SourceType TempGE0;
			SourceType TempGE1;

			TempGE0.CopyReplicatedFieldsFrom(SourceValue0);
			TempGE1.CopyReplicatedFieldsFrom(SourceValue1);

			FNetIsEqualArgs GEIsEqualArgs = Args;
			GEIsEqualArgs.NetSerializerConfig = &StructNetSerializerConfigForGE;
			GEIsEqualArgs.Source0 = NetSerializerValuePointer(&TempGE0);
			GEIsEqualArgs.Source1 = NetSerializerValuePointer(&TempGE1);

			if (!StructNetSerializer->IsEqual(Context, GEIsEqualArgs))
			{
				return false;
			}
		}
	}

	return true;
}

bool FGameplayEffectContextNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);

	// Because of conditionals we need to create a temporary copy of the source data.
	{
		SourceType TempGE;
		TempGE.CopyReplicatedFieldsFrom(SourceValue);

		FNetValidateArgs GEValidateArgs = {};
		GEValidateArgs.NetSerializerConfig = &StructNetSerializerConfigForGE;
		GEValidateArgs.Source = NetSerializerValuePointer(&TempGE);
		if (!StructNetSerializer->Validate(Context, GEValidateArgs))
		{
			return false;
		}
	}

	if (const FHitResult* HitResult = SourceValue.GetHitResult())
	{
		FNetValidateArgs HitEffectValidateArgs = {};
		HitEffectValidateArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
		HitEffectValidateArgs.Source = NetSerializerValuePointer(HitResult);
		if (!StructNetSerializer->Validate(Context, HitEffectValidateArgs))
		{
			return false;
		}
	}

	return true;
}

void FGameplayEffectContextNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	
	{
		FNetCollectReferencesArgs GECollectReferencesArgs = Args;
		GECollectReferencesArgs.NetSerializerConfig = &StructNetSerializerConfigForGE;
		GECollectReferencesArgs.Source = NetSerializerValuePointer(&Value.EffectContext);
		StructNetSerializer->CollectNetReferences(Context, GECollectReferencesArgs);
	}

	if (Value.ReplicationFlags & EReplicationFlags::ReplicateHitResult)
	{
		FNetCollectReferencesArgs HitEffectCollectReferencesArgs = Args;
		HitEffectCollectReferencesArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
		HitEffectCollectReferencesArgs.Source = NetSerializerValuePointer(&Value.HitResult);
		StructNetSerializer->CollectNetReferences(Context, HitEffectCollectReferencesArgs);
	}
}

void FGameplayEffectContextNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	
	if (EnumHasAnyFlags(GEStateTraits, UE::Net::EReplicationStateTraits::HasDynamicState))
	{
		FNetCloneDynamicStateArgs GECloneDynamicStateArgs = {};
		GECloneDynamicStateArgs.NetSerializerConfig = &StructNetSerializerConfigForGE;
		GECloneDynamicStateArgs.Source = NetSerializerValuePointer(&SourceValue.EffectContext);
		GECloneDynamicStateArgs.Target = NetSerializerValuePointer(&TargetValue.EffectContext);
		StructNetSerializer->CloneDynamicState(Context, GECloneDynamicStateArgs);
	}

	if (EnumHasAnyFlags(HitResultStateTraits, UE::Net::EReplicationStateTraits::HasDynamicState) && (SourceValue.ReplicationFlags & EReplicationFlags::ReplicateHitResult))
	{
		FNetCloneDynamicStateArgs HitEffectCloneDynamicStateArgs = {};
		HitEffectCloneDynamicStateArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
		HitEffectCloneDynamicStateArgs.Source = NetSerializerValuePointer(&SourceValue.HitResult);
		HitEffectCloneDynamicStateArgs.Target = NetSerializerValuePointer(&TargetValue.HitResult);
		StructNetSerializer->CloneDynamicState(Context, HitEffectCloneDynamicStateArgs);
	}
}

void FGameplayEffectContextNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	
	if (EnumHasAnyFlags(GEStateTraits, UE::Net::EReplicationStateTraits::HasDynamicState))
	{
		FNetFreeDynamicStateArgs GEFreeDynamicStateArgs = {};
		GEFreeDynamicStateArgs.NetSerializerConfig = &StructNetSerializerConfigForGE;
		GEFreeDynamicStateArgs.Source = NetSerializerValuePointer(&SourceValue.EffectContext);
		StructNetSerializer->FreeDynamicState(Context, GEFreeDynamicStateArgs);
	}

	if (EnumHasAnyFlags(HitResultStateTraits, UE::Net::EReplicationStateTraits::HasDynamicState) && (SourceValue.ReplicationFlags & EReplicationFlags::ReplicateHitResult))
	{
		FNetFreeDynamicStateArgs HitEffectFreeDynamicStateArgs = {};
		HitEffectFreeDynamicStateArgs.NetSerializerConfig = &StructNetSerializerConfigForHitResult;
		HitEffectFreeDynamicStateArgs.Source = NetSerializerValuePointer(&SourceValue.HitResult);
		StructNetSerializer->FreeDynamicState(Context, HitEffectFreeDynamicStateArgs);
	}
}

static const FName PropertyNetSerializerRegistry_NAME_GameplayEffectContext("GameplayEffectContext");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayEffectContext, FGameplayEffectContextNetSerializer);

FGameplayEffectContextNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayEffectContext);
}

void FGameplayEffectContextNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayEffectContext);
}

void FGameplayEffectContextNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	// Setup serializer for most members. The HitResult requires special care and is handled later.
	{
		const UStruct* GEStruct = FGameplayEffectContextAccessorForNetSerializer::StaticStruct();
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (GEStruct->GetStructureSize() != 176 || GEStruct->GetMinAlignment() != 8)
#else
		if (GEStruct->GetStructureSize() != 128 || GEStruct->GetMinAlignment() != 8)
#endif
		{
			LowLevelFatalError(TEXT("%s Size: %d Alignment: %d"), TEXT("FGameplayEffectContext layout has changed. Need to update FGameplayEffectContextNetSerializer."), GEStruct->GetStructureSize(), GEStruct->GetMinAlignment());
		}

		// In this case we want to build a descriptor based on the struct members rather than the serializer we try to register
		FReplicationStateDescriptorBuilder::FParameters Params;
		Params.SkipCheckForCustomNetSerializerForStruct = true;

		StructNetSerializerConfigForGE.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(GEStruct, Params);
		const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfigForGE.StateDescriptor.GetReference();
		check(Descriptor != nullptr);
		GEStateTraits = Descriptor->Traits;

		// Build property -> index lookup table
		FName PropertyNames[FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_ReplicatedPropertyCount];
		PropertyNames[FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_Instigator] = FName("Instigator");
		PropertyNames[FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_EffectCauser] = FName("EffectCauser");
		PropertyNames[FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_AbilityCDO] = FName("AbilityCDO");
		PropertyNames[FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_AbilityLevel] = FName("AbilityLevel");
		PropertyNames[FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_SourceObject] = FName("SourceObject");
		PropertyNames[FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_Actors] = FName("Actors");
		PropertyNames[FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_WorldOrigin] = FName("WorldOrigin");

		uint32 FoundPropertyMask = 0;

		// Find all replicated properties of interest.
		const FProperty*const*MemberProperties = Descriptor->MemberProperties;
		for (uint32 PropertyIndex = 0; PropertyIndex != FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_ReplicatedPropertyCount; ++PropertyIndex)
		{
			for (const FProperty*const& MemberProperty : MakeArrayView(MemberProperties, Descriptor->MemberCount))
			{
				const SIZE_T MemberIndex = &MemberProperty - MemberProperties;
				if (MemberProperty->GetFName() == PropertyNames[PropertyIndex])
				{
					FoundPropertyMask |= 1U << PropertyIndex;

					FGameplayEffectContextAccessorForNetSerializer::PropertyToMemberIndex[PropertyIndex] = MemberIndex;
					break;
				}
			}
		}

		if (FoundPropertyMask != (1U <<  FGameplayEffectContextAccessorForNetSerializer::EPropertyName::PropertyName_ReplicatedPropertyCount) - 1U)
		{
			LowLevelFatalError(TEXT("%s"), TEXT("Couldn't find expected replicated members in FGameplayEffectContext.")); 
		}

		// Validate our assumptions regarding quantized state size and alignment.
		constexpr SIZE_T OffsetOfEffectContext = offsetof(FQuantizedType, EffectContext);
		if ((sizeof(FQuantizedType::EffectContext) < Descriptor->InternalSize) || (((OffsetOfEffectContext/Descriptor->InternalAlignment)*Descriptor->InternalAlignment) != OffsetOfEffectContext))
		{
			LowLevelFatalError(TEXT("FQuantizedType::EffectContext has size %u but requires size %u and alignment %u."), uint32(sizeof(FQuantizedType::EffectContext)), uint32(Descriptor->InternalSize), uint32(Descriptor->InternalAlignment));
		}
	}

	// Setup serializer for HitResult
	{
		const UStruct* HitResultStruct = FHitResult::StaticStruct();
		StructNetSerializerConfigForHitResult.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(HitResultStruct);
		const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfigForHitResult.StateDescriptor.GetReference();
		check(Descriptor != nullptr);
		HitResultStateTraits = Descriptor->Traits;

		// Validate our assumptions regarding quantized state size and alignment.
		constexpr SIZE_T OffsetOfHitResult = offsetof(FQuantizedType, HitResult);
		if ((sizeof(FQuantizedType::HitResult) < Descriptor->InternalSize) || (((OffsetOfHitResult/Descriptor->InternalAlignment)*Descriptor->InternalAlignment) != OffsetOfHitResult))
		{
			LowLevelFatalError(TEXT("FQuantizedType::HitResult has size %u but requires size %u and alignment %u."), uint32(sizeof(FQuantizedType::HitResult)), uint32(Descriptor->InternalSize), uint32(Descriptor->InternalAlignment));
		}
	}

	// Verify traits
	ValidateForwardingNetSerializerTraits(&UE_NET_GET_SERIALIZER(FGameplayEffectContextNetSerializer), GEStateTraits | HitResultStateTraits);
}

}


void FGameplayEffectContextAccessorForNetSerializer::CopyReplicatedFieldsFrom(const FGameplayEffectContextAccessorForNetSerializer& GE)
{
	this->AbilityCDO = GE.AbilityCDO;
	this->AbilityLevel = GE.AbilityLevel;
	this->Actors = GE.Actors;
	if (GE.bReplicateEffectCauser)
	{
		this->EffectCauser = GE.EffectCauser;
	}

	if (GE.bReplicateInstigator)
	{
		this->Instigator = GE.Instigator;
	}

	this->bHasWorldOrigin = GE.bHasWorldOrigin;
	if (GE.bHasWorldOrigin)
	{
		this->WorldOrigin = GE.WorldOrigin;
	}
	this->bReplicateSourceObject = GE.bReplicateSourceObject;
	this->bReplicateInstigator = GE.bReplicateInstigator;
	this->bReplicateEffectCauser = GE.bReplicateEffectCauser;

	if (GE.bReplicateSourceObject)
	{
		this->SourceObject = GE.SourceObject;
	}
}

void FGameplayEffectContextAccessorForNetSerializer::AssignReplicatedFieldsTo(FGameplayEffectContextAccessorForNetSerializer& GE) const
{
	GE.AbilityCDO = this->AbilityCDO;
	GE.AbilityLevel = this->AbilityLevel;
	GE.Actors = this->Actors;
	GE.EffectCauser = this->EffectCauser;
	GE.Instigator = this->Instigator;
	GE.bHasWorldOrigin = this->bHasWorldOrigin;
	if (this->bHasWorldOrigin)
	{
		GE.WorldOrigin = this->WorldOrigin;
	}
	GE.bReplicateSourceObject = this->bReplicateSourceObject;
	if (this->bReplicateSourceObject)
	{
		GE.SourceObject = this->SourceObject;
	}
	if (this->HitResult.IsValid())
	{
		GE.HitResult = this->HitResult;
	}

	GE.AddInstigator(GE.Instigator.Get(), GE.EffectCauser.Get());
}

const FHitResult* FGameplayEffectContextAccessorForNetSerializer::GetHitResult() const
{
	return HitResult.Get();
}

void FGameplayEffectContextAccessorForNetSerializer::SetHitResult(TSharedRef<FHitResult> InHitResult)
{
	HitResult = InHitResult;
}
