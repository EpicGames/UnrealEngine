// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffectTypes.h"
#include "Engine/EngineTypes.h"
#include "Templates/SharedPointer.h"

// Implementation is located in GameplayEffectContextNetSerializer.cpp
struct FGameplayEffectContextAccessorForNetSerializer : public FGameplayEffectContext
{

public:
	void CopyReplicatedFieldsFrom(const FGameplayEffectContextAccessorForNetSerializer& GE);	
	void AssignReplicatedFieldsTo(FGameplayEffectContextAccessorForNetSerializer& GE) const;

	void SetShouldReplicateInstigator(bool bShouldReplicateInstigator) { bReplicateInstigator = bShouldReplicateInstigator; }
	bool ShouldReplicateInstigator() const { return bReplicateInstigator; }

	void SetShouldReplicateEffectCauser(bool bShouldReplicateEffectCauser) { bReplicateEffectCauser = bShouldReplicateEffectCauser; }
	bool ShouldReplicateEffectCauser() const { return bReplicateEffectCauser; }

	void SetShouldReplicateWorldOrigin(bool bShouldReplicateWorldOrigin) { bHasWorldOrigin = bShouldReplicateWorldOrigin; }
	bool ShouldReplicateWorldOrigin() const { return bHasWorldOrigin; }

	void SetShouldReplicateSourceObject(bool bShouldReplicateSourceObject) { bReplicateSourceObject = bShouldReplicateSourceObject; }
	bool ShouldReplicateSourceObject() const { return bReplicateSourceObject; }

	const FHitResult* GetHitResult() const;
	void SetHitResult(TSharedRef<FHitResult> HitResult);


	enum EPropertyName : uint32
	{
		PropertyName_Instigator,
		PropertyName_EffectCauser,
		PropertyName_AbilityCDO,
		PropertyName_AbilityLevel,
		PropertyName_SourceObject,
		PropertyName_Actors,
		PropertyName_WorldOrigin,

		PropertyName_ReplicatedPropertyCount
	};

	static uint16 PropertyToMemberIndex[EPropertyName::PropertyName_ReplicatedPropertyCount];
};
