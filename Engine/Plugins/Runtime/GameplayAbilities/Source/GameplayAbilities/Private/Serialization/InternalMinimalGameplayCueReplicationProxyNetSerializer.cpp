// Copyright Epic Games, Inc. All Rights Reserved.

#include "InternalMinimalGameplayCueReplicationProxyNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InternalMinimalGameplayCueReplicationProxyNetSerializer)


#include "GameplayCueInterface.h"

void FMinimalGameplayCueReplicationProxyForNetSerializer::CopyReplicatedFieldsFrom(const FMinimalGameplayCueReplicationProxy& ReplicationProxy)
{
	Tags = ReplicationProxy.ReplicatedTags;
	Locations = ReplicationProxy.ReplicatedLocations;
}

void FMinimalGameplayCueReplicationProxyForNetSerializer::AssignReplicatedFieldsTo(FMinimalGameplayCueReplicationProxy& ReplicationProxy) const
{
	ReplicationProxy.ReplicatedTags = Tags;
	ReplicationProxy.ReplicatedLocations = Locations;
}

