// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Containers/Array.h"
#include "Engine/NetSerialization.h"
#include "GameplayTagContainer.h"
#include "InternalMinimalGameplayCueReplicationProxyNetSerializer.generated.h"

class UAbilitySystemComponent;
struct FMinimalGameplayCueReplicationProxy;

USTRUCT()
struct FMinimalGameplayCueReplicationProxyForNetSerializer
{
	GENERATED_BODY()

public:
	void CopyReplicatedFieldsFrom(const FMinimalGameplayCueReplicationProxy& ReplicationProxy);	
	void AssignReplicatedFieldsTo(FMinimalGameplayCueReplicationProxy& ReplicationProxy) const;

private:
	UPROPERTY()
	TArray<FGameplayTag> Tags;
	UPROPERTY()
	TArray<FVector_NetQuantize> Locations;
};
