// Copyright Epic Games, Inc. All Rights Reserved.

#include "InternalMinimalReplicationTagCountMapNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InternalMinimalReplicationTagCountMapNetSerializer)


#include "GameplayEffectTypes.h"

void FMinimalReplicationTagCountMapForNetSerializer::CopyReplicatedFieldsFrom(const FMinimalReplicationTagCountMap& TagCountMap)
{
	TagCountMap.TagMap.GetKeys(Tags);
}

void FMinimalReplicationTagCountMapForNetSerializer::AssignReplicatedFieldsTo(FMinimalReplicationTagCountMap& TagCountMap) const
{
	TMap<FGameplayTag, int32>& TagMap = TagCountMap.TagMap;
	for (const FGameplayTag& Tag : Tags)
	{
		TagMap.FindOrAdd(Tag) = 1;
	}
}

void FMinimalReplicationTagCountMapForNetSerializer::ClampTagCount(SIZE_T MaxTagCount)
{
	if (static_cast<uint32>(Tags.Num()) > MaxTagCount)
	{
		// This struct is only used on stack with a very limited lifetime. Let's not spend CPU time on re-allocating.
		Tags.SetNum(static_cast<int32>(static_cast<SSIZE_T>(MaxTagCount)), EAllowShrinking::No);
	}
}

