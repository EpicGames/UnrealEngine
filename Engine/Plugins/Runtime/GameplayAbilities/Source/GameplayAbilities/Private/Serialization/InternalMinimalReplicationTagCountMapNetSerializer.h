// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Containers/Array.h"
#include "GameplayTagContainer.h"
#include "InternalMinimalReplicationTagCountMapNetSerializer.generated.h"

struct FMinimalReplicationTagCountMap;

USTRUCT()
struct FMinimalReplicationTagCountMapForNetSerializer
{
	GENERATED_BODY()

public:
	void CopyReplicatedFieldsFrom(const FMinimalReplicationTagCountMap& TagCountMap);	
	void AssignReplicatedFieldsTo(FMinimalReplicationTagCountMap& TagCountMap) const;

	SIZE_T GetTagCount() const;
	void ClampTagCount(SIZE_T MaxTagCount);

private:
	UPROPERTY()
	TArray<FGameplayTag> Tags;
};

inline SIZE_T FMinimalReplicationTagCountMapForNetSerializer::GetTagCount() const
{
	return static_cast<uint32>(Tags.Num());
}