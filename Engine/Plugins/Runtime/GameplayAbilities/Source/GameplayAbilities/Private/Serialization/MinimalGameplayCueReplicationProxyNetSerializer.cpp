// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/MinimalGameplayCueReplicationProxyNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MinimalGameplayCueReplicationProxyNetSerializer)


#include "GameplayCueInterface.h"
#include "Iris/Core/NetObjectReference.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Serialization/InternalMinimalGameplayCueReplicationProxyNetSerializer.h"

namespace UE::Net
{

extern FReplicationFragment* CreateAndRegisterMinimalGameplayCueReplicationProxyReplicationFragment(UObject* Owner, const FReplicationStateDescriptor* Descriptor, FFragmentRegistrationContext& Context);

struct FMinimalGameplayCueReplicationProxyNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true;
	static constexpr bool bHasDynamicState = true;
	// Few fields are replicated.
	static constexpr bool bUseSerializerIsEqual = true;

	struct FQuantizedType
	{
		alignas(16) uint8 Buffer[32];
	};

	typedef FMinimalGameplayCueReplicationProxy SourceType;
	typedef FQuantizedType QuantizedType;
	typedef FMinimalGameplayCueReplicationProxyNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
		virtual void OnPostFreezeNetSerializerRegistry() override;
	};

	inline static FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	inline static FStructNetSerializerConfig StructNetSerializerConfig;
	static const FNetSerializer* StructNetSerializer;
};

UE_NET_IMPLEMENT_SERIALIZER(FMinimalGameplayCueReplicationProxyNetSerializer);

const FNetSerializer* FMinimalGameplayCueReplicationProxyNetSerializer::StructNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);

void FMinimalGameplayCueReplicationProxyNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	FNetSerializeArgs NetSerializeArgs = Args;
	NetSerializeArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->Serialize(Context, NetSerializeArgs);
}

void FMinimalGameplayCueReplicationProxyNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetDeserializeArgs NetDeserializeArgs = Args;
	NetDeserializeArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->Deserialize(Context, NetDeserializeArgs);
}

void FMinimalGameplayCueReplicationProxyNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	FNetSerializeDeltaArgs NetSerializeDeltaArgs = Args;
	NetSerializeDeltaArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->SerializeDelta(Context, NetSerializeDeltaArgs);
}

void FMinimalGameplayCueReplicationProxyNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetDeserializeDeltaArgs NetDeserializeDeltaArgs = Args;
	NetDeserializeDeltaArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->DeserializeDelta(Context, NetDeserializeDeltaArgs);
}

void FMinimalGameplayCueReplicationProxyNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	
	FMinimalGameplayCueReplicationProxyForNetSerializer TempValue;
	TempValue.CopyReplicatedFieldsFrom(SourceValue);

	FNetQuantizeArgs QuantizeArgs = Args;
	QuantizeArgs.NetSerializerConfig = &StructNetSerializerConfig;
	QuantizeArgs.Source = NetSerializerValuePointer(&TempValue);
	StructNetSerializer->Quantize(Context, QuantizeArgs);
}

void FMinimalGameplayCueReplicationProxyNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);
	
	FMinimalGameplayCueReplicationProxyForNetSerializer TempValue;

	FNetDequantizeArgs DequantizeArgs = Args;
	DequantizeArgs.NetSerializerConfig = &StructNetSerializerConfig;
	DequantizeArgs.Target = NetSerializerValuePointer(&TempValue);
	StructNetSerializer->Dequantize(Context, DequantizeArgs);

	TempValue.AssignReplicatedFieldsTo(TargetValue);
}

bool FMinimalGameplayCueReplicationProxyNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		FNetIsEqualArgs IsEqualArgs = Args;
		IsEqualArgs.NetSerializerConfig = &StructNetSerializerConfig;
		if (!StructNetSerializer->IsEqual(Context, IsEqualArgs))
		{
			return false;
		}
	}
	else
	{
		const SourceType& Source0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Source1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		FMinimalGameplayCueReplicationProxyForNetSerializer TempValue0;
		FMinimalGameplayCueReplicationProxyForNetSerializer TempValue1;

		TempValue0.CopyReplicatedFieldsFrom(Source0);
		TempValue1.CopyReplicatedFieldsFrom(Source1);

		FNetIsEqualArgs IsEqualArgs = Args;
		IsEqualArgs.Source0 = NetSerializerValuePointer(&TempValue0);
		IsEqualArgs.Source1 = NetSerializerValuePointer(&TempValue1);
		IsEqualArgs.NetSerializerConfig = &StructNetSerializerConfig;
		if (!StructNetSerializer->IsEqual(Context, IsEqualArgs))
		{
			return false;
		}
	}

	return true;
}

bool FMinimalGameplayCueReplicationProxyNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	
	FMinimalGameplayCueReplicationProxyForNetSerializer TempValue;
	TempValue.CopyReplicatedFieldsFrom(SourceValue);

	FNetValidateArgs ValidateArgs = Args;
	ValidateArgs.NetSerializerConfig = &StructNetSerializerConfig;
	ValidateArgs.Source = NetSerializerValuePointer(&TempValue);
	return StructNetSerializer->Validate(Context, ValidateArgs);
}

void FMinimalGameplayCueReplicationProxyNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	FNetCollectReferencesArgs CollectReferencesArgs = Args;
	CollectReferencesArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->CollectNetReferences(Context, CollectReferencesArgs);
}

void FMinimalGameplayCueReplicationProxyNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	FNetCloneDynamicStateArgs CloneDynamicStateArgs = Args;
	CloneDynamicStateArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->CloneDynamicState(Context, CloneDynamicStateArgs);
}

void FMinimalGameplayCueReplicationProxyNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	FNetFreeDynamicStateArgs FreeDynamicStateArgs = Args;
	FreeDynamicStateArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->FreeDynamicState(Context, FreeDynamicStateArgs);
}

static const FName PropertyNetSerializerRegistry_NAME_MinimalGameplayCueReplicationProxyNetSerializer("MinimalGameplayCueReplicationProxy");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_WITH_CUSTOM_FRAGMENT_INFO(PropertyNetSerializerRegistry_NAME_MinimalGameplayCueReplicationProxyNetSerializer, FMinimalGameplayCueReplicationProxyNetSerializer, CreateAndRegisterMinimalGameplayCueReplicationProxyReplicationFragment);

FMinimalGameplayCueReplicationProxyNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{	
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_MinimalGameplayCueReplicationProxyNetSerializer);
}

void FMinimalGameplayCueReplicationProxyNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_MinimalGameplayCueReplicationProxyNetSerializer);
}

void FMinimalGameplayCueReplicationProxyNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	constexpr SIZE_T ExpectedSizeOfFMinimalGameplayCueReplicationProxy = 992;
	constexpr SIZE_T ExpectedAlignOfFMinimalGameplayCueReplicationProxy = 16;

	constexpr SIZE_T ActualSizeOfFMinimalGameplayCueReplicationProxy = sizeof(FMinimalGameplayCueReplicationProxy);
	constexpr SIZE_T ActualAlignOfFMinimalGameplayCueReplicationProxy = alignof(FMinimalGameplayCueReplicationProxy);

	// Do our best to detect changes to FMinimalGameplayCueReplicationProxy
	// If this assert triggers, this NetSerializer implementation must be verified against FMinimalGameplayCueReplicationProxy::NetSerialize before updating the size and alignment.
	//static_assert(ExpectedSizeOfFMinimalGameplayCueReplicationProxy == ActualSizeOfFMinimalGameplayCueReplicationProxy && ActualAlignOfFMinimalGameplayCueReplicationProxy == ExpectedAlignOfFMinimalGameplayCueReplicationProxy, "FMinimalGameplayCueReplicationProxy layout has changed. Might need to update FMinimalGameplayCueReplicationProxyNetSerializer to include new data or update the size.");

	const UStruct* ReplicationProxyStruct = FMinimalGameplayCueReplicationProxyForNetSerializer::StaticStruct();
	FReplicationStateDescriptorBuilder::FParameters Params;
	StructNetSerializerConfig.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(ReplicationProxyStruct, Params);

	const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfig.StateDescriptor.GetReference();

	// Validate our assumptions regarding quantized state size and alignment.
	if ((sizeof(QuantizedType) < Descriptor->InternalSize) || (alignof(QuantizedType) < Descriptor->InternalAlignment))
	{
		LowLevelFatalError(TEXT("FMinimalGameplayCueReplicationProxyNetSerializer::FQuantizedType has size %u and alignment %u but requires size %u and alignment %u."), uint32(sizeof(FQuantizedType)), uint32(alignof(FQuantizedType)), Descriptor->InternalSize, Descriptor->InternalAlignment);
	}

	ensureMsgf(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasDynamicState), TEXT("FMinimalGameplayCueReplicationProxyNetSerializer seems to no longer have dynamic state."));
	ensureMsgf(!EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasObjectReference), TEXT("FMinimalGameplayCueReplicationProxyNetSerializer seems to have object references."));
}

}

