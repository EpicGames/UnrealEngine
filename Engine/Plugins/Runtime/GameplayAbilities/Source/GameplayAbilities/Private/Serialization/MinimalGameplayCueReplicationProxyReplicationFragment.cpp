// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/MinimalGameplayCueReplicationProxyReplicationFragment.h"


#include "Engine/NetConnection.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/NetSerializer.h"
#include "AbilitySystemComponent.h"
#include "GameplayCueInterface.h"
#include <type_traits>

namespace UE::Net
{

bool bShouldMarkStructDirtyAfterRemoval = true;
static FAutoConsoleVariableRef CVarShouldMarkStructDirtyAfterRemoval(
	TEXT("Net.GameplayCues.ShouldMarkStructDirtyAfterRemoval"), bShouldMarkStructDirtyAfterRemoval,
	TEXT("Whether to increment LastSourceArrayReplicationKey after Gameplay Cue Removals"));

FReplicationFragment* CreateAndRegisterMinimalGameplayCueReplicationProxyReplicationFragment(UObject* Owner, const FReplicationStateDescriptor* Descriptor, FFragmentRegistrationContext& Context)
{
	if (FMinimalGameplayCueReplicationProxyReplicationFragment* Fragment = new FMinimalGameplayCueReplicationProxyReplicationFragment(Context.GetFragmentTraits() | EReplicationFragmentTraits::DeleteWithInstanceProtocol, Owner, Descriptor))
	{
		Fragment->Register(Context);
		return Fragment;
	}

	return nullptr;
}

FMinimalGameplayCueReplicationProxyReplicationFragment::FMinimalGameplayCueReplicationProxyReplicationFragment(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor)
: FReplicationFragment(InTraits)
, ReplicationStateDescriptor(InDescriptor)
, Owner(InOwner)
{
	// We don't want to create temporary states when applying replicated data as we only want to update the replicated fields.
	Traits |= EReplicationFragmentTraits::HasPersistentTargetStateBuffer;

	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReplicate))
	{
		Traits |= EReplicationFragmentTraits::NeedsPoll;

		// We assume the struct only replicates a few members and the serializer knows which ones. We don't want to mark the struct as dirty if it isn't.
		check(EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::UseSerializerIsEqual));

		SrcReplicationState = MakeUnique<FPropertyReplicationState>(InDescriptor);
	}

#if WITH_PUSH_MODEL
	if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasPushBasedDirtiness))
	{
		Traits |= EReplicationFragmentTraits::HasPushBasedDirtiness;
		if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasFullPushBasedDirtiness))
		{
			Traits |= EReplicationFragmentTraits::HasFullPushBasedDirtiness;
		}
	}
#endif
	
	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReceive))
	{
		if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasRepNotifies))
		{
			ensureMsgf(!EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::KeepPreviousState), TEXT("FMinimalGameplayCueReplicationProxyReplicationFragment doesn't support OnRep calls with previous value."));
		}

		// We do custom callbacks in CallRepNotifies rather than in ApplyReplicatedState.
		Traits |= EReplicationFragmentTraits::HasRepNotifies | EReplicationFragmentTraits::NeedsLegacyCallbacks;
	}
}

void FMinimalGameplayCueReplicationProxyReplicationFragment::Register(FFragmentRegistrationContext& Context)
{
	FPropertyReplicationState* ReplicationState = SrcReplicationState.Get();
	Context.RegisterReplicationFragment(this, ReplicationStateDescriptor.GetReference(), (ReplicationState ? ReplicationState->GetStateBuffer() : nullptr));
}

void FMinimalGameplayCueReplicationProxyReplicationFragment::ApplyReplicatedState(FReplicationStateApplyContext& ApplyContext) const
{
	uint8* ExternalStatePointer = reinterpret_cast<uint8*>(Owner) + ReplicationStateDescriptor->MemberProperties[0]->GetOffset_ForGC();

	// Copy replicated tags to local tags to detect what was replicated. This information is used in MimicMinimalGameplayCueReplicationProxyReceiveLogic.
	FMinimalGameplayCueReplicationProxy* ExternalSourceState = reinterpret_cast<FMinimalGameplayCueReplicationProxy*>(ExternalStatePointer);
	ExternalSourceState->LocalTags = MoveTemp(ExternalSourceState->ReplicatedTags);
	ExternalSourceState->LocalBitMask.Init(true /* initial value */, ExternalSourceState->LocalTags.Num());

	// Dequantize replicated members to external source state. Need to call the serializer directly with the appropriate arguments as the external state doesn't have a ReplicationStateHeader in front of it.
	FNetDequantizeArgs DequantizeArgs = {};
	DequantizeArgs.Source = NetSerializerValuePointer(ApplyContext.StateBufferData.RawStateBuffer);
	DequantizeArgs.Target = NetSerializerValuePointer(ExternalSourceState);
	DequantizeArgs.NetSerializerConfig = ReplicationStateDescriptor->MemberSerializerDescriptors[0].SerializerConfig;
	const FNetSerializer* Serializer = ReplicationStateDescriptor->MemberSerializerDescriptors[0].Serializer;
	Serializer->Dequantize(*ApplyContext.NetSerializationContext, DequantizeArgs);

	MimicMinimalGameplayCueReplicationProxyReceiveLogic(ApplyContext);
}

bool FMinimalGameplayCueReplicationProxyReplicationFragment::PollReplicatedState(EReplicationFragmentPollFlags PollOption)
{
	// We can early out if we are pushbased and not dirty for polling
	const bool bPoll = EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::PollAllState) ||
	(EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::PollDirtyState) && (!EnumHasAnyFlags(EReplicationFragmentTraits::HasPushBasedDirtiness, Traits) || SrcReplicationState->IsDirtyForPolling()));

	if (bPoll)
	{
		const uint8* ExternalStateBuffer = reinterpret_cast<uint8*>(Owner) + ReplicationStateDescriptor->MemberProperties[0]->GetOffset_ForGC();
		const FMinimalGameplayCueReplicationProxy* ExternalSourceState = reinterpret_cast<const FMinimalGameplayCueReplicationProxy*>(ExternalStateBuffer);

		void* CachedStateBuffer = SrcReplicationState->GetStateBuffer();
		FMinimalGameplayCueReplicationProxy* CachedState = reinterpret_cast<FMinimalGameplayCueReplicationProxy*>(static_cast<uint8*>(CachedStateBuffer) + ReplicationStateDescriptor->MemberDescriptors[0].ExternalMemberOffset);

		if (ExternalSourceState->LastSourceArrayReplicationKey != CachedState->LastSourceArrayReplicationKey)
		{
			CachedState->LastSourceArrayReplicationKey = ExternalSourceState->LastSourceArrayReplicationKey;
			return SrcReplicationState->PollPropertyReplicationState(Owner);
		}
	}

	return SrcReplicationState->IsDirty(0);
}

void FMinimalGameplayCueReplicationProxyReplicationFragment::CollectOwner(FReplicationStateOwnerCollector* Owners) const
{
	Owners->AddOwner(Owner);
}

void FMinimalGameplayCueReplicationProxyReplicationFragment::CallRepNotifies(FReplicationStateApplyContext& Context)
{
	CallRepNotify(Context);
}

void FMinimalGameplayCueReplicationProxyReplicationFragment::MimicMinimalGameplayCueReplicationProxyReceiveLogic(FReplicationStateApplyContext& Context) const
{
	uint8* ExternalStatePointer = reinterpret_cast<uint8*>(Owner) + ReplicationStateDescriptor->MemberProperties[0]->GetOffset_ForGC();
	FMinimalGameplayCueReplicationProxy* ExternalSourceState = reinterpret_cast<FMinimalGameplayCueReplicationProxy*>(ExternalStatePointer);

	// Check whether we need to perform the complex logic or not.
	UAbilitySystemComponent* StateOwner = ExternalSourceState->Owner;
	bool UpdateOwnerTagMap = StateOwner != nullptr;
	if (ExternalSourceState->bRequireNonOwningNetConnection && StateOwner)
	{
		if (AActor* OwningActor = StateOwner->GetOwner())
		{			
			if (const UNetConnection* OwnerNetConnection = OwningActor->GetNetConnection())
			{
				if (OwnerNetConnection->GetConnectionHandle().GetParentConnectionId() == Context.NetSerializationContext->GetLocalConnectionId())
				{
					UpdateOwnerTagMap = false;
				}
			}
		}
	}

	if (!UpdateOwnerTagMap)
	{
		return;
	}

	FGameplayCueParameters Parameters;
	ExternalSourceState->InitGameplayCueParametersFunc(Parameters, ExternalSourceState->Owner);
	const FVector OriginalLocationParameter = Parameters.Location;

	const int32 TagCount = ExternalSourceState->ReplicatedTags.Num();
	for (int32 TagIt = 0; TagIt < TagCount; ++TagIt)
	{
		FGameplayTag& ReplicatedTag = ExternalSourceState->ReplicatedTags[TagIt];
		FVector_NetQuantize& ReplicatedLocation = ExternalSourceState->ReplicatedLocations[TagIt];

		const bool bHasReplicatedLocation = !ReplicatedLocation.IsZero();
		if (bHasReplicatedLocation)
		{
			Parameters.Location = ReplicatedLocation;
		}
		else
		{
			Parameters.Location = OriginalLocationParameter;
		}

		const int32 LocalIdx = ExternalSourceState->LocalTags.IndexOfByKey(ReplicatedTag);
		if (LocalIdx != INDEX_NONE)
		{
			// This tag already existed and is accounted for
			ExternalSourceState->LocalBitMask[LocalIdx] = false;
		}
		else
		{
			// Not sure this needs to be set here as it's unconditionally set later. Keeping to mimic original behavior as much as possible.
			ExternalSourceState->bCachedModifiedOwnerTags = true;

			// This is a new tag, we need to invoke the WhileActive gameplay cue event
			StateOwner->SetTagMapCount(ReplicatedTag, 1);
			StateOwner->InvokeGameplayCueEvent(ReplicatedTag, EGameplayCueEvent::WhileActive, Parameters);

			// The demo recorder needs to believe that this structure is dirty so it will get saved into the demo stream
			ExternalSourceState->LastSourceArrayReplicationKey++;
		}
	}

	// Restore the location in case we touched it
	Parameters.Location = OriginalLocationParameter;

	ExternalSourceState->bCachedModifiedOwnerTags = true;
	for (auto It = ExternalSourceState->LocalTags.CreateConstIterator(); It; ++It)
	{
		if (!ExternalSourceState->LocalBitMask[It.GetIndex()])
		{
			continue;
		}

		FGameplayTag RemovedTag = *It;
		StateOwner->SetTagMapCount(RemovedTag, 0);

		StateOwner->InvokeGameplayCueEvent(RemovedTag, EGameplayCueEvent::Removed, Parameters);

		if(bShouldMarkStructDirtyAfterRemoval)
		{
			// The demo recorder needs to believe that this structure is dirty so it will get saved into the demo stream
			ExternalSourceState->LastSourceArrayReplicationKey++;
		}
	}
}

void FMinimalGameplayCueReplicationProxyReplicationFragment::CallRepNotify(FReplicationStateApplyContext& ApplyContext)
{
	if (const UFunction* RepNotifyFunction = ReplicationStateDescriptor->MemberPropertyDescriptors[0].RepNotifyFunction)
	{
		if (ApplyContext.bIsInit)
		{
			const uint8* ReceivedState = ApplyContext.StateBufferData.RawStateBuffer;
			const uint8* DefaultState = ReplicationStateDescriptor->DefaultStateBuffer;

			if (FReplicationStateOperations::IsEqualQuantizedState(*ApplyContext.NetSerializationContext, ReceivedState, DefaultState, ReplicationStateDescriptor))
			{
				return;
			}
		}

		Owner->ProcessEvent(const_cast<UFunction*>(RepNotifyFunction), nullptr);
	}
}

}

