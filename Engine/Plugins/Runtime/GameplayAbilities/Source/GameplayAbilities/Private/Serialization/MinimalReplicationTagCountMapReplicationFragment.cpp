// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/MinimalReplicationTagCountMapReplicationFragment.h"


#include "Engine/NetConnection.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/NetSerializer.h"
#include "AbilitySystemComponent.h"
#include "GameplayEffectTypes.h"

namespace UE::Net
{

FReplicationFragment* CreateAndRegisterMinimalReplicationTagCountMapReplicationFragment(UObject* Owner, const FReplicationStateDescriptor* Descriptor, FFragmentRegistrationContext& Context)
{
	if (FMinimalReplicationTagCountMapReplicationFragment* Fragment = new FMinimalReplicationTagCountMapReplicationFragment(Context.GetFragmentTraits() | EReplicationFragmentTraits::DeleteWithInstanceProtocol, Owner, Descriptor))
	{
		Fragment->Register(Context);
		return Fragment;
	}

	return nullptr;
}

FMinimalReplicationTagCountMapReplicationFragment::FMinimalReplicationTagCountMapReplicationFragment(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor)
: FReplicationFragment(InTraits)
, ReplicationStateDescriptor(InDescriptor)
, Owner(InOwner)
{
	// We don't want to create temporary states when applying replicated data as we only want to update the replicated fields.
	Traits |= EReplicationFragmentTraits::HasPersistentTargetStateBuffer;

	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReplicate))
	{
		Traits |= EReplicationFragmentTraits::NeedsPoll;

		// We assume the struct only replicates a few members and the serializer knows which ones. We don't want to mark the struct as dirty if it isn't.
		check(EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::UseSerializerIsEqual));

		SrcReplicationState = MakeUnique<FPropertyReplicationState>(InDescriptor);
	}

#if WITH_PUSH_MODEL
	if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasPushBasedDirtiness))
	{
		Traits |= EReplicationFragmentTraits::HasPushBasedDirtiness;
		if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasFullPushBasedDirtiness))
		{
			Traits |= EReplicationFragmentTraits::HasFullPushBasedDirtiness;
		}
	}
#endif
		
	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReceive))
	{
		if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasRepNotifies))
		{
			ensureMsgf(!EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::KeepPreviousState), TEXT("FMinimalReplicationTagCountMapReplicationFragment doesn't support OnRep calls with previous value."));
		}

		// We do custom callbacks in CallRepNotifies rather than in ApplyReplicatedState.
		Traits |= EReplicationFragmentTraits::HasRepNotifies | EReplicationFragmentTraits::NeedsLegacyCallbacks;
	}
}

void FMinimalReplicationTagCountMapReplicationFragment::Register(FFragmentRegistrationContext& Context)
{
	FPropertyReplicationState* ReplicationState = SrcReplicationState.Get();
	Context.RegisterReplicationFragment(this, ReplicationStateDescriptor.GetReference(), (ReplicationState ? ReplicationState->GetStateBuffer() : nullptr));
}

void FMinimalReplicationTagCountMapReplicationFragment::ApplyReplicatedState(FReplicationStateApplyContext& ApplyContext) const
{
	uint8* ExternalStatePointer = reinterpret_cast<uint8*>(Owner) + ReplicationStateDescriptor->MemberProperties[0]->GetOffset_ForGC();

	// Clear tag counts in the map. This information is used in MimicMinimalReplicationTagCountMapReceiveLogic.
	FMinimalReplicationTagCountMap* ExternalSourceState = reinterpret_cast<FMinimalReplicationTagCountMap*>(ExternalStatePointer);
	for (auto& Pair : ExternalSourceState->TagMap)
	{
		Pair.Value = 0;
	}

	// Dequantize replicated members to external source state. Need to call the serializer directly with the appropriate arguments as the external state doesn't have a ReplicationStateHeader in front of it.
	FNetDequantizeArgs DequantizeArgs = {};
	DequantizeArgs.Source = NetSerializerValuePointer(ApplyContext.StateBufferData.RawStateBuffer);
	DequantizeArgs.Target = NetSerializerValuePointer(ExternalSourceState);
	DequantizeArgs.NetSerializerConfig = ReplicationStateDescriptor->MemberSerializerDescriptors[0].SerializerConfig;
	const FNetSerializer* Serializer = ReplicationStateDescriptor->MemberSerializerDescriptors[0].Serializer;
	Serializer->Dequantize(*ApplyContext.NetSerializationContext, DequantizeArgs);

	MimicMinimalReplicationTagCountMapReceiveLogic(ApplyContext);
}

bool FMinimalReplicationTagCountMapReplicationFragment::PollReplicatedState(EReplicationFragmentPollFlags PollOption)
{
	// We can early out if we are pushbased and not dirty for polling
	const bool bPoll = EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::PollAllState) ||
	(EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::PollDirtyState) && (!EnumHasAnyFlags(EReplicationFragmentTraits::HasPushBasedDirtiness, Traits) || SrcReplicationState->IsDirtyForPolling()));

	if (bPoll)
	{
		const uint8* ExternalStateBuffer = reinterpret_cast<uint8*>(Owner) + ReplicationStateDescriptor->MemberProperties[0]->GetOffset_ForGC();
		const FMinimalReplicationTagCountMap* ExternalSourceState = reinterpret_cast<const FMinimalReplicationTagCountMap*>(ExternalStateBuffer);

		void* CachedStateBuffer = SrcReplicationState->GetStateBuffer();
		FMinimalReplicationTagCountMap* CachedState = reinterpret_cast<FMinimalReplicationTagCountMap*>(static_cast<uint8*>(CachedStateBuffer) + ReplicationStateDescriptor->MemberDescriptors[0].ExternalMemberOffset);

		if (ExternalSourceState->MapID != CachedState->MapID)
		{
			CachedState->MapID = ExternalSourceState->MapID;
			return SrcReplicationState->PollPropertyReplicationState(Owner);
		}
	}

	return SrcReplicationState->IsDirty(0);
}

void FMinimalReplicationTagCountMapReplicationFragment::CollectOwner(FReplicationStateOwnerCollector* Owners) const
{
	Owners->AddOwner(Owner);
}

void FMinimalReplicationTagCountMapReplicationFragment::CallRepNotifies(FReplicationStateApplyContext& Context)
{
	CallRepNotify(Context);
}

void FMinimalReplicationTagCountMapReplicationFragment::MimicMinimalReplicationTagCountMapReceiveLogic(FReplicationStateApplyContext& Context) const
{
	uint8* ExternalStatePointer = reinterpret_cast<uint8*>(Owner) + ReplicationStateDescriptor->MemberProperties[0]->GetOffset_ForGC();
	FMinimalReplicationTagCountMap* ExternalSourceState = reinterpret_cast<FMinimalReplicationTagCountMap*>(ExternalStatePointer);

	// Mark map dirty for replay
	ExternalSourceState->MapID++;

	// UpdateOwnerTagMap performs most of the logic regarding whether it should update or not and needs LastConnection to do it.
	const uint32 ConnectionId = Context.NetSerializationContext->GetLocalConnectionId();
	UObject* UserData = Context.NetSerializationContext->GetLocalConnectionUserData(ConnectionId);
	ExternalSourceState->LastConnection = Cast<UNetConnection>(UserData);

	const UAbilitySystemComponent* StateOwner = ExternalSourceState->Owner;
	const bool bUpdateOwnerTagMap = StateOwner != nullptr;
	if (bUpdateOwnerTagMap)
	{
		ExternalSourceState->UpdateOwnerTagMap();
	}
}

void FMinimalReplicationTagCountMapReplicationFragment::CallRepNotify(FReplicationStateApplyContext& ApplyContext)
{
	if (const UFunction* RepNotifyFunction = ReplicationStateDescriptor->MemberPropertyDescriptors[0].RepNotifyFunction)
	{
		if (ApplyContext.bIsInit)
		{
			const uint8* ReceivedState = ApplyContext.StateBufferData.RawStateBuffer;
			const uint8* DefaultState = ReplicationStateDescriptor->DefaultStateBuffer;

			if (FReplicationStateOperations::IsEqualQuantizedState(*ApplyContext.NetSerializationContext, ReceivedState, DefaultState, ReplicationStateDescriptor))
			{
				return;
			}
		}

		Owner->ProcessEvent(const_cast<UFunction*>(RepNotifyFunction), nullptr);
	}
}

}

