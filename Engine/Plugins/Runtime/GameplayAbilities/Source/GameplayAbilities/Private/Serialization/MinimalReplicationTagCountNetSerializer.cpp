// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/MinimalReplicationTagCountNetSerializer.h"

#include "GameplayAbilitiesDeveloperSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MinimalReplicationTagCountNetSerializer)


#include "GameplayEffectTypes.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Serialization/InternalMinimalReplicationTagCountMapNetSerializer.h"

namespace UE::Net
{

extern FReplicationFragment* CreateAndRegisterMinimalReplicationTagCountMapReplicationFragment(UObject* Owner, const FReplicationStateDescriptor* Descriptor, FFragmentRegistrationContext& Context);

struct FMinimalReplicationTagCountMapNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true;
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bUseSerializerIsEqual = true;

	struct FQuantizedType
	{
		alignas(16) uint8 Buffer[16];
	};

	typedef FMinimalReplicationTagCountMap SourceType;
	typedef FQuantizedType QuantizedType;
	typedef FMinimalReplicationTagCountMapNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	static SIZE_T GetMaxReplicatedTagCount();

	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
		virtual void OnPostFreezeNetSerializerRegistry() override;
	};

	inline static FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	inline static FStructNetSerializerConfig StructNetSerializerConfig;
	static const FNetSerializer* StructNetSerializer;
};

UE_NET_IMPLEMENT_SERIALIZER(FMinimalReplicationTagCountMapNetSerializer);

const FNetSerializer* FMinimalReplicationTagCountMapNetSerializer::StructNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);

void FMinimalReplicationTagCountMapNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	FNetSerializeArgs NetSerializeArgs = Args;
	NetSerializeArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->Serialize(Context, NetSerializeArgs);
}

void FMinimalReplicationTagCountMapNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetDeserializeArgs NetDeserializeArgs = Args;
	NetDeserializeArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->Deserialize(Context, NetDeserializeArgs);
}

void FMinimalReplicationTagCountMapNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	FNetSerializeDeltaArgs NetSerializeDeltaArgs = Args;
	NetSerializeDeltaArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->SerializeDelta(Context, NetSerializeDeltaArgs);
}

void FMinimalReplicationTagCountMapNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetDeserializeDeltaArgs NetDeserializeDeltaArgs = Args;
	NetDeserializeDeltaArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->DeserializeDelta(Context, NetDeserializeDeltaArgs);
}

void FMinimalReplicationTagCountMapNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	
	FMinimalReplicationTagCountMapForNetSerializer TempValue;
	TempValue.CopyReplicatedFieldsFrom(SourceValue);

	// Cap the number of replicated tags.
	const SIZE_T MaxTagCount = GetMaxReplicatedTagCount();
	TempValue.ClampTagCount(MaxTagCount);

	FNetQuantizeArgs QuantizeArgs = Args;
	QuantizeArgs.NetSerializerConfig = &StructNetSerializerConfig;
	QuantizeArgs.Source = NetSerializerValuePointer(&TempValue);
	StructNetSerializer->Quantize(Context, QuantizeArgs);
}

void FMinimalReplicationTagCountMapNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);
	
	// The temporary value is required because of two reasons; the tag count needs to be verified and the real arget is a Map, which isn't supported for replication.
	FMinimalReplicationTagCountMapForNetSerializer TempValue;

	FNetDequantizeArgs DequantizeArgs = Args;
	DequantizeArgs.NetSerializerConfig = &StructNetSerializerConfig;
	DequantizeArgs.Target = NetSerializerValuePointer(&TempValue);
	StructNetSerializer->Dequantize(Context, DequantizeArgs);

	// Don't allow too large arrays.
	if (TempValue.GetTagCount() > GetMaxReplicatedTagCount())
	{
		Context.SetError(GNetError_ArraySizeTooLarge);
		return;
	}

	TempValue.AssignReplicatedFieldsTo(TargetValue);
}

bool FMinimalReplicationTagCountMapNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		FNetIsEqualArgs IsEqualArgs = Args;
		IsEqualArgs.NetSerializerConfig = &StructNetSerializerConfig;
		if (!StructNetSerializer->IsEqual(Context, IsEqualArgs))
		{
			return false;
		}
	}
	else
	{
		const SourceType& Source0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Source1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		FMinimalReplicationTagCountMapForNetSerializer TempValue0;
		FMinimalReplicationTagCountMapForNetSerializer TempValue1;

		const SIZE_T MaxTagCount = GetMaxReplicatedTagCount();
		TempValue0.CopyReplicatedFieldsFrom(Source0);
		TempValue0.ClampTagCount(MaxTagCount);

		TempValue1.CopyReplicatedFieldsFrom(Source1);
		TempValue1.ClampTagCount(MaxTagCount);

		FNetIsEqualArgs IsEqualArgs = Args;
		IsEqualArgs.Source0 = NetSerializerValuePointer(&TempValue0);
		IsEqualArgs.Source1 = NetSerializerValuePointer(&TempValue1);
		IsEqualArgs.NetSerializerConfig = &StructNetSerializerConfig;
		if (!StructNetSerializer->IsEqual(Context, IsEqualArgs))
		{
			return false;
		}
	}

	return true;
}

bool FMinimalReplicationTagCountMapNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	
	FMinimalReplicationTagCountMapForNetSerializer TempValue;
	TempValue.CopyReplicatedFieldsFrom(SourceValue);

	// MinimalReplicationTagCountMap logs an error when too many tags are detected. We return false during validation. 
	// Similar to the original implementation we clamp the number of tags that will be replicated. This is done during quantization.
	const SIZE_T MaxTagCount = GetMaxReplicatedTagCount();
	if (TempValue.GetTagCount() > MaxTagCount)
	{
		return false;
	}

	FNetValidateArgs ValidateArgs = Args;
	ValidateArgs.NetSerializerConfig = &StructNetSerializerConfig;
	ValidateArgs.Source = NetSerializerValuePointer(&TempValue);
	return StructNetSerializer->Validate(Context, ValidateArgs);
}

void FMinimalReplicationTagCountMapNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
}

void FMinimalReplicationTagCountMapNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	FNetCloneDynamicStateArgs CloneDynamicStateArgs = Args;
	CloneDynamicStateArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->CloneDynamicState(Context, CloneDynamicStateArgs);
}

void FMinimalReplicationTagCountMapNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	FNetFreeDynamicStateArgs FreeDynamicStateArgs = Args;
	FreeDynamicStateArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->FreeDynamicState(Context, FreeDynamicStateArgs);
}

SIZE_T FMinimalReplicationTagCountMapNetSerializer::GetMaxReplicatedTagCount()
{
	const UGameplayAbilitiesDeveloperSettings* DeveloperSettings = GetDefault<UGameplayAbilitiesDeveloperSettings>();
	const int32 BitCount = DeveloperSettings->MinimalReplicationTagCountBits;
	const int32 ClampedBitCount = FMath::Clamp(BitCount, 1, 16);

	const uint32 MaxCount = (1U << static_cast<uint32>(ClampedBitCount)) - 1U;
	return MaxCount;
}

static const FName PropertyNetSerializerRegistry_NAME_MinimalReplicationTagCountMap("MinimalReplicationTagCountMap");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_WITH_CUSTOM_FRAGMENT_INFO(PropertyNetSerializerRegistry_NAME_MinimalReplicationTagCountMap, FMinimalReplicationTagCountMapNetSerializer, CreateAndRegisterMinimalReplicationTagCountMapReplicationFragment);

FMinimalReplicationTagCountMapNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{	
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_MinimalReplicationTagCountMap);
}

void FMinimalReplicationTagCountMapNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_MinimalReplicationTagCountMap);
}

void FMinimalReplicationTagCountMapNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	constexpr SIZE_T ExpectedSizeOfFMinimalReplicationTagCountMap = 32 + sizeof(TMap<FGameplayTag, int32>);
#else
	constexpr SIZE_T ExpectedSizeOfFMinimalReplicationTagCountMap = 24 + sizeof(TMap<FGameplayTag, int32>);
#endif
	constexpr SIZE_T ExpectedAlignOfFMinimalReplicationTagCountMap = 8;

	constexpr SIZE_T ActualSizeOfFMinimalReplicationTagCountMap = sizeof(FMinimalReplicationTagCountMap);
	constexpr SIZE_T ActualAlignOfFMinimalReplicationTagCountMap = alignof(FMinimalReplicationTagCountMap);

	// Try to detect changes to FMinimalReplicationTagCountMap. Ensure instead of check as it's not known to be an error until verified.
	ensureMsgf(ExpectedSizeOfFMinimalReplicationTagCountMap == ActualSizeOfFMinimalReplicationTagCountMap && ExpectedAlignOfFMinimalReplicationTagCountMap == ActualAlignOfFMinimalReplicationTagCountMap, TEXT("%s"), TEXT("FMinimalReplicationTagCountMap layout has changed. Might need to update FMinimalReplicationTagCountMapNetSerializer to include new data."));

	const UStruct* TagCountMapStruct = FMinimalReplicationTagCountMapForNetSerializer::StaticStruct();
	FReplicationStateDescriptorBuilder::FParameters Params;
	StructNetSerializerConfig.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(TagCountMapStruct, Params);

	const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfig.StateDescriptor.GetReference();

	// Validate our assumptions regarding quantized state size and alignment.
	if ((sizeof(QuantizedType) < Descriptor->InternalSize) || (alignof(QuantizedType) < Descriptor->InternalAlignment))
	{
		LowLevelFatalError(TEXT("FMinimalReplicationTagCountMapNetSerializer::FQuantizedType has size %u and alignment %u but requires size %u and alignment %u."), uint32(sizeof(FQuantizedType)), uint32(alignof(FQuantizedType)), Descriptor->InternalSize, Descriptor->InternalAlignment);
	}

	ensureMsgf(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasDynamicState), TEXT("FMinimalReplicationTagCountMapNetSerializer seems to no longer have dynamic state. Should remove trait from the NetSerializer."));
	ensureMsgf(!EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasObjectReference), TEXT("FMinimalReplicationTagCountMapNetSerializer seems to have object references. Need to add trait to the NetSerializer."));
}

}

