// Copyright Epic Games, Inc. All Rights Reserved.

#include "PredictionKeyNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PredictionKeyNetSerializer)


#include "GameplayPrediction.h"
#include "Engine/NetConnection.h"
#include "Engine/PackageMapClient.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializerDelegates.h"

namespace UE::Net
{

struct FPredictionKeyNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bHasConnectionSpecificSerialization = true;

	// Types
	struct FQuantizedType
	{
		uint16 CurrentKey;
		uint16 OwningConnectionId;
		uint16 bIsServerInitiated : 1;
	};

	typedef FPredictionKey SourceType;
	typedef FQuantizedType QuantizedType;
	typedef FPredictionKeyNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	// 
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
	};

	static FPredictionKeyNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};

UE_NET_IMPLEMENT_SERIALIZER(FPredictionKeyNetSerializer);

const FPredictionKeyNetSerializer::ConfigType FPredictionKeyNetSerializer::DefaultConfig;
FPredictionKeyNetSerializer::FNetSerializerRegistryDelegates FPredictionKeyNetSerializer::NetSerializerRegistryDelegates;

const UNetConnection* GetNetConnectionEnsured(const FNetSerializationContext& Context, uint32 ConnectionId)
{
	// We won't always have UserData e.g. we could be (de-)serializing a default state for delta serialization
	if (UObject* UserData = Context.GetLocalConnectionUserData(ConnectionId))
	{
		const UNetConnection* NetConnection = Cast<UNetConnection>(UserData);
		ensureMsgf(NetConnection, TEXT("Expected FNetSerializationContext::GetLocalConnectionUserData to always return a NetConnection. Instead we received %s"), *GetNameSafe(UserData));

		return NetConnection;
	}

	return nullptr;
}

void FPredictionKeyNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	Writer->WriteBool(Value.bIsServerInitiated);

	const UNetConnection* NetConnection = GetNetConnectionEnsured(Context, Context.GetLocalConnectionId());
	const bool bIsServerSendingToProxy = NetConnection && NetConnection->IsProxyConnection();
	const bool bIsProxySendingToServer = NetConnection && (NetConnection->GetDriver()->GetProxyType() == UE::Net::EProxyType::Backend);
	const bool bShouldWriteKey = (Value.CurrentKey > 0) && (Value.bIsServerInitiated || (Value.OwningConnectionId == 0) || (Value.OwningConnectionId == Context.GetLocalConnectionId()) || bIsServerSendingToProxy || bIsProxySendingToServer);
	if (Writer->WriteBool(bShouldWriteKey))
	{
		Writer->WriteBits(Value.CurrentKey, 15U);
	}
}

void FPredictionKeyNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	QuantizedType TempValue = {};
	TempValue.bIsServerInitiated = Reader->ReadBool();
	if (Reader->ReadBool())
	{
		TempValue.CurrentKey = Reader->ReadBits(15U);
	}

	if (TempValue.CurrentKey > 0 && !TempValue.bIsServerInitiated)
	{
		TempValue.OwningConnectionId = Context.GetLocalConnectionId();
	}

	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = TempValue;
}

void FPredictionKeyNetSerializer::Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = {};

	Target.bIsServerInitiated = Source.IsServerInitiatedKey();
	if (Source.IsValidKey())
	{
		Target.CurrentKey = uint16(Source.Current);

		// The owning connection is only relevant if the key is valid and we've _received_ this key previously (PredictiveConnectionObjectKey was set)
		if (UPackageMapClient* PackageMap = Cast<UPackageMapClient>(Source.PredictiveConnectionObjectKey.ResolveObjectPtr()))
		{
			if (const UNetConnection* Connection = PackageMap->GetConnection())
			{
				uint32 OwningConnectionId = Connection->GetConnectionHandle().GetParentConnectionId();

				ensureMsgf(!Target.OwningConnectionId || Target.OwningConnectionId == OwningConnectionId,
					TEXT("FPredictionKeyNetSerializer::Quantize is stomping an existing OwningConnectionId with a new one (this should never happen)"));

				Target.OwningConnectionId = OwningConnectionId;
			}
		}
	}
}

void FPredictionKeyNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.bIsServerInitiated = Source.bIsServerInitiated;
	Target.Current = int16(Source.CurrentKey);

	// This is set in Deserialize if the key is valid.
	if (Source.OwningConnectionId != 0)
	{
		if (const UNetConnection* NetConnection = GetNetConnectionEnsured(Context, Source.OwningConnectionId))
		{
			Target.PredictiveConnectionObjectKey = FObjectKey(NetConnection->PackageMap);
		}
	}
}

bool FPredictionKeyNetSerializer::IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		const bool bIsEqual = (Value0.CurrentKey == Value1.CurrentKey) & (Value0.bIsServerInitiated == Value1.bIsServerInitiated) & (Value0.OwningConnectionId == Value1.OwningConnectionId);

		return bIsEqual;
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		if (Value0.IsValidKey() != Value1.IsValidKey())
		{
			return false;
		}

		if (Value0.IsValidKey())
		{
			const bool bIsSameKey = (Value0.Current == Value1.Current) & (Value0.bIsServerInitiated == Value1.bIsServerInitiated);
			if (!bIsSameKey)
			{
				return false;
			}

			if (Value0.GetPredictiveConnectionKey() != Value1.GetPredictiveConnectionKey())
			{
				return false;
			}

			return true;
		}
		else
		{
			// If both keys are invalid then they're net equal as the receiving side will get the same key.
			return true;
		}
	}
}

bool FPredictionKeyNetSerializer::Validate(FNetSerializationContext&, const FNetValidateArgs& Args)
{
	return true;
}

static const FName PropertyNetSerializerRegistry_NAME_PredictionKey("PredictionKey");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_PredictionKey, FPredictionKeyNetSerializer);

FPredictionKeyNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_PredictionKey);
}

void FPredictionKeyNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_PredictionKey);
}

}

