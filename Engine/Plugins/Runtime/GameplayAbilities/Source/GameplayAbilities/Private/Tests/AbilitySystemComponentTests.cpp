// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Engine/DataTable.h"
#include "Engine/Engine.h"
#include "GameplayAbilitiesModule.h"
#include "GameplayTagsModule.h"
#include "GameFramework/PlayerController.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemTestPawn.h"
#include "AbilitySystemGlobals.h"
#include "NativeGameplayTags.h"
#include "Engine/LocalPlayer.h"

#if WITH_EDITOR

/** 
 * This is just a scoped class that helps register for all of the AbilitySystemComponent Callbacks
 */
struct TestAllAbilitySystemComponentCallbacks
{
	bool bReceivedAbilityActivated = false;
	bool bReceivedAbilityCommitted = false;
	bool bReceiveAbilityFailed = false;
	bool bReceiveAbilityEnded = false;

	static bool IsSameAbility(const UGameplayAbility* GameplayAbility, const UGameplayAbility* ExpectedAbility)
	{
		if (ExpectedAbility->GetInstancingPolicy() == EGameplayAbilityInstancingPolicy::NonInstanced)
		{
			return GameplayAbility == ExpectedAbility;
		}
		else
		{
			return GameplayAbility->GetClass()->GetDefaultObject() == ExpectedAbility;
		}
	}

	TestAllAbilitySystemComponentCallbacks(UAbilitySystemComponent* InAbilitySystemComponent, FAutomationTestBase* Test, const UGameplayAbility* ExpectedAbility)
		: AbilitySystemComponent(InAbilitySystemComponent)
	{
		check(AbilitySystemComponent);
		check(Test);
		check(ExpectedAbility);

		// Register All ASC Callbacks so we can detect the flow
		OnAbilityActivated = AbilitySystemComponent->AbilityActivatedCallbacks.AddLambda([this, ExpectedAbility, Test](UGameplayAbility* InGameplayAbility)
			{
				const bool bIsCorrectAbility = IsSameAbility(InGameplayAbility, ExpectedAbility);
				Test->TestTrue(TEXT(" AbilityActivatedCallbacks with Expected GameplayAbility Instance"), bIsCorrectAbility);
				bReceivedAbilityActivated = true;
			});

		OnAbilityCommitted = AbilitySystemComponent->AbilityCommittedCallbacks.AddLambda([this, ExpectedAbility, Test](UGameplayAbility* InGameplayAbility)
			{
				const bool bIsCorrectAbility = IsSameAbility(InGameplayAbility, ExpectedAbility);
				Test->TestTrue(TEXT(" AbilityCommittedCallbacks with Expected GameplayAbility Instance"), bIsCorrectAbility);
				bReceivedAbilityCommitted = true;
			});

		OnAbilityFailed = AbilitySystemComponent->AbilityFailedCallbacks.AddLambda([this, ExpectedAbility, Test](const UGameplayAbility* InGameplayAbility, const FGameplayTagContainer&)
			{
				const bool bIsCorrectAbility = IsSameAbility(InGameplayAbility, ExpectedAbility);
				Test->TestTrue(TEXT(" AbilityFailedCallbacks with Expected GameplayAbility Instance"), bIsCorrectAbility);
				bReceiveAbilityFailed = true;
			});

		OnAbilityEnded = AbilitySystemComponent->AbilityEndedCallbacks.AddLambda([this, ExpectedAbility, Test](UGameplayAbility* InGameplayAbility)
			{
				const bool bIsCorrectAbility = IsSameAbility(InGameplayAbility, ExpectedAbility);
				Test->TestTrue(TEXT(" AbilityEndedCallbacks with Expected GameplayAbility Instance"), bIsCorrectAbility);
				bReceiveAbilityEnded = true;
			});
	}

	~TestAllAbilitySystemComponentCallbacks()
	{
		AbilitySystemComponent->AbilityActivatedCallbacks.Remove(OnAbilityActivated);
		AbilitySystemComponent->AbilityCommittedCallbacks.Remove(OnAbilityCommitted);
		AbilitySystemComponent->AbilityFailedCallbacks.Remove(OnAbilityFailed);
		AbilitySystemComponent->AbilityEndedCallbacks.Remove(OnAbilityEnded);
	}

private:
	UAbilitySystemComponent* AbilitySystemComponent = nullptr;
	FDelegateHandle OnAbilityActivated;
	FDelegateHandle OnAbilityCommitted;
	FDelegateHandle OnAbilityFailed;
	FDelegateHandle OnAbilityEnded;
}; // End TestAllAbilitySystemComponentCallbacks

/**
 * The implementation of the test suite
 */
class AbilitySystemComponentTestSuite
{
public:
	AbilitySystemComponentTestSuite(UWorld* WorldIn, FAutomationTestBase* TestIn)
		: World(WorldIn)
		, Test(TestIn)
	{
		// run before each test

		// set up the source actor
		SourceActor = World->SpawnActor<AAbilitySystemTestPawn>();
		SourceController = World->SpawnActor<APlayerController>();
		SourceController->Possess(SourceActor);
		SourceLocalPlayer = NewObject<ULocalPlayer>(GEngine, ULocalPlayer::StaticClass());
		SourceController->SetPlayer(SourceLocalPlayer);
		SourceComponent = SourceActor->GetAbilitySystemComponent();

		// set up the destination actor
		DestActor = World->SpawnActor<AAbilitySystemTestPawn>();
		DestComponent = DestActor->GetAbilitySystemComponent();
	}

	~AbilitySystemComponentTestSuite()
	{
		// run after each test

		// destroy the actors
		if (SourceActor)
		{
			World->EditorDestroyActor(SourceActor, false);
		}
		if (DestActor)
		{
			World->EditorDestroyActor(DestActor, false);
		}
	}

public: // the tests

	void Test_ActivateAbilityFlow()
	{
		// Tricky part here:  You pass in the FGameplayAbilitySpec to GiveAbility, but it makes a copy and leaves yours unchanged.
		FGameplayAbilitySpec TempAbilitySpec{ UGameplayAbility::StaticClass(), 1 };
		FGameplayAbilitySpecHandle GivenAbilitySpecHandle = SourceComponent->GiveAbility(TempAbilitySpec);
		FGameplayAbilitySpec* AbilitySpec = SourceComponent->FindAbilitySpecFromHandle(GivenAbilitySpecHandle);

		TArray<FGameplayAbilitySpecHandle> GameplayAbilitySpecHandles;
		SourceComponent->GetAllAbilities(GameplayAbilitySpecHandles);
		const bool bHasAbility = GameplayAbilitySpecHandles.Num() > 0;
		const bool bHasCorrectAbility = bHasAbility && (GameplayAbilitySpecHandles[0] == GivenAbilitySpecHandle);
		Test->TestTrue(TEXT("GiveAbility"), bHasCorrectAbility);

		const TArray<FGameplayAbilitySpec>& ActivatableAbilities = SourceComponent->GetActivatableAbilities();
		const bool bHasActivatableAbility = ActivatableAbilities.Num() > 0;
		const bool bBothArraysMatch = bHasAbility && bHasActivatableAbility && (GameplayAbilitySpecHandles[0] == ActivatableAbilities[0].Handle);
		Test->TestTrue(TEXT("GetAllAbilities() == GetActivatableAbilities().Handle"), bBothArraysMatch);

		if (!Test->HasAnyErrors())
		{
			TestAllAbilitySystemComponentCallbacks TestCallbacks{ SourceComponent, Test, AbilitySpec->Ability };

			// Since we have a Controller for the SourceActor in this case, we're executing this all locally.
			// Try to activate the ability, and see that we've gone through the correct flow.
			constexpr bool bAllowRemoteActivation = false;
			const bool bLocalActivation = SourceComponent->TryActivateAbility(GivenAbilitySpecHandle, bAllowRemoteActivation);
			Test->TestTrue(TEXT("TryActivateAbility executes successfully (using FGameplayAbilitySpecHandle)"), bLocalActivation);
			Test->TestTrue(TEXT(" AbilitySpec.IsActive() after TryActivateAbility (using FGameplayAbilitySpecHandle)"), AbilitySpec->IsActive());
			Test->TestTrue(TEXT(" AbilityActivated after TryActivateAbility (using FGameplayAbilitySpecHandle)"), TestCallbacks.bReceivedAbilityActivated);
			Test->TestFalse(TEXT(" AbilityCommitted after TryActivateAbility (using FGameplayAbilitySpecHandle)"), TestCallbacks.bReceivedAbilityCommitted);
			Test->TestFalse(TEXT(" AbilityEnded (prematurely) after TryActivateAbility (using FGameplayAbilitySpecHandle)"), TestCallbacks.bReceiveAbilityEnded);
			Test->TestFalse(TEXT(" AbilityFailed after TryActivateAbility (with an Ability that should succeed)"), TestCallbacks.bReceiveAbilityFailed);

			// Now Cancel the ability and see if we handle that correctly.
			SourceComponent->CancelAbilityHandle(GivenAbilitySpecHandle);
			Test->TestTrue(TEXT(" AbilityEnded (after CancelAbilityHandle)"), TestCallbacks.bReceiveAbilityEnded);
			Test->TestFalse(TEXT(" AbilitySpec.IsActive() (after CancelAbilityHandle)"), AbilitySpec->IsActive());
		}
	}

	void Test_FailedAbilityFlow()
	{
		// Tricky part here:  You pass in the FGameplayAbilitySpec to GiveAbility, but it makes a copy and leaves yours unchanged.
		FGameplayAbilitySpec TempAbilitySpec{ UGameplayAbility::StaticClass(), 1 };
		FGameplayAbilitySpecHandle GivenAbilitySpecHandle = SourceComponent->GiveAbility(TempAbilitySpec);
		FGameplayAbilitySpec* AbilitySpec = SourceComponent->FindAbilitySpecFromHandle(GivenAbilitySpecHandle);

		TArray<FGameplayAbilitySpecHandle> GameplayAbilitySpecHandles;
		SourceComponent->GetAllAbilities(GameplayAbilitySpecHandles);
		const bool bHasAbility = GameplayAbilitySpecHandles.Num() > 0;
		const bool bHasCorrectAbility = bHasAbility && (GameplayAbilitySpecHandles[0] == GivenAbilitySpecHandle);
		Test->TestTrue(TEXT("GiveAbility"), bHasCorrectAbility);

		const TArray<FGameplayAbilitySpec>& ActivatableAbilities = SourceComponent->GetActivatableAbilities();
		const bool bHasActivatableAbility = ActivatableAbilities.Num() > 0;
		const bool bBothArraysMatch = bHasAbility && bHasActivatableAbility && (GameplayAbilitySpecHandles[0] == ActivatableAbilities[0].Handle);
		Test->TestTrue(TEXT("GetAllAbilities() == GetActivatableAbilities().Handle"), bBothArraysMatch);

		if (!Test->HasAnyErrors())
		{
			TestAllAbilitySystemComponentCallbacks TestCallbacks{ SourceComponent, Test, AbilitySpec->Ability };

			// Since we have a Controller for the SourceActor in this case, we're executing this all locally.
			// Inhibit Activation, then try to Activate it to trigger the failed flow.
			SourceComponent->SetUserAbilityActivationInhibited(true);

			constexpr bool bAllowRemoteActivation = false;
			const bool bLocalActivation = SourceComponent->TryActivateAbility(GivenAbilitySpecHandle, bAllowRemoteActivation);

			Test->TestFalse(TEXT("TryActivateAbility fails (using FGameplayAbilitySpecHandle)"), bLocalActivation);
			Test->TestFalse(TEXT(" AbilitySpec.IsActive() after TryActivateAbility (using FGameplayAbilitySpecHandle)"), AbilitySpec->IsActive());
			Test->TestFalse(TEXT(" AbilityActivated after TryActivateAbility (using FGameplayAbilitySpecHandle)"), TestCallbacks.bReceivedAbilityActivated);
			Test->TestFalse(TEXT(" AbilityCommitted after TryActivateAbility (using FGameplayAbilitySpecHandle)"), TestCallbacks.bReceivedAbilityCommitted);
			Test->TestFalse(TEXT(" AbilityEnded (prematurely) after TryActivateAbility (using FGameplayAbilitySpecHandle)"), TestCallbacks.bReceiveAbilityEnded);
			Test->TestTrue(TEXT(" AbilityFailed after TryActivateAbility (with an Ability that should fail)"), TestCallbacks.bReceiveAbilityFailed);
		}
	}

private: // test helpers

	void TickWorld(float Time)
	{
		const float step = 0.1f;
		while (Time > 0.f)
		{
			World->Tick(ELevelTick::LEVELTICK_All, FMath::Min(Time, step));
			Time -= step;

			// This is terrible but required for subticking like this.
			// we could always cache the real GFrameCounter at the start of our tests and restore it when finished.
			GFrameCounter++;
		}
	}

private:
	UWorld* World;
	FAutomationTestBase* Test;

	AAbilitySystemTestPawn* SourceActor;
	UAbilitySystemComponent* SourceComponent;
	APlayerController* SourceController;
	ULocalPlayer* SourceLocalPlayer;

	AAbilitySystemTestPawn* DestActor;
	UAbilitySystemComponent* DestComponent;
}; // End AbilitySystemComponentTestSuite

/**
 * This is the Automation Test definition that lets the automation system know which tests we can run (and how to run them)
 * The implementation will take care of setting up the World and tearing it down.
 * It will instantiate a fresh AbilitySystemComponentTestSuite for each test.
 */
class FAbilitySystemComponentTest : public FAutomationTestBase
{
public:
	typedef void (AbilitySystemComponentTestSuite::*TestFunc)();

	FAbilitySystemComponentTest(const FString& InName)
		: FAutomationTestBase(InName, false)
	{
		auto AddTest = [&](const TestFunc& InTestFunction, const FString& InTestName)
		{
			TestFunctions.Add(InTestFunction);
			TestFunctionNames.Add(InTestName);
		};

		// list all test functions here
		AddTest(&AbilitySystemComponentTestSuite::Test_ActivateAbilityFlow, TEXT("ActivateAbilityFlow"));
		AddTest(&AbilitySystemComponentTestSuite::Test_FailedAbilityFlow, TEXT("FailedAbilityFlow"));
	}

	virtual EAutomationTestFlags GetTestFlags() const override { return EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter; }
	virtual bool IsStressTest() const { return false; }
	virtual uint32 GetRequiredDeviceNum() const override { return 1; }

protected:
	virtual FString GetBeautifiedTestName() const override { return "System.AbilitySystem.AbilitySystemComponent"; }
	virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const override
	{
		for (const FString& TestFuncName : TestFunctionNames)
		{
			OutBeautifiedNames.Add(TestFuncName);
			OutTestCommands.Add(TestFuncName);
		}
	}

	bool RunTest(const FString& Parameters) override
	{
		// find the matching test
		TestFunc TestFunction = nullptr;
		for (int32 i = 0; i < TestFunctionNames.Num(); ++i)
		{
			if (TestFunctionNames[i] == Parameters)
			{
				TestFunction = TestFunctions[i];
				break;
			}
		}
		if (TestFunction == nullptr)
		{
			return false;
		}

		// get the current data table (to restore later)
		UDataTable *DataTable = IGameplayAbilitiesModule::Get().GetAbilitySystemGlobals()->GetGlobalAttributeMetaDataTable();

		UWorld *World = UWorld::CreateWorld(EWorldType::Game, false);
		FWorldContext &WorldContext = GEngine->CreateNewWorldContext(EWorldType::Game);
		WorldContext.SetCurrentWorld(World);

		FURL URL;
		//World->GetGameInstanceChecked<UGameInstance>()->EnableListenServer(true);
		World->InitializeActorsForPlay(URL);
		World->BeginPlay();

		// run the matching test
		uint64 InitialFrameCounter = GFrameCounter;
		{
			AbilitySystemComponentTestSuite Tester(World, this);
			(Tester.*TestFunction)();
		}
		GFrameCounter = InitialFrameCounter;

		World->EndPlay(EEndPlayReason::Quit);
		GEngine->DestroyWorldContext(World);
		World->DestroyWorld(false);

		IGameplayAbilitiesModule::Get().GetAbilitySystemGlobals()->AutomationTestOnly_SetGlobalAttributeDataTable(DataTable);
		return true;
	}

	TArray<TestFunc> TestFunctions;
	TArray<FString> TestFunctionNames;
};

namespace
{
	// Global instance that will register itself to participate in the automated tests
	FAbilitySystemComponentTest FGameplayAbilitySystemComponentAutomationTestInstance(TEXT("FAbilitySystemComponentTest"));
}

#endif //WITH_EDITOR
