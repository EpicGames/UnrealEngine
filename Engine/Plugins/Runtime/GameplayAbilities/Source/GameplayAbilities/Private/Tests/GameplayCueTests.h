// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayCueNotify_Static.h"
#include "NativeGameplayTags.h"
#include "GameplayCueTests.generated.h"

namespace UE::GameplayTags
{
	UE_DECLARE_GAMEPLAY_TAG_EXTERN(GameplayCue_Test);
}

/** Helper GameplayCueNotify that is used for unit testing.  Note: Since this is a GCN_Static, we're using the CDO during testing */
UCLASS(NotBlueprintType, MinimalAPI)
class UGameplayCueNotify_UnitTest : public UGameplayCueNotify_Static
{
	GENERATED_BODY()

public:
	virtual void PostInitProperties() override
	{
		Super::PostInitProperties();
		GameplayCueTag = UE::GameplayTags::GameplayCue_Test;
	}

	/** Called when a GameplayCue is executed, this is used for instant effects or periodic ticks */
	virtual bool OnExecute_Implementation(AActor* MyTarget, const FGameplayCueParameters& Parameters) const override { ++NumOnExecuteCalls; return true; }

	/** Called when a GameplayCue with duration is first activated, this will only be called if the client witnessed the activation */
	virtual bool OnActive_Implementation(AActor* MyTarget, const FGameplayCueParameters& Parameters) const override { ++NumOnActiveCalls; return true; }

	/** Called when a GameplayCue with duration is first seen as active, even if it wasn't actually just applied (Join in progress, etc) */
	virtual bool WhileActive_Implementation(AActor* MyTarget, const FGameplayCueParameters& Parameters) const override { ++NumWhileActiveCalls; return true; }

	/** Called when a GameplayCue with duration is removed */
	virtual bool OnRemove_Implementation(AActor* MyTarget, const FGameplayCueParameters& Parameters) const override { ++NumOnRemoveCalls; return true; }

	/** These are counters for how many times the functions are called (so we can verify we get the right amounts).  All mutable because the UGCN_Static function overrides are const. */
	mutable int NumOnExecuteCalls = 0;
	mutable int NumOnActiveCalls = 0;
	mutable int NumWhileActiveCalls = 0;
	mutable int NumOnRemoveCalls = 0;

	/** Reset all of the counts */
	inline void ResetCallCounts()
	{
		NumOnExecuteCalls = NumOnActiveCalls = NumWhileActiveCalls = NumOnRemoveCalls = 0;
	}
};

