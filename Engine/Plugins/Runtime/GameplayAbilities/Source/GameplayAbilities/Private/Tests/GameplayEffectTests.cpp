// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/DataTable.h"
#include "Engine/Engine.h"
#include "AttributeSet.h"
#include "GameplayEffectTypes.h"
#include "GameplayEffect.h"
#include "GameplayAbilitiesModule.h"
#include "GameplayTagsManager.h"
#include "GameplayTagsModule.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemTestPawn.h"
#include "AbilitySystemTestAttributeSet.h"
#include "AbilitySystemGlobals.h"

// For Testing Gameplay Cues
#include "GameplayCueTests.h"
#include "NativeGameplayTags.h"
#include "GameplayCueManager.h"
#include "GameplayCueSet.h"
#include "GameplayCueNotify_Static.h"

#define SKILL_TEST_TEXT( Format, ... ) FString::Printf(TEXT("%s - %d: %s"), TEXT(__FILE__) , __LINE__ , *FString::Printf(TEXT(Format), ##__VA_ARGS__) )

namespace UE::GameplayTags
{
	UE_DEFINE_GAMEPLAY_TAG_COMMENT(GameplayCue_Test, "GameplayCue.Test", "GameplayCue Test Tag used for Unit Tests");
}

#if WITH_EDITOR

static UDataTable* CreateGameplayDataTable()
{
	FString CSV(TEXT("---,Tag,DevComment"));
	CSV.Append(TEXT("\r\n0,Damage,"));
	CSV.Append(TEXT("\r\n1,Damage.Basic,"));
	CSV.Append(TEXT("\r\n2,Damage.Type1,"));
	CSV.Append(TEXT("\r\n3,Damage.Type2,"));
	CSV.Append(TEXT("\r\n4,Damage.Reduce,"));
	CSV.Append(TEXT("\r\n5,Damage.Buffable,"));
	CSV.Append(TEXT("\r\n6,Damage.Buff,"));
	CSV.Append(TEXT("\r\n7,Damage.Physical,"));
	CSV.Append(TEXT("\r\n8,Damage.Fire,"));
	CSV.Append(TEXT("\r\n9,Damage.Buffed.FireBuff,"));
	CSV.Append(TEXT("\r\n10,Damage.Mitigated.Armor,"));
	CSV.Append(TEXT("\r\n11,Lifesteal,"));
	CSV.Append(TEXT("\r\n12,Shield,"));
	CSV.Append(TEXT("\r\n13,Buff,"));
	CSV.Append(TEXT("\r\n14,Immune,"));
	CSV.Append(TEXT("\r\n15,FireDamage,"));
	CSV.Append(TEXT("\r\n16,ShieldAbsorb,"));
	CSV.Append(TEXT("\r\n17,Stackable,"));
	CSV.Append(TEXT("\r\n18,Stack,"));
	CSV.Append(TEXT("\r\n19,Stack.CappedNumber,"));
	CSV.Append(TEXT("\r\n20,Stack.DiminishingReturns,"));
	CSV.Append(TEXT("\r\n21,Protect.Damage,"));
	CSV.Append(TEXT("\r\n22,SpellDmg.Buff,"));
	CSV.Append(TEXT("\r\n23,GameplayCue.Burning,"));

	UDataTable* DataTable = NewObject<UDataTable>(GetTransientPackage(), FName(TEXT("TempDataTable")));
	DataTable->RowStruct = FGameplayTagTableRow::StaticStruct();
	DataTable->CreateTableFromCSVString(CSV);

	const FGameplayTagTableRow* Row = (const FGameplayTagTableRow*)DataTable->GetRowMap()["0"];
	if (Row)
	{
		check(Row->Tag == TEXT("Damage"));
	}
	return DataTable;
}

#define GET_FIELD_CHECKED(Class, Field) FindFieldChecked<FProperty>(Class::StaticClass(), GET_MEMBER_NAME_CHECKED(Class, Field))
#define CONSTRUCT_CLASS(Class, Name) Class* Name = NewObject<Class>(GetTransientPackage(), FName(TEXT(#Name)))

class GameplayEffectsTestSuite
{
public:
	GameplayEffectsTestSuite(UWorld* WorldIn, FAutomationTestBase* TestIn)
	: World(WorldIn)
	, Test(TestIn)
	{
		// run before each test

		const float StartingHealth = 100.f;
		const float StartingMana = 200.f;

		// set up the source actor
		SourceActor = World->SpawnActor<AAbilitySystemTestPawn>();
		SourceComponent = SourceActor->GetAbilitySystemComponent();
		SourceComponent->GetSet<UAbilitySystemTestAttributeSet>()->Health = StartingHealth;
		SourceComponent->GetSet<UAbilitySystemTestAttributeSet>()->MaxHealth = StartingHealth;
		SourceComponent->GetSet<UAbilitySystemTestAttributeSet>()->Mana = FGameplayAttributeData(StartingMana);
		SourceComponent->GetSet<UAbilitySystemTestAttributeSet>()->MaxMana = StartingMana;

		// set up the destination actor
		DestActor = World->SpawnActor<AAbilitySystemTestPawn>();
		DestComponent = DestActor->GetAbilitySystemComponent();
		DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Health = StartingHealth;
		DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->MaxHealth = StartingHealth;
		DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Mana = FGameplayAttributeData(StartingMana);
		DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->MaxMana = StartingMana;
	}

	~GameplayEffectsTestSuite()
	{
		// run after each test

		// destroy the actors
		if (SourceActor)
		{
			World->EditorDestroyActor(SourceActor, false);
		}
		if (DestActor)
		{
			World->EditorDestroyActor(DestActor, false);
		}
	}

public: // the tests

	void Test_InstantDamage()
	{
		const float DamageValue = 5.f;
		const float StartingHealth = DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Health;

		// just try and reduce the health attribute
		{
			
			CONSTRUCT_CLASS(UGameplayEffect, BaseDmgEffect);
			AddModifier(BaseDmgEffect, GET_FIELD_CHECKED(UAbilitySystemTestAttributeSet, Health), EGameplayModOp::Additive, FScalableFloat(-DamageValue));
			BaseDmgEffect->DurationPolicy = EGameplayEffectDurationType::Instant;
			
			SourceComponent->ApplyGameplayEffectToTarget(BaseDmgEffect, DestComponent, 1.f);
		}

		// make sure health was reduced
		Test->TestEqual(TEXT("Health Reduced"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Health, StartingHealth - DamageValue);
	}

	void Test_InstantDamageRemap()
	{
		const float DamageValue = 5.f;
		const float StartingHealth = DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Health;

		// This is the same as GameplayEffectsTest_InstantDamage but modifies the Damage attribute and confirms it is remapped to -Health
		{
			CONSTRUCT_CLASS(UGameplayEffect, BaseDmgEffect);
			AddModifier(BaseDmgEffect, GET_FIELD_CHECKED(UAbilitySystemTestAttributeSet, Damage), EGameplayModOp::Additive, FScalableFloat(DamageValue));
			BaseDmgEffect->DurationPolicy = EGameplayEffectDurationType::Instant;

			SourceComponent->ApplyGameplayEffectToTarget(BaseDmgEffect, DestComponent, 1.f);
		}

		// Now we should have lost some health
		Test->TestEqual(TEXT("Health Reduced"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Health, StartingHealth - DamageValue);

		// Confirm the damage attribute itself was reset to 0 when it was applied to health
		Test->TestEqual(TEXT("Damage Applied"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Damage, 0.f);
	}

	void Test_ManaBuff()
	{
		const float BuffValue = 30.f;
		const float StartingMana = DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Mana.GetCurrentValue();

		FActiveGameplayEffectHandle BuffHandle;

		// apply the buff
		{
			CONSTRUCT_CLASS(UGameplayEffect, DamageBuffEffect);
			AddModifier(DamageBuffEffect, GET_FIELD_CHECKED(UAbilitySystemTestAttributeSet, Mana), EGameplayModOp::Additive, FScalableFloat(BuffValue));
			DamageBuffEffect->DurationPolicy = EGameplayEffectDurationType::Infinite;

			BuffHandle = SourceComponent->ApplyGameplayEffectToTarget(DamageBuffEffect, DestComponent, 1.f);
		}

		// check that the value changed
		Test->TestEqual(TEXT("Mana Buffed"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Mana.GetCurrentValue(), StartingMana + BuffValue);

		// remove the effect
		{
			DestComponent->RemoveActiveGameplayEffect(BuffHandle);
		}

		// check that the value changed back
		Test->TestEqual(TEXT("Mana Restored"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Mana.GetCurrentValue(), StartingMana);
	}

	void Test_AttributeAggregators()
	{
		constexpr float BuffValue = 2.0f;
		const float ManaBaseValue = DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Mana.GetBaseValue();

		// Define a common lambda for applying a GE with a specific GameplayModOp
		auto ApplyGameplayModOp = [this, BuffValue](EGameplayModOp::Type GameplayModOp, UGameplayEffect* InEffect)
		{
			AddModifier(InEffect, GET_FIELD_CHECKED(UAbilitySystemTestAttributeSet, Mana), GameplayModOp, FScalableFloat(BuffValue));
			InEffect->DurationPolicy = EGameplayEffectDurationType::Infinite;

			return SourceComponent->ApplyGameplayEffectToTarget(InEffect, DestComponent, 1.f);
		};

		// Define a common lambda for testing if an op applied correctly
		auto TestGameplayModOp = [this, &ApplyGameplayModOp, BuffValue](EGameplayModOp::Type GameplayModOp, UGameplayEffect* InEffect)
		{
			const float PrevValue = DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Mana.GetCurrentValue();
			const float ExpectedValue = FAggregator::StaticExecModOnBaseValue(PrevValue, GameplayModOp, BuffValue);

			FActiveGameplayEffectHandle AGEHandle = ApplyGameplayModOp(GameplayModOp, InEffect);
			const float CurrentValue = DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Mana.GetCurrentValue();
			Test->TestEqual(FString::Printf(TEXT("Attribute GameplayModOp %s"), *UEnum::GetValueAsString(GameplayModOp)), CurrentValue, ExpectedValue);
			
			return AGEHandle;
		};

		// Test all of the ops (order matters here due to the implementation of the above StaticExecModOnBaseValue)
		CONSTRUCT_CLASS(UGameplayEffect, AdditiveEffect);
		FActiveGameplayEffectHandle AdditiveHandle = TestGameplayModOp(EGameplayModOp::Additive, AdditiveEffect);
		CONSTRUCT_CLASS(UGameplayEffect, MultiplyEffect);
		FActiveGameplayEffectHandle MultiplicativeHandle = TestGameplayModOp(EGameplayModOp::Multiplicitive, MultiplyEffect);
		CONSTRUCT_CLASS(UGameplayEffect, DivisionEffect);
		FActiveGameplayEffectHandle DivisionHandle = TestGameplayModOp(EGameplayModOp::Division, DivisionEffect);
		CONSTRUCT_CLASS(UGameplayEffect, CompoundEffect);
		FActiveGameplayEffectHandle CompoundHandle = TestGameplayModOp(EGameplayModOp::MultiplyCompound, CompoundEffect);
		CONSTRUCT_CLASS(UGameplayEffect, FinalEffect);
		FActiveGameplayEffectHandle FinalAddHandle = TestGameplayModOp(EGameplayModOp::AddFinal, FinalEffect);

		// Test the override quickly here before testing the aggregation is as expected
		{
			CONSTRUCT_CLASS(UGameplayEffect, OverrideEffect);
			FActiveGameplayEffectHandle OverrideHandle = TestGameplayModOp(EGameplayModOp::Override, OverrideEffect);
			DestComponent->RemoveActiveGameplayEffect(OverrideHandle);
		}

		Test->TestEqual(TEXT("Mana BaseValue is Unchanged"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Mana.GetBaseValue(), ManaBaseValue);

		// Add some compounding GameplayEffects and manually test our equation to see if it's all compounded correctly.
		{
			CONSTRUCT_CLASS(UGameplayEffect, Add2Effect);
			FActiveGameplayEffectHandle BaseAdd2 = ApplyGameplayModOp(EGameplayModOp::Additive, Add2Effect);
			CONSTRUCT_CLASS(UGameplayEffect, Multiply2Effect);
			FActiveGameplayEffectHandle BaseMultiply2 = ApplyGameplayModOp(EGameplayModOp::Multiplicitive, Multiply2Effect);
			CONSTRUCT_CLASS(UGameplayEffect, Compound2Effect);
			FActiveGameplayEffectHandle Compound2 = ApplyGameplayModOp(EGameplayModOp::MultiplyCompound, Compound2Effect);
			CONSTRUCT_CLASS(UGameplayEffect, Final2Effect);
			FActiveGameplayEffectHandle FinalAdd2 = ApplyGameplayModOp(EGameplayModOp::AddFinal, Final2Effect);
			
			// ExpectedResult = ((BaseValue + Additive) * Multiplicative / Division * CompoundMultiply) + FinalAdd;
			// Multiplicative and Division Compound as: 1.0f + ForEachValue(Value - 1.0f). E.g. two applications of 1.5 = 2.
			// CompoundMultiply Compounds as: 1.0f *= ForEachValue(Value) e.g. two applications of 1.5 = 2.25.
			constexpr float X = BuffValue;
			const float ExpectedResult = ((ManaBaseValue + X + X) * (1.0f + (X - 1.0f) + (X - 1.0f)) / (1.0f + (X - 1.0f)) * (X * X)) + X + X;
			const float CurrentValue = DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Mana.GetCurrentValue();
			Test->TestEqual(TEXT("Aggregation Equation Works as Expected"), CurrentValue, ExpectedResult);

			// Remove the compounding GE's
			DestComponent->RemoveActiveGameplayEffect(FinalAdd2);
			DestComponent->RemoveActiveGameplayEffect(Compound2);
			DestComponent->RemoveActiveGameplayEffect(BaseMultiply2);
			DestComponent->RemoveActiveGameplayEffect(BaseAdd2);
		}

		// Remove the base GE's
		{
			DestComponent->RemoveActiveGameplayEffect(FinalAddHandle);
			DestComponent->RemoveActiveGameplayEffect(CompoundHandle);
			DestComponent->RemoveActiveGameplayEffect(DivisionHandle);
			DestComponent->RemoveActiveGameplayEffect(MultiplicativeHandle);
			DestComponent->RemoveActiveGameplayEffect(AdditiveHandle);
		}

		// check that the value changed back
		Test->TestEqual(TEXT("Mana Restored to BaseValue"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Mana.GetCurrentValue(), ManaBaseValue);
	}

	void Test_PeriodicDamage()
	{
		const int32 NumPeriods = 10;
		const float PeriodSecs = 1.0f;
		const float DamagePerPeriod = 5.f; 
		const float StartingHealth = DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Health;

		// just try and reduce the health attribute
		{
			CONSTRUCT_CLASS(UGameplayEffect, BaseDmgEffect);
			AddModifier(BaseDmgEffect, GET_FIELD_CHECKED(UAbilitySystemTestAttributeSet, Health), EGameplayModOp::Additive, FScalableFloat(-DamagePerPeriod));
			BaseDmgEffect->DurationPolicy = EGameplayEffectDurationType::HasDuration;
			BaseDmgEffect->DurationMagnitude = FGameplayEffectModifierMagnitude(FScalableFloat(NumPeriods * PeriodSecs));
			BaseDmgEffect->Period.Value = PeriodSecs;

			SourceComponent->ApplyGameplayEffectToTarget(BaseDmgEffect, DestComponent, 1.f);
		}

		int32 NumApplications = 0;

		// Tick a small number to verify the application tick
		TickWorld(SMALL_NUMBER);
		++NumApplications;

		Test->TestEqual(TEXT("Health Reduced"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Health, StartingHealth - (DamagePerPeriod * NumApplications));

		// Tick a bit more to address possible floating point issues
		TickWorld(PeriodSecs * .1f);

		for (int32 i = 0; i < NumPeriods; ++i)
		{
			// advance time by one period
			TickWorld(PeriodSecs);

			++NumApplications;

			// check that health has been reduced
			Test->TestEqual(TEXT("Health Reduced"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Health, StartingHealth - (DamagePerPeriod * NumApplications));
		}

		// advance time by one extra period
		TickWorld(PeriodSecs);

		// should not have reduced further
		Test->TestEqual(TEXT("Health Reduced"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->Health, StartingHealth - (DamagePerPeriod * NumApplications));

		// TODO: test that the effect is no longer applied
	}

	void Test_StackLimit()
	{
		const float Duration = 10.0f;
		const float HalfDuration = Duration / 2.f;
		const float ChangePerGE = 5.f;
		const uint32 StackLimit = 2;
		const float StartingAttributeValue = DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->StackingAttribute1;

		// Apply one copy of the stacking GE
		CONSTRUCT_CLASS(UGameplayEffect, StackingEffect);
		AddModifier(StackingEffect, GET_FIELD_CHECKED(UAbilitySystemTestAttributeSet, StackingAttribute1), EGameplayModOp::Additive, FScalableFloat(ChangePerGE));
		StackingEffect->DurationPolicy = EGameplayEffectDurationType::HasDuration;
		StackingEffect->DurationMagnitude = FGameplayEffectModifierMagnitude(FScalableFloat(Duration));
		StackingEffect->StackLimitCount = StackLimit;
		StackingEffect->SetStackingType(EGameplayEffectStackingType::AggregateByTarget);
		StackingEffect->StackDurationRefreshPolicy = EGameplayEffectStackingDurationPolicy::NeverRefresh;
		StackingEffect->StackExpirationPolicy = EGameplayEffectStackingExpirationPolicy::ClearEntireStack;

		// Apply the GE StackLimit + 1 times
		for (uint32 Idx = 0; Idx <= StackLimit; ++Idx)
		{
			SourceComponent->ApplyGameplayEffectToTarget(StackingEffect, DestComponent, 1.f);
		}

		Test->TestEqual(TEXT("Stacking GEs"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->StackingAttribute1, StartingAttributeValue + (StackLimit * ChangePerGE));
	}

	void Test_SetByCallerStackingDuration()
	{
		const float Duration = 10.0f;
		const float HalfDuration = Duration / 2.f;
		const float ChangePerGE = 5.f;
		const uint32 StackLimit = 2;
		const float StartingAttributeValue = DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->StackingAttribute1;

		const FName DurationName(TEXT("Duration"));
		FSetByCallerFloat SetByCallerDuration;
		SetByCallerDuration.DataName = DurationName;

		// Apply one copy of the stacking GE
		CONSTRUCT_CLASS(UGameplayEffect, StackingEffect);
		AddModifier(StackingEffect, GET_FIELD_CHECKED(UAbilitySystemTestAttributeSet, StackingAttribute1), EGameplayModOp::Additive, FScalableFloat(ChangePerGE));
		StackingEffect->DurationPolicy = EGameplayEffectDurationType::HasDuration;
		StackingEffect->DurationMagnitude = FGameplayEffectModifierMagnitude(SetByCallerDuration);
		StackingEffect->StackLimitCount = StackLimit;
		StackingEffect->SetStackingType(EGameplayEffectStackingType::AggregateByTarget);
		StackingEffect->StackDurationRefreshPolicy = EGameplayEffectStackingDurationPolicy::NeverRefresh;
		StackingEffect->StackExpirationPolicy = EGameplayEffectStackingExpirationPolicy::RemoveSingleStackAndRefreshDuration;

		// create a spec, set the magnitude and apply the GE
		{
			FGameplayEffectSpec	Spec(StackingEffect, FGameplayEffectContextHandle(), 1.f);
			Spec.SetSetByCallerMagnitude(DurationName, Duration);
			SourceComponent->ApplyGameplayEffectSpecToTarget(Spec, DestComponent, FPredictionKey());
		}

		// Tick to partway through the GE's duration and apply a second copy of the GE
		TickWorld(HalfDuration);

		Test->TestEqual(TEXT("Stacking GEs"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->StackingAttribute1, StartingAttributeValue + ChangePerGE);

		// apply second copy of GE
		{
			FGameplayEffectSpec	Spec(StackingEffect, FGameplayEffectContextHandle(), 1.f);
			Spec.SetSetByCallerMagnitude(DurationName, Duration);
			SourceComponent->ApplyGameplayEffectSpecToTarget(Spec, DestComponent, FPredictionKey());
		}

		Test->TestEqual(TEXT("Stacking GEs"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->StackingAttribute1, StartingAttributeValue + (2 * ChangePerGE));

		// Tick to just after the first GE should have expired
		TickWorld(HalfDuration + 0.1f);

		// we should have removed one copy and still have the second copy
		Test->TestEqual(TEXT("Stacking GEs"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->StackingAttribute1, StartingAttributeValue + ChangePerGE);

		// check again near the end of the remaining GE's duration
		TickWorld(Duration - 0.2f);

		Test->TestEqual(TEXT("Stacking GEs"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->StackingAttribute1, StartingAttributeValue + ChangePerGE);

		// Tick to a point just after where the remaining GE should have been removed
		TickWorld(0.2f);

		Test->TestEqual(TEXT("Stacking GEs"), DestComponent->GetSet<UAbilitySystemTestAttributeSet>()->StackingAttribute1, StartingAttributeValue);
	}

	void Test_GameplayCues_DirectAPI()
	{
		// Construct a TestGameplayCuesGE Gameplay Effect that triggers the GameplayCue.Test tag.
		FGameplayEffectCue GameplayEffectCue{ UE::GameplayTags::GameplayCue_Test, 0.0f, 0.0f };
		CONSTRUCT_CLASS(UGameplayEffect, TestGameplayCuesGE);
		TestGameplayCuesGE->GameplayCues.Emplace(GameplayEffectCue);
		TestGameplayCuesGE->DurationPolicy = EGameplayEffectDurationType::Infinite;
		TestGameplayCuesGE->SetStackingType(EGameplayEffectStackingType::AggregateByTarget);
		TestGameplayCuesGE->StackLimitCount = 5; // Make it high enough to not confuse between "expected 1 and got 2" etc.

		UGameplayCueNotify_UnitTest* GCNotify_Test_CDO = GetMutableDefault<UGameplayCueNotify_UnitTest>();
		const FGameplayEffectSpec GESpecInfinite{ TestGameplayCuesGE, FGameplayEffectContextHandle{ new FGameplayEffectContext{SourceActor, SourceActor} }, 1.f };

		// Setup some parameters so we can test all of the routes we can execute a Cue through
		FGameplayCueParameters GameplayCueParameters;
		{
			UAbilitySystemGlobals::Get().InitGameplayCueParameters_GESpec(GameplayCueParameters, GESpecInfinite);
			GameplayCueParameters.EffectCauser = SourceActor;
			GameplayCueParameters.Instigator = SourceActor;
		}

		Test->AddInfo(TEXT(" Testing Duration GameplayCues via UAbilitySystemComponent::AddGameplayCue"));
		{
			GCNotify_Test_CDO->ResetCallCounts();

			// We should test with EGameplayCueExecutionOptions::IgnoreInterfaces and then with EGameplayCueExecutionOptions::IgnoreNotifies to see the functionality.
			// Add GameplayCue should execute OnActive / While Active (used for duration)
			DestComponent->AddGameplayCue(UE::GameplayTags::GameplayCue_Test, GameplayCueParameters);

			Test->TestTrue(TEXT("  IsGameplayCueActive"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));
			Test->TestEqual(TEXT("  OnActive Calls"), GCNotify_Test_CDO->NumOnActiveCalls, DestComponent->bSuppressGameplayCues ? 0 : 1);
			Test->TestEqual(TEXT("  WhileActive Calls"), GCNotify_Test_CDO->NumWhileActiveCalls, DestComponent->bSuppressGameplayCues ? 0 : 1);
			Test->TestEqual(TEXT("  OnExecute Calls"), GCNotify_Test_CDO->NumOnExecuteCalls, 0);
			Test->TestEqual(TEXT("  OnRemove Calls"), GCNotify_Test_CDO->NumOnRemoveCalls, 0);
			Test->TestEqual(TEXT("  OwnerTagCount"), DestComponent->GetTagCount(UE::GameplayTags::GameplayCue_Test), 1);
		}

		Test->AddInfo(TEXT(" Testing Duration GameplayCues via UAbilitySystemComponent::RemoveGameplayCue"));
		{
			GCNotify_Test_CDO->ResetCallCounts();

			// Remove GameplayCue should execute OnRemove.
			DestComponent->RemoveGameplayCue(UE::GameplayTags::GameplayCue_Test);

			Test->TestFalse(TEXT("  IsGameplayCueActive"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));
			Test->TestEqual(TEXT("  OnActive Calls"), GCNotify_Test_CDO->NumOnActiveCalls, 0);
			Test->TestEqual(TEXT("  WhileActive Calls"), GCNotify_Test_CDO->NumWhileActiveCalls, 0);
			Test->TestEqual(TEXT("  OnExecute Calls"), GCNotify_Test_CDO->NumOnExecuteCalls, 0);
			Test->TestEqual(TEXT("  OnRemove Calls"), GCNotify_Test_CDO->NumOnRemoveCalls, DestComponent->bSuppressGameplayCues ? 0 : 1);
			Test->TestEqual(TEXT("  OwnerTagCount"), DestComponent->GetTagCount(UE::GameplayTags::GameplayCue_Test), 0);
		}

		Test->AddInfo(TEXT(" Testing Duration GameplayCues via UAbilitySystemComponent::AddGameplayCue_MinimalReplication"));
		{
			GCNotify_Test_CDO->ResetCallCounts();

			// Add the Minimal Replication Cue should execute OnActive/WhileActive
			DestComponent->AddGameplayCue_MinimalReplication(UE::GameplayTags::GameplayCue_Test, GESpecInfinite.GetEffectContext());

			Test->TestTrue(TEXT("  IsGameplayCueActive"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));
			Test->TestEqual(TEXT("  OnActive Calls"), GCNotify_Test_CDO->NumOnActiveCalls, DestComponent->bSuppressGameplayCues ? 0 : 1);
			Test->TestEqual(TEXT("  WhileActive Calls"), GCNotify_Test_CDO->NumWhileActiveCalls, DestComponent->bSuppressGameplayCues ? 0 : 1);
			Test->TestEqual(TEXT("  OnExecute Calls"), GCNotify_Test_CDO->NumOnExecuteCalls, 0);
			Test->TestEqual(TEXT("  OnRemove Calls"), GCNotify_Test_CDO->NumOnRemoveCalls, 0);
			Test->TestEqual(TEXT("  OwnerTagCount"), DestComponent->GetTagCount(UE::GameplayTags::GameplayCue_Test), 1);
		}

		Test->AddInfo(TEXT(" Testing Duration GameplayCues via UAbilitySystemComponent::RemoveGameplayCue_MinimalReplication"));
		{
			GCNotify_Test_CDO->ResetCallCounts();

			// Remove GameplayCue should execute OnRemove.
			DestComponent->RemoveGameplayCue_MinimalReplication(UE::GameplayTags::GameplayCue_Test);

			Test->TestFalse(TEXT("  IsGameplayCueActive"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));
			Test->TestEqual(TEXT("  OnActive Calls"), GCNotify_Test_CDO->NumOnActiveCalls, 0);
			Test->TestEqual(TEXT("  WhileActive Calls"), GCNotify_Test_CDO->NumWhileActiveCalls, 0);
			Test->TestEqual(TEXT("  OnExecute Calls"), GCNotify_Test_CDO->NumOnExecuteCalls, 0);
			Test->TestEqual(TEXT("  OnRemove Calls"), GCNotify_Test_CDO->NumOnRemoveCalls, DestComponent->bSuppressGameplayCues ? 0 : 1);
			Test->TestEqual(TEXT("  OwnerTagCount"), DestComponent->GetTagCount(UE::GameplayTags::GameplayCue_Test), 0);
		}

		Test->AddInfo(TEXT(" Testing UAbilitySystemComponent::InvokeGameplayCueEvent"));
		{
			GCNotify_Test_CDO->ResetCallCounts();

			Test->TestFalse(TEXT("  IsGameplayCueActive"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));

			DestComponent->InvokeGameplayCueEvent(GESpecInfinite, EGameplayCueEvent::OnActive);
			DestComponent->InvokeGameplayCueEvent(GESpecInfinite, EGameplayCueEvent::WhileActive);
			DestComponent->InvokeGameplayCueEvent(GESpecInfinite, EGameplayCueEvent::Executed);
			DestComponent->InvokeGameplayCueEvent(GESpecInfinite, EGameplayCueEvent::Removed);

			// One extra one, just to make sure
			DestComponent->InvokeGameplayCueEvent(GESpecInfinite, EGameplayCueEvent::WhileActive);

			Test->TestFalse(TEXT("  IsGameplayCueActive"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));
			Test->TestEqual(TEXT("  OnActive Calls"), GCNotify_Test_CDO->NumOnActiveCalls, DestComponent->bSuppressGameplayCues ? 0 : 1);
			Test->TestEqual(TEXT("  WhileActive Calls"), GCNotify_Test_CDO->NumWhileActiveCalls, DestComponent->bSuppressGameplayCues ? 0 : 2);
			Test->TestEqual(TEXT("  OnExecute Calls"), GCNotify_Test_CDO->NumOnExecuteCalls, DestComponent->bSuppressGameplayCues ? 0 : 1);
			Test->TestEqual(TEXT("  OnRemove Calls"), GCNotify_Test_CDO->NumOnRemoveCalls, DestComponent->bSuppressGameplayCues ? 0 : 1);
			Test->TestEqual(TEXT("  OwnerTagCount"), DestComponent->GetTagCount(UE::GameplayTags::GameplayCue_Test), 0);
		}

		Test->AddInfo(TEXT(" Testing UAbilitySystemComponent::ExecuteGameplayCue"));
		{
			GCNotify_Test_CDO->ResetCallCounts();

			Test->TestFalse(TEXT("  IsGameplayCueActive"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));

			DestComponent->ExecuteGameplayCue(UE::GameplayTags::GameplayCue_Test, GameplayCueParameters);
			Test->TestFalse(TEXT("  IsGameplayCueActive"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));
			Test->TestEqual(TEXT("  OnActive Calls"), GCNotify_Test_CDO->NumOnActiveCalls, 0);
			Test->TestEqual(TEXT("  WhileActive Calls"), GCNotify_Test_CDO->NumWhileActiveCalls, 0);
			Test->TestEqual(TEXT("  OnExecute Calls"), GCNotify_Test_CDO->NumOnExecuteCalls, DestComponent->bSuppressGameplayCues ? 0 : 1);
			Test->TestEqual(TEXT("  OnRemove Calls"), GCNotify_Test_CDO->NumOnRemoveCalls, 0);
			Test->TestEqual(TEXT("  OwnerTagCount"), DestComponent->GetTagCount(UE::GameplayTags::GameplayCue_Test), 0);
		}
	}

	void Test_GameplayCues_GameplayEffectsAPI()
	{
		// Construct a TestGameplayCuesGE Gameplay Effect that triggers the GameplayCue.Test tag.
		FGameplayEffectCue GameplayEffectCue{ UE::GameplayTags::GameplayCue_Test, 0.0f, 0.0f };
		CONSTRUCT_CLASS(UGameplayEffect, TestGameplayCuesGE);
		TestGameplayCuesGE->GameplayCues.Emplace(GameplayEffectCue);
		TestGameplayCuesGE->DurationPolicy = EGameplayEffectDurationType::Infinite;
		TestGameplayCuesGE->SetStackingType(EGameplayEffectStackingType::AggregateByTarget);
		TestGameplayCuesGE->StackDurationRefreshPolicy = EGameplayEffectStackingDurationPolicy::NeverRefresh;
		TestGameplayCuesGE->StackPeriodResetPolicy = EGameplayEffectStackingPeriodPolicy::NeverReset;
		TestGameplayCuesGE->StackLimitCount = 5; // Make it high enough to not confuse between "expected 1 and got 2" etc.

		UGameplayCueNotify_UnitTest* GCNotify_Test_CDO = GetMutableDefault<UGameplayCueNotify_UnitTest>();
		const FGameplayEffectSpec GESpecInfinite{ TestGameplayCuesGE, FGameplayEffectContextHandle{ new FGameplayEffectContext{SourceActor, SourceActor} }, 1.f };

		auto TestGameplayEffectsTriggerGameplayCuesProperly = [&](const FGameplayEffectSpec& GESpec)
			{
				GCNotify_Test_CDO->ResetCallCounts();

				Test->TestFalse(TEXT("  IsGameplayCueActive (Before Apply)"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));

				// Apply the GE.  We expect OnActive/WhileActive/OnRemove for non-instant; OnExecute for the instant ones.
				FActiveGameplayEffectHandle ActiveGEHandle = DestComponent->ApplyGameplayEffectSpecToSelf(GESpec, FPredictionKey::CreateNewPredictionKey(DestComponent));

				const bool bPredicted = !DestComponent->IsOwnerActorAuthoritative();
				const bool bInstant = (TestGameplayCuesGE->DurationPolicy != EGameplayEffectDurationType::Instant);
				const bool bExpectedAdd = (!DestComponent->bSuppressGameplayCues) && (bPredicted || bInstant);
				const int ExpectedOnActive = bExpectedAdd && (TestGameplayCuesGE->DurationPolicy != EGameplayEffectDurationType::Instant);
				const int ExpectedOnExecute = ((TestGameplayCuesGE->DurationPolicy == EGameplayEffectDurationType::Instant) && (!DestComponent->bSuppressGameplayCues)) ? 1 : 0;
				const int ExpectedWhileActive = ExpectedOnActive;
				const int ExpectedOnRemove = bExpectedAdd;
				const int ExpectedOwnerTagCount = bExpectedAdd;

				Test->TestEqual(TEXT("  IsGameplayCueActive (After Apply)"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test), bExpectedAdd);
				Test->TestEqual(TEXT("  OnActive Calls"), GCNotify_Test_CDO->NumOnActiveCalls, ExpectedOnActive);
				Test->TestEqual(TEXT("  WhileActive Calls"), GCNotify_Test_CDO->NumWhileActiveCalls, ExpectedWhileActive);
				Test->TestEqual(TEXT("  OnExecute Calls"), GCNotify_Test_CDO->NumOnExecuteCalls, ExpectedOnExecute);
				Test->TestEqual(TEXT("  OnRemove Calls"), GCNotify_Test_CDO->NumOnRemoveCalls, 0);
				Test->TestEqual(TEXT("  OwnerTagCount"), DestComponent->GetTagCount(UE::GameplayTags::GameplayCue_Test), ExpectedOwnerTagCount);

				// Now try to remove it
				DestComponent->RemoveActiveGameplayEffect_NoReturn(ActiveGEHandle);

				Test->TestFalse(TEXT("  IsGameplayCueActive (After RemoveActiveEffects)"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));
				Test->TestEqual(TEXT("  OnRemove Calls"), GCNotify_Test_CDO->NumOnRemoveCalls, ExpectedOnRemove);
				Test->TestEqual(TEXT("  OwnerTagCount"), DestComponent->GetTagCount(UE::GameplayTags::GameplayCue_Test), 0);
			};

		auto TestGameplayEffectsStackTriggerGameplayCuesProperly = [&](const FGameplayEffectSpec& GESpec)
			{
				GCNotify_Test_CDO->ResetCallCounts();

				Test->TestFalse(TEXT("  IsGameplayCueActive (Before Stack Application)"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));

				// Apply the GE until the StackCount.  We expect OnActive/WhileActive/OnRemove for non-instant; OnExecute cannot stack.
				FActiveGameplayEffectHandle ActiveGEHandle = DestComponent->ApplyGameplayEffectSpecToSelf(GESpec, FPredictionKey::CreateNewPredictionKey(DestComponent));
				const int StackLimitCount = GESpec.Def->GetStackLimitCount();
				for (int Index = 1; Index < StackLimitCount; ++Index)
				{
					DestComponent->ApplyGameplayEffectSpecToSelf(GESpec, FPredictionKey::CreateNewPredictionKey(DestComponent));
				}

				// More complex rules:  We expect as many OnActive's as there is applications, unless we're predicting.
				// Unexpected behavior:  We can never predict a stacked application (why?!) so it ends up with 1.
				const bool bPredicting = !DestComponent->IsOwnerActorAuthoritative();
				bool bExpectedAdded = (!DestComponent->bSuppressGameplayCues);
				int ExpectedOnActive = bExpectedAdded ? StackLimitCount : 0;
				int ExpectedOnExecute = 0;
				int ExpectedWhileActive = ExpectedOnActive;
				int ExpectedOnRemove = (ExpectedOnActive > 0) ? 1 : 0;		// Even with stacking, the GameplayCue is actually only applied once (and therefore removed once when fully unstacked)
				int ExpectedOwnerTagCount = (ExpectedOnActive > 0) ? 1 : 0;	// Even with stacking, we only have the tag applied once

				const bool bInstant = (TestGameplayCuesGE->DurationPolicy == EGameplayEffectDurationType::Instant);
				if (bInstant)
				{
					// Instant Cues should not actually stack, so we don't get any of the Added/While/Remove calls, just Execute...
					// Except if we're predicted, in which case we're added so that we can reconcile the prediction.
					bExpectedAdded = (!DestComponent->bSuppressGameplayCues) && bPredicting;
					ExpectedOnExecute = (!DestComponent->bSuppressGameplayCues) ? StackLimitCount : 0;
					ExpectedOnActive = ExpectedWhileActive = 0;

					// Unexpected behavior here:  If we're predicting, we get Tags.  If not, we don't get tags.
					ExpectedOwnerTagCount = bPredicting && (!DestComponent->bSuppressGameplayCues) ? ExpectedOnExecute : 0;
					ExpectedOnRemove = bExpectedAdded && (!DestComponent->bSuppressGameplayCues) ? ExpectedOnExecute : 0;
				}

				Test->TestEqual(TEXT("  IsGameplayCueActive (After Stack Application)"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test), bExpectedAdded);
				Test->TestEqual(TEXT("  OnActive Calls"), GCNotify_Test_CDO->NumOnActiveCalls, ExpectedOnActive);
				Test->TestEqual(TEXT("  WhileActive Calls"), GCNotify_Test_CDO->NumWhileActiveCalls, ExpectedWhileActive);
				Test->TestEqual(TEXT("  OnExecute Calls"), GCNotify_Test_CDO->NumOnExecuteCalls, ExpectedOnExecute);
				Test->TestEqual(TEXT("  OnRemove Calls"), GCNotify_Test_CDO->NumOnRemoveCalls, 0);
				Test->TestEqual(TEXT("  OwnerTagCount"), DestComponent->GetTagCount(UE::GameplayTags::GameplayCue_Test), ExpectedOwnerTagCount);

				// Now try to remove it.  If not instant, try one at a time.
				if (!bInstant)
				{
					constexpr int StacksToRemove = 1;
					for (int Index = 0; Index < ExpectedOnActive; ++Index)
					{
						Test->TestEqual(TEXT("  IsGameplayCueActive before RemoveActiveEffects Unstack"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test), (ExpectedOnActive > 0));
						DestComponent->RemoveActiveGameplayEffect_NoReturn(ActiveGEHandle, StacksToRemove);
					}
				}
				else
				{
					// If instant, since we ActiveGEHandle was different, we need to remove this way.  This function for some reason allows us to bypass the Authority check.
					DestComponent->RemoveActiveGameplayEffectBySourceEffect(UGameplayEffect::StaticClass(), SourceComponent);
				}

				Test->TestFalse(TEXT("  IsGameplayCueActive (After Stack Removal)"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));
				Test->TestEqual(TEXT("  OnRemove Calls"), GCNotify_Test_CDO->NumOnRemoveCalls, ExpectedOnRemove);
				Test->TestEqual(TEXT("  OwnerTagCount"), DestComponent->GetTagCount(UE::GameplayTags::GameplayCue_Test), 0);
			};

		Test->AddInfo(TEXT(" Testing Infinite Duration GameplayCues via UAbilitySystemComponent::ApplyGameplayEffectSpecToSelf"));
		TestGameplayCuesGE->DurationPolicy = EGameplayEffectDurationType::Infinite;
		TestGameplayEffectsTriggerGameplayCuesProperly(GESpecInfinite);

		Test->AddInfo(TEXT(" Testing Stacking Infinite Duration GameplayCues via UAbilitySystemComponent::ApplyGameplayEffectSpecToSelf"));
		TestGameplayEffectsStackTriggerGameplayCuesProperly(GESpecInfinite);

		Test->AddInfo(TEXT(" Testing HasDuration GameplayCues via UAbilitySystemComponent::ApplyGameplayEffectSpecToSelf"));
		TestGameplayCuesGE->DurationPolicy = EGameplayEffectDurationType::HasDuration;
		FGameplayEffectSpec GESpecHasDuration{ GESpecInfinite, GESpecInfinite.GetContext() };
		GESpecHasDuration.SetDuration(5.0f, false);
		TestGameplayEffectsTriggerGameplayCuesProperly(GESpecHasDuration);

		Test->AddInfo(TEXT(" Testing Stacking HasDuration GameplayCues via UAbilitySystemComponent::ApplyGameplayEffectSpecToSelf"));
		TestGameplayEffectsStackTriggerGameplayCuesProperly(GESpecHasDuration);

		Test->AddInfo(TEXT(" Testing Instant GameplayCues via UAbilitySystemComponent::ApplyGameplayEffectSpecToSelf"));
		TestGameplayCuesGE->DurationPolicy = EGameplayEffectDurationType::Instant;
		FGameplayEffectSpec GESpecInstant{ GESpecInfinite, GESpecInfinite.GetContext() };
		GESpecInstant.SetDuration(FGameplayEffectConstants::INSTANT_APPLICATION, false);
		TestGameplayEffectsTriggerGameplayCuesProperly(GESpecInstant);

		Test->AddInfo(TEXT(" Testing Stacking Instant GameplayCues via UAbilitySystemComponent::ApplyGameplayEffectSpecToSelf"));
		TestGameplayEffectsStackTriggerGameplayCuesProperly(GESpecInstant);

		Test->TestFalse(TEXT(" Ending with IsGameplayCueActive?"), DestComponent->IsGameplayCueActive(UE::GameplayTags::GameplayCue_Test));
	}

	void Test_GameplayCues()
	{
		Test->TestTrue(TEXT("DestComponent IsReadyForGameplayCues()"), DestComponent->IsReadyForGameplayCues());

		// Setup a temporary UGameplayCueNotify_Test instance that will intercept the GameplayCue.Test tag
		// We are poking the data that should be internal (GameplayCueData) because we want to manually populate
		// the LoadedGameplayCueClass (to avoid the async load path).  That means we will eventually need to
		const UGameplayCueNotify_UnitTest* GCNotify_Test_CDO = GetDefault<UGameplayCueNotify_UnitTest>();
		FGameplayCueNotifyData GameplayCueNotifyData;
		GameplayCueNotifyData.GameplayCueTag = UE::GameplayTags::GameplayCue_Test;
		GameplayCueNotifyData.GameplayCueNotifyObj = GCNotify_Test_CDO->GetPathName();
		GameplayCueNotifyData.LoadedGameplayCueClass = UGameplayCueNotify_UnitTest::StaticClass();

		UGameplayCueManager* GameplayCueManager = UAbilitySystemGlobals::Get().GetGameplayCueManager();
		UGameplayCueSet* RuntimeCueSet = GameplayCueManager->GetRuntimeCueSet();
		RuntimeCueSet->GameplayCueData.Emplace(GameplayCueNotifyData);

		// Now just force a rebuild of the internal table since we weren't supposed to poke the data manually.
		RuntimeCueSet->UpdateCueByStringRefs(GameplayCueNotifyData.GameplayCueNotifyObj, GCNotify_Test_CDO->GetPathName());
		ON_SCOPE_EXIT{ RuntimeCueSet->RemoveCuesByStringRefs({ GCNotify_Test_CDO->GetPathName() }); };

		// Perform all of the Tests once as an Authority (Server) and once as an AutonomousProxy (Client)
		Test->AddInfo(TEXT("--- Testing as ROLE_Authority ---"));
		// Full Mode (Gameplay Effects are replicated to all clients)
		Test->AddInfo("Testing SetReplicationMode(Full)");
		DestComponent->SetReplicationMode(EGameplayEffectReplicationMode::Full);
		Test_GameplayCues_DirectAPI();
		Test_GameplayCues_GameplayEffectsAPI();

		// Mixed Mode (GE's are replicated to Autonomous Proxies, Cues from the GE's are replicated to Simulated as they won't have the GE's)
		Test->AddInfo("Testing SetReplicationMode(Mixed)");
		DestComponent->SetReplicationMode(EGameplayEffectReplicationMode::Mixed);
		Test_GameplayCues_DirectAPI();
		Test_GameplayCues_GameplayEffectsAPI();

		// Minimal Mode (GE's are never replicated; Cues from the GE's are replicated to the Simulated as they won't have the GE's.  There should be no autonomous proxies when this is set (e.g. used for AI).)
		Test->AddInfo("Testing SetReplicationMode(Minimal)");
		DestComponent->SetReplicationMode(EGameplayEffectReplicationMode::Minimal);
		Test_GameplayCues_DirectAPI();
		Test_GameplayCues_GameplayEffectsAPI();

		// Test Suppression
		DestComponent->bSuppressGameplayCues = true;
		DestComponent->SetReplicationMode(EGameplayEffectReplicationMode::Full);
		Test->AddInfo(TEXT("Testing bSuppressGameplayCues (expecting no Gameplay Cues)"));
		Test_GameplayCues_DirectAPI();
		Test_GameplayCues_GameplayEffectsAPI();
		DestComponent->bSuppressGameplayCues = false;

		// As Autonomous Proxy, we are mimicking the prediction of Gameplay Effects.  As such, we should not
		// be calling the Direct API, only the Gameplay Effects API.
		Test->AddInfo(TEXT("--- Testing as ROLE_AutonomousProxy ---"));
		SourceActor->SetRole(ENetRole::ROLE_AutonomousProxy);
		SourceComponent->CacheIsNetSimulated();
		DestActor->SetRole(ENetRole::ROLE_AutonomousProxy);
		DestComponent->CacheIsNetSimulated();

		// Replication modes are server-side only, so we really only need to test the one...
		Test->AddInfo("Testing SetReplicationMode(Mixed)");
		DestComponent->SetReplicationMode(EGameplayEffectReplicationMode::Mixed);
		Test_GameplayCues_GameplayEffectsAPI();

		// Reset the variables
		SourceActor->SetRole(ENetRole::ROLE_Authority);
		SourceComponent->CacheIsNetSimulated();
		DestActor->SetRole(ENetRole::ROLE_Authority);
		DestComponent->CacheIsNetSimulated();
	}
private: // test helpers

	template<typename MODIFIER_T>
	FGameplayModifierInfo& AddModifier(UGameplayEffect* Effect, FProperty* Property, EGameplayModOp::Type Op, const MODIFIER_T& Magnitude)
	{
		int32 Idx = Effect->Modifiers.Num();
		Effect->Modifiers.SetNum(Idx+1);
		FGameplayModifierInfo& Info = Effect->Modifiers[Idx];
		Info.ModifierMagnitude = Magnitude;
		Info.ModifierOp = Op;
		Info.Attribute.SetUProperty(Property);
		return Info;
	}

	void TickWorld(float Time)
	{
		const float step = 0.1f;
		while (Time > 0.f)
		{
			World->Tick(ELevelTick::LEVELTICK_All, FMath::Min(Time, step));
			Time -= step;

			// This is terrible but required for subticking like this.
			// we could always cache the real GFrameCounter at the start of our tests and restore it when finished.
			GFrameCounter++;
		}
	}

private:
	UWorld* World;
	FAutomationTestBase* Test;

	AAbilitySystemTestPawn* SourceActor;
	UAbilitySystemComponent* SourceComponent;

	AAbilitySystemTestPawn* DestActor;
	UAbilitySystemComponent* DestComponent;
};

#define ADD_TEST(Name) \
	TestFunctions.Add(&GameplayEffectsTestSuite::Name); \
	TestFunctionNames.Add(TEXT(#Name))

class FGameplayEffectsTest : public FAutomationTestBase
{
public:
	typedef void (GameplayEffectsTestSuite::*TestFunc)();

	FGameplayEffectsTest(const FString& InName)
	: FAutomationTestBase(InName, false)
	{
		// list all test functions here
		ADD_TEST(Test_InstantDamage);
		ADD_TEST(Test_InstantDamageRemap);
		ADD_TEST(Test_ManaBuff);
		ADD_TEST(Test_AttributeAggregators);
		ADD_TEST(Test_PeriodicDamage);
		ADD_TEST(Test_StackLimit);
		ADD_TEST(Test_SetByCallerStackingDuration);
		ADD_TEST(Test_GameplayCues);
	}

	virtual EAutomationTestFlags GetTestFlags() const override { return EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter; }
	virtual bool IsStressTest() const { return false; }
	virtual uint32 GetRequiredDeviceNum() const override { return 1; }

protected:
	virtual FString GetBeautifiedTestName() const override { return "System.AbilitySystem.GameplayEffects"; }
	virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const override
	{
		for (const FString& TestFuncName : TestFunctionNames)
		{
			OutBeautifiedNames.Add(TestFuncName);
			OutTestCommands.Add(TestFuncName);
		}
	}

	bool RunTest(const FString& Parameters) override
	{
		// find the matching test
		TestFunc TestFunction = nullptr;
		for (int32 i = 0; i < TestFunctionNames.Num(); ++i)
		{
			if (TestFunctionNames[i] == Parameters)
			{
				TestFunction = TestFunctions[i];
				break;
			}
		}
		if (TestFunction == nullptr)
		{
			return false;
		}

		// get the current data table (to restore later)
		UDataTable *DataTable = IGameplayAbilitiesModule::Get().GetAbilitySystemGlobals()->GetGlobalAttributeMetaDataTable();

		// setup required GameplayTags
		UDataTable* TagTable = CreateGameplayDataTable();

		UGameplayTagsManager::Get().PopulateTreeFromDataTable(TagTable);

		UWorld *World = UWorld::CreateWorld(EWorldType::Game, false);
		FWorldContext &WorldContext = GEngine->CreateNewWorldContext(EWorldType::Game);
		WorldContext.SetCurrentWorld(World);

		FURL URL;
		World->InitializeActorsForPlay(URL);
		World->BeginPlay();

		// run the matching test
		uint64 InitialFrameCounter = GFrameCounter;
		{
			GameplayEffectsTestSuite Tester(World, this);
			(Tester.*TestFunction)();
		}
		GFrameCounter = InitialFrameCounter;

		World->EndPlay(EEndPlayReason::Quit);
		GEngine->DestroyWorldContext(World);
		World->DestroyWorld(false);

		IGameplayAbilitiesModule::Get().GetAbilitySystemGlobals()->AutomationTestOnly_SetGlobalAttributeDataTable(DataTable);
		return true;
	}

	TArray<TestFunc> TestFunctions;
	TArray<FString> TestFunctionNames;
};

namespace
{
	FGameplayEffectsTest FGameplayEffectsTestAutomationTestInstance(TEXT("FGameplayEffectsTest"));
}

#endif //WITH_EDITOR
