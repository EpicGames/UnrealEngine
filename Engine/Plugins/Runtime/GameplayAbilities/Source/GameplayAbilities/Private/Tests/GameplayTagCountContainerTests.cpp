// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "GameplayEffectTypes.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGameplayTagCountContainerTests, "System.AbilitySystem.GameplayTagCountContainer", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

#include "NativeGameplayTags.h"

namespace UE::AbilitySystem::Private::GameplayTagCountContainerTests
{
	UE_DEFINE_GAMEPLAY_TAG_STATIC(TestsDotGenericTag, "Tests.GenericTag");
	UE_DEFINE_GAMEPLAY_TAG_STATIC(TestsDotGenericTagDotOne, "Tests.GenericTag.One");
	UE_DEFINE_GAMEPLAY_TAG_STATIC(TestsDotGenericTagDotTwo, "Tests.GenericTag.Two");
}

bool FGameplayTagCountContainerTests::RunTest(const FString& Parameters)
{
	using namespace UE::AbilitySystem::Private::GameplayTagCountContainerTests;

	// Create the container and add the first tag
	FGameplayTagCountContainer TagCountContainer;
	TagCountContainer.SetTagCount(TestsDotGenericTagDotOne, 1);

	// Test for the first tag after adding
	FGameplayTagContainer ContainerOne{ TestsDotGenericTagDotOne };
	TestTrue(TEXT("HasAllMatchingGameplayTags after Adding"), TagCountContainer.HasAllMatchingGameplayTags(ContainerOne));
	TestTrue(TEXT("HasAnyMatchingGameplayTags after Adding"), TagCountContainer.HasAnyMatchingGameplayTags(ContainerOne));
	TestTrue(TEXT("HasMatchingGameplayTag after Adding"), TagCountContainer.HasMatchingGameplayTag(TestsDotGenericTagDotOne));
	TestTrue(TEXT("GetTagCount of One == 1"), TagCountContainer.GetTagCount(TestsDotGenericTagDotOne) == 1);
	TestTrue(TEXT("GetTagCount of Parent == 1"), TagCountContainer.GetTagCount(TestsDotGenericTag) == 1);

	// False for the second tag before adding
	FGameplayTagContainer ContainerTwo{ TestsDotGenericTagDotTwo };
	TestFalse(TEXT("HasAllMatchingGameplayTags before Adding"), TagCountContainer.HasAllMatchingGameplayTags(ContainerTwo));
	TestFalse(TEXT("HasAnyMatchingGameplayTags before Adding"), TagCountContainer.HasAnyMatchingGameplayTags(ContainerTwo));
	TestFalse(TEXT("HasMatchingGameplayTag before Adding"), TagCountContainer.HasMatchingGameplayTag(TestsDotGenericTagDotTwo));

	// True for the second tag after adding
	TagCountContainer.SetTagCount(TestsDotGenericTagDotTwo, 2);
	TestTrue(TEXT("GetTagCount of One == 1"), TagCountContainer.GetTagCount(TestsDotGenericTagDotOne) == 1);
	TestTrue(TEXT("GetTagCount of Two == 2"), TagCountContainer.GetTagCount(TestsDotGenericTagDotTwo) == 2);
	TestTrue(TEXT("GetTagCount of Parent == 3"), TagCountContainer.GetTagCount(TestsDotGenericTag) == 3);

	// Remove one
	TagCountContainer.UpdateTagCount(TestsDotGenericTagDotOne, -1, EGameplayTagReplicationState::None);
	TestFalse(TEXT("HasAllMatchingGameplayTags after Removing"), TagCountContainer.HasAllMatchingGameplayTags(ContainerOne));
	TestFalse(TEXT("HasAnyMatchingGameplayTags after Removing"), TagCountContainer.HasAnyMatchingGameplayTags(ContainerOne));
	TestTrue(TEXT("GetTagCount of One == 0"), TagCountContainer.GetTagCount(TestsDotGenericTagDotOne) == 0);
	TestTrue(TEXT("GetTagCount of Two == 2"), TagCountContainer.GetTagCount(TestsDotGenericTagDotTwo) == 2);
	TestTrue(TEXT("GetTagCount of Parent == 2"), TagCountContainer.GetTagCount(TestsDotGenericTag) == 2);

	// Explicit Tags
	TestTrue(TEXT("GetExplicitTagCount of Parent == 0"), TagCountContainer.GetExplicitTagCount(TestsDotGenericTag) == 0);
	TestTrue(TEXT("GetExplicitTagCount of One == 0"), TagCountContainer.GetExplicitTagCount(TestsDotGenericTagDotOne) == 0);
	TestTrue(TEXT("GetExplicitTagCount of Two == 2"), TagCountContainer.GetExplicitTagCount(TestsDotGenericTagDotTwo) == 2);

	// Test the Explicit Tag Container
	FGameplayTagContainer ExplicitTags = TagCountContainer.GetExplicitGameplayTags();
	TestFalse(TEXT("GetExplicitGameplayTags Contains Parent == False"), ExplicitTags.HasTagExact(TestsDotGenericTag));
	TestFalse(TEXT("GetExplicitGameplayTags Contains One == False"), ExplicitTags.HasTagExact(TestsDotGenericTagDotOne));
	TestTrue(TEXT("GetExplicitGameplayTags Contains Two == True"), ExplicitTags.HasTagExact(TestsDotGenericTagDotTwo));

	return true;
}
