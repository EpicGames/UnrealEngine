// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

#include "GameplayAbilitiesModule.h"
#include "GameplayEffectTypes.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGameplayTagQueryMatchesTagRequirementsTest, "System.AbilitySystem.GameplayTagRequirements.ConvertTagFieldsToTagQuery", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

#include "GameplayTagsManager.h"

// You can run this on the command line like this to get a longer test
//  UnrealEditor.exe -game -unattended -nullrhi -ExecCmds="Automation RunTest System.AbilitySystem.GameplayTagRequirements.ConvertTagFieldsToTagQuery;Now;Quit" -GameplayTagQueryMatchesTagRequirementsTest_NumTestsPerScenario=65536
bool FGameplayTagQueryMatchesTagRequirementsTest::RunTest(const FString& Parameters)
{
	constexpr int NumDebugTags = 4;
	constexpr int NumTestScenarios = 1024;
	const int32 SomeSeed = FMath::GetRandSeed(); // Need to sync two random streams so we get the same results

	int NumTestsPerScenario = 0;
	FParse::Value(FCommandLine::Get(), TEXT("GameplayTagQueryMatchesTagRequirementsTest_NumTestsPerScenario="), NumTestsPerScenario);
	if (NumTestsPerScenario <= 0)
	{
		NumTestsPerScenario = 1024;
	}

	// Stats to Collect
	uint32 NumMetRequirements = 0;
	uint64 MetRequirementsCycles = 0;

	uint32 NumQueryMatches = 0;
	uint64 QueryMatchesCycles = 0;

	// Fill up possible tags with just some random tags from our project
	TArray<FGameplayTag> PossibleTags;
	{
		FGameplayTagContainer AllGameplayTags;
		UGameplayTagsManager::Get().RequestAllGameplayTags(AllGameplayTags, true);

		if (AllGameplayTags.Num() < NumDebugTags)
		{
			AddInfo(FString::Printf(TEXT("There are only %d defined tags in the Project.  We need at least %d to run this test (skipping)."), AllGameplayTags.Num(), NumDebugTags));
			return true;
		}

		int Step = FMath::DivideAndRoundDown(AllGameplayTags.Num(), NumDebugTags);
		for (int Index = 0; Index < NumDebugTags; ++Index)
		{
			FGameplayTag RandomGameplayTag = AllGameplayTags.GetByIndex(Index * Step);
			PossibleTags.Add(RandomGameplayTag);
		}
	}

	// Compute all permutations with those tags
	TArray<FGameplayTagContainer> PossibleTagContainers;
	for (int Index = 0; Index < NumDebugTags; ++Index)
	{
		FGameplayTagContainer Container;
		Container.AddTag(PossibleTags[Index]);

		for (int InnerIndex = Index; InnerIndex < NumDebugTags; ++InnerIndex)
		{
			Container.AddTag(PossibleTags[InnerIndex]);
			PossibleTagContainers.Add(Container);
		}
	}
	const int NumPossibleContainers = PossibleTagContainers.Num();

	// Build up scenarios and then test them
	for (int ScenarioNum = 0; ScenarioNum < NumTestScenarios; ++ScenarioNum)
	{
		// Assemble the TagRequirements
		FGameplayTagRequirements TagReqs;
		{
			FRandomStream RandomStream{ FMath::GetRandSeed() };
			for (int TagNum = 0; TagNum < NumDebugTags; ++TagNum)
			{
				FGameplayTag GameplayTag = PossibleTags[TagNum];
				switch (RandomStream.RandHelper(3))
				{
				case 0:
					TagReqs.RequireTags.AddTag(GameplayTag);
					break;
				case 1:
					TagReqs.IgnoreTags.AddTag(GameplayTag);
					break;
				case 2:
				default:
					break;
				};
			}
		}

		// Do TagRequirements Test
		{
			uint64 StartTime = FPlatformTime::Cycles64();

			FRandomStream RandomStream{ SomeSeed };
			for (int i = 0; i < NumTestsPerScenario; ++i)
			{
				FGameplayTagContainer& TestAgainstTags = PossibleTagContainers[RandomStream.RandHelper(NumPossibleContainers)];
				NumMetRequirements += TagReqs.RequirementsMet(TestAgainstTags) ? 1 : 0;
			}

			MetRequirementsCycles += FPlatformTime::Cycles64() - StartTime;
		}

		// Do TagQuery Test
		FGameplayTagQuery TagQuery = TagReqs.ConvertTagFieldsToTagQuery();
		{
			uint64 StartTime = FPlatformTime::Cycles64();

			FRandomStream RandomStream{ SomeSeed };
			for (int i = 0; i < NumTestsPerScenario; ++i)
			{
				FGameplayTagContainer& TestAgainstTags = PossibleTagContainers[RandomStream.RandHelper(NumPossibleContainers)];
				NumQueryMatches += TagQuery.Matches(TestAgainstTags) ? 1 : 0;
			}

			QueryMatchesCycles += FPlatformTime::Cycles64() - StartTime;
		}
	} // for each scenario

	AddInfo(FString::Printf(TEXT(" Ran %d tests against FGameplayTagRequirements and FGameplayTagQuery in %.4lf ms"), NumTestScenarios * NumTestsPerScenario, FPlatformTime::ToMilliseconds64(MetRequirementsCycles + QueryMatchesCycles)));
	AddInfo(FString::Printf(TEXT("	FGameplayTagRequirements took %" UINT64_FMT " cycles (%.4lf ms) and gave %u matches"), MetRequirementsCycles, FPlatformTime::ToMilliseconds64(MetRequirementsCycles), NumMetRequirements));
	AddInfo(FString::Printf(TEXT("	FGameplayTagQuery took %" UINT64_FMT " cycles (%.4lf ms) and gave %u matches"), QueryMatchesCycles, FPlatformTime::ToMilliseconds64(QueryMatchesCycles), NumQueryMatches));

	TestEqual(TEXT("FGameplayTagRequirements has same matching rules as FGameplayTagQuery"), NumQueryMatches, NumMetRequirements);
	return NumMetRequirements == NumQueryMatches;
}
