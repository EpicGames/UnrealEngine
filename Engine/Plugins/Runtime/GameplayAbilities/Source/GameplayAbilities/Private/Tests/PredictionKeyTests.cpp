// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "GameplayPrediction.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemTestPawn.h"

#include "GameFramework/PlayerController.h"
#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "UObject/StrongObjectPtr.h"

DEFINE_LOG_CATEGORY_STATIC(LogPredictionKeyTests, Warning, All);

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGameplayPredictionKeyTest_UnitTest, "System.AbilitySystem.PredictionKey.UnitTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGameplayPredictionKeyTest_ScopedPredictionsTest, "System.AbilitySystem.PredictionKey.ScopedPredictions", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

namespace UE::AbilitySystem::Private
{
	// This can affect the outcome of the tests, so we need to query it.
	extern int32 CVarDependentChainBehaviorValue;
}

/**
 * FPredictionKey Wrapper class that allows us to accept/reject keys and also query
 * if a key has been acknowledged/accepted/rejected
 */
struct FPredictionKeyTestWrapper
{
	FPredictionKeyTestWrapper() = default;
	~FPredictionKeyTestWrapper()
	{
		UnbindDelegates();
	}

	// No copying
	FPredictionKeyTestWrapper(const FPredictionKeyTestWrapper& Other) = delete;
	FPredictionKeyTestWrapper& operator=(const FPredictionKeyTestWrapper&) = delete;

	// Allow moving
	FPredictionKeyTestWrapper& operator=(FPredictionKeyTestWrapper&& Other)
	{
		Discard();
		Other.Discard();

		bAcknowledged = bAccepted = bRejected = false;
		Key = MoveTemp(Other.Key);
		BindDelegates();

		// Make sure it's zero'd out or it will then try to Unbind the newly bound delegates
		Other.Key = FPredictionKey{};
		return *this;
	}
	FPredictionKeyTestWrapper(FPredictionKeyTestWrapper&& Other)
	{
		*this = MoveTemp(Other);
	}

	static FPredictionKeyTestWrapper CreateDependentKey(const FPredictionKey& BasedOn)
	{
		FPredictionKey Key = BasedOn;
		Key.GenerateDependentPredictionKey();
		return FPredictionKeyTestWrapper { Key };
	}

	static FPredictionKeyTestWrapper CreateDependentKey(const FPredictionKeyTestWrapper& BasedOn)
	{
		return CreateDependentKey(BasedOn.GetKey());
	}

	static FPredictionKeyTestWrapper CreateNewClientKey()
	{
		return CreateDependentKey(FPredictionKey{});
	}

	static FPredictionKeyTestWrapper CopyFromASC(const UAbilitySystemComponent* ASC)
	{
		return FPredictionKeyTestWrapper(ASC->ScopedPredictionKey);
	}

	void Reject()
	{
		FPredictionKeyDelegates::Reject(Key.Current);
	}

	void Accept()
	{
		FPredictionKeyDelegates::CatchUpTo(Key.Current);
	}

	bool GetRejected() const { return bRejected; }
	bool GetAccepted() const { return bAccepted; }
	bool GetAcknowledged() const { return bAcknowledged; }
	const FPredictionKey& GetKey() const { return Key; }

	// Allow us to discard the results safely
	void Discard() { UnbindDelegates(false); }

private:

	FPredictionKeyTestWrapper(const FPredictionKey& InKey)
	: Key(InKey)
	{
		BindDelegates();
	}

	void BindDelegates()
	{
		if (Key.Current > 0)
		{
			Key.NewRejectedDelegate().BindLambda([this]() { bRejected = bAcknowledged = true; });
			Key.NewCaughtUpDelegate().BindLambda([this]() { bAccepted = bAcknowledged = true; });
			Key.NewRejectOrCaughtUpDelegate(FPredictionKeyEvent::CreateLambda([this]() { bAcknowledged = true; }));
		}
	}

	void UnbindDelegates(bool bWarnIfNotTriggered = true)
	{
		if (Key.Current > 0)
		{
			int32 NumRemoved = FPredictionKeyDelegates::Get().DelegateMap.Remove(Key.Current);
			UE_CLOGF(bWarnIfNotTriggered && NumRemoved > 0, LogPredictionKeyTests, Warning, "%s FPredictionKeyDelegate still had entry for %d on destruction", __func__, Key.Current);
		}
	}

private:
	// The underlying Key
	FPredictionKey Key;

	// Are we either Accepted or Rejected?
	bool bAccepted = false;
	bool bRejected = false;

	// This is either Accepted or Rejected (but we don't know which one)
	bool bAcknowledged = false;
};

bool FGameplayPredictionKeyTest_UnitTest::RunTest(const FString& Parameters)
{
	// Test basic functionality:  BaseKey and DependentKey
	{
		FPredictionKeyTestWrapper BaseKey = FPredictionKeyTestWrapper::CreateNewClientKey();
		FPredictionKeyTestWrapper DependentKey = FPredictionKeyTestWrapper::CreateDependentKey(BaseKey);

		TestEqual(TEXT("DependentKey is Based on BaseKey"), BaseKey.GetKey().Current, DependentKey.GetKey().Base);
		TestTrue(TEXT("DependentKey is Greater than BaseKey"), DependentKey.GetKey().Current > BaseKey.GetKey().Current);

		DependentKey.Discard();
		BaseKey.Discard();
	}

	if (!(UE::AbilitySystem::Private::CVarDependentChainBehaviorValue & 0x1))
	{
		// Sadly we can't make these warnings/errors or Horde will complain about tests failing
		AddInfo(TEXT("AbilitySystem.PredictionKey.DepChainBehavior needs bitflag & 0x01 (new keys accepted imply old keys accepted) to make tests meangingful"));
		return true;
	}

	// Test Branch:
	// Base -> DependentKey	-> AcceptKey
	//				  		|-> RejectKey
	AddInfo(TEXT("Testing Branching Keys:  Base->DependentKey->(AcceptKey, RejectKey)"));
	{

		FPredictionKeyTestWrapper BaseKey = FPredictionKeyTestWrapper::CreateNewClientKey();
		FPredictionKeyTestWrapper DependentKey = FPredictionKeyTestWrapper::CreateDependentKey(BaseKey);
		FPredictionKeyTestWrapper AcceptKey = FPredictionKeyTestWrapper::CreateDependentKey(DependentKey);
		FPredictionKeyTestWrapper RejectKey = FPredictionKeyTestWrapper::CreateDependentKey(DependentKey);

		AddInfo(TEXT("  Rejecting RejectKey only."));
		RejectKey.Reject();

		TestTrue(	TEXT("    RejectKey is Acknowledged."), RejectKey.GetAcknowledged());
		TestTrue(	TEXT("    RejectKey is Rejected."), RejectKey.GetRejected());
		TestFalse(	TEXT("    RejectKey is Accepted."), RejectKey.GetAccepted());

		TestFalse(	TEXT("    AcceptKey is Acknowledged."), AcceptKey.GetAcknowledged());
		TestFalse(	TEXT("    AcceptKey is Rejected."), AcceptKey.GetRejected());
		TestFalse(	TEXT("    AcceptKey is Accepted."), AcceptKey.GetAccepted());

		TestFalse(	TEXT("    DependentKey is Acknowledged."), DependentKey.GetAcknowledged());
		TestFalse(	TEXT("    DependentKey is Rejected."), DependentKey.GetRejected());
		TestFalse(	TEXT("    DependentKey is Accepted."), DependentKey.GetAccepted());

		TestFalse(	TEXT("    BaseKey is Acknowledged."), BaseKey.GetAcknowledged());
		TestFalse(	TEXT("    BaseKey is Rejected."), BaseKey.GetRejected());
		TestFalse(	TEXT("    BaseKey is Accepted."), BaseKey.GetAccepted());

		AddInfo(TEXT("  Accepting AcceptKey."));
		AcceptKey.Accept();

		TestTrue(	TEXT("    RejectKey is Acknowledged."), RejectKey.GetAcknowledged());
		TestTrue(	TEXT("    RejectKey is Rejected."), RejectKey.GetRejected());
		TestFalse(	TEXT("    RejectKey is Accepted."), RejectKey.GetAccepted());

		TestTrue(	TEXT("    AcceptKey is Acknowledged."), AcceptKey.GetAcknowledged());
		TestFalse(	TEXT("    AcceptKey is Rejected."), AcceptKey.GetRejected());
		TestTrue(	TEXT("    AcceptKey is Accepted."), AcceptKey.GetAccepted());

		TestTrue(	TEXT("    DependentKey is Acknowledged."), DependentKey.GetAcknowledged());
		TestFalse(	TEXT("    DependentKey is Rejected."), DependentKey.GetRejected());
		TestTrue(	TEXT("    DependentKey is Accepted."), DependentKey.GetAccepted());

		TestTrue(	TEXT("    BaseKey is Acknowledged."), BaseKey.GetAcknowledged());
		TestFalse(	TEXT("    BaseKey is Rejected."), BaseKey.GetRejected());
		TestTrue(	TEXT("    BaseKey is Accepted."), BaseKey.GetAccepted());
	}

	// Test Chain: RejectKey should not reject any previous keys.  AcceptKey accepts DependentKey & BaseKey.
	// Base -> DependentKey -> AcceptKey -> RejectKey
	AddInfo(TEXT("Testing Chained Keys:  Base->DependentKey->AcceptKey->RejectKey"));
	{
		FPredictionKeyTestWrapper BaseKey = FPredictionKeyTestWrapper::CreateNewClientKey();
		FPredictionKeyTestWrapper DependentKey = FPredictionKeyTestWrapper::CreateDependentKey(BaseKey);
		FPredictionKeyTestWrapper AcceptKey = FPredictionKeyTestWrapper::CreateDependentKey(DependentKey);
		FPredictionKeyTestWrapper RejectKey = FPredictionKeyTestWrapper::CreateDependentKey(AcceptKey);

		AddInfo(TEXT("  Rejecting RejectKey only."));
		RejectKey.Reject();

		TestTrue(	TEXT("    RejectKey is Acknowledged."), RejectKey.GetAcknowledged());
		TestTrue(	TEXT("    RejectKey is Rejected."), RejectKey.GetRejected());
		TestFalse(	TEXT("    RejectKey is Accepted."), RejectKey.GetAccepted());

		TestFalse(	TEXT("    AcceptKey is Acknowledged."), AcceptKey.GetAcknowledged());
		TestFalse(	TEXT("    AcceptKey is Rejected."), AcceptKey.GetRejected());
		TestFalse(	TEXT("    AcceptKey is Accepted."), AcceptKey.GetAccepted());

		TestFalse(	TEXT("    DependentKey is Acknowledged."), DependentKey.GetAcknowledged());
		TestFalse(	TEXT("    DependentKey is Rejected."), DependentKey.GetRejected());
		TestFalse(	TEXT("    DependentKey is Accepted."), DependentKey.GetAccepted());

		TestFalse(	TEXT("    BaseKey is Acknowledged."), BaseKey.GetAcknowledged());
		TestFalse(	TEXT("    BaseKey is Rejected."), BaseKey.GetRejected());
		TestFalse(	TEXT("    BaseKey is Accepted."), BaseKey.GetAccepted());

		AddInfo(TEXT("  Accepting AcceptKey."));
		AcceptKey.Accept();

		TestTrue(	TEXT("    RejectKey is Acknowledged."), RejectKey.GetAcknowledged());
		TestTrue(	TEXT("    RejectKey is Rejected."), RejectKey.GetRejected());
		TestFalse(	TEXT("    RejectKey is Accepted."), RejectKey.GetAccepted());

		TestTrue(	TEXT("    AcceptKey is Acknowledged."), AcceptKey.GetAcknowledged());
		TestFalse(	TEXT("    AcceptKey is Rejected."), AcceptKey.GetRejected());
		TestTrue(	TEXT("    AcceptKey is Accepted."), AcceptKey.GetAccepted());

		TestTrue(	TEXT("    DependentKey is Acknowledged."), DependentKey.GetAcknowledged());
		TestFalse(	TEXT("    DependentKey is Rejected."), DependentKey.GetRejected());
		TestTrue(	TEXT("    DependentKey is Accepted."), DependentKey.GetAccepted());

		TestTrue(	TEXT("    BaseKey is Acknowledged."), BaseKey.GetAcknowledged());
		TestFalse(	TEXT("    BaseKey is Rejected."), BaseKey.GetRejected());
		TestTrue(	TEXT("    BaseKey is Accepted."), BaseKey.GetAccepted());
	}

	// Test Chain: AcceptKey accepts DependentKey & BaseKey.  It should not accept RejectKey.
	// Base -> DependentKey -> AcceptKey -> RejectKey
	AddInfo(TEXT("Testing Chained Keys:  Base->DependentKey->AcceptKey->RejectKey"));
	if (UE::AbilitySystem::Private::CVarDependentChainBehaviorValue > 1)
	{
		FPredictionKeyTestWrapper BaseKey = FPredictionKeyTestWrapper::CreateNewClientKey();
		FPredictionKeyTestWrapper DependentKey = FPredictionKeyTestWrapper::CreateDependentKey(BaseKey);
		FPredictionKeyTestWrapper AcceptKey = FPredictionKeyTestWrapper::CreateDependentKey(DependentKey);
		FPredictionKeyTestWrapper RejectKey = FPredictionKeyTestWrapper::CreateDependentKey(AcceptKey);

		AddInfo(TEXT("  Accepting AcceptKey."));
		AcceptKey.Accept();

		TestFalse(	TEXT("    RejectKey is Acknowledged."), RejectKey.GetAcknowledged());
		TestFalse(	TEXT("    RejectKey is Rejected."), RejectKey.GetRejected());
		TestFalse(	TEXT("    RejectKey is Accepted."), RejectKey.GetAccepted());

		TestTrue(	TEXT("    AcceptKey is Acknowledged."), AcceptKey.GetAcknowledged());
		TestFalse(	TEXT("    AcceptKey is Rejected."), AcceptKey.GetRejected());
		TestTrue(	TEXT("    AcceptKey is Accepted."), AcceptKey.GetAccepted());

		TestTrue(	TEXT("    DependentKey is Acknowledged."), DependentKey.GetAcknowledged());
		TestFalse(	TEXT("    DependentKey is Rejected."), DependentKey.GetRejected());
		TestTrue(	TEXT("    DependentKey is Accepted."), DependentKey.GetAccepted());

		TestTrue(	TEXT("    BaseKey is Acknowledged."), BaseKey.GetAcknowledged());
		TestFalse(	TEXT("    BaseKey is Rejected."), BaseKey.GetRejected());
		TestTrue(	TEXT("    BaseKey is Accepted."), BaseKey.GetAccepted());

		AddInfo(TEXT("  Rejecting RejectKey."));
		RejectKey.Reject();

		TestTrue(	TEXT("    RejectKey is Acknowledged."), RejectKey.GetAcknowledged());
		TestTrue(	TEXT("    RejectKey is Rejected."), RejectKey.GetRejected());
		TestFalse(	TEXT("    RejectKey is Accepted."), RejectKey.GetAccepted());

		TestTrue(	TEXT("    AcceptKey is Acknowledged."), AcceptKey.GetAcknowledged());
		TestFalse(	TEXT("    AcceptKey is Rejected."), AcceptKey.GetRejected());
		TestTrue(	TEXT("    AcceptKey is Accepted."), AcceptKey.GetAccepted());

		TestTrue(	TEXT("    DependentKey is Acknowledged."), DependentKey.GetAcknowledged());
		TestFalse(	TEXT("    DependentKey is Rejected."), DependentKey.GetRejected());
		TestTrue(	TEXT("    DependentKey is Accepted."), DependentKey.GetAccepted());

		TestTrue(	TEXT("    BaseKey is Acknowledged."), BaseKey.GetAcknowledged());
		TestFalse(	TEXT("    BaseKey is Rejected."), BaseKey.GetRejected());
		TestTrue(	TEXT("    BaseKey is Accepted."), BaseKey.GetAccepted());
	}
	else
	{
		AddInfo(TEXT("  Skip: AbilitySystem.PredictionKey.DepChainBehavior needs to be >= 2 for this test to have correct results"));
	}

	// Test Chain: RejectKey rejects AcceptKey.  It does not affect DependentKey or BaseKey.
	// Base -> DependentKey -> RejectKey -> AcceptKey
	{
		AddInfo(TEXT("Testing Chained Keys:  Base->DependentKey->RejectKey->AcceptKey"));

		FPredictionKeyTestWrapper BaseKey = FPredictionKeyTestWrapper::CreateNewClientKey();
		FPredictionKeyTestWrapper DependentKey = FPredictionKeyTestWrapper::CreateDependentKey(BaseKey);
		FPredictionKeyTestWrapper RejectKey = FPredictionKeyTestWrapper::CreateDependentKey(DependentKey);
		FPredictionKeyTestWrapper AcceptKey = FPredictionKeyTestWrapper::CreateDependentKey(RejectKey);

		AddInfo(TEXT("  Rejecting RejectKey."));
		RejectKey.Reject();

		TestTrue(	TEXT("    RejectKey is Acknowledged."), RejectKey.GetAcknowledged());
		TestTrue(	TEXT("    RejectKey is Rejected."), RejectKey.GetRejected());
		TestFalse(	TEXT("    RejectKey is Accepted."), RejectKey.GetAccepted());

		TestTrue(	TEXT("    AcceptKey is Acknowledged."), AcceptKey.GetAcknowledged());
		TestTrue(	TEXT("    AcceptKey is Rejected."), AcceptKey.GetRejected());
		TestFalse(	TEXT("    AcceptKey is Accepted."), AcceptKey.GetAccepted());

		TestFalse(	TEXT("    DependentKey is Acknowledged."), DependentKey.GetAcknowledged());
		TestFalse(	TEXT("    DependentKey is Rejected."), DependentKey.GetRejected());
		TestFalse(	TEXT("    DependentKey is Accepted."), DependentKey.GetAccepted());

		TestFalse(	TEXT("    BaseKey is Acknowledged."), BaseKey.GetAcknowledged());
		TestFalse(	TEXT("    BaseKey is Rejected."), BaseKey.GetRejected());
		TestFalse(	TEXT("    BaseKey is Accepted."), BaseKey.GetAccepted());

		AddInfo(TEXT("  Accepting AcceptKey (but it has already been rejected)."));
		AcceptKey.Accept();

		TestTrue(	TEXT("    RejectKey is Acknowledged."), RejectKey.GetAcknowledged());
		TestTrue(	TEXT("    RejectKey is Rejected."), RejectKey.GetRejected());
		TestFalse(	TEXT("    RejectKey is Accepted."), RejectKey.GetAccepted());

		TestTrue(	TEXT("    AcceptKey is Acknowledged."), AcceptKey.GetAcknowledged());
		TestTrue(	TEXT("    AcceptKey is Rejected."), AcceptKey.GetRejected());
		TestFalse(	TEXT("    AcceptKey is Accepted."), AcceptKey.GetAccepted());

		// Since AcceptKey was already rejected, it loses its dependency chain and these go unack'd
		TestFalse(	TEXT("    DependentKey is Acknowledged."), DependentKey.GetAcknowledged());
		TestFalse(	TEXT("    DependentKey is Rejected."), DependentKey.GetRejected());
		TestFalse(	TEXT("    DependentKey is Accepted."), DependentKey.GetAccepted());

		TestFalse(	TEXT("    BaseKey is Acknowledged."), BaseKey.GetAcknowledged());
		TestFalse(	TEXT("    BaseKey is Rejected."), BaseKey.GetRejected());
		TestFalse(	TEXT("    BaseKey is Accepted."), BaseKey.GetAccepted());

		AddInfo(TEXT("  Accepting DependentKey (it should be detached from the dep chain)."));
		DependentKey.Accept();

		TestTrue(	TEXT("    RejectKey is Acknowledged."), RejectKey.GetAcknowledged());
		TestTrue(	TEXT("    RejectKey is Rejected."), RejectKey.GetRejected());
		TestFalse(	TEXT("    RejectKey is Accepted."), RejectKey.GetAccepted());

		TestTrue(	TEXT("    AcceptKey is Acknowledged."), AcceptKey.GetAcknowledged());
		TestTrue(	TEXT("    AcceptKey is Rejected."), AcceptKey.GetRejected());
		TestFalse(	TEXT("    AcceptKey is Accepted."), AcceptKey.GetAccepted());

		// Since AcceptKey was already rejected, it loses its dependency chain and these go unack'd
		TestTrue(	TEXT("    DependentKey is Acknowledged."), DependentKey.GetAcknowledged());
		TestFalse(	TEXT("    DependentKey is Rejected."), DependentKey.GetRejected());
		TestTrue(	TEXT("    DependentKey is Accepted."), DependentKey.GetAccepted());

		TestTrue(	TEXT("    BaseKey is Acknowledged."), BaseKey.GetAcknowledged());
		TestFalse(	TEXT("    BaseKey is Rejected."), BaseKey.GetRejected());
		TestTrue(	TEXT("    BaseKey is Accepted."), BaseKey.GetAccepted());
	}

	// Test Chain: AcceptKey accepts all Keys.  Rejecting RejectKey has no affect.
	// Base -> DependentKey -> RejectKey -> AcceptKey
	{
		AddInfo(TEXT("Testing Chained Keys:  Base->DependentKey->RejectKey->AcceptKey"));

		FPredictionKeyTestWrapper BaseKey = FPredictionKeyTestWrapper::CreateNewClientKey();
		FPredictionKeyTestWrapper DependentKey = FPredictionKeyTestWrapper::CreateDependentKey(BaseKey);
		FPredictionKeyTestWrapper RejectKey = FPredictionKeyTestWrapper::CreateDependentKey(DependentKey);
		FPredictionKeyTestWrapper AcceptKey = FPredictionKeyTestWrapper::CreateDependentKey(RejectKey);

		AddInfo(TEXT("  Accepting AcceptKey."));
		AcceptKey.Accept();

		TestTrue(	TEXT("    RejectKey is Acknowledged."), RejectKey.GetAcknowledged());
		TestFalse(	TEXT("    RejectKey is Rejected."), RejectKey.GetRejected());
		TestTrue(	TEXT("    RejectKey is Accepted."), RejectKey.GetAccepted());

		TestTrue(	TEXT("    AcceptKey is Acknowledged."), AcceptKey.GetAcknowledged());
		TestFalse(	TEXT("    AcceptKey is Rejected."), AcceptKey.GetRejected());
		TestTrue(	TEXT("    AcceptKey is Accepted."), AcceptKey.GetAccepted());

		TestTrue(	TEXT("    DependentKey is Acknowledged."), DependentKey.GetAcknowledged());
		TestFalse(	TEXT("    DependentKey is Rejected."), DependentKey.GetRejected());
		TestTrue(	TEXT("    DependentKey is Accepted."), DependentKey.GetAccepted());

		TestTrue(	TEXT("    BaseKey is Acknowledged."), BaseKey.GetAcknowledged());
		TestFalse(	TEXT("    BaseKey is Rejected."), BaseKey.GetRejected());
		TestTrue(	TEXT("    BaseKey is Accepted."), BaseKey.GetAccepted());

		AddInfo(TEXT("  Rejecting RejectKey (but it has already been accepted)."));
		RejectKey.Reject();

		TestTrue(	TEXT("    RejectKey is Acknowledged."), RejectKey.GetAcknowledged());
		TestFalse(	TEXT("    RejectKey is Rejected."), RejectKey.GetRejected());
		TestTrue(	TEXT("    RejectKey is Accepted."), RejectKey.GetAccepted());

		TestTrue(	TEXT("    AcceptKey is Acknowledged."), AcceptKey.GetAcknowledged());
		TestFalse(	TEXT("    AcceptKey is Rejected."), AcceptKey.GetRejected());
		TestTrue(	TEXT("    AcceptKey is Accepted."), AcceptKey.GetAccepted());

		TestTrue(	TEXT("    DependentKey is Acknowledged."), DependentKey.GetAcknowledged());
		TestFalse(	TEXT("    DependentKey is Rejected."), DependentKey.GetRejected());
		TestTrue(	TEXT("    DependentKey is Accepted."), DependentKey.GetAccepted());

		TestTrue(	TEXT("    BaseKey is Acknowledged."), BaseKey.GetAcknowledged());
		TestFalse(	TEXT("    BaseKey is Rejected."), BaseKey.GetRejected());
		TestTrue(	TEXT("    BaseKey is Accepted."), BaseKey.GetAccepted());
	}

	return true;
}

bool FGameplayPredictionKeyTest_ScopedPredictionsTest::RunTest(const FString& Parameters)
{
	// This will get cleaned up when it leaves scope
	FTestWorldWrapper WorldWrapper;
	WorldWrapper.CreateTestWorld(EWorldType::Game);
	UWorld* World = WorldWrapper.GetTestWorld();

	if (!World)
	{
		return false;
	}

	WorldWrapper.BeginPlayInTestWorld();

	// set up the source actor
	TStrongObjectPtr<AAbilitySystemTestPawn> SourceActor{ World->SpawnActor<AAbilitySystemTestPawn>() };
	TStrongObjectPtr<APlayerController> SourceController{ World->SpawnActor<APlayerController>() };
	if (!SourceActor || !SourceController)
	{
		AddError(TEXT("Could not Spawn SourceActor or SourceController"));
		return false;
	}

	SourceController->Possess(SourceActor.Get());
	UAbilitySystemComponent* SourceASC = SourceActor->GetAbilitySystemComponent();
	if (!TestNotNull(TEXT("Could Not Obtain AbilitySystemComponent"), SourceASC))
	{
		return false;
	}

	// Masquerade as a client
	SourceActor->SetRole(ENetRole::ROLE_SimulatedProxy);
	SourceController->SetRole(ENetRole::ROLE_AutonomousProxy);
	SourceASC->InitAbilityActorInfo(SourceController.Get(), SourceActor.Get());
	SourceASC->CacheIsNetSimulated();

	// Tests start here
	AddInfo(TEXT("Initial State"));
	TestFalse(TEXT("  CanPredict"), SourceASC->CanPredict());
	TestFalse(TEXT("  ScopedPredictionKey IsValid"), SourceASC->GetPredictionKeyForNewAction().IsValidKey());

	// Let's start a scoped prediction window
	AddInfo(TEXT("Starting Main Prediction Window"));
	{
		FScopedPredictionWindow ScopedPrediction(SourceASC);
		TestTrue(	TEXT("  CanPredict"), SourceASC->CanPredict());
		TestTrue(	TEXT("  ScopedPredictionKey IsValid"), SourceASC->GetPredictionKeyForNewAction().IsValidKey());
		TestTrue(	TEXT("  ScopedPredictionKey IsLocalClientKey"), SourceASC->GetPredictionKeyForNewAction().IsLocalClientKey());
		TestFalse(	TEXT("  ScopedPredictionKey IsServerInitiatedKey"), SourceASC->GetPredictionKeyForNewAction().IsServerInitiatedKey());

		AddInfo(TEXT("  Discard Prediction Window (SilentlyDrop)"));
		{
			FPredictionKeyTestWrapper DiscardPredictionKey;
			{
				FScopedDiscardPredictions ScopedDiscardPredictions(SourceASC, EGasPredictionKeyResult::SilentlyDrop);
				DiscardPredictionKey = FPredictionKeyTestWrapper::CopyFromASC(SourceASC);
			}
			TestFalse(TEXT("    DiscardPredictionKey Acknowledged"), DiscardPredictionKey.GetAcknowledged());
			TestFalse(TEXT("    DiscardPredictionKey Accepted"), DiscardPredictionKey.GetAccepted());
			TestFalse(TEXT("    DiscardPredictionKey Rejected"), DiscardPredictionKey.GetRejected());

			DiscardPredictionKey.Discard();
		}

		AddInfo(TEXT("  Discard Prediction Window (Auto-Accept)"));
		{
			FPredictionKeyTestWrapper DiscardPredictionKey;
			{
				FScopedDiscardPredictions ScopedDiscardPredictions(SourceASC, EGasPredictionKeyResult::Accept);
				DiscardPredictionKey = FPredictionKeyTestWrapper::CopyFromASC(SourceASC);
			}
			TestTrue(	TEXT("    DiscardPredictionKey Acknowledged"), DiscardPredictionKey.GetAcknowledged());
			TestTrue(	TEXT("    DiscardPredictionKey Accepted"), DiscardPredictionKey.GetAccepted());
			TestFalse(	TEXT("    DiscardPredictionKey Rejected"), DiscardPredictionKey.GetRejected());
		}

		AddInfo(TEXT("  Discard Prediction Window (Auto-Reject)"));
		{
			FPredictionKeyTestWrapper DiscardPredictionKey;
			{
				FScopedDiscardPredictions ScopedDiscardPredictions(SourceASC, EGasPredictionKeyResult::Reject);
				DiscardPredictionKey = FPredictionKeyTestWrapper::CopyFromASC(SourceASC);
			}
			TestTrue(	TEXT("    DiscardPredictionKey Acknowledged"), DiscardPredictionKey.GetAcknowledged());
			TestFalse(	TEXT("    DiscardPredictionKey Accepted"), DiscardPredictionKey.GetAccepted());
			TestTrue(	TEXT("    DiscardPredictionKey Rejected"), DiscardPredictionKey.GetRejected());
		}

		// Make sure none of those discard windows messed up the original key
		TestTrue(	TEXT("  CanPredict"), SourceASC->CanPredict());
		TestTrue(	TEXT("  ScopedPredictionKey IsValid"), SourceASC->GetPredictionKeyForNewAction().IsValidKey());
		TestTrue(	TEXT("  ScopedPredictionKey IsLocalClientKey"), SourceASC->GetPredictionKeyForNewAction().IsLocalClientKey());
		TestFalse(	TEXT("  ScopedPredictionKey IsServerInitiatedKey"), SourceASC->GetPredictionKeyForNewAction().IsServerInitiatedKey());
	}
	AddInfo(TEXT("Ending Main Prediction Window"));
	TestFalse(TEXT("  CanPredict"), SourceASC->CanPredict());
	TestFalse(TEXT("  ScopedPredictionKey IsValid"), SourceASC->GetPredictionKeyForNewAction().IsValidKey());

	WorldWrapper.ForwardErrorMessages(this);
	return !HasAnyErrors();
}