// Copyright Epic Games, Inc. All Rights Reserved.

#include "TickableAttributeSetInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TickableAttributeSetInterface)

UTickableAttributeSetInterface::UTickableAttributeSetInterface(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

