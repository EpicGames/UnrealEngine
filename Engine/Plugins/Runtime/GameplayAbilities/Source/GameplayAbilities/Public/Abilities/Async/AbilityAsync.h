// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "GameplayEffectTypes.h"
#include "Abilities/GameplayAbilityTypes.h"
#include "Engine/CancellableAsyncAction.h"
#include "AbilityAsync.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

/**
 * AbilityAsync is a base class for ability-specific BlueprintAsyncActions.
 * These are similar to ability tasks, but they can be executed from any blueprint like an actor and are not tied to a specific ability lifespan.
 * By default these actions are only kept alive by the blueprint graph that spawns them and will eventually be destroyed if the graph instance is deleted or spawns a replacement.
 * EndAction should be called when a one-time action has succeeded or failed, but for longer-lived actions with multiple triggers it can be called from blueprints.
 */
UCLASS(Abstract, meta = (ExposedAsyncProxy = AsyncAction), MinimalAPI)
class UAbilityAsync : public UCancellableAsyncAction
{
	GENERATED_BODY()
public:

	UE_API virtual void Cancel() override;

	/** Explicitly end the action, will disable any callbacks and allow action to be destroyed */
	UFUNCTION(BlueprintCallable, Category = "Ability|Async")
	UE_API virtual void EndAction();

	/** This should be called prior to broadcasting delegates back into the event graph, this ensures the action and ability system component are still valid */
	UE_API virtual bool ShouldBroadcastDelegates() const;

	/** Returns the ability system component this action is bound to */
	UE_API virtual UAbilitySystemComponent* GetAbilitySystemComponent() const;

	/** Sets the bound component */
	UE_API virtual void SetAbilitySystemComponent(UAbilitySystemComponent* InAbilitySystemComponent);

	/** Sets the bound component by searching actor for one */
	UE_API virtual void SetAbilityActor(AActor* InActor);

private:
	TWeakObjectPtr<UAbilitySystemComponent> AbilitySystemComponent;

};

#undef UE_API
