// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "AbilityAsync.h"
#include "GameplayEffectTypes.h"
#include "Abilities/GameplayAbilityTypes.h"
#include "AbilityAsync_WaitAttributeChanged.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

UCLASS(MinimalAPI)
class UAbilityAsync_WaitAttributeChanged : public UAbilityAsync
{
	GENERATED_BODY()

public:

	/**
	 * Wait until the specified gameplay attribute is changed on a target ability system component
	 * It will keep listening as long as OnlyTriggerOnce = false
	 * If used in an ability graph, this async action will wait even after activation ends. It's recommended to use WaitForAttributeChange instead.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (DefaultToSelf = "TargetActor", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityAsync_WaitAttributeChanged* WaitForAttributeChanged(AActor* TargetActor, FGameplayAttribute Attribute, bool OnlyTriggerOnce = false);

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FAsyncWaitAttributeChangedDelegate, FGameplayAttribute, Attribute, float, NewValue, float, OldValue);
	UPROPERTY(BlueprintAssignable)
	FAsyncWaitAttributeChangedDelegate Changed;

protected:

	UE_API virtual void Activate() override;
	UE_API virtual void EndAction() override;

	UE_API void OnAttributeChanged(const FOnAttributeChangeData& ChangeData);

	FGameplayAttribute Attribute;
	bool OnlyTriggerOnce = false;

	FDelegateHandle MyHandle;
};

#undef UE_API
