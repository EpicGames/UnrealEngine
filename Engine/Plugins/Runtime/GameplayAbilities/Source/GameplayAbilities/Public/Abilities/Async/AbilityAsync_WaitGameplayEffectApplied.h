// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "AbilityAsync.h"
#include "Abilities/GameplayAbilityTargetDataFilter.h"
#include "GameplayEffectTypes.h"
#include "Delegates/IDelegateInstance.h"
#include "AbilityAsync_WaitGameplayEffectApplied.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

/**
 * This action listens for specific gameplay effect applications based off specified tags. 
 * Effects themselves are not replicated; rather the tags they grant, the attributes they 
 * change, and the gameplay cues they emit are replicated.
 * This will only listen for local server or predicted client effects.
 */
UCLASS(MinimalAPI)
class UAbilityAsync_WaitGameplayEffectApplied : public UAbilityAsync
{
	GENERATED_BODY()

public:
	/**
	 * Wait until a GameplayEffect is applied to a target actor.
	 * 
	 * @param TargetActor The actor on which to listen for a GameplayEffect being applied to it
	 * @param SourceFilter Actor filter for the source actor applying this effect that must be passed to trigger this task
	 * @param SourceTagRequirements Gameplay tag requirements for the source actor applying this effect
	 * @param TargetTagRequirements Gameplay tag requirements for the target actor receiving this effect (self in this case)
	 * @param AssetTagRequirements Requirements for the asset tags of the applied effect: 'Tags this Effect Has' on the GameplayEffect class
	 * @param GrantedTagRequirements Requirements for the tags granted by the applied effect: 'Grant Tags to Target Actor' on the GameplayEffect class
	 * @param TriggerOnce If TriggerOnce is true, this task will only return one time. Otherwise it will return everytime a GE is applied that meets the requirements over the life of the ability.
	 * @param OptionalExternalOwner Can be used to run this task on someone else (not the owner of the ability). By default you can leave this empty.
	 * @param ListenForPeriodicEffects Whether to also fire on periodic ticks of the applied effect
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|Async", meta = (DefaultToSelf = "TargetActor", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityAsync_WaitGameplayEffectApplied* WaitGameplayEffectAppliedToActor(AActor* TargetActor, const FGameplayTargetDataFilterHandle SourceFilter, FGameplayTagRequirements SourceTagRequirements, FGameplayTagRequirements TargetTagRequirements, FGameplayTagRequirements AssetTagRequirements, FGameplayTagRequirements GrantedTagRequirements, bool TriggerOnce = false, bool ListenForPeriodicEffect = false);

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnAppliedDelegate, AActor*, Source, FGameplayEffectSpecHandle, SpecHandle, FActiveGameplayEffectHandle, ActiveHandle);
	UPROPERTY(BlueprintAssignable)
	FOnAppliedDelegate OnApplied;

protected:
	UE_API virtual void Activate() override;
	UE_API virtual void EndAction() override;

	UE_API FString GetBroadcastingEffectStackString() const;

	UE_API void OnApplyGameplayEffectCallback(UAbilitySystemComponent* Target, const FGameplayEffectSpec& SpecApplied, FActiveGameplayEffectHandle ActiveHandle);

	// Actor filter for the source actor applying this effect that must be passed to trigger this task
	FGameplayTargetDataFilterHandle Filter;
	// Tag requirements for source actor that applies the GameplayEffect
	FGameplayTagRequirements SourceTagRequirements;
	// Tag requirements for target actor to which the GameplayEffect is applied
	FGameplayTagRequirements TargetTagRequirements;
	// Tag requirements for asset tags ('Tags this Effect Has') on the GameplayEffect being applied
	FGameplayTagRequirements AssetTagRequirements;
	// Tag requirements for granted tags ('Grant Tags to Target Actor') that the GameplayEffect applies to the target actor
	FGameplayTagRequirements GrantedTagRequirements;

	bool TriggerOnce = false;
	bool ListenForPeriodicEffects = false;

	FDelegateHandle OnApplyGameplayEffectCallbackDelegateHandle;
	FDelegateHandle OnPeriodicGameplayEffectExecuteCallbackDelegateHandle;
	
	// If we are in the process of broadcasting, we do not allow a GameplayEffect class to trigger application of another instance of the same class.
	TArray<TSubclassOf<class UGameplayEffect>> BroadcastingEffectStack;
};

#undef UE_API
