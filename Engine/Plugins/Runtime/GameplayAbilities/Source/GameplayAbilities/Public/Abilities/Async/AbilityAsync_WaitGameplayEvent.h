// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "AbilityAsync.h"
#include "GameplayEffectTypes.h"
#include "Abilities/GameplayAbilityTypes.h"
#include "AbilityAsync_WaitGameplayEvent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

UCLASS(MinimalAPI)
class UAbilityAsync_WaitGameplayEvent : public UAbilityAsync
{
	GENERATED_BODY()

public:
	/**
	 * Wait until the specified gameplay tag event is triggered on a target ability system component
	 * It will keep listening as long as OnlyTriggerOnce = false
	 * If OnlyMatchExact = false it will trigger for nested tags
	 * If used in an ability graph, this async action will wait even after activation ends. It's recommended to use WaitGameplayEvent instead.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|Async", meta = (DefaultToSelf = "TargetActor", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityAsync_WaitGameplayEvent* WaitGameplayEventToActor(AActor* TargetActor, UPARAM(meta=(GameplayTagFilter="GameplayEventTagsCategory")) FGameplayTag EventTag, bool OnlyTriggerOnce = false, bool OnlyMatchExact = true);

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FEventReceivedDelegate, FGameplayEventData, Payload);

	UPROPERTY(BlueprintAssignable)
	FEventReceivedDelegate EventReceived;

protected:
	UE_API virtual void Activate() override;
	UE_API virtual void EndAction() override;

	UE_API virtual void GameplayEventCallback(const FGameplayEventData* Payload);
	UE_API virtual void GameplayEventContainerCallback(FGameplayTag MatchingTag, const FGameplayEventData* Payload);

	FGameplayTag Tag;
	bool OnlyTriggerOnce = false;
	bool OnlyMatchExact = false;

	FDelegateHandle MyHandle;
};

#undef UE_API
