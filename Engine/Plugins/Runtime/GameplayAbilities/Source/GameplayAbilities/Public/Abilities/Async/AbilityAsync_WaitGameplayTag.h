// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "AbilityAsync.h"
#include "GameplayEffectTypes.h"
#include "Abilities/GameplayAbilityTypes.h"
#include "AbilityAsync_WaitGameplayTag.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

UCLASS(Abstract, MinimalAPI)
class UAbilityAsync_WaitGameplayTag : public UAbilityAsync
{
	GENERATED_BODY()
protected:

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FAsyncWaitGameplayTagDelegate);

	UE_API virtual void Activate() override;
	UE_API virtual void EndAction() override;

	UE_API virtual void GameplayTagCallback(const FGameplayTag Tag, int32 NewCount);
	UE_API virtual void BroadcastDelegate();

	int32 TargetCount = -1;
	FGameplayTag Tag;
	bool OnlyTriggerOnce = false;

	FDelegateHandle MyHandle;
};

UCLASS(MinimalAPI)
class UAbilityAsync_WaitGameplayTagAdded : public UAbilityAsync_WaitGameplayTag
{
	GENERATED_BODY()
public:

	UPROPERTY(BlueprintAssignable)
	FAsyncWaitGameplayTagDelegate Added;

	/**
	 * Wait until the specified gameplay tag is Added to Target Actor's ability component
	 * If the tag is already present when this task is started, it will immediately broadcast the Added event. It will keep listening as long as OnlyTriggerOnce = false.
	 * If used in an ability graph, this async action will wait even after activation ends. It's recommended to use WaitGameplayTagAdd instead.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (DefaultToSelf = "TargetActor", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityAsync_WaitGameplayTagAdded* WaitGameplayTagAddToActor(AActor* TargetActor, FGameplayTag Tag, bool OnlyTriggerOnce=false);

	UE_API virtual void BroadcastDelegate() override;
};

UCLASS(MinimalAPI)
class UAbilityAsync_WaitGameplayTagRemoved : public UAbilityAsync_WaitGameplayTag
{
	GENERATED_BODY()
public:

	UPROPERTY(BlueprintAssignable)
	FAsyncWaitGameplayTagDelegate Removed;

	/**
	 * Wait until the specified gameplay tag is Removed from Target Actor's ability component
	 * If the tag is not present when this task is started, it will immediately broadcast the Removed event. It will keep listening as long as OnlyTriggerOnce = false.
	 * If used in an ability graph, this async action will wait even after activation ends. It's recommended to use WaitGameplayTagRemove instead.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (DefaultToSelf = "TargetActor", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityAsync_WaitGameplayTagRemoved* WaitGameplayTagRemoveFromActor(AActor* TargetActor, FGameplayTag Tag, bool OnlyTriggerOnce=false);

	UE_API virtual void BroadcastDelegate() override;
};

#undef UE_API
