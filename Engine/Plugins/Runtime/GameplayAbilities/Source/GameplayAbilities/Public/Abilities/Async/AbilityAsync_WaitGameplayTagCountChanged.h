// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AbilityAsync.h"
#include "AbilityAsync_WaitGameplayTagCountChanged.generated.h"

#define UE_API GAMEPLAYABILITIES_API

UCLASS(MinimalAPI)
class UAbilityAsync_WaitGameplayTagCountChanged : public UAbilityAsync
{
	GENERATED_BODY()
protected:

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FAsyncWaitGameplayTagCountDelegate, int32, TagCount);

	UE_API virtual void Activate() override;
	UE_API virtual void EndAction() override;

	UE_API virtual void GameplayTagCallback(const FGameplayTag Tag, int32 NewCount);

	FGameplayTag Tag;
	FDelegateHandle GameplayTagCountChangedHandle;

public:
	UPROPERTY(BlueprintAssignable)
	FAsyncWaitGameplayTagCountDelegate TagCountChanged;

	/**
	 * Wait until the specified gameplay tag count changes on Target Actor's ability component
	 * If used in an ability graph, this async action will wait even after activation ends. It's recommended to use WaitGameplayTagCountChange instead.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (DefaultToSelf = "TargetActor", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityAsync_WaitGameplayTagCountChanged* WaitGameplayTagCountChangedOnActor(AActor* TargetActor, FGameplayTag Tag);
};

#undef UE_API
