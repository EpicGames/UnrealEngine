// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Abilities/Tasks/AbilityTask_WaitGameplayTagQuery.h"
#include "AbilityAsync.h"

#include "AbilityAsync_WaitGameplayTagQuery.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FAsyncWaitGameplayTagQueryDelegate);

/** This class defines an async node to wait on a gameplay tag query. */
UCLASS(MinimalAPI)
class UAbilityAsync_WaitGameplayTagQuery : public UAbilityAsync
{
	GENERATED_BODY()

public:
	/** This delegate will be triggered when the trigger condition has been met. */
	UPROPERTY(BlueprintAssignable)
	FAsyncWaitGameplayTagQueryDelegate Triggered;

	/**
	 * 	Wait until the given gameplay tag query has become true or false, based on TriggerCondition, looking at the target actors ASC.
	 *  If the the tag query already satisfies the TriggerCondition when this task is started, it will immediately broadcast the Triggered
	 *  event. It will keep listening as long as bOnlyTriggerOnce = false.
	 *  If used in an ability graph, this async action will wait even after activation ends. It's recommended to use WaitGameplayTagQuery instead.
	 */
 	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (DefaultToSelf = "TargetActor", BlueprintInternalUseOnly = "TRUE"))
 	static UE_API UAbilityAsync_WaitGameplayTagQuery* WaitGameplayTagQueryOnActor(AActor* TargetActor, 
																		   const FGameplayTagQuery TagQuery, 
																		   const EWaitGameplayTagQueryTriggerCondition TriggerCondition = EWaitGameplayTagQueryTriggerCondition::WhenTrue,
																		   const bool bOnlyTriggerOnce=false);

	/** Activates this AbilityAsync node. */
	UE_API virtual void Activate() override;

protected:
	/** Ends this AbilityAsync node, unregistering all tag changed delagates. */
	UE_API virtual void EndAction() override;

	/** Returns if this AbilityAsync node should currently broadcast it's delegates. */
	UE_API virtual bool ShouldBroadcastDelegates() const;

	/** This will update the tags in the TargetTags container to reflect that tags that are on the target ASC. */
	UE_API void UpdateTargetTags(const FGameplayTag Tag, int32 NewCount);

	/** This will evaluate the TargetTags using the given TagQuery, executing the Trigger delegate if needed. */
	UE_API void EvaluateTagQuery();

	/** This is the tag query to evaluate for triggering this node. */
	FGameplayTagQuery TagQuery;

	/** This indicates when to Trigger the Triggered output for this node. */
	EWaitGameplayTagQueryTriggerCondition TriggerCondition = EWaitGameplayTagQueryTriggerCondition::WhenTrue;

	/** This indicates if this node should only trigger once, or any number of times. */
	bool bOnlyTriggerOnce = false;

	/** Indicates if the callbacks for tag changes have been registered. */
	bool bRegisteredCallbacks = false;

	/** This was the last result of evaluating the TagQuery. */
	bool bQueryState = false;

	/** This is the tag container of the targets the target ASC currently has.
		NOTE: This will only contain the tags referenced in the TagQuery. */
	FGameplayTagContainer TargetTags;

	/** This is the handles to the tag changed delegate for each gameplay tag in the TagQuery. */
	TMap<FGameplayTag, FDelegateHandle> TagHandleMap;
};

#undef UE_API
