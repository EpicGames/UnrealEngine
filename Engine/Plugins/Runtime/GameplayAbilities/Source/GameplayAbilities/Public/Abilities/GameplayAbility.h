// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "GameplayTagContainer.h"
#include "GameplayEffectTypes.h"
#include "GameplayAbilitySpec.h"
#include "GameplayEffect.h"
#include "Abilities/GameplayAbilityTypes.h"
#include "GameplayTaskOwnerInterface.h"
#include "Abilities/GameplayAbilityTargetTypes.h"
#include "Net/Core/PushModel/PushModelMacros.h"
#include "GameplayAbility.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;
class UAnimMontage;
class UGameplayAbility;
class UGameplayTask;
class UGameplayTasksComponent;

struct FScopedCanActivateAbilityLogEnabler
{
	FScopedCanActivateAbilityLogEnabler() { ++LogEnablerCounter; }

	~FScopedCanActivateAbilityLogEnabler() { --LogEnablerCounter; }

	static bool IsLoggingEnabled() { return LogEnablerCounter > 0; }

private:

	static int32 LogEnablerCounter;
};

/**
 * UGameplayAbility
 *	
 *	Abilities define custom gameplay logic that can be activated or triggered.
 *	
 *	The main features provided by the AbilitySystem for GameplayAbilities are: 
 *		-CanUse functionality:
 *			-Cooldowns
 *			-Costs (mana, stamina, etc)
 *			-etc
 *			
 *		-Replication support
 *			-Client/Server communication for ability activation
 *			-Client prediction for ability activation
 *			
 *		-Instancing support
 *			-Abilities can be non-instanced (native only)
 *			-Instanced per owner
 *			-Instanced per execution (default)
 *			
 *		-Basic, extendable support for:
 *			-Input binding
 *			-'Giving' abilities (that can be used) to actors
 *	
 *
 *	See GameplayAbility_Montage for an example of a non-instanced ability
 *		-Plays a montage and applies a GameplayEffect to its target while the montage is playing.
 *		-When finished, removes GameplayEffect.
 *	
 *	Note on replication support:
 *		-Non instanced abilities have limited replication support. 
 *			-Cannot have state (obviously) so no replicated properties
 *			-RPCs on the ability class are not possible either.
 *			
 *	To support state or event replication, an ability must be instanced. This can be done with the InstancingPolicy property.
 */

/** Notification delegate definition for when the gameplay ability ends */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGameplayAbilityEnded, UGameplayAbility*);

/** Notification delegate definition for when the gameplay ability is cancelled */
DECLARE_MULTICAST_DELEGATE(FOnGameplayAbilityCancelled);

/** Used to notify ability state tasks that a state is being ended */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGameplayAbilityStateEnded, FName);

/** Used to delay execution until we leave a critical section */
DECLARE_DELEGATE(FPostLockDelegate);

/** Abilities define custom gameplay logic that can be activated by players or external game logic */
UCLASS(Blueprintable, MinimalAPI)
class UGameplayAbility : public UObject, public IGameplayTaskOwnerInterface
{
	GENERATED_UCLASS_BODY()
	REPLICATED_BASE_CLASS(UGameplayAbility)

	friend class UAbilitySystemComponent;
	friend class UGameplayAbilitySet;
	friend struct FScopedTargetListLock;

public:

	// ----------------------------------------------------------------------------------------------------------------
	//
	//	The important functions:
	//	
	//		CanActivateAbility()	- const function to see if ability is activatable. Callable by UI etc
	//
	//		TryActivateAbility()	- Attempts to activate the ability. Calls CanActivateAbility(). Input events can call this directly.
	//								- Also handles instancing-per-execution logic and replication/prediction calls.
	//		
	//		CallActivateAbility()	- Protected, non virtual function. Does some boilerplate 'pre activate' stuff, then calls ActivateAbility()
	//
	//		ActivateAbility()		- What the abilities *does*. This is what child classes want to override.
	//	
	//		CommitAbility()			- Commits reources/cooldowns etc. ActivateAbility() must call this!
	//		
	//		CancelAbility()			- Interrupts the ability (from an outside source).
	//
	//		EndAbility()			- The ability has ended. This is intended to be called by the ability to end itself.
	//	
	// ----------------------------------------------------------------------------------------------------------------

	// --------------------------------------
	//	Accessors
	// --------------------------------------

	/** Returns how the ability is instanced when executed. This limits what an ability can do in its implementation. */
	UE_API EGameplayAbilityInstancingPolicy::Type GetInstancingPolicy() const;

	/** How an ability replicates state/events to everyone on the network */
	EGameplayAbilityReplicationPolicy::Type GetReplicationPolicy() const
	{
		return ReplicationPolicy;
	}

	/** Where does an ability execute on the network? Does a client "ask and predict", "ask and wait", "don't ask (just do it)" */
	EGameplayAbilityNetExecutionPolicy::Type GetNetExecutionPolicy() const
	{
		return NetExecutionPolicy;
	}

	/** Where should an ability execute on the network? Provides protection from clients attempting to execute restricted abilities. */
	EGameplayAbilityNetSecurityPolicy::Type GetNetSecurityPolicy() const
	{
		return NetSecurityPolicy;
	}

	/** Returns the actor info associated with this ability, has cached pointers to useful objects */
	UFUNCTION(BlueprintCallable, Category=Ability)
	UE_API FGameplayAbilityActorInfo GetActorInfo() const;

	/** Returns the actor that owns this ability, which may not have a physical location */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API AActor* GetOwningActorFromActorInfo() const;

	/** Returns the physical actor that is executing this ability. May be null */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API AActor* GetAvatarActorFromActorInfo() const;

	/** Convenience method for abilities to get skeletal mesh component - useful for aiming abilities */
	UFUNCTION(BlueprintCallable, DisplayName = "GetSkeletalMeshComponentFromActorInfo", Category = Ability)
	UE_API USkeletalMeshComponent* GetOwningComponentFromActorInfo() const;

	/** Returns the AbilitySystemComponent that is activating this ability */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API UAbilitySystemComponent* GetAbilitySystemComponentFromActorInfo() const;

	UE_DEPRECATED(5.5, "Use GetAbilitySystemComponentFromActorInfo_Ensured")
	UE_API UAbilitySystemComponent* GetAbilitySystemComponentFromActorInfo_Checked() const;
	UE_API UAbilitySystemComponent* GetAbilitySystemComponentFromActorInfo_Ensured() const;

	/** The ability is considered to have these tags. */
	UE_API const FGameplayTagContainer& GetAssetTags() const;

	/** Gets the current actor info bound to this ability - can only be called on instanced abilities. */
	UE_API const FGameplayAbilityActorInfo* GetCurrentActorInfo() const;

	/** Gets the current activation info bound to this ability - can only be called on instanced abilities. */
	UE_API FGameplayAbilityActivationInfo GetCurrentActivationInfo() const;

	/** Gets the current activation info bound to this ability - can only be called on instanced abilities. */
	FGameplayAbilityActivationInfo& GetCurrentActivationInfoRef()
	{
		checkf(IsInstantiated(), TEXT("%s: GetCurrentActivationInfoRef cannot be called on a non-instanced ability. Check the instancing policy."), *GetPathName());
		return CurrentActivationInfo;
	}

	/** Gets the current AbilitySpecHandle- can only be called on instanced abilities. */
	UE_API FGameplayAbilitySpecHandle GetCurrentAbilitySpecHandle() const;

	/** Retrieves the actual AbilitySpec for this ability. Can only be called on instanced abilities. */
	UE_API FGameplayAbilitySpec* GetCurrentAbilitySpec() const;

	/** Retrieves the EffectContext of the GameplayEffect that granted this ability. Can only be called on instanced abilities. */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API FGameplayEffectContextHandle GetGrantedByEffectContext() const;

	/** Generates a GameplayEffectContextHandle from our owner and an optional TargetData.*/
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API virtual FGameplayEffectContextHandle GetContextFromOwner(FGameplayAbilityTargetDataHandle OptionalTargetData) const;

	/** Returns an effect context, given a specified actor info */
	UE_API virtual FGameplayEffectContextHandle MakeEffectContext(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo *ActorInfo) const;

	/** Convenience method for abilities to get outgoing gameplay effect specs (for example, to pass on to projectiles to apply to whoever they hit) */
	UFUNCTION(BlueprintCallable, Category=Ability)
	UE_API FGameplayEffectSpecHandle MakeOutgoingGameplayEffectSpec(TSubclassOf<UGameplayEffect> GameplayEffectClass, float Level=1.f) const;

	/** Native version of above function */
	UE_API virtual FGameplayEffectSpecHandle MakeOutgoingGameplayEffectSpec(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, TSubclassOf<UGameplayEffect> GameplayEffectClass, float Level = 1.f) const;

	/** Add the Ability's tags to the given GameplayEffectSpec. This is likely to be overridden per project. */
	UE_API virtual void ApplyAbilityTagsToGameplayEffectSpec(FGameplayEffectSpec& Spec, FGameplayAbilitySpec* AbilitySpec) const;

	/** Returns true if the ability is currently active */
	UE_API bool IsActive() const;

	/** Is this ability triggered from TriggerData (or is it triggered explicitly through input/game code) */
	UE_API bool IsTriggered() const;

	/** Is this ability running on a a predicting client, this is false in single player */
	UE_API bool IsPredictingClient() const;

	/** True if this is on the server and is being executed for a non-local player, false in single player */
	UE_API bool IsForRemoteClient() const;

	/** True if the owning actor is locally controlled, true in single player */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = Ability, Meta = (ExpandBoolAsExecs = "ReturnValue"))
	UE_API bool IsLocallyControlled() const;

	/** True if this is the server or single player */
	UE_API bool HasAuthority(const FGameplayAbilityActivationInfo* ActivationInfo) const;

	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = Ability, DisplayName = "HasAuthority", Meta = (ExpandBoolAsExecs = "ReturnValue"))
	UE_API bool K2_HasAuthority() const;

	/** True if we are authority or we have a valid prediciton key that is expected to work */
	UE_API bool HasAuthorityOrPredictionKey(const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo* ActivationInfo) const;

	/** True if this has been instanced, always true for blueprints */
	UE_API bool IsInstantiated() const;

	/** Notification that the ability has ended.  Set using TryActivateAbility. */
	FOnGameplayAbilityEnded OnGameplayAbilityEnded;

	/** Notification that the ability has ended with data on how it was ended */
	FGameplayAbilityEndedDelegate OnGameplayAbilityEndedWithData;

	/** Notification that the ability is being cancelled.  Called before OnGameplayAbilityEnded. */
	FOnGameplayAbilityCancelled OnGameplayAbilityCancelled;

	/** Used by the ability state task to handle when a state is ended */
	FOnGameplayAbilityStateEnded OnGameplayAbilityStateEnded;

	/** Callback for when this ability has been confirmed by the server */
	FGenericAbilityDelegate	OnConfirmDelegate;

	// --------------------------------------
	//	CanActivateAbility
	// --------------------------------------

	/** Returns true if this ability can be activated right now. Has no side effects */
	UE_API virtual bool CanActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayTagContainer* SourceTags = nullptr, const FGameplayTagContainer* TargetTags = nullptr, OUT FGameplayTagContainer* OptionalRelevantTags = nullptr) const;

	/** Returns true if this ability can be triggered right now. Has no side effects */
	UE_API virtual bool ShouldAbilityRespondToEvent(const FGameplayAbilityActorInfo* ActorInfo, const FGameplayEventData* Payload) const;

	/** Returns true if an an ability should be activated */
	UE_API virtual bool ShouldActivateAbility(ENetRole Role) const;

	/** Returns the time in seconds remaining on the currently active cooldown. */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API float GetCooldownTimeRemaining() const;

	/** Returns the time in seconds remaining on the currently active cooldown. */
	UE_API virtual float GetCooldownTimeRemaining(const FGameplayAbilityActorInfo* ActorInfo) const;

	/** Returns the time in seconds remaining on the currently active cooldown and the original duration for this cooldown. */
	UE_API virtual void GetCooldownTimeRemainingAndDuration(FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, float& TimeRemaining, float& CooldownDuration) const;

	/** Returns all tags that can put this ability into cooldown */
	UE_API virtual const FGameplayTagContainer* GetCooldownTags() const;
	
	/** Returns true if none of the ability's tags are blocked and if it doesn't have a "Blocking" tag and has all "Required" tags. */
	UE_API virtual bool DoesAbilitySatisfyTagRequirements(const UAbilitySystemComponent& AbilitySystemComponent, const FGameplayTagContainer* SourceTags = nullptr, const FGameplayTagContainer* TargetTags = nullptr, OUT FGameplayTagContainer* OptionalRelevantTags = nullptr) const;

	/** Returns true if this ability is blocking other abilities */
	UE_API virtual bool IsBlockingOtherAbilities() const;

	/** Sets rather ability block flags are enabled or disabled. Only valid on instanced abilities */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API virtual void SetShouldBlockOtherAbilities(bool bShouldBlockAbilities);

	// --------------------------------------
	//	CancelAbility
	// --------------------------------------

	/** Destroys instanced-per-execution abilities. Instance-per-actor abilities should 'reset'. Any active ability state tasks receive the 'OnAbilityStateInterrupted' event. Non instance abilities - what can we do? */
	UE_API virtual void CancelAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, bool bReplicateCancelAbility);

	/** Call from Blueprint to cancel the ability naturally */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName = "CancelAbility", meta=(ScriptName = "CancelAbility"))
	UE_API void K2_CancelAbility();

	/** Returns true if this ability can be canceled */
	UE_API virtual bool CanBeCanceled() const;

	/** Sets whether the ability should ignore cancel requests. Only valid on instanced abilities */
	UFUNCTION(BlueprintCallable, Category=Ability)
	UE_API virtual void SetCanBeCanceled(bool bCanBeCanceled);

	// --------------------------------------
	//	CommitAbility
	// --------------------------------------

	/** Attempts to commit the ability (spend resources, etc). This our last chance to fail. Child classes that override ActivateAbility must call this themselves! */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName = "CommitAbility", meta=(ScriptName = "CommitAbility"))
	UE_API virtual bool K2_CommitAbility();

	/** Attempts to commit the ability's cooldown only. If BroadcastCommitEvent is true, it will broadcast the commit event that tasks like WaitAbilityCommit are listening for. */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName = "CommitAbilityCooldown", meta=(ScriptName = "CommitAbilityCooldown"))
	UE_API virtual bool K2_CommitAbilityCooldown(bool BroadcastCommitEvent=false, bool ForceCooldown=false);

	/** Attempts to commit the ability's cost only. If BroadcastCommitEvent is true, it will broadcast the commit event that tasks like WaitAbilityCommit are listening for. */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName = "CommitAbilityCost", meta=(ScriptName = "CommitAbilityCost"))
	UE_API virtual bool K2_CommitAbilityCost(bool BroadcastCommitEvent=false);

	/** Checks the ability's cooldown, but does not apply it.*/
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName = "CheckAbilityCooldown", meta=(ScriptName = "CheckAbilityCooldown"))
	UE_API virtual bool K2_CheckAbilityCooldown();

	/** Checks the ability's cost, but does not apply it. */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName = "CheckAbilityCost", meta=(ScriptName = "CheckAbilityCost"))
	UE_API virtual bool K2_CheckAbilityCost();

	UE_API virtual bool CommitAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, OUT FGameplayTagContainer* OptionalRelevantTags = nullptr);
	UE_API virtual bool CommitAbilityCooldown(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const bool ForceCooldown, OUT FGameplayTagContainer* OptionalRelevantTags = nullptr);
	UE_API virtual bool CommitAbilityCost(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, OUT FGameplayTagContainer* OptionalRelevantTags = nullptr);

	/** The last chance to fail before committing, this will usually be the same as CanActivateAbility. Some abilities may need to do extra checks here if they are consuming extra stuff in CommitExecute */
	UE_API virtual bool CommitCheck(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, OUT FGameplayTagContainer* OptionalRelevantTags = nullptr);

	/** BP event called from CommitAbility */
	UFUNCTION(BlueprintImplementableEvent, Category = Ability, DisplayName = "CommitExecute", meta = (ScriptName = "CommitExecute"))
	UE_API void K2_CommitExecute();

	/** Does the commit atomically (consume resources, do cooldowns, etc) */
	UE_API virtual void CommitExecute(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo);

	/** Returns the gameplay effect used to determine cooldown */
	UE_API virtual UGameplayEffect* GetCooldownGameplayEffect() const;

	/** Returns the gameplay effect used to apply cost */
	UE_API virtual UGameplayEffect* GetCostGameplayEffect() const;

	/** Checks cooldown. returns true if we can be used again. False if not */
	UE_API virtual bool CheckCooldown(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, OUT FGameplayTagContainer* OptionalRelevantTags = nullptr) const;

	/** Applies CooldownGameplayEffect to the target */
	UE_API virtual void ApplyCooldown(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo) const;

	/** Checks cost. returns true if we can pay for the ability. False if not */
	UE_API virtual bool CheckCost(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, OUT FGameplayTagContainer* OptionalRelevantTags = nullptr) const;

	/** Applies the ability's cost to the target */
	UE_API virtual void ApplyCost(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo) const;

	// --------------------------------------
	//	Input
	// --------------------------------------

	/** Input binding stub. */
	virtual void InputPressed(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo) {};

	/** Input binding stub. */
	virtual void InputReleased(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo) {};

	/** Called from AbilityTask_WaitConfirmCancel to handle input confirming */
	virtual void OnWaitingForConfirmInputBegin() {}
	virtual void OnWaitingForConfirmInputEnd() {}

	// --------------------------------------
	//	Animation
	// --------------------------------------

	/** Returns the currently playing montage for this ability, if any */
	UFUNCTION(BlueprintCallable, Category = Animation)
	UE_API UAnimMontage* GetCurrentMontage() const;

	/** Call to set/get the current montage from a montage task. Set to allow hooking up montage events to ability events */
	UE_API virtual void SetCurrentMontage(class UAnimMontage* InCurrentMontage);

	// ----------------------------------------------------------------------------------------------------------------
	//	Ability Levels and source objects 
	// ----------------------------------------------------------------------------------------------------------------
	
	/** Returns current level of the Ability */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API int32 GetAbilityLevel() const;

	/** Returns current ability level for non instanced abilities. You must call this version in these contexts! */
	UE_API int32 GetAbilityLevel(FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo) const;

	/** Returns current ability level for non instanced abilities. You must call this version in these contexts! */
	UFUNCTION(BlueprintCallable, Category = Ability, meta = (DisplayName = "GetAbilityLevelNonInstanced", ReturnDisplayName = "AbilityLevel"))
	UE_API int32 GetAbilityLevel_BP(FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo& ActorInfo) const;

	/** Retrieves the SourceObject associated with this ability. Can only be called on instanced abilities. */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API UObject* GetCurrentSourceObject() const;

	/** Retrieves the SourceObject associated with this ability. Callable on non instanced */
	UE_API UObject* GetSourceObject(FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo) const;

	/** Retrieves the SourceObject associated with this ability. Callable on non instanced */
	UFUNCTION(BlueprintCallable, Category = Ability, meta = (DisplayName = "GetSourceObjectNonInstanced", ReturnDisplayName = "SourceObject"))
	UE_API UObject* GetSourceObject_BP(FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo& ActorInfo) const;

	// --------------------------------------
	//	Interaction with ability system component
	// --------------------------------------

	/** Called by ability system component to inform this ability instance the remote instance was ended */
	UE_API virtual void SetRemoteInstanceHasEnded();

	/** Called to inform the ability that the AvatarActor has been replaced. If the ability is dependent on avatar state, it may want to end itself. */
	UE_API virtual void NotifyAvatarDestroyed();

	/** Called to inform the ability that a task is waiting for the player to do something */
	UE_API virtual void NotifyAbilityTaskWaitingOnPlayerData(class UAbilityTask* AbilityTask);

	/** Called to inform the ability that a task is waiting for the player's avatar to do something in world */
	UE_API virtual void NotifyAbilityTaskWaitingOnAvatar(class UAbilityTask* AbilityTask);

	/** Called when the ability is given to an AbilitySystemComponent */
	UE_API virtual void OnGiveAbility(const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilitySpec& Spec);

	/** Called when the ability is removed from an AbilitySystemComponent */
	virtual void OnRemoveAbility(const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilitySpec& Spec) {}

	/** Called when the avatar actor is set/changes */
	UE_API virtual void OnAvatarSet(const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilitySpec& Spec);

	/** Takes in the ability spec and checks if we should allow replication on the ability spec, this will NOT stop replication of the ability UObject just the spec inside the UAbilitySystemComponenet ActivatableAbilities for this ability */
	virtual bool ShouldReplicateAbilitySpec(const FGameplayAbilitySpec& AbilitySpec) const
	{
		return true;
	}

	/** 
	 * Invalidates the current prediction key. This should be used in cases where there is a valid prediction window, but the server is doing logic that only it can do, and afterwards performs an action that the client could predict (had the client been able to run the server-only code prior).
	 * This returns instantly and has no other side effects other than clearing the current prediction key.
	 */ 
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API void InvalidateClientPredictionKey() const;

	/** Removes the GameplayEffect that granted this ability. Can only be called on instanced abilities. */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API virtual void RemoveGrantedByEffect();

	/** Adds a debug message to display to the user */
	UE_API void AddAbilityTaskDebugMessage(UGameplayTask* AbilityTask, FString DebugMessage);

#if WITH_EDITOR
	/** Allow modification of the AssetTags (AbilityTags) while in editor */
	UE_API FGameplayTagContainer& EditorGetAssetTags();
#endif // WITH_EDITOR

	// --------------------------------------
	//	Public variables, exposed for backwards compatibility
	// --------------------------------------

	/** This ability has these tags */
	UE_DEPRECATED_FORGAME(5.5, "Use GetAssetTags(). This is being made non-mutable, private and renamed to AssetTags in the future. Use SetAssetTags to set defaults (in constructor only).")
	UPROPERTY(EditDefaultsOnly, Category = Tags, DisplayName="AssetTags (Default AbilityTags)", meta=(Categories="AbilityTagCategory"))
	FGameplayTagContainer AbilityTags;

	/** If true, this ability will always replicate input press/release events to the server. */
	UPROPERTY(EditDefaultsOnly, Category = Input)
	bool bReplicateInputDirectly;

	/** Set when the remote instance of this ability has ended (but the local instance may still be running or finishing up */
	UPROPERTY()
	bool RemoteInstanceEnded;

	// --------------------------------------
	//	UObject overrides
	// --------------------------------------	
	UE_API virtual UWorld* GetWorld() const override;
	UE_API virtual int32 GetFunctionCallspace(UFunction* Function, FFrame* Stack) override;
	UE_API virtual bool CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack) override;
	UE_API virtual bool IsSupportedForNetworking() const override;
	UE_API virtual void PreDestroyFromReplication() override;

#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(FDataValidationContext& Context) const override;
#endif

	/** Overridden to allow Blueprint replicated properties to work */
	UE_API virtual void GetLifetimeReplicatedProps(TArray< class FLifetimeProperty >& OutLifetimeProps) const;

	/** Register all replication fragments */
	UE_API virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	// --------------------------------------
	//	IGameplayTaskOwnerInterface
	// --------------------------------------	
	UE_API virtual UGameplayTasksComponent* GetGameplayTasksComponent(const UGameplayTask& Task) const override;
	UE_API virtual AActor* GetGameplayTaskOwner(const UGameplayTask* Task) const override;
	UE_API virtual AActor* GetGameplayTaskAvatar(const UGameplayTask* Task) const override;
	UE_API virtual void OnGameplayTaskInitialized(UGameplayTask& Task) override;
	UE_API virtual void OnGameplayTaskActivated(UGameplayTask& Task) override;
	UE_API virtual void OnGameplayTaskDeactivated(UGameplayTask& Task) override;

protected:

	/**
 	 * Allows a derived class to set the default GameplayTags that this Ability is considered to have (formerly AbilityTags).
	 * This can only be called during construction.
	 * At runtime, the AbilitySpec is queried through a Gameplay Ability's CDO for its AbilityTags which can be a combination of these
	 * Asset Tags and specifically granted DynamicAbilityTags (all instances generated from an AbilitySpec are expected to share the same AbilityTags).
	 */
	UE_API void SetAssetTags(const FGameplayTagContainer& InAbilityTags);

	// --------------------------------------
	//	ShouldAbilityRespondToEvent
	// --------------------------------------

	/** Returns true if this ability can be activated right now. Has no side effects */
	UFUNCTION(BlueprintImplementableEvent, Category = Ability, DisplayName = "ShouldAbilityRespondToEvent", meta=(ScriptName = "ShouldAbilityRespondToEvent"))
	UE_API bool K2_ShouldAbilityRespondToEvent(FGameplayAbilityActorInfo ActorInfo, FGameplayEventData Payload) const;

	bool bHasBlueprintShouldAbilityRespondToEvent;

	/** Sends a gameplay event, also creates a prediction window */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API virtual void SendGameplayEvent(UPARAM(meta=(GameplayTagFilter="GameplayEventTagsCategory")) FGameplayTag EventTag, FGameplayEventData Payload);

	// --------------------------------------
	//	CanActivate
	// --------------------------------------
	
	/** Returns true if this ability can be activated right now. Has no side effects */
	UFUNCTION(BlueprintImplementableEvent, Category = Ability, DisplayName="CanActivateAbility", meta=(ScriptName="CanActivateAbility"))
	UE_API bool K2_CanActivateAbility(FGameplayAbilityActorInfo ActorInfo, const FGameplayAbilitySpecHandle Handle, FGameplayTagContainer& RelevantTags) const;

	bool bHasBlueprintCanUse;

	// --------------------------------------
	//	ActivateAbility
	// --------------------------------------

	/**
	 * The main function that defines what an ability does.
	 *  -Child classes will want to override this
	 *  -This function graph should call CommitAbility
	 *  -This function graph should call EndAbility
	 *  
	 *  Latent/async actions are ok in this graph. Note that Commit and EndAbility calling requirements speak to the K2_ActivateAbility graph. 
	 *  In C++, the call to K2_ActivateAbility() may return without CommitAbility or EndAbility having been called. But it is expected that this
	 *  will only occur when latent/async actions are pending. When K2_ActivateAbility logically finishes, then we will expect Commit/End to have been called.
	 *  
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = Ability, DisplayName = "ActivateAbility", meta=(ScriptName = "ActivateAbility"))
	UE_API void K2_ActivateAbility();

	UFUNCTION(BlueprintImplementableEvent, Category = Ability, DisplayName = "ActivateAbilityFromEvent", meta=(ScriptName = "ActivateAbilityFromEvent"))
	UE_API void K2_ActivateAbilityFromEvent(const FGameplayEventData& EventData);

	bool bHasBlueprintActivate;
	bool bHasBlueprintActivateFromEvent;

	/** Actually activate ability, do not call this directly */
	UE_API virtual void ActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayEventData* TriggerEventData);

	/** Do boilerplate init stuff and then call ActivateAbility */
	UE_API virtual void PreActivate(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, FOnGameplayAbilityEnded::FDelegate* OnGameplayAbilityEndedDelegate, const FGameplayEventData* TriggerEventData = nullptr);

	/** Executes PreActivate and ActivateAbility */
	UE_API void CallActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, FOnGameplayAbilityEnded::FDelegate* OnGameplayAbilityEndedDelegate = nullptr, const FGameplayEventData* TriggerEventData = nullptr);

	/** Called on a predictive ability when the server confirms its execution */
	UE_API virtual void ConfirmActivateSucceed();

	// -------------------------------------
	//	EndAbility
	// -------------------------------------
	/** Call from blueprints to forcibly end the ability without canceling it. This will replicate the end ability to the client or server which can interrupt tasks */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName="End Ability", meta=(ScriptName = "EndAbility"))
	UE_API virtual void K2_EndAbility();

	/** Call from blueprints to end the ability naturally. This will only end predicted abilities locally, allowing it end naturally on the client or server */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName = "End Ability Locally", meta = (ScriptName = "EndAbilityLocally"))
	UE_API virtual void K2_EndAbilityLocally();

	/** Blueprint event, will be called if an ability ends normally or abnormally */
	UFUNCTION(BlueprintImplementableEvent, Category = Ability, DisplayName = "OnEndAbility", meta=(ScriptName = "OnEndAbility"))
	UE_API void K2_OnEndAbility(bool bWasCancelled);

	/** Check if the ability can be ended */
	UE_API bool IsEndAbilityValid(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo) const;

	/** Native function, called if an ability ends normally or abnormally. If bReplicate is set to true, try to replicate the ending to the client/server */
	UE_API virtual void EndAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, bool bReplicateEndAbility, bool bWasCancelled);

	// -------------------------------------
	//  Apply Gameplay effects to Self
	// -------------------------------------

	/** Apply a gameplay effect to the owner of this ability */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName="ApplyGameplayEffectToOwner", meta=(ScriptName="ApplyGameplayEffectToOwner"))
	UE_API FActiveGameplayEffectHandle BP_ApplyGameplayEffectToOwner(TSubclassOf<UGameplayEffect> GameplayEffectClass, int32 GameplayEffectLevel = 1, int32 Stacks = 1);

	/** Non blueprintcallable, safe to call on CDO/NonInstance abilities */
	UE_API FActiveGameplayEffectHandle ApplyGameplayEffectToOwner(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const UGameplayEffect* GameplayEffect, float GameplayEffectLevel, int32 Stacks = 1) const;

	/** Apply a previously created gameplay effect spec to the owner of this ability */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName = "ApplyGameplayEffectSpecToOwner", meta=(ScriptName = "ApplyGameplayEffectSpecToOwner"))
	UE_API FActiveGameplayEffectHandle K2_ApplyGameplayEffectSpecToOwner(const FGameplayEffectSpecHandle EffectSpecHandle);

	UE_API FActiveGameplayEffectHandle ApplyGameplayEffectSpecToOwner(const FGameplayAbilitySpecHandle AbilityHandle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayEffectSpecHandle SpecHandle) const;

	/** Change Gameplay effect values before they get applied */
	UFUNCTION(BlueprintImplementableEvent, Category = Ability, DisplayName = "EditSpecValues", meta = (ScriptName = "EditSpecValues"))
	UE_API void BP_EditSpecValues(FGameplayEffectSpecHandle SpecHandle) const;
	
	// -------------------------------------
	//  Apply Gameplay effects to Target
	// -------------------------------------

	/** Apply a gameplay effect to a Target */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName = "ApplyGameplayEffectToTarget", meta=(ScriptName = "ApplyGameplayEffectToTarget"))
	UE_API TArray<FActiveGameplayEffectHandle> BP_ApplyGameplayEffectToTarget(FGameplayAbilityTargetDataHandle TargetData, TSubclassOf<UGameplayEffect> GameplayEffectClass, int32 GameplayEffectLevel = 1, int32 Stacks = 1);

	/** Non blueprintcallable, safe to call on CDO/NonInstance abilities */
	UE_API TArray<FActiveGameplayEffectHandle> ApplyGameplayEffectToTarget(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayAbilityTargetDataHandle& Target, TSubclassOf<UGameplayEffect> GameplayEffectClass, float GameplayEffectLevel, int32 Stacks = 1) const;

	/** Apply a previously created gameplay effect spec to a target */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName = "ApplyGameplayEffectSpecToTarget", meta=(ScriptName = "ApplyGameplayEffectSpecToTarget"))
	UE_API TArray<FActiveGameplayEffectHandle> K2_ApplyGameplayEffectSpecToTarget(const FGameplayEffectSpecHandle EffectSpecHandle, FGameplayAbilityTargetDataHandle TargetData);

	UE_API TArray<FActiveGameplayEffectHandle> ApplyGameplayEffectSpecToTarget(const FGameplayAbilitySpecHandle AbilityHandle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayEffectSpecHandle SpecHandle, const FGameplayAbilityTargetDataHandle& TargetData) const;

	// -------------------------------------
	//  Remove Gameplay effects from Self
	// -------------------------------------
	
	/** Removes GameplayEffects from owner which match the given asset level tags */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName="RemoveGameplayEffectFromOwnerWithAssetTags", meta=(ScriptName="RemoveGameplayEffectFromOwnerWithAssetTags"))
	UE_API void BP_RemoveGameplayEffectFromOwnerWithAssetTags(FGameplayTagContainer WithAssetTags, int32 StacksToRemove = -1);

	/** Removes GameplayEffects from owner which grant the given tags */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName="RemoveGameplayEffectFromOwnerWithGrantedTags", meta=(ScriptName="RemoveGameplayEffectFromOwnerWithGrantedTags"))
	UE_API void BP_RemoveGameplayEffectFromOwnerWithGrantedTags(FGameplayTagContainer WithGrantedTags, int32 StacksToRemove = -1);

	/** Removes GameplayEffect from owner that match the given handle */
	UFUNCTION(BlueprintCallable, Category = Ability, DisplayName = "RemoveGameplayEffectFromOwnerWithHandle", meta=(ScriptName = "RemoveGameplayEffectFromOwnerWithHandle"))
	UE_API void BP_RemoveGameplayEffectFromOwnerWithHandle(FActiveGameplayEffectHandle Handle, int32 StacksToRemove = -1);

	// -------------------------------------
	//	GameplayCue
	//	Abilities can invoke GameplayCues without having to create GameplayEffects
	// -------------------------------------

	/** Invoke a gameplay cue on the ability owner */
	UFUNCTION(BlueprintCallable, Category = Ability, meta=(GameplayTagFilter="GameplayCue"), DisplayName="Execute GameplayCue On Owner", meta=(ScriptName="ExecuteGameplayCue"))
	UE_API virtual void K2_ExecuteGameplayCue(FGameplayTag GameplayCueTag, FGameplayEffectContextHandle Context);

	/** Invoke a gameplay cue on the ability owner, with extra parameters */
	UFUNCTION(BlueprintCallable, Category = Ability, meta = (GameplayTagFilter = "GameplayCue"), DisplayName = "Execute GameplayCueWithParams On Owner", meta=(ScriptName = "ExecuteGameplayCueWithParams"))
	UE_API virtual void K2_ExecuteGameplayCueWithParams(FGameplayTag GameplayCueTag, const FGameplayCueParameters& GameplayCueParameters);

	/** Adds a persistent gameplay cue to the ability owner. Optionally will remove if ability ends */
	UFUNCTION(BlueprintCallable, Category = Ability, meta=(GameplayTagFilter="GameplayCue"), DisplayName="Add GameplayCue To Owner", meta=(ScriptName="AddGameplayCue"))
	UE_API virtual void K2_AddGameplayCue(FGameplayTag GameplayCueTag, FGameplayEffectContextHandle Context, bool bRemoveOnAbilityEnd = true);

	/** Adds a persistent gameplay cue to the ability owner. Optionally will remove if ability ends */
	UFUNCTION(BlueprintCallable, Category = Ability, meta = (GameplayTagFilter = "GameplayCue"), DisplayName = "Add GameplayCueWithParams To Owner", meta=(ScriptName = "AddGameplayCueWithParams"))
	UE_API virtual void K2_AddGameplayCueWithParams(FGameplayTag GameplayCueTag, const FGameplayCueParameters& GameplayCueParameter, bool bRemoveOnAbilityEnd = true);

	/** Removes a persistent gameplay cue from the ability owner */
	UFUNCTION(BlueprintCallable, Category = Ability, meta=(GameplayTagFilter="GameplayCue"), DisplayName="Remove GameplayCue From Owner", meta=(ScriptName="RemoveGameplayCue"))
	UE_API virtual void K2_RemoveGameplayCue(FGameplayTag GameplayCueTag);

	// -------------------------------------
	//	Protected properties
	// -------------------------------------

	/** How an ability replicates state/events to everyone on the network. Replication is not required for NetExecutionPolicy. */
	UPROPERTY(EditDefaultsOnly, Category = Advanced)
	TEnumAsByte<EGameplayAbilityReplicationPolicy::Type> ReplicationPolicy;

	/** How the ability is instanced when executed. This limits what an ability can do in its implementation. */
	UPROPERTY(EditDefaultsOnly, Category = Advanced)
	TEnumAsByte<EGameplayAbilityInstancingPolicy::Type>	InstancingPolicy;

	/** If this is set, the server-side version of the ability can be canceled by the client-side version. The client-side version can always be canceled by the server. */
	UPROPERTY(EditDefaultsOnly, Category = Advanced)
	bool bServerRespectsRemoteAbilityCancellation;

	/** if true, and trying to activate an already active instanced ability, end it and re-trigger it. */
	UPROPERTY(EditDefaultsOnly, Category = Advanced)
	bool bRetriggerInstancedAbility;

	/** This is information specific to this instance of the ability. E.g, whether it is predicting, authoring, confirmed, etc. */
	UPROPERTY(BlueprintReadOnly, Category = Ability)
	FGameplayAbilityActivationInfo	CurrentActivationInfo;

	/** Information specific to this instance of the ability, if it was activated by an event */
	UPROPERTY(BlueprintReadOnly, Category = Ability)
	FGameplayEventData CurrentEventData;

	/** How does an ability execute on the network. Does a client "ask and predict", "ask and wait", "don't ask (just do it)". */
	UPROPERTY(EditDefaultsOnly, Category=Advanced)
	TEnumAsByte<EGameplayAbilityNetExecutionPolicy::Type> NetExecutionPolicy;

	/** What protections does this ability have? Should the client be allowed to request changes to the execution of the ability? */
	UPROPERTY(EditDefaultsOnly, Category = Advanced)
	TEnumAsByte<EGameplayAbilityNetSecurityPolicy::Type> NetSecurityPolicy;

	/** This GameplayEffect represents the cost (mana, stamina, etc) of the ability. It will be applied when the ability is committed. */
	UPROPERTY(EditDefaultsOnly, Category = Costs)
	TSubclassOf<class UGameplayEffect> CostGameplayEffectClass;

	/** Triggers to determine if this ability should execute in response to an event */
	UPROPERTY(EditDefaultsOnly, Category = Triggers)
	TArray<FAbilityTriggerData> AbilityTriggers;
			
	/** This GameplayEffect represents the cooldown. It will be applied when the ability is committed and the ability cannot be used again until it is expired. */
	UPROPERTY(EditDefaultsOnly, Category = Cooldowns)
	TSubclassOf<class UGameplayEffect> CooldownGameplayEffectClass;

	// ----------------------------------------------------------------------------------------------------------------
	//	Ability exclusion / canceling
	// ----------------------------------------------------------------------------------------------------------------

	/** Abilities with these tags are cancelled when this ability is executed */
	UPROPERTY(EditDefaultsOnly, Category = Tags, meta=(Categories="AbilityTagCategory"))
	FGameplayTagContainer CancelAbilitiesWithTag;

	/** Abilities with these tags are blocked while this ability is active */
	UPROPERTY(EditDefaultsOnly, Category = Tags, meta=(Categories="AbilityTagCategory"))
	FGameplayTagContainer BlockAbilitiesWithTag;

	/** Tags to apply to activating owner while this ability is active. These are replicated if ReplicateActivationOwnedTags is enabled in AbilitySystemGlobals. */
	UPROPERTY(EditDefaultsOnly, Category = Tags, meta=(Categories="OwnedTagsCategory"))
	FGameplayTagContainer ActivationOwnedTags;

	/** This ability can only be activated if the activating actor/component has all of these tags */
	UPROPERTY(EditDefaultsOnly, Category = Tags, meta=(Categories="OwnedTagsCategory"))
	FGameplayTagContainer ActivationRequiredTags;

	/** This ability is blocked if the activating actor/component has any of these tags */
	UPROPERTY(EditDefaultsOnly, Category = Tags, meta=(Categories="OwnedTagsCategory"))
	FGameplayTagContainer ActivationBlockedTags;

	/** This ability can only be activated if the source actor/component has all of these tags */
	UPROPERTY(EditDefaultsOnly, Category = Tags, meta=(Categories="SourceTagsCategory"))
	FGameplayTagContainer SourceRequiredTags;

	/** This ability is blocked if the source actor/component has any of these tags */
	UPROPERTY(EditDefaultsOnly, Category = Tags, meta=(Categories="SourceTagsCategory"))
	FGameplayTagContainer SourceBlockedTags;

	/** This ability can only be activated if the target actor/component has all of these tags */
	UPROPERTY(EditDefaultsOnly, Category = Tags, meta=(Categories="TargetTagsCategory"))
	FGameplayTagContainer TargetRequiredTags;

	/** This ability is blocked if the target actor/component has any of these tags */
	UPROPERTY(EditDefaultsOnly, Category = Tags, meta=(Categories="TargetTagsCategory"))
	FGameplayTagContainer TargetBlockedTags;

	// ----------------------------------------------------------------------------------------------------------------
	//	Ability Tasks
	// ----------------------------------------------------------------------------------------------------------------

	/** Finds all currently active tasks named InstanceName and confirms them. What this means depends on the individual task. By default, this does nothing other than ending if bEndTask is true. */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API void ConfirmTaskByInstanceName(FName InstanceName, bool bEndTask);

	/** Internal function, cancels all the tasks we asked to cancel last frame (by instance name). */
	UE_API void EndOrCancelTasksByInstanceName();
	TArray<FName> CancelTaskInstanceNames;

	/** Add any task with this instance name to a list to be ended (not canceled) next frame.  See also CancelTaskByInstanceName. */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API void EndTaskByInstanceName(FName InstanceName);
	TArray<FName> EndTaskInstanceNames;

	/** Add any task with this instance name to a list to be canceled (not ended) next frame.  See also EndTaskByInstanceName. */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API void CancelTaskByInstanceName(FName InstanceName);

	/** Ends any active ability state task with the given name. If name is 'None' all active states will be ended (in an arbitrary order). */
	UFUNCTION(BlueprintCallable, Category = Ability)
	UE_API void EndAbilityState(FName OptionalStateNameToEnd);

	/** List of currently active tasks, do not modify directly */
	UPROPERTY()
	TArray<TObjectPtr<UGameplayTask>>	ActiveTasks;

	/** Tasks can emit debug messages throughout their life for debugging purposes. Saved on the ability so that they persist after task is finished */
	TArray<FAbilityTaskDebugMessage> TaskDebugMessages;

	// ----------------------------------------------------------------------------------------------------------------
	//	Animation
	// ----------------------------------------------------------------------------------------------------------------

	/** Immediately jumps the active montage to a section */
	UFUNCTION(BlueprintCallable, Category="Ability|Animation")
	UE_API void MontageJumpToSection(FName SectionName);

	/** Sets pending section on active montage */
	UFUNCTION(BlueprintCallable, Category = "Ability|Animation")
	UE_API void MontageSetNextSectionName(FName FromSectionName, FName ToSectionName);

	/**
	 * Stops the current animation montage.
	 *
	 * @param OverrideBlendTime If >= 0, will override the BlendOutTime parameter on the AnimMontage instance
	 */
	UFUNCTION(BlueprintCallable, Category="Ability|Animation", Meta = (AdvancedDisplay = "OverrideBlendOutTime"))
	UE_API void MontageStop(float OverrideBlendOutTime = -1.0f);

	/** Active montage being played by this ability */
	UPROPERTY()
	TObjectPtr<class UAnimMontage> CurrentMontage;

	// ----------------------------------------------------------------------------------------------------------------
	//	Target Data
	// ----------------------------------------------------------------------------------------------------------------

	/** Creates a target location from where the owner avatar is */
	UFUNCTION(BlueprintPure, Category = Ability)
	UE_API FGameplayAbilityTargetingLocationInfo MakeTargetLocationInfoFromOwnerActor();

	/** Creates a target location from a socket on the owner avatar's skeletal mesh */
	UFUNCTION(BlueprintPure, Category = Ability)
	UE_API FGameplayAbilityTargetingLocationInfo MakeTargetLocationInfoFromOwnerSkeletalMeshComponent(FName SocketName);

	// ----------------------------------------------------------------------------------------------------------------
	//	Setters for temporary execution data
	// ----------------------------------------------------------------------------------------------------------------

	/** Called to initialize after being created due to replication */
	UE_API virtual void PostNetInit();

	/** Modifies actor info, only safe on instanced abilities */
	UE_API virtual void SetCurrentActorInfo(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo) const;
	
	/** Modifies activation info, only safe on instanced abilities */
	UE_API virtual void SetCurrentActivationInfo(const FGameplayAbilityActivationInfo ActivationInfo);
	
	/** Sets both actor and activation info */
	UE_API void SetCurrentInfo(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo);
	
	/** 
	 *  This is shared, cached information about the thing using us
	 *	 E.g, Actor*, MovementComponent*, AnimInstance, etc.
	 *	 This is hopefully allocated once per actor and shared by many abilities.
	 *	 The actual struct may be overridden per game to include game specific data.
	 *	 (E.g, child classes may want to cast to FMyGameAbilityActorInfo)
	 */
	mutable const FGameplayAbilityActorInfo* CurrentActorInfo;

	/** For instanced abilities */
	mutable FGameplayAbilitySpecHandle CurrentSpecHandle;

	/** GameplayCues that were added during this ability that will get automatically removed when it ends */
	TSet<FGameplayTag> TrackedGameplayCues;

	/** True if the ability is currently active. For instance per owner abilities */
	UPROPERTY()
	bool bIsActive;
	
	/** True if the end ability has been called, but has not yet completed. */
	UPROPERTY()
	bool bIsAbilityEnding = false;

	/** True if the ability is currently cancelable, if not will only be canceled by hard EndAbility calls */
	UPROPERTY()
	bool bIsCancelable;

	/** True if the ability block flags are currently enabled */
	UPROPERTY()
	bool bIsBlockingOtherAbilities;

	/** A count of all the current scope locks. */
	mutable int8 ScopeLockCount;

	/** A list of all the functions waiting for the scope lock to end so they can run. */
	mutable TArray<FPostLockDelegate> WaitingToExecute;

	/** Increases the scope lock count. */
	UE_API void IncrementListLock() const;

	/** Decreases the scope lock count. Runs the waiting to execute delegates if the count drops to zero. */
	UE_API void DecrementListLock() const;

	/** We need to serialize non-property fields in certain circumstances */
	UE_API virtual void Serialize(FArchive& Ar) override;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	virtual void PostMigrate(const struct FUObjectMigrationContext& MigrationContext) override;
#endif
};

#undef UE_API
