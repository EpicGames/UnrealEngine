// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayPrediction.h"
#include "GameplayAbilityRepAnimMontage.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAnimMontage;
class UAnimSequenceBase;

/** Enum used by the Ability Rep Anim Montage struct to rep the quantized position or the current section id */
UENUM()
enum class ERepAnimPositionMethod
{
	Position = 0,			// reps the position in the montage to keep the client in sync (heavier, quantized, more precise)
	CurrentSectionId = 1,	// reps the current section id we want to play on the client (compact, less precise)
};

/** Data about montages that is replicated to simulated clients */
USTRUCT()
struct FGameplayAbilityRepAnimMontage
{
	GENERATED_USTRUCT_BODY()

	/** Animation ref. When playing a dynamic montage this points to the AnimSequence the montage was created from */
	UPROPERTY()
	TObjectPtr<UAnimSequenceBase> Animation;

	/** Optional slot name used by dynamic montages. */
	UPROPERTY()
	FName SlotName;

	/** Play Rate */
	UPROPERTY()
	float PlayRate;

	/** Montage position */
	UPROPERTY(NotReplicated)
	float Position;

	/** Montage current blend time */
	UPROPERTY()
	float BlendTime;

	/** Optional blend out used by dynamic montages. */
	UPROPERTY(NotReplicated)
	float BlendOutTime;

	/** Fractional loop count for non-montages */
	UPROPERTY()
	float PlayCount;

	/** NextSectionID */
	UPROPERTY()
	uint8 NextSectionID;

	/** ID incremented every time a montage is played, used to trigger replication when the same montage is played multiple times. This ID wraps around when it reaches its max value. */
	UPROPERTY()
	uint8 PlayInstanceId;

	/** flag indicating we should serialize the position or the current section id */
	UPROPERTY()
	uint8 bRepPosition : 1;

	/** Bit set when montage has been stopped. */
	UPROPERTY()
	uint8 IsStopped : 1;
	
	/** Stops montage position from replicating at all to save bandwidth */
	UPROPERTY()
	uint8 SkipPositionCorrection : 1;

	/** Stops PlayRate from replicating to save bandwidth. PlayRate will be assumed to be 1.f. */
	UPROPERTY()
	uint8 bSkipPlayRate : 1;

	UPROPERTY()
	FPredictionKey PredictionKey;

	/** The current section Id used by the montage. Will only be valid if bRepPosition is false */
	UPROPERTY()
	uint8 SectionIdToPlay;

	FGameplayAbilityRepAnimMontage()
	: Animation(nullptr),
	SlotName(NAME_None),
	PlayRate(0.f),
	Position(0.f),
	BlendTime(0.f),
	BlendOutTime(0.f),
	PlayCount(0.f),
	NextSectionID(0),
	PlayInstanceId(0),
	bRepPosition(true),
	IsStopped(true),
	SkipPositionCorrection(false),
	bSkipPlayRate(false),
	SectionIdToPlay(0)
	{
	}

	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	UE_API void SetRepAnimPositionMethod(ERepAnimPositionMethod InMethod);

	UE_API UAnimMontage* GetAnimMontage() const;
};

template<>
struct TStructOpsTypeTraits<FGameplayAbilityRepAnimMontage> : public TStructOpsTypeTraitsBase2<FGameplayAbilityRepAnimMontage>
{
	enum
	{
		WithNetSerializer = true,
	};
};

#undef UE_API
