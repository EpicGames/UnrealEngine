// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "Abilities/GameplayAbilityWorldReticle.h"
#include "Abilities/GameplayAbilityTargetTypes.h"
#include "Abilities/GameplayAbilityTargetDataFilter.h"
#include "GameplayAbilityTargetActor.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class APlayerController;
class UAbilitySystemComponent;
class UGameplayAbility;

/**
 * TargetActors are spawned to assist with ability targeting. They are spawned by ability tasks and create/determine the outgoing targeting data passed from one task to another
 *
 * WARNING: These actors are spawned once per ability activation and in their default form are not very efficient
 * For most games you will need to subclass and heavily modify this actor, or you will want to implement similar functions in a game-specific actor or blueprint to avoid actor spawn costs
 * This class is not well tested by internal games, but it is a useful class to look at to learn how target replication occurs
 */
UCLASS(Blueprintable, abstract, notplaceable, MinimalAPI)
class AGameplayAbilityTargetActor : public AActor
{
	GENERATED_UCLASS_BODY()

public:

	UE_API void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	/** The TargetData this class produces can be entirely generated on the server. We don't require the client to send us full or partial TargetData (possibly just a 'confirm') */
	UPROPERTY(EditAnywhere, Category=Advanced)
	bool ShouldProduceTargetDataOnServer;

	/** Describes where the targeting action starts, usually the player character or a socket on the player character. */
	//UPROPERTY(BlueprintReadOnly, meta=(ExposeOnSpawn=true), Category=Targeting)
	UPROPERTY(BlueprintReadOnly, meta = (ExposeOnSpawn = true), Replicated, Category = Targeting)
	FGameplayAbilityTargetingLocationInfo StartLocation;
	
	/** Initialize and begin targeting logic  */
	UE_API virtual void StartTargeting(UGameplayAbility* Ability);

	UE_API virtual bool IsConfirmTargetingAllowed();

	/** Requesting targeting data, but not necessarily stopping/destroying the task. Useful for external target data requests. */
	UE_API virtual void ConfirmTargetingAndContinue();

	/** Outside code is saying 'stop and just give me what you have.' Returns true if the ability accepts this and can be forgotten. */
	UFUNCTION()
	UE_API virtual void ConfirmTargeting();

	/** Outside code is saying 'stop everything and just forget about it' */
	UFUNCTION()
	UE_API virtual void CancelTargeting();

	UE_API virtual void BindToConfirmCancelInputs();

	UE_API virtual bool ShouldProduceTargetData() const;

	/** Replicated target data was received from a client. Possibly sanitize/verify. return true if data is good and we should broadcast it as valid data. */
	UE_API virtual bool OnReplicatedTargetDataReceived(FGameplayAbilityTargetDataHandle& Data) const;

	// ------------------------------
	
	FAbilityTargetData	TargetDataReadyDelegate;
	FAbilityTargetData	CanceledDelegate;

	UE_API virtual bool IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const override;
	
	UPROPERTY(BlueprintReadOnly, Category = "Targeting")
	TObjectPtr<APlayerController> PrimaryPC;

	UPROPERTY()
	TObjectPtr<UGameplayAbility> OwningAbility;

	UPROPERTY(BlueprintReadOnly, Replicated, Category = Targeting)
	bool bDestroyOnConfirmation;

	UPROPERTY(BlueprintReadOnly, Replicated, Category = Targeting)
	TObjectPtr<AActor> SourceActor;

	/** Parameters for world reticle. Usage of these parameters is dependent on the reticle. */
	UPROPERTY(BlueprintReadWrite, meta = (ExposeOnSpawn = true), Category = Targeting)
	FWorldReticleParameters ReticleParams;

	/** Reticle that will appear on top of acquired targets. Reticles will be spawned/despawned as targets are acquired/lost. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, meta = (ExposeOnSpawn = true), Category = Targeting)
	TSubclassOf<AGameplayAbilityWorldReticle> ReticleClass;		//Using a special class for replication purposes.

	UPROPERTY(BlueprintReadWrite, Replicated, meta = (ExposeOnSpawn = true), Category = Targeting)
	FGameplayTargetDataFilterHandle Filter;

	/** Draw the debug information (if applicable) for this targeting actor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Replicated, meta = (ExposeOnSpawn = true), Category = Targeting)
	bool bDebug;

	FDelegateHandle GenericConfirmHandle;
	FDelegateHandle GenericCancelHandle;

	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> GenericDelegateBoundASC;
};

#undef UE_API
