// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/GameplayAbilityTargetActor_GroundTrace.h"
#include "GameplayAbilityTargetActor_ActorPlacement.generated.h"

class AGameplayAbilityWorldReticle_ActorVisualization;
class UGameplayAbility;
class UMaterialInterface;

UCLASS(Blueprintable)
class AGameplayAbilityTargetActor_ActorPlacement : public AGameplayAbilityTargetActor_GroundTrace
{
	GENERATED_UCLASS_BODY()

	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	virtual void StartTargeting(UGameplayAbility* InAbility) override;

	/** Actor we intend to place. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, meta = (ExposeOnSpawn = true), Category = Targeting)
	TObjectPtr<UClass> PlacedActorClass;		//Using a special class for replication purposes. (Not implemented yet)
	
	/** Override Material 0 on our placed actor's meshes with this material for visualization. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, meta = (ExposeOnSpawn = true), Category = Projectile)
	TObjectPtr<UMaterialInterface> PlacedActorMaterial;

	/** Visualization for the intended location of the placed actor. */
	TWeakObjectPtr<AGameplayAbilityWorldReticle_ActorVisualization> ActorVisualizationReticle;
};
