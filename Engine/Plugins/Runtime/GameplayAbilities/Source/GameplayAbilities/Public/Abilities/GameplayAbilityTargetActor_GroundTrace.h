// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"
#include "Abilities/GameplayAbilityTargetActor_Trace.h"
#include "GameplayAbilityTargetActor_GroundTrace.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UGameplayAbility;

UCLASS(Blueprintable, MinimalAPI)
class AGameplayAbilityTargetActor_GroundTrace : public AGameplayAbilityTargetActor_Trace
{
	GENERATED_UCLASS_BODY()

	UE_API virtual void StartTargeting(UGameplayAbility* InAbility) override;

	/** Radius for a sphere or capsule. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, meta = (ExposeOnSpawn = true), Category = Targeting)
	float CollisionRadius;

	/** Height for a capsule. Implicitly indicates a capsule is desired if this is greater than zero. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, meta = (ExposeOnSpawn = true), Category = Targeting)
	float CollisionHeight;

	//TODO: Consider adding the ability to rotate the capsule with a supplied quaternion.

protected:
	UE_API virtual FHitResult PerformTrace(AActor* InSourceActor) override;

	UE_API virtual bool IsConfirmTargetingAllowed() override;

	UE_API bool AdjustCollisionResultForShape(const FVector OriginalStartPoint, const FVector OriginalEndPoint, const FCollisionQueryParams Params, FHitResult& OutHitResult) const;

	FCollisionShape CollisionShape;
	float CollisionHeightOffset;		//When tracing, give this much extra height to avoid start-in-ground problems. Dealing with thick placement actors while standing near walls may be trickier.
	bool bLastTraceWasGood;
};

#undef UE_API
