// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/GameplayAbilityTargetTypes.h"
#include "Abilities/GameplayAbilityTargetActor.h"
#include "GameplayAbilityTargetActor_Radius.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UGameplayAbility;

/** Selects everything within a given radius of the source actor. */
UCLASS(Blueprintable, notplaceable, MinimalAPI)
class AGameplayAbilityTargetActor_Radius : public AGameplayAbilityTargetActor
{
	GENERATED_UCLASS_BODY()

public:

	UE_API virtual void StartTargeting(UGameplayAbility* Ability) override;
	
	UE_API virtual void ConfirmTargetingAndContinue() override;

	/** Radius of target acquisition around the ability's start location. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, meta = (ExposeOnSpawn = true), Category = Radius)
	float Radius;

protected:

	UE_API TArray<TWeakObjectPtr<AActor> >	PerformOverlap(const FVector& Origin);

	UE_API FGameplayAbilityTargetDataHandle MakeTargetData(const TArray<TWeakObjectPtr<AActor>>& Actors, const FVector& Origin) const;

};

#undef UE_API
