// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Abilities/GameplayAbilityTargetActor_Trace.h"
#include "GameplayAbilityTargetActor_SingleLineTrace.generated.h"

#define UE_API GAMEPLAYABILITIES_API

UCLASS(Blueprintable, MinimalAPI)
class AGameplayAbilityTargetActor_SingleLineTrace : public AGameplayAbilityTargetActor_Trace
{
	GENERATED_UCLASS_BODY()

protected:
	UE_API virtual FHitResult PerformTrace(AActor* InSourceActor) override;
};

#undef UE_API
