// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"
#include "Engine/CollisionProfile.h"
#include "Abilities/GameplayAbilityTargetTypes.h"
#include "Abilities/GameplayAbilityTargetDataFilter.h"
#include "Abilities/GameplayAbilityTargetActor.h"
#include "GameplayAbilityTargetActor_Trace.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UGameplayAbility;

/** Intermediate base class for all line-trace type targeting actors. */
UCLASS(Abstract, Blueprintable, notplaceable, config=Game, MinimalAPI)
class AGameplayAbilityTargetActor_Trace : public AGameplayAbilityTargetActor
{
	GENERATED_UCLASS_BODY()

public:
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	/** Traces as normal, but will manually filter all hit actors */
	static UE_API void LineTraceWithFilter(FHitResult& OutHitResult, const UWorld* World, const FGameplayTargetDataFilterHandle FilterHandle, const FVector& Start, const FVector& End, FName ProfileName, const FCollisionQueryParams Params);

	/** Sweeps as normal, but will manually filter all hit actors */
	static UE_API void SweepWithFilter(FHitResult& OutHitResult, const UWorld* World, const FGameplayTargetDataFilterHandle FilterHandle, const FVector& Start, const FVector& End, const FQuat& Rotation, const FCollisionShape CollisionShape, FName ProfileName, const FCollisionQueryParams Params);

	UE_API void AimWithPlayerController(const AActor* InSourceActor, FCollisionQueryParams Params, const FVector& TraceStart, FVector& OutTraceEnd, bool bIgnorePitch = false) const;

	static UE_API bool ClipCameraRayToAbilityRange(FVector CameraLocation, FVector CameraDirection, FVector AbilityCenter, float AbilityRange, FVector& ClippedPosition);

	UE_API virtual void StartTargeting(UGameplayAbility* Ability) override;

	UE_API virtual void ConfirmTargetingAndContinue() override;

	UE_API virtual void Tick(float DeltaSeconds) override;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, meta = (ExposeOnSpawn = true), Category = Trace)
	float MaxRange;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, config, meta = (ExposeOnSpawn = true), Category = Trace)
	FCollisionProfileName TraceProfile;

	// Does the trace affect the aiming pitch
	UPROPERTY(BlueprintReadWrite, EditAnywhere, meta = (ExposeOnSpawn = true), Category = Trace)
	bool bTraceAffectsAimPitch;

protected:
	virtual FHitResult PerformTrace(AActor* InSourceActor) PURE_VIRTUAL(AGameplayAbilityTargetActor_Trace, return FHitResult(););

	UE_API FGameplayAbilityTargetDataHandle MakeTargetData(const FHitResult& HitResult) const;

	TWeakObjectPtr<AGameplayAbilityWorldReticle> ReticleActor;
};

#undef UE_API
