// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "GameplayAbilityTargetDataFilter.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** Set rather it is possible to target self */
UENUM(BlueprintType)
namespace ETargetDataFilterSelf
{
	enum Type : int
	{
		TDFS_Any 			UMETA(DisplayName = "Allow self or others"),
		TDFS_NoSelf 		UMETA(DisplayName = "Filter self out"),
		TDFS_NoOthers		UMETA(DisplayName = "Filter others out")
	};
}

/** Simple actor target filter, games can subclass this */
USTRUCT(BlueprintType)
struct FGameplayTargetDataFilter
{
	GENERATED_USTRUCT_BODY()

	virtual ~FGameplayTargetDataFilter() = default;

	/** Returns true if the actor passes the filter and will be targeted */
	virtual bool FilterPassesForActor(const AActor* ActorToBeFiltered) const
	{
		switch (SelfFilter.GetValue())
		{
		case ETargetDataFilterSelf::Type::TDFS_NoOthers:
			if (ActorToBeFiltered != SelfActor)
			{
				return (bReverseFilter ^ false);
			}
			break;
		case ETargetDataFilterSelf::Type::TDFS_NoSelf:
			if (ActorToBeFiltered == SelfActor)
			{
				return (bReverseFilter ^ false);
			}
			break;
		case ETargetDataFilterSelf::Type::TDFS_Any:
		default:
			break;
		}

		if (RequiredActorClass && !ActorToBeFiltered->IsA(RequiredActorClass))
		{
			return (bReverseFilter ^ false);
		}

		return (bReverseFilter ^ true);
	}

	/** Initializes SelfActor */
	UE_API void InitializeFilterContext(AActor* FilterActor);

	/** Actor we're comparing against. */
	UPROPERTY()
	TObjectPtr<AActor> SelfActor = nullptr;

	/** Subclass actors must be to pass the filter. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, meta = (ExposeOnSpawn = true), Category = Filter)
	TSubclassOf<AActor> RequiredActorClass;

	/** Filter based on whether or not this actor is "self." */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, meta = (ExposeOnSpawn = true), Category = Filter)
	TEnumAsByte<ETargetDataFilterSelf::Type> SelfFilter = ETargetDataFilterSelf::TDFS_Any;

	/** Reverses the meaning of the filter, so it will exclude all actors that pass. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, meta = (ExposeOnSpawn = true), Category = Filter)
	bool bReverseFilter = false;
};

/** Polymorphic handle to filter structure that handles checking if actors should be targeted */
USTRUCT(BlueprintType)
struct FGameplayTargetDataFilterHandle
{
	GENERATED_USTRUCT_BODY()

	TSharedPtr<FGameplayTargetDataFilter> Filter;

	/** Returns true if the actor passes the filter and will be targeted */
	bool FilterPassesForActor(const AActor* ActorToBeFiltered) const
	{
		if (!ActorToBeFiltered)
		{
			// If no filter is set, then always return true even if there is no actor.
			// If there is no actor and there is a filter, then always fail.
			return (Filter.IsValid() == false);
		}
		//Eventually, this might iterate through multiple filters. We'll need to decide how to designate OR versus AND functionality.
		if (Filter.IsValid())
		{
			if (!Filter.Get()->FilterPassesForActor(ActorToBeFiltered))
			{
				return false;
			}
		}
		return true;
	}

	bool FilterPassesForActor(const TWeakObjectPtr<AActor> ActorToBeFiltered) const
	{
		return FilterPassesForActor(ActorToBeFiltered.Get());
	}

	bool operator()(const TWeakObjectPtr<AActor> ActorToBeFiltered) const
	{
		return FilterPassesForActor(ActorToBeFiltered.Get());
	}

	bool operator()(const AActor* ActorToBeFiltered) const
	{
		return FilterPassesForActor(ActorToBeFiltered);
	}
};

#undef UE_API
