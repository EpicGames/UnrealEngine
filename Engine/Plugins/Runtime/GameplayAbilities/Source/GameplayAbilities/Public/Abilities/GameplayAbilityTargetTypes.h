// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/HitResult.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Engine/NetSerialization.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "GameplayTagContainer.h"
#include "GameplayEffectTypes.h"
#include "GameplayPrediction.h"

#include "GameplayAbilityTargetTypes.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UGameplayAbility;
class UGameplayEffect;
class UMeshComponent;
struct FGameplayEffectSpec;

UENUM(BlueprintType)
namespace EGameplayTargetingConfirmation
{
	/** Describes how the targeting information is confirmed */
	enum Type : int
	{
		/** The targeting happens instantly without special logic or user input deciding when to 'fire' */
		Instant,

		/** The targeting happens when the user confirms the targeting */
		UserConfirmed,

		/** The GameplayTargeting Ability is responsible for deciding when the targeting data is ready. Not supported by all TargetingActors */
		Custom,

		/** The GameplayTargeting Ability is responsible for deciding when the targeting data is ready. Not supported by all TargetingActors. Should not destroy upon data production */
		CustomMulti,
	};
}

/** 
 * This type should be provided as the Deleter when creating a TSharedPtr<FGameplayAbilityTargetData> so that
 * the Object's ScriptStruct data will be destructed correctly.
 */
struct FGameplayAbilityTargetDataDeleter
{
	UE_API void operator()(FGameplayAbilityTargetData* Object) const;
};

/**
 *	A generic structure for targeting data. We want generic functions to produce this data and other generic
 *	functions to consume this data.
 *
 *	We expect this to be able to hold specific actors/object reference and also generic location/direction/origin
 *	information.
 *
 *	Some example producers:
 *		-Overlap/Hit collision event generates TargetData about who was hit in a melee attack
 *		-A mouse input causes a hit trace and the actor infront of the crosshair is turned into TargetData
 *		-A mouse input causes TargetData to be generated from the owner's crosshair view origin/direction
 *		-An AOE/aura pulses and all actors in a radius around the instigator are added to TargetData
 *		-Panzer Dragoon style 'painting' targeting mode
 *		-MMORPG style ground AOE targeting style (potentially both a location on the ground and actors that were targeted)
 *
 *	Some example consumers:
 *		-Apply a GameplayEffect to all actors in TargetData
 *		-Find closest actor from all in TargetData
 *		-Call some function on all actors in TargetData
 * 		-Filter or merge TargetDatas
 *		-Spawn a new actor at a TargetData location
 *
 *	Maybe it is better to distinguish between actor list targeting vs positional targeting data?
 *		-AOE/aura type of targeting data blurs the line
 */
USTRUCT()
struct FGameplayAbilityTargetData
{
	GENERATED_USTRUCT_BODY()

	virtual ~FGameplayAbilityTargetData() = default;

	/** Applies a gameplay effect to each target represented */
	UE_API TArray<FActiveGameplayEffectHandle> ApplyGameplayEffect(const UGameplayEffect* GameplayEffect, const FGameplayEffectContextHandle& InEffectContext, float Level, FPredictionKey PredictionKey = FPredictionKey());

	/** Applies a previously created gameplay effect spec to each target represented */
	UE_API virtual TArray<FActiveGameplayEffectHandle> ApplyGameplayEffectSpec(FGameplayEffectSpec& Spec, FPredictionKey PredictionKey = FPredictionKey());

	/** Modifies the context and adds this target data to the target data handle stored within */
	UE_API virtual void AddTargetDataToContext(FGameplayEffectContextHandle& Context, bool bIncludeActorArray) const;

	/** Returns all actors targeted, almost always overridden */
	virtual TArray<TWeakObjectPtr<AActor>> GetActors() const
	{
		return TArray<TWeakObjectPtr<AActor>>();
	}

	/** Modify the actor list */
	virtual bool SetActors(TArray<TWeakObjectPtr<AActor>> NewActorArray)
	{
		//By default, we don't keep this data, and therefore can't set it.
		return false;
	}

	/** Return true in subclasses if GetHitResult will work */
	virtual bool HasHitResult() const
	{
		return false;
	}

	/** Override to return a hit result */
	virtual const FHitResult* GetHitResult() const
	{
		return nullptr;
	}

	/** Override to true if GetOrigin will work */
	virtual bool HasOrigin() const
	{
		return false;
	}

	/** Override to return an origin point, which may be derived from other data */
	virtual FTransform GetOrigin() const
	{
		return FTransform::Identity;
	}

	/** Override to true if GetEndPoint/Transform will work */
	virtual bool HasEndPoint() const
	{
		return false;
	}

	/** Override to return a target/end point */
	virtual FVector GetEndPoint() const
	{
		return FVector::ZeroVector;
	}

	/** Override to return a transform, default will create one from just the location */
	virtual FTransform GetEndPointTransform() const
	{
		return FTransform(GetEndPoint());
	}

	/** Returns the serialization data, must always be overridden */
	virtual UScriptStruct* GetScriptStruct() const
	{
		return FGameplayAbilityTargetData::StaticStruct();
	}

	/** Returns a debug string representation */
	UE_API virtual FString ToString() const;

	/** See notes on delegate definition FOnTargetActorSwapped */
	virtual bool ShouldCheckForTargetActorSwap() const
	{
		return false;
	}

	virtual void ReplaceHitWith(AActor* NewHitActor, const FHitResult* NewHitResult)
	{
		// Intended to be implemented in derived structs.
	}
};

UENUM(BlueprintType)
namespace EGameplayAbilityTargetingLocationType
{
	/** What type of location calculation to use when an ability asks for our transform */
	enum Type : int
	{
		/** We report an actual raw transform. This is also the final fallback if other methods fail */
		LiteralTransform		UMETA(DisplayName = "Literal Transform"),

		/** We pull the transform from an associated actor directly */
		ActorTransform			UMETA(DisplayName = "Actor Transform"),

		/** We aim from a named socket on the player's skeletal mesh component */
		SocketTransform			UMETA(DisplayName = "Socket Transform"),		
	};
}

/**
*	Handle for Targeting Data. This servers two main purposes:
*		-Avoid us having to copy around the full targeting data structure in Blueprints
*		-Allows us to leverage polymorphism in the target data structure
*		-Allows us to implement NetSerialize and replicate by value between clients/server
*
*		-Avoid using UObjects could be used to give us polymorphism and by reference passing in blueprints.
*		-However we would still be screwed when it came to replication
*
*		-Replication by value
*		-Pass by reference in blueprints
*		-Polymophism in TargetData structure
*/
USTRUCT(BlueprintType)
struct FGameplayAbilityTargetDataHandle
{
	GENERATED_USTRUCT_BODY()

	FGameplayAbilityTargetDataHandle() { }
	FGameplayAbilityTargetDataHandle(FGameplayAbilityTargetData* DataPtr)
	{
		Data.Add(TSharedPtr<FGameplayAbilityTargetData>(DataPtr));
	}

	FGameplayAbilityTargetDataHandle(FGameplayAbilityTargetDataHandle&& Other) : UniqueId(Other.UniqueId), Data(MoveTemp(Other.Data))	{ }
	FGameplayAbilityTargetDataHandle(const FGameplayAbilityTargetDataHandle& Other) : UniqueId(Other.UniqueId), Data(Other.Data) { }

	FGameplayAbilityTargetDataHandle& operator=(FGameplayAbilityTargetDataHandle&& Other) { UniqueId = Other.UniqueId; Data = MoveTemp(Other.Data); return *this; }
	FGameplayAbilityTargetDataHandle& operator=(const FGameplayAbilityTargetDataHandle& Other) { UniqueId = Other.UniqueId; Data = Other.Data; return *this; }

	uint8 UniqueId = 0;

	/** Raw storage of target data, do not modify this directly */
	TArray<TSharedPtr<FGameplayAbilityTargetData>, TInlineAllocator<1> >	Data;

	/** Resets handle to have no targets */
	void Clear()
	{
		Data.Reset();
	}

	/** Returns number of target data, not number of actors/targets as target data may contain multiple actors */
	int32 Num() const
	{
		return Data.Num();
	}

	/** Returns true if there are any valid targets */
	bool IsValid(int32 Index) const
	{
		return (Data.IsValidIndex(Index) && Data[Index].IsValid());
	}

	/** Returns data at index, or nullptr if invalid */
	const FGameplayAbilityTargetData* Get(int32 Index) const
	{
		return IsValid(Index) ? Data[Index].Get() : nullptr;
	}

	/** Returns data at index, or nullptr if invalid */
	FGameplayAbilityTargetData* Get(int32 Index)
	{
		return IsValid(Index) ? Data[Index].Get() : nullptr;
	}

	/** Adds a new target data to handle, it must have been created with new */
	void Add(FGameplayAbilityTargetData* DataPtr)
	{
		Data.Add(TSharedPtr<FGameplayAbilityTargetData>(DataPtr));
	}

	/** Does a shallow copy of target data from one handle to another */
	void Append(const FGameplayAbilityTargetDataHandle& OtherHandle)
	{
		Data.Append(OtherHandle.Data);
	}

	/** Serialize for networking, handles polymorphism */
	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** Comparison operator */
	bool operator==(const FGameplayAbilityTargetDataHandle& Other) const
	{
		// Both invalid structs or both valid and Pointer compare (???) // deep comparison equality
		if (Data.Num() != Other.Data.Num())
		{
			return false;
		}
		for (int32 i = 0; i < Data.Num(); ++i)
		{
			if (Data[i].IsValid() != Other.Data[i].IsValid())
			{
				return false;
			}
			if (Data[i].Get() != Other.Data[i].Get())
			{
				return false;
			}
		}
		return true;
	}

	/** Comparison operator */
	bool operator!=(const FGameplayAbilityTargetDataHandle& Other) const
	{
		return !(FGameplayAbilityTargetDataHandle::operator==(Other));
	}
};

template<>
struct TStructOpsTypeTraits<FGameplayAbilityTargetDataHandle> : public TStructOpsTypeTraitsBase2<FGameplayAbilityTargetDataHandle>
{
	enum
	{
		WithCopy = true,		// Necessary so that TSharedPtr<FGameplayAbilityTargetData> Data is copied around
		WithNetSerializer = true,
		WithIdenticalViaEquality = true,
	};
};

// Hacky base class to avoid 8 bytes of padding after the vtable
struct FGameplayAbilityTargetingLocationInfoFixLayout
{
	virtual ~FGameplayAbilityTargetingLocationInfoFixLayout() = default;
};

/** Structure that stores a location in one of several different formats */
USTRUCT(BlueprintType)
struct FGameplayAbilityTargetingLocationInfo
#if CPP
	: public FGameplayAbilityTargetingLocationInfoFixLayout
#endif
{
	GENERATED_USTRUCT_BODY()

	FGameplayAbilityTargetingLocationInfo()
	: SourceActor(nullptr)
	, SourceComponent(nullptr)
	, SourceAbility(nullptr)
	, LocationType(EGameplayAbilityTargetingLocationType::LiteralTransform)
	{ }

	virtual ~FGameplayAbilityTargetingLocationInfo() = default;

	void operator=(const FGameplayAbilityTargetingLocationInfo& Other)
	{
		LocationType = Other.LocationType;
		LiteralTransform = Other.LiteralTransform;
		SourceActor = Other.SourceActor;
		SourceComponent = Other.SourceComponent;
		SourceAbility = Other.SourceAbility;
		SourceSocketName = Other.SourceSocketName;
	}

	/** Converts internal format into a literal world space transform */
	UE_API FTransform GetTargetingTransform() const;
	/** Initializes new target data and fills in with hit results */
	UE_API FGameplayAbilityTargetDataHandle MakeTargetDataHandleFromHitResult(TWeakObjectPtr<UGameplayAbility> Ability, const FHitResult& HitResult) const;
	UE_API FGameplayAbilityTargetDataHandle MakeTargetDataHandleFromHitResults(TWeakObjectPtr<UGameplayAbility> Ability, const TArray<FHitResult>& HitResults) const;

	/** Initializes new actor list target data, and sets this as the origin */
	UE_API FGameplayAbilityTargetDataHandle MakeTargetDataHandleFromActors(const TArray<TWeakObjectPtr<AActor>>& TargetActors, bool OneActorPerHandle = false) const;

	/** A source actor is needed for Actor-based targeting, but not for Socket-based targeting. */
	UPROPERTY(BlueprintReadWrite, meta = (ExposeOnSpawn = true), Category = Targeting)
	TObjectPtr<AActor> SourceActor;

	/** Socket-based targeting requires a skeletal mesh component to check for the named socket. */
	UPROPERTY(BlueprintReadWrite, meta = (ExposeOnSpawn = true), Category = Targeting)
	TObjectPtr<UMeshComponent> SourceComponent;

	/** Ability that will be using the targeting data */
	UPROPERTY(BlueprintReadWrite, meta = (ExposeOnSpawn = true), Category = Targeting)
	TObjectPtr<UGameplayAbility> SourceAbility;

	/** A literal world transform can be used, if one has been calculated outside of the actor using the ability. */
	UPROPERTY(BlueprintReadWrite, meta = (ExposeOnSpawn = true), Category = Targeting)
	FTransform LiteralTransform;

	/** If SourceComponent is valid, this is the name of the socket transform that will be used. If no Socket is provided, SourceComponent's transform will be used. */
	UPROPERTY(BlueprintReadWrite, meta = (ExposeOnSpawn = true), Category = Targeting)
	FName SourceSocketName;

	/** Type of location used - will determine what data is transmitted over the network and what fields are used when calculating position. */
	UPROPERTY(BlueprintReadWrite, meta = (ExposeOnSpawn = true), Category = Targeting)
	TEnumAsByte<EGameplayAbilityTargetingLocationType::Type> LocationType;

	/** Optimized serialize function */
	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);
};

template<>
struct TStructOpsTypeTraits<FGameplayAbilityTargetingLocationInfo> : public TStructOpsTypeTraitsBase2<FGameplayAbilityTargetingLocationInfo>
{
	enum
	{
		WithNetSerializer = true	// For now this is REQUIRED for FGameplayAbilityTargetDataHandle net serialization to work
	};
};

/** Target data with just a source and target location in space */
USTRUCT(BlueprintType)
struct FGameplayAbilityTargetData_LocationInfo : public FGameplayAbilityTargetData
{
	GENERATED_USTRUCT_BODY()

	/** Generic location data for source */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Targeting)
	FGameplayAbilityTargetingLocationInfo SourceLocation;

	/** Generic location data for target */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Targeting)
	FGameplayAbilityTargetingLocationInfo TargetLocation;

	// -------------------------------------

	virtual bool HasOrigin() const override
	{
		return true;
	}

	virtual FTransform GetOrigin() const override
	{
		return SourceLocation.GetTargetingTransform();
	}

	// -------------------------------------

	virtual bool HasEndPoint() const override
	{
		return true;
	}

	virtual FVector GetEndPoint() const override
	{
		return TargetLocation.GetTargetingTransform().GetLocation();
	}

	virtual FTransform GetEndPointTransform() const override
	{
		return TargetLocation.GetTargetingTransform();
	}

	// -------------------------------------

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FGameplayAbilityTargetData_LocationInfo::StaticStruct();
	}

	virtual FString ToString() const override
	{
		return TEXT("FGameplayAbilityTargetData_LocationInfo");
	}

	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);
};

template<>
struct TStructOpsTypeTraits<FGameplayAbilityTargetData_LocationInfo> : public TStructOpsTypeTraitsBase2<FGameplayAbilityTargetData_LocationInfo>
{
	enum
	{
		WithNetSerializer = true	// For now this is REQUIRED for FGameplayAbilityTargetDataHandle net serialization to work
	};
};

/** Target data with a source location and a list of targeted actors, makes sense for AOE attacks */
USTRUCT(BlueprintType)
struct FGameplayAbilityTargetData_ActorArray : public FGameplayAbilityTargetData
{
	GENERATED_USTRUCT_BODY()

	/** We could be selecting this group of actors from any type of location, so use a generic location type */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Targeting)
	FGameplayAbilityTargetingLocationInfo SourceLocation;

	/** Rather than targeting a single point, this type of targeting selects multiple actors. */
	UPROPERTY(EditAnywhere, Category = Targeting)
	TArray<TWeakObjectPtr<AActor> > TargetActorArray;

	virtual TArray<TWeakObjectPtr<AActor> >	GetActors() const override
	{
		return TargetActorArray;
	}

	virtual bool SetActors(TArray<TWeakObjectPtr<AActor>> NewActorArray) override
	{
		TargetActorArray = NewActorArray;
		return true;
	}

	// -------------------------------------

	virtual bool HasOrigin() const override
	{
		return true;
	}

	virtual FTransform GetOrigin() const override
	{
		FTransform ReturnTransform = SourceLocation.GetTargetingTransform();

		//Aim at first valid target, if we have one. Duplicating GetEndPoint() code here so we don't iterate through the target array twice.
		for (int32 i = 0; i < TargetActorArray.Num(); ++i)
		{
			if (TargetActorArray[i].IsValid())
			{
				FVector Direction = (TargetActorArray[i].Get()->GetActorLocation() - ReturnTransform.GetLocation()).GetSafeNormal();
				if (Direction.IsNormalized())
				{
					ReturnTransform.SetRotation(Direction.Rotation().Quaternion());
					break;
				}
			}
		}
		return ReturnTransform;
	}

	// -------------------------------------

	virtual bool HasEndPoint() const override
	{
		//We have an endpoint if we have at least one valid actor in our target array
		for (int32 i = 0; i < TargetActorArray.Num(); ++i)
		{
			if (TargetActorArray[i].IsValid())
			{
				return true;
			}
		}
		return false;
	}

	virtual FVector GetEndPoint() const override
	{
		for (int32 i = 0; i < TargetActorArray.Num(); ++i)
		{
			if (TargetActorArray[i].IsValid())
			{
				return TargetActorArray[i].Get()->GetActorLocation();
			}
		}
		return FVector::ZeroVector;
	}

	// -------------------------------------

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FGameplayAbilityTargetData_ActorArray::StaticStruct();
	}

	virtual FString ToString() const override
	{
		return TEXT("FGameplayAbilityTargetData_ActorArray");
	}

	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);
};

template<>
struct TStructOpsTypeTraits<FGameplayAbilityTargetData_ActorArray> : public TStructOpsTypeTraitsBase2<FGameplayAbilityTargetData_ActorArray>
{
	enum
	{
		WithNetSerializer = true	// For now this is REQUIRED for FGameplayAbilityTargetDataHandle net serialization to work
	};
};

/** Target data with a single hit result, data is packed into the hit result */
USTRUCT(BlueprintType)
struct FGameplayAbilityTargetData_SingleTargetHit : public FGameplayAbilityTargetData
{
	GENERATED_USTRUCT_BODY()

	FGameplayAbilityTargetData_SingleTargetHit()
	{ }

	FGameplayAbilityTargetData_SingleTargetHit(FHitResult InHitResult)
		: HitResult(MoveTemp(InHitResult))
	{ }

	// -------------------------------------

	virtual TArray<TWeakObjectPtr<AActor> >	GetActors() const override
	{
		TArray<TWeakObjectPtr<AActor> >	Actors;
		if (HitResult.HasValidHitObjectHandle())
		{
			Actors.Push(HitResult.HitObjectHandle.FetchActor());
		}
		return Actors;
	}

	// SetActors() will not work here because the actor "array" is drawn from the hit result data, and changing that doesn't make sense.

	// -------------------------------------

	virtual bool HasHitResult() const override
	{
		return true;
	}

	virtual const FHitResult* GetHitResult() const override
	{
		return &HitResult;
	}

	virtual bool HasOrigin() const override
	{
		return true;
	}

	virtual FTransform GetOrigin() const override
	{
		return FTransform((HitResult.TraceEnd - HitResult.TraceStart).Rotation(), HitResult.TraceStart);
	}

	virtual bool HasEndPoint() const override
	{
		return true;
	}

	virtual FVector GetEndPoint() const override
	{
		return HitResult.Location;
	}

	virtual void ReplaceHitWith(AActor* NewHitActor, const FHitResult* NewHitResult)
	{
		bHitReplaced = true;

		HitResult = FHitResult();
		if (NewHitResult != nullptr)
		{
			HitResult = *NewHitResult;
		}
	}

	// -------------------------------------

	/** Hit result that stores data */
	UPROPERTY()
	FHitResult	HitResult;

	UPROPERTY(NotReplicated)
	bool bHitReplaced = false;

	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FGameplayAbilityTargetData_SingleTargetHit::StaticStruct();
	}
};

template<>
struct TStructOpsTypeTraits<FGameplayAbilityTargetData_SingleTargetHit> : public TStructOpsTypeTraitsBase2<FGameplayAbilityTargetData_SingleTargetHit>
{
	enum
	{
		WithNetSerializer = true	// For now this is REQUIRED for FGameplayAbilityTargetDataHandle net serialization to work
	};
};

/** Generic callback for returning when target data is available */
DECLARE_MULTICAST_DELEGATE_OneParam(FAbilityTargetData, const FGameplayAbilityTargetDataHandle&);

/** Generic callback for returning when target data is available */
DECLARE_MULTICAST_DELEGATE_TwoParams(FAbilityTargetDataSetDelegate, const FGameplayAbilityTargetDataHandle&, FGameplayTag);

/** These are generic, nonpayload carrying events that are replicated between the client and server */
UENUM()
namespace EAbilityGenericReplicatedEvent
{
	enum Type : int
	{	
		/** A generic confirmation to commit the ability */
		GenericConfirm = 0,
		/** A generic cancellation event. Not necessarily a canellation of the ability or targeting. Could be used to cancel out of a channelling portion of ability. */
		GenericCancel,
		/** Additional input presses of the ability (Press X to activate ability, press X again while it is active to do other things within the GameplayAbility's logic) */
		InputPressed,	
		/** Input release event of the ability */
		InputReleased,
		/** A generic event from the client */
		GenericSignalFromClient,
		/** A generic event from the server */
		GenericSignalFromServer,
		/** Custom events for game use */
		GameCustom1,
		GameCustom2,
		GameCustom3,
		GameCustom4,
		GameCustom5,
		GameCustom6,
		MAX
	};
}

/** Payload for generic replicated events */
struct FAbilityReplicatedData
{
	FAbilityReplicatedData() : bTriggered(false), VectorPayload(ForceInitToZero) {}

	/** Event has triggered */
	bool bTriggered;

	/** Optional Vector payload for event */
	FVector_NetQuantize100 VectorPayload;

	/** Delegate that will be called on replication */
	FSimpleMulticastDelegate Delegate;
};

#undef UE_API
