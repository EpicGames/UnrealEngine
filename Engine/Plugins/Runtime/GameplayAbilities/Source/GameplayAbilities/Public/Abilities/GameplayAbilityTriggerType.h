// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "GameplayAbilityTriggerType.generated.h"

UENUM(BlueprintType)
namespace EGameplayAbilityTriggerSource
{
	/**	Defines what type of trigger will activate the ability, paired to a tag */
	enum Type : int
	{
		/** Triggered by an external gameplay event. The ability will receive a GameplayEvent payload. */
		GameplayEvent      UMETA(DisplayName = "On Gameplay Event"),
		
		/** Triggered when the owner gains the specified tag. Will not cancel when the tag is removed. */
		OwnedTagAdded      UMETA(DisplayName = "When Tag Is Added"),
		
		/** Triggered when the owner has the specified tag. The ability will be canceled if the tag is later removed. */
		OwnedTagPresent    UMETA(DisplayName = "When Tag Is Present"),
	};
}

/** Structure that defines how an ability will be triggered by external events */
USTRUCT()
struct FAbilityTriggerData
{
	GENERATED_USTRUCT_BODY()

	FAbilityTriggerData() 
	: TriggerSource(EGameplayAbilityTriggerSource::GameplayEvent)
	{}

	/** The tag to respond to */
	UPROPERTY(EditAnywhere, Category=TriggerData, meta=(Categories="TriggerTagCategory"))
	FGameplayTag TriggerTag;

	/** The type of trigger to respond to */
	UPROPERTY(EditAnywhere, Category=TriggerData)
	TEnumAsByte<EGameplayAbilityTriggerSource::Type> TriggerSource;
};
