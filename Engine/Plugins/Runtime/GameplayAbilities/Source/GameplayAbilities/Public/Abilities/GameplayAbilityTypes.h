// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "GameplayTagContainer.h"
#include "AttributeSet.h"
#include "GameplayEffectTypes.h"
#include "GameplayPrediction.h"
#include "GameplayAbilitySpec.h"
#include "Abilities/GameplayAbilityTriggerType.h"
#include "Abilities/GameplayAbilityTargetTypes.h"

#include "GameplayAbilityTypes.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class APlayerController;
class UAbilitySystemComponent;
class UAnimInstance;
class UAnimMontage;
class UDataTable;
class UGameplayAbility;
class UGameplayTask;
class UMovementComponent;
class USkeletalMeshComponent;

GAMEPLAYABILITIES_API DECLARE_LOG_CATEGORY_EXTERN(LogAbilitySystemComponent, Log, All);

#define ENABLE_ABILITYTASK_DEBUGMSG !(UE_BUILD_SHIPPING | UE_BUILD_TEST)


UENUM(BlueprintType)
namespace EGameplayAbilityInstancingPolicy
{
	/**
	 *	How the ability is instanced when executed. This limits what an ability can do in its implementation. For example, a NonInstanced
	 *	Ability cannot have state. It is probably unsafe for an InstancedPerActor ability to have latent actions, etc.
	 */
	enum Type : int
	{
		/** This ability is never instanced. Anything that executes the ability is operating on the CDO. */
		NonInstanced UE_DEPRECATED_FORGAME(5.5, "Use InstancedPerActor as the default to avoid confusing corner cases"),

		/** Each actor gets it's own single instance of this ability, which supports replication to its owning actor to allow RPC's
		 * within the ability. As it is a single instance, only one ability maybe active at any time. State is saved between activations. */
		InstancedPerActor 		UMETA(DisplayName = "Instanced Per Actor"),

		/** We instance this ability each time it is executed. Replication currently unsupported. These can have multiple running
		 * at the same time, and no state is saved between execution. */
		InstancedPerExecution		UMETA(DisplayName = "Instanced Per Execution"),
	};
}

UENUM(BlueprintType)
namespace EGameplayAbilityNetExecutionPolicy
{
	/** Where does an ability execute on the network. Does a client "ask and predict", "ask and wait", "don't ask (just do it)" */
	enum Type : int
	{
		/** Part of this ability runs predictively on the local client if there is one */
		LocalPredicted		UMETA(DisplayName = "Local Predicted"),

		/** This ability will only run on the client or server that has local control */
		LocalOnly			UMETA(DisplayName = "Local Only"),

		/** This ability is initiated by the server, but will also run on the local client if one exists */
		ServerInitiated		UMETA(DisplayName = "Server Initiated"),

		/** This ability will only run on the server */
		ServerOnly			UMETA(DisplayName = "Server Only"),
	};
}

UENUM(BlueprintType)
namespace EGameplayAbilityNetSecurityPolicy
{
	/** What protections does this ability have? Should the client be allowed to request changes to the execution of the ability? */
	enum Type : int
	{
		/** No security requirements. Client or server can trigger execution and termination of this ability freely. */
		ClientOrServer			UMETA(DisplayName = "Client Or Server"),

		/** A client requesting execution of this ability will be ignored by the server. Clients can still request that the server cancel or end this ability. */
		ServerOnlyExecution		UMETA(DisplayName = "Server Only Execution"),

		/** A client requesting cancellation or ending of this ability will be ignored by the server. Clients can still request execution of the ability. */
		ServerOnlyTermination	UMETA(DisplayName = "Server Only Termination"),

		/** Server controls both execution and termination of this ability. A client making any requests will be ignored. */
		ServerOnly				UMETA(DisplayName = "Server Only"),
	};
}

UENUM(BlueprintType)
namespace EGameplayAbilityReplicationPolicy
{
	/** How an ability replicates state/events to everyone on the network */
	enum Type : int
	{
		/** We don't replicate the instance of the ability to anyone. */
		ReplicateNo			UMETA(DisplayName = "Do Not Replicate"),

		/** We replicate the instance of the ability to the owner. */
		ReplicateYes		UMETA(DisplayName = "Replicate"),
	};
}

/**
 *	FGameplayAbilityActorInfo
 *
 *	Cached data associated with an Actor using an Ability.
 *		-Initialized from an AActor* in InitFromActor
 *		-Abilities use this to know what to actor upon. E.g., instead of being coupled to a specific actor class.
 *		-These are generally passed around as pointers to support polymorphism.
 *		-Projects can override UAbilitySystemGlobals::AllocAbilityActorInfo to override the default struct type that is created.
 *
 */
USTRUCT(BlueprintType)
struct FGameplayAbilityActorInfo
{
	GENERATED_USTRUCT_BODY()

	virtual ~FGameplayAbilityActorInfo() = default;

	/** The actor that owns the abilities, shouldn't be null */
	UPROPERTY(BlueprintReadOnly, Category = "ActorInfo")
	TWeakObjectPtr<AActor>	OwnerActor;

	/** The physical representation of the owner, used for targeting and animation. This will often be null! */
	UPROPERTY(BlueprintReadOnly, Category = "ActorInfo")
	TWeakObjectPtr<AActor>	AvatarActor;

	/** PlayerController associated with the owning actor. This will often be null! */
	UPROPERTY(BlueprintReadOnly, Category = "ActorInfo")
	TWeakObjectPtr<APlayerController>	PlayerController;

	/** Ability System component associated with the owner actor, shouldn't be null */
	UPROPERTY(BlueprintReadOnly, Category = "ActorInfo")
	TWeakObjectPtr<UAbilitySystemComponent>	AbilitySystemComponent;

	/** Skeletal mesh of the avatar actor. Often null */
	UPROPERTY(BlueprintReadOnly, Category = "ActorInfo")
	TWeakObjectPtr<USkeletalMeshComponent>	SkeletalMeshComponent;

 	/** Anim instance of the avatar actor. Often null */
	UPROPERTY(BlueprintReadOnly, Category = "ActorInfo")
	TWeakObjectPtr<UAnimInstance>	AnimInstance;

	/** Movement component of the avatar actor. Often null */
	UPROPERTY(BlueprintReadOnly, Category = "ActorInfo")
	TWeakObjectPtr<UMovementComponent>	MovementComponent;
	
	/** The linked Anim Instance that this component will play montages in. Use NAME_None for the main anim instance. */
	UPROPERTY(BlueprintReadOnly, Category = "ActorInfo")
	FName AffectedAnimInstanceTag; 
	
	/** Accessor to get the affected anim instance from the SkeletalMeshComponent */
	UE_API UAnimInstance* GetAnimInstance() const;
	
	/** Returns true if this actor is locally controlled. Only true for players on the client that owns them (differs from APawn::IsLocallyControlled which requires a Controller) */
	UE_API bool IsLocallyControlled() const;

	/** Returns true if this actor has a PlayerController that is locally controlled. */
	UE_API bool IsLocallyControlledPlayer() const;

	/** Returns true if the owning actor has net authority */
	UE_API bool IsNetAuthority() const;

	/** Initializes the info from an owning actor. Will set both owner and avatar */
	UE_API virtual void InitFromActor(AActor *OwnerActor, AActor *AvatarActor, UAbilitySystemComponent* InAbilitySystemComponent);

	/** Sets a new avatar actor, keeps same owner and ability system component */
	UE_API virtual void SetAvatarActor(AActor *AvatarActor);

	/** Clears out any actor info, both owner and avatar */
	UE_API virtual void ClearActorInfo();
};



/** Data about montages that were played locally (all montages in case of server. predictive montages in case of client). Never replicated directly. */
USTRUCT()
struct FGameplayAbilityLocalAnimMontage
{
	GENERATED_USTRUCT_BODY()

	FGameplayAbilityLocalAnimMontage()
		: AnimMontage(nullptr), PlayInstanceId(0), AnimatingAbility(nullptr)
	{
	}

	/** What montage is being played */
	UPROPERTY()
	TObjectPtr<UAnimMontage> AnimMontage;

	/** ID tied to a particular play of a montage, used to trigger replication when the same montage is played multiple times. This ID wraps around when it reaches its max value.  */
	UPROPERTY()
	uint8 PlayInstanceId;

	/** Prediction key that started the montage play */
	UPROPERTY()
	FPredictionKey PredictionKey;

	/** The ability, if any, that instigated this montage */
	UPROPERTY()
	TWeakObjectPtr<UGameplayAbility> AnimatingAbility;
};


/** Metadata for a tag-based Gameplay Event, that can activate other abilities or run ability-specific logic */
USTRUCT(BlueprintType)
struct FGameplayEventData
{
	GENERATED_USTRUCT_BODY()

	FGameplayEventData()
		: Instigator(nullptr)
		, Target(nullptr)
		, OptionalObject(nullptr)
		, OptionalObject2(nullptr)
		, EventMagnitude(0.f)
	{
	}
	
	/** Tag of the event that triggered this */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayAbilityTriggerPayload, meta=(GameplayTagFilter="GameplayEventTagsCategory"))
	FGameplayTag EventTag;

	/** The instigator of the event */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayAbilityTriggerPayload)
	TObjectPtr<const AActor> Instigator;

	/** The target of the event */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayAbilityTriggerPayload)
	TObjectPtr<const AActor> Target;

	/** An optional ability-specific object to be passed though the event */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayAbilityTriggerPayload)
	TObjectPtr<const UObject> OptionalObject;

	/** A second optional ability-specific object to be passed though the event */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayAbilityTriggerPayload)
	TObjectPtr<const UObject> OptionalObject2;

	/** Polymorphic context information */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayAbilityTriggerPayload)
	FGameplayEffectContextHandle ContextHandle;

	/** Tags that the instigator has */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayAbilityTriggerPayload)
	FGameplayTagContainer InstigatorTags;

	/** Tags that the target has */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayAbilityTriggerPayload)
	FGameplayTagContainer TargetTags;

	/** The magnitude of the triggering event */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayAbilityTriggerPayload)
	float EventMagnitude;

	/** The polymorphic target information for the event */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayAbilityTriggerPayload)
	FGameplayAbilityTargetDataHandle TargetData;
};

/** Delegate for handling gameplay event data */
DECLARE_MULTICAST_DELEGATE_OneParam(FGameplayEventMulticastDelegate, const FGameplayEventData*);

/** Delegate for handling gameplay event data, includes tag as the Event Data does not always have it filled out */
DECLARE_MULTICAST_DELEGATE_TwoParams(FGameplayEventTagMulticastDelegate, FGameplayTag, const FGameplayEventData*);


/** Ability Ended Data */
USTRUCT(BlueprintType)
struct FAbilityEndedData
{
	GENERATED_USTRUCT_BODY()

	FAbilityEndedData()
		: AbilityThatEnded(nullptr)
		, bReplicateEndAbility(false)
		, bWasCancelled(false)
	{
	}

	FAbilityEndedData(UGameplayAbility* InAbility, FGameplayAbilitySpecHandle InHandle, bool bInReplicateEndAbility, bool bInWasCancelled)
		: AbilityThatEnded(InAbility)
		, AbilitySpecHandle(InHandle)
		, bReplicateEndAbility(bInReplicateEndAbility)
		, bWasCancelled(bInWasCancelled)
	{
	}

	/** Ability that ended, normally instance but could be CDO */
	UPROPERTY()
	TObjectPtr<UGameplayAbility> AbilityThatEnded;

	/** Specific ability spec that ended */
	UPROPERTY()
	FGameplayAbilitySpecHandle AbilitySpecHandle;

	/** Rather to replicate the ability to ending */
	UPROPERTY()
	bool bReplicateEndAbility;

	/** True if this was cancelled deliberately, false if it ended normally */
	UPROPERTY()
	bool bWasCancelled;
};

/** Notification delegate definition for when the gameplay ability ends */
DECLARE_MULTICAST_DELEGATE_OneParam(FGameplayAbilityEndedDelegate, const FAbilityEndedData&);


/** Structure that tells AbilitySystemComponent what to bind to an InputComponent (see BindAbilityActivationToInputComponent) */
struct FGameplayAbilityInputBinds
{
	FGameplayAbilityInputBinds(FString InConfirmTargetCommand, FString InCancelTargetCommand, FTopLevelAssetPath InEnumPathName, int32 InConfirmTargetInputID = INDEX_NONE, int32 InCancelTargetInputID = INDEX_NONE)
		: ConfirmTargetCommand(InConfirmTargetCommand)
		, CancelTargetCommand(InCancelTargetCommand)
		, EnumPathName(InEnumPathName)
		, ConfirmTargetInputID(InConfirmTargetInputID)
		, CancelTargetInputID(InCancelTargetInputID)
	{
	}

	FGameplayAbilityInputBinds(FGameplayAbilityInputBinds&&) = default;
	FGameplayAbilityInputBinds(const FGameplayAbilityInputBinds&) = default;
	FGameplayAbilityInputBinds& operator=(FGameplayAbilityInputBinds&&) = default;
	FGameplayAbilityInputBinds& operator=(const FGameplayAbilityInputBinds&) = default;

	/** Defines command string that will be bound to Confirm Targeting */
	FString ConfirmTargetCommand;

	/** Defines command string that will be bound to Cancel Targeting */
	FString CancelTargetCommand;

	/** Returns enum to use for ability binds. E.g., "Ability1"-"Ability9" input commands will be bound to ability activations inside the AbiltiySystemComponent */
	FTopLevelAssetPath EnumPathName;

	/** If >=0, Confirm is bound to an entry in the enum */
	int32 ConfirmTargetInputID;

	/** If >=0, Cancel is bound to an entry in the enum */
	int32 CancelTargetInputID;

	UEnum* GetBindEnum() 
	{
		return FindObject<UEnum>(EnumPathName);
	}
};


/** Used to initialize default values for attributes */
USTRUCT()
struct FAttributeDefaults
{
	GENERATED_USTRUCT_BODY()

	FAttributeDefaults()
		: DefaultStartingTable(nullptr)
	{ }

	UPROPERTY(EditAnywhere, Category = "AttributeTest")
	TSubclassOf<UAttributeSet> Attributes;

	UPROPERTY(EditAnywhere, Category = "AttributeTest")
	TObjectPtr<UDataTable> DefaultStartingTable;
};


/** Debug message emitted by ability tasks */
USTRUCT()
struct FAbilityTaskDebugMessage
{
	GENERATED_USTRUCT_BODY()

	FAbilityTaskDebugMessage()
		: FromTask(nullptr)
	{ }

	UPROPERTY()
	TObjectPtr<UGameplayTask>	FromTask;

	UPROPERTY()
	FString Message;
};

/** Used for cleaning up predicted data on network clients */
DECLARE_MULTICAST_DELEGATE(FAbilitySystemComponentPredictionKeyClear);

/** Generic delegate for ability 'events'/notifies */
DECLARE_MULTICAST_DELEGATE_OneParam(FGenericAbilityDelegate, UGameplayAbility*);


/** This struct holds state to batch server RPC calls: ServerTryActivateAbility, ServerSetReplicatedTargetData, ServerEndAbility.  */
USTRUCT()
struct FServerAbilityRPCBatch
{
	GENERATED_BODY()

	FServerAbilityRPCBatch() 
		: InputPressed(false), Ended(false), Started(false)
	{
	}

	UPROPERTY()
	FGameplayAbilitySpecHandle	AbilitySpecHandle;

	UPROPERTY()
	FPredictionKey	PredictionKey;

	UPROPERTY()
	FGameplayAbilityTargetDataHandle	TargetData;

	UPROPERTY()
	bool InputPressed;

	UPROPERTY()
	bool Ended;

	/** Safety bool to make sure ServerTryActivate was called exactly one time in a batch */
	UPROPERTY(NotReplicated)
	bool Started;

	// To allow FindByKey etc
	bool operator==(const FGameplayAbilitySpecHandle& InHandle) const
	{
		return AbilitySpecHandle == InHandle;
	}
};


/** Helper struct for defining ServerRPC batch windows. If null ASC is passed in, this becomes a noop. */
struct FScopedServerAbilityRPCBatcher
{
	UE_API FScopedServerAbilityRPCBatcher(UAbilitySystemComponent* InASC, FGameplayAbilitySpecHandle InAbilityHandle);
	UE_API ~FScopedServerAbilityRPCBatcher();

private:

	UAbilitySystemComponent* ASC;
	FGameplayAbilitySpecHandle AbilityHandle;
	FScopedPredictionWindow ScopedPredictionWindow;
};


/** Used as a key for storing internal ability data */
USTRUCT()
struct FGameplayAbilitySpecHandleAndPredictionKey
{
	GENERATED_USTRUCT_BODY()

	FGameplayAbilitySpecHandleAndPredictionKey()
		: PredictionKeyAtCreation(0)
	{}

	FGameplayAbilitySpecHandleAndPredictionKey(const FGameplayAbilitySpecHandle& HandleRef, const FPredictionKey& PredictionKeyAtCreationRef)
		: AbilityHandle(HandleRef), PredictionKeyAtCreation(PredictionKeyAtCreationRef.Current)
	{}

	bool operator==(const FGameplayAbilitySpecHandleAndPredictionKey& Other) const
	{
		return AbilityHandle == Other.AbilityHandle && PredictionKeyAtCreation == Other.PredictionKeyAtCreation;
	}

	bool operator!=(const FGameplayAbilitySpecHandleAndPredictionKey& Other) const
	{
		return AbilityHandle != Other.AbilityHandle || PredictionKeyAtCreation != Other.PredictionKeyAtCreation;
	}

	friend uint32 GetTypeHash(const FGameplayAbilitySpecHandleAndPredictionKey& Handle)
	{
		return GetTypeHash(Handle.AbilityHandle) ^ Handle.PredictionKeyAtCreation;
	}

	UPROPERTY()
	FGameplayAbilitySpecHandle AbilityHandle;

	UPROPERTY()
	int32 PredictionKeyAtCreation;
};

/** Struct defining the cached data for a specific gameplay ability. This data is generally synchronized client->server in a network game. */
struct FAbilityReplicatedDataCache
{
	/** What elements this activation is targeting */
	FGameplayAbilityTargetDataHandle TargetData;

	/** What tag to pass through when doing an application */
	FGameplayTag ApplicationTag;

	/** True if we've been positively confirmed our targeting, false if we don't know */
	bool bTargetConfirmed;

	/** True if we've been positively cancelled our targeting, false if we don't know */
	bool bTargetCancelled;

	/** Delegate to call whenever this is modified */
	FAbilityTargetDataSetDelegate TargetSetDelegate;

	/** Delegate to call whenever this is confirmed (without target data) */
	FSimpleMulticastDelegate TargetCancelledDelegate;

	/** Generic events that contain no payload data */
	FAbilityReplicatedData	GenericEvents[EAbilityGenericReplicatedEvent::MAX];

	/** Prediction Key when this data was set */
	FPredictionKey PredictionKey;

	FAbilityReplicatedDataCache() : bTargetConfirmed(false), bTargetCancelled(false) {}
	virtual ~FAbilityReplicatedDataCache() = default;

	/** Resets any cached data, leaves delegates up */
	void Reset()
	{
		bTargetConfirmed = bTargetCancelled = false;
		TargetData = FGameplayAbilityTargetDataHandle();
		ApplicationTag = FGameplayTag();
		PredictionKey = FPredictionKey();
		for (int32 i=0; i < (int32) EAbilityGenericReplicatedEvent::MAX; ++i)
		{
			GenericEvents[i].bTriggered = false;
			GenericEvents[i].VectorPayload = FVector::ZeroVector;
		}
	}

	/** Resets cached data and clears delegates. */
	void ResetAll()
	{
		Reset();
		TargetSetDelegate.Clear();
		TargetCancelledDelegate.Clear();
		for (int32 i=0; i < (int32) EAbilityGenericReplicatedEvent::MAX; ++i)
		{
			GenericEvents[i].Delegate.Clear();
		}
	}
};


/** 
 *	Associative container of GameplayAbilitySpecs + PredictionKeys --> FAbilityReplicatedDataCache. Basically, it holds replicated data on the ability system component that abilities access in their scripting.
 *	This was refactored from a normal TMap. This mainly serves to:
 *		1. Return shared ptrs to the cached data so that callsites are not vulnerable to the underlying map shifting around (E.g invoking a replicated event ends the ability or activates a new one and causes memory to move, invalidating the pointer).
 *		2. Data is cleared on ability end via ::Remove.
 *		3. The FAbilityReplicatedDataCache instances are recycled rather than allocated each time via ::FreeData.
 * 
 **/
struct FGameplayAbilityReplicatedDataContainer
{
	GAMEPLAYABILITIES_API TSharedPtr<FAbilityReplicatedDataCache> Find(const FGameplayAbilitySpecHandleAndPredictionKey& Key) const;
	GAMEPLAYABILITIES_API TSharedRef<FAbilityReplicatedDataCache> FindOrAdd(const FGameplayAbilitySpecHandleAndPredictionKey& Key);

	void Remove(const FGameplayAbilitySpecHandleAndPredictionKey& Key);
	void PrintDebug();

private:

	typedef TPair<FGameplayAbilitySpecHandleAndPredictionKey, TSharedRef<FAbilityReplicatedDataCache>> FKeyDataPair;

	TArray<FKeyDataPair> InUseData;
	TArray<TSharedRef<FAbilityReplicatedDataCache>> FreeData;
};

#undef UE_API
