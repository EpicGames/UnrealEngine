// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "GameplayAbilityWorldReticle.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class APlayerController;
class AGameplayAbilityTargetActor;

USTRUCT(BlueprintType)
struct FWorldReticleParameters
{
	GENERATED_USTRUCT_BODY()

	//Use this so that we can't slip in new parameters without some actor knowing about it.
	void Initialize(FVector InAOEScale)
	{
		AOEScale = InAOEScale;
	}

	UPROPERTY(BlueprintReadWrite, Category = Reticle)
	FVector AOEScale = FVector(0.f);
};

/** Reticles allow targeting to be visualized. Tasks can spawn these. Artists/designers can create BPs for these. */
UCLASS(Blueprintable, notplaceable, MinimalAPI)
class AGameplayAbilityWorldReticle : public AActor
{
	GENERATED_UCLASS_BODY()

public:

	UE_API virtual void Tick(float DeltaSeconds) override;

	// ------------------------------

	UE_API virtual bool IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const override;

	UE_API void InitializeReticle(AGameplayAbilityTargetActor* InTargetingActor, APlayerController* PlayerController, FWorldReticleParameters InParameters);

	UE_API void SetIsTargetValid(bool bNewValue);
	UE_API void SetIsTargetAnActor(bool bNewValue);

	/** Called whenever bIsTargetValid changes value. */
	UFUNCTION(BlueprintImplementableEvent, Category = Reticle)
	UE_API void OnValidTargetChanged(bool bNewValue);

	/** Called whenever bIsTargetAnActor changes value. */
	UFUNCTION(BlueprintImplementableEvent, Category = Reticle)
	UE_API void OnTargetingAnActor(bool bNewValue);

	UFUNCTION(BlueprintImplementableEvent, Category = Reticle)
	UE_API void OnParametersInitialized();

	UFUNCTION(BlueprintImplementableEvent, Category = Reticle)
	UE_API void SetReticleMaterialParamFloat(FName ParamName, float value);

	UFUNCTION(BlueprintImplementableEvent, Category = Reticle)
	UE_API void SetReticleMaterialParamVector(FName ParamName, FVector value);

	UFUNCTION(BlueprintCallable, Category = Reticle)
	UE_API void FaceTowardSource(bool bFaceIn2D);

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, meta = (ExposeOnSpawn = "true"), Category = "Reticle")
	FWorldReticleParameters Parameters;

	/** Makes the reticle's default owner-facing behavior operate in 2D (flat) instead of 3D (pitched). Defaults to true. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ExposeOnSpawn = "true"), Category = "Reticle")
	bool bFaceOwnerFlat;

	// If the target is an actor snap to it's location 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ExposeOnSpawn = "true"), Category = "Reticle")
	bool bSnapToTargetedActor;

protected:
	/** This indicates whether or not the targeting actor considers the current target to be valid. Defaults to true. */
	UPROPERTY(BlueprintReadOnly, Category = "Network")
	bool bIsTargetValid;

	/** This indicates whether or not the targeting reticle is pointed at an actor. Defaults to false. */
	UPROPERTY(BlueprintReadOnly, Category = "Network")
	bool bIsTargetAnActor;
	
	/** This is used in the process of determining whether we should replicate to a specific client. */
	UPROPERTY(BlueprintReadOnly, Category = "Network")
	TObjectPtr<APlayerController> PrimaryPC;

	/** In the future, we may want to grab things like sockets off of this. */
	UPROPERTY(BlueprintReadOnly, Category = "Network")
	TObjectPtr<AGameplayAbilityTargetActor> TargetingActor;
};

#undef UE_API
