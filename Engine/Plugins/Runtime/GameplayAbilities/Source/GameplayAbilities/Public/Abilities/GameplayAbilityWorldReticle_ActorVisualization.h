// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/GameplayAbilityWorldReticle.h"

#include "GameplayAbilityWorldReticle_ActorVisualization.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class AGameplayAbilityTargetActor;
class UMaterialInterface;

/** This is a dummy reticle for internal use by visualization placement tasks. It builds a custom visual model of the visualization being placed. */
UCLASS(notplaceable)
class AGameplayAbilityWorldReticle_ActorVisualization : public AGameplayAbilityWorldReticle
{
	GENERATED_UCLASS_BODY()

public:
	void InitializeReticleVisualizationInformation(AGameplayAbilityTargetActor* InTargetingActor, AActor* VisualizationActor, UMaterialInterface *VisualizationMaterial);

private:
	/** Hardcoded collision component, so other objects don't think they can collide with the visualization actor */
	UPROPERTY()
	TObjectPtr<class UCapsuleComponent> CollisionComponent;
public:

	UPROPERTY()
	TArray<TObjectPtr<UActorComponent>> VisualizationComponents;

	/** Overridable function called whenever this actor is being removed from a level */
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	/** Returns CollisionComponent subobject **/
	class UCapsuleComponent* GetCollisionComponent() { return CollisionComponent; }
};

#undef UE_API
