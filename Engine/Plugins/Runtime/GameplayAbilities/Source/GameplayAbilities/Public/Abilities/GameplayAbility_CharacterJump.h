// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayAbilitySpec.h"
#include "Abilities/GameplayAbility.h"
#include "GameplayAbility_CharacterJump.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/**
 *	Ability that jumps with a character.
 */
UCLASS(MinimalAPI)
class UGameplayAbility_CharacterJump : public UGameplayAbility
{
	GENERATED_UCLASS_BODY()

public:

	UE_API virtual bool CanActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayTagContainer* SourceTags = nullptr, const FGameplayTagContainer* TargetTags = nullptr, OUT FGameplayTagContainer* OptionalRelevantTags = nullptr) const override;

	UE_API virtual void ActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* OwnerInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayEventData* TriggerEventData) override;

	UE_API virtual void InputReleased(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo) override;

	UE_API virtual void CancelAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, bool bReplicateCancelAbility) override;
};

#undef UE_API
