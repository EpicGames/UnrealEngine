// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameplayAbilitySpec.h"
#include "GameplayEffect.h"
#include "Abilities/GameplayAbility.h"
#include "GameplayAbility_Montage.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;
class UAnimMontage;

/**
 *	A gameplay ability that plays a single montage and applies a GameplayEffect
 */
UCLASS(MinimalAPI)
class UGameplayAbility_Montage : public UGameplayAbility
{
	GENERATED_UCLASS_BODY()

public:
	
	UE_API virtual void ActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* OwnerInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayEventData* TriggerEventData) override;

	UPROPERTY(EditDefaultsOnly, Category = MontageAbility)
	TObjectPtr<UAnimMontage> 	MontageToPlay;

	UPROPERTY(EditDefaultsOnly, Category = MontageAbility)
	float	PlayRate;

	UPROPERTY(EditDefaultsOnly, Category = MontageAbility)
	FName	SectionName;

	/** GameplayEffects to apply and then remove while the animation is playing */
	UPROPERTY(EditDefaultsOnly, Category = MontageAbility)
	TArray<TSubclassOf<UGameplayEffect>> GameplayEffectClassesWhileAnimating;

	/** Deprecated. Use GameplayEffectClassesWhileAnimating instead. */
	UPROPERTY(VisibleDefaultsOnly, Category = Deprecated)
	TArray<TObjectPtr<const UGameplayEffect>>	GameplayEffectsWhileAnimating;

	UE_API void OnMontageEnded(UAnimMontage* Montage, bool bInterrupted, TWeakObjectPtr<UAbilitySystemComponent> AbilitySystemComponent, TArray<struct FActiveGameplayEffectHandle>	AppliedEffects);

	UE_API void GetGameplayEffectsWhileAnimating(TArray<const UGameplayEffect*>& OutEffects) const;
};

#undef UE_API
