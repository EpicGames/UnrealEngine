// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameplayPrediction.h"
#include "GameplayAbilitySpec.h"
#include "Abilities/GameplayAbilityTypes.h"
#include "GameplayTask.h"
#include "Abilities/GameplayAbility.h"
#include "AbilityTask.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;
class UGameplayTasksComponent;

/**
 *	AbilityTasks are small, self contained operations that can be performed while executing an ability.
 *	They are latent/asynchronous is nature. They will generally follow the pattern of 'start something and wait until it is finished or interrupted'
 *	
 *	We have code in K2Node_LatentAbilityCall to make using these in blueprints streamlined. The best way to become familiar with AbilityTasks is to 
 *	look at existing tasks like UAbilityTask_WaitOverlap (very simple) and UAbilityTask_WaitTargetData (much more complex).
 *	
 *	These are the basic requirements for using an ability task:
 *	
 *	1) Define dynamic multicast, BlueprintAssignable delegates in your AbilityTask. These are the OUTPUTs of your task. When these delegates fire,
 *	execution resumes in the calling blueprints.
 *	
 *	2) Your inputs are defined by a static factory function which will instantiate an instance of your task. The parameters of this function define
 *	the INPUTs into your task. All the factory function should do is instantiate your task and possibly set starting parameters. It should NOT invoke
 *	any of the callback delegates!
 *	
 *	3) Implement a Activate() function (defined here in base class). This function should actually start/execute your task logic. It is safe to invoke
 *	callback delegates here.
 *	
 *	
 *	This is all you need for basic AbilityTasks. 
 *	
 *	
 *	CheckList:
 *		-Override ::OnDestroy() and unregister any callbacks that the task registered. Call Super::EndTask too!
 *		-Implemented an Activate function which truly 'starts' the task. Do not 'start' the task in your static factory function!
 *	
 *	
 *	--------------------------------------
 *	
 *	We have additional support for AbilityTasks that want to spawn actors. Though this could be accomplished in an Activate() function, it would not be
 *	possible to pass in dynamic "ExposeOnSpawn" actor properties. This is a powerful feature of blueprints, in order to support this, you need to implement 
 *	a different step 3:
 *	
 *	Instead of an Activate() function, you should implement a BeginSpawningActor() and FinishSpawningActor() function.
 *	
 *	BeginSpawningActor() must take in a TSubclassOf<YourActorClassToSpawn> parameters named 'Class'. It must also have a out reference parameters of type 
 *	YourActorClassToSpawn*& named SpawnedActor. This function is allowed to decide whether it wants to spawn the actor or not (useful if wishing to
 *	predicate actor spawning on network authority).
 *	
 *	BeginSpawningActor() can instantiate an actor with SpawnActorDeferred. This is important, otherwise the UCS will run before spawn parameters are set.
 *	BeginSpawningActor() should also set the SpawnedActor parameter to the actor it spawned.
 *	
 *	[Next, the generated byte code will set the expose on spawn parameters to whatever the user has set]
 *	
 *	If you spawned something, FinishSpawningActor() will be called and pass in the same actor that was just spawned. You MUST call ExecuteConstruction + PostActorConstruction
 *	on this actor!
 *	
 *	This is a lot of steps but in general, AbilityTask_SpawnActor() gives a clear, minimal example.
 *	
 *	
 */

/**
 *	Latent tasks are waiting on something. This is to differeniate waiting on the user to do something vs waiting on the game to do something.
 *	Tasks start WaitingOnGame, and are set to WaitingOnUser when appropriate (see WaitTargetData, WaitIiputPress, etc)
 */
UENUM()
enum class EAbilityTaskWaitState : uint8
{
	/** Task is waiting for the game to do something */
	WaitingOnGame = 0x01,

	/** Waiting for the user to do something */
	WaitingOnUser = 0x02,

	/** Waiting on Avatar (Character/Pawn/Actor) to do something (usually something physical in the world, like land, move, etc) */
	WaitingOnAvatar = 0x04
};

UCLASS(Abstract, MinimalAPI)
class UAbilityTask : public UGameplayTask
{
	GENERATED_UCLASS_BODY()

	UE_API virtual void OnDestroy(bool bInOwnerFinished) override;
	UE_API virtual void BeginDestroy() override;

	/**
	 * Clears potential dangling pointers. Added to prevent dangling pointer to GameplayAbility that issued the task.
	 * When overriding use EndTask() instead.
	 */
	UE_API virtual void PreDestroyFromReplication() override;
	
	/** Returns spec handle for owning ability */
	UE_API FGameplayAbilitySpecHandle GetAbilitySpecHandle() const;

	UE_API void SetAbilitySystemComponent(UAbilitySystemComponent* InAbilitySystemComponent);

	/** GameplayAbility that created us */
	UPROPERTY()
	TObjectPtr<UGameplayAbility> Ability;

	UPROPERTY()
	TWeakObjectPtr<UAbilitySystemComponent> AbilitySystemComponent;

	/** Returns true if the ability is a locally predicted ability running on a client. Usually this means we need to tell the server something. */
	UE_API bool IsPredictingClient() const;

	/** Returns true if we are executing the ability on the server for a non locally controlled client */
	UE_API bool IsForRemoteClient() const;

	/** Returns true if we are executing the ability on the locally controlled client */
	UE_API bool IsLocallyControlled() const;

	/** Returns ActivationPredictionKey of owning ability */
	UE_API FPredictionKey GetActivationPredictionKey() const;

	/** This should be called prior to broadcasting delegates back into the ability graph. This makes sure the ability is still active.  */
	UE_API bool ShouldBroadcastAbilityTaskDelegates() const;

	UE_API virtual void InitSimulatedTask(UGameplayTasksComponent& InGameplayTasksComponent) override;

	static UE_API void DebugRecordAbilityTaskCreatedByAbility(const UObject* Ability);

	/** Helper function for instantiating and initializing a new task */
	template <class T>
	static T* NewAbilityTask(UGameplayAbility* ThisAbility, FName InstanceName = FName())
	{
		check(ThisAbility);

		T* MyObj = NewObject<T>();
		MyObj->InitTask(*ThisAbility, ThisAbility->GetGameplayTaskDefaultPriority());

		UAbilityTask::DebugRecordAbilityTaskCreatedByAbility(ThisAbility);

		MyObj->InstanceName = InstanceName;
		return MyObj;
	}

	template<typename T>
	static bool DelayedFalse()
	{
		return false;
	}

	// this function has been added to make sure AbilityTasks don't use this method
	template <class T>
	inline static T* NewTask(UObject* WorldContextObject, FName InstanceName = FName())
	{
		static_assert(DelayedFalse<T>(), "UAbilityTask::NewTask should never be used. Use NewAbilityTask instead");
		return nullptr;
	}

	/** Called when the ability task is waiting on remote player data. IF the remote player ends the ability prematurely, and a task with this set is still running, the ability is killed. */
	UE_API void SetWaitingOnRemotePlayerData();
	UE_API void ClearWaitingOnRemotePlayerData();
	UE_API virtual bool IsWaitingOnRemotePlayerdata() const override;

	/** same as RemotePlayerData but for ACharacter type of state (movement state, etc) */
	UE_API void SetWaitingOnAvatar();
	UE_API void ClearWaitingOnAvatar();
	UE_API virtual bool IsWaitingOnAvatar() const override;

	/** What we are waiting on */
	uint8 WaitStateBitMask;
	uint8 bWasSuccessfullyDestroyed : 1;

protected:
	/** Helper method for registering client replicated callbacks */
	UE_API bool CallOrAddReplicatedDelegate(EAbilityGenericReplicatedEvent::Type Event, FSimpleMulticastDelegate::FDelegate Delegate);
};

//For searching through lists of ability instances
struct FAbilityInstanceNamePredicate
{
	FAbilityInstanceNamePredicate(FName DesiredInstanceName)
	{
		InstanceName = DesiredInstanceName;
	}

	bool operator()(const TWeakObjectPtr<UAbilityTask> A) const
	{
		return (A.IsValid() && !A.Get()->GetInstanceName().IsNone() && A.Get()->GetInstanceName().IsValid() && (A.Get()->GetInstanceName() == InstanceName));
	}

	FName InstanceName;
};

struct FAbilityInstanceClassPredicate
{
	FAbilityInstanceClassPredicate(TSubclassOf<UAbilityTask> Class)
	{
		TaskClass = Class;
	}

	bool operator()(const TWeakObjectPtr<UAbilityTask> A) const
	{
		return (A.IsValid() && (A.Get()->GetClass() == TaskClass));
	}

	TSubclassOf<UAbilityTask> TaskClass;
};

#define ABILITYTASK_MSG(Format, ...) \
	if (ENABLE_ABILITYTASK_DEBUGMSG) \
	{ \
		if (Ability) \
			Ability->AddAbilityTaskDebugMessage(this, FString::Printf(TEXT(Format), ##__VA_ARGS__)); \
	} 

#undef UE_API
