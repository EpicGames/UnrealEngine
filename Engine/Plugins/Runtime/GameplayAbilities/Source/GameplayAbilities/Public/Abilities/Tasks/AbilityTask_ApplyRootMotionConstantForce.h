// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "Abilities/Tasks/AbilityTask_ApplyRootMotion_Base.h"
#include "AbilityTask_ApplyRootMotionConstantForce.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UCharacterMovementComponent;
class UCurveFloat;
class UGameplayTasksComponent;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FApplyRootMotionConstantForceDelegate);

class AActor;

/**
 *	Applies force to character's movement
 */
UCLASS(MinimalAPI)
class UAbilityTask_ApplyRootMotionConstantForce : public UAbilityTask_ApplyRootMotion_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FApplyRootMotionConstantForceDelegate OnFinish;

	/** Apply force to character's movement */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_ApplyRootMotionConstantForce* ApplyRootMotionConstantForce
	(
		UGameplayAbility* OwningAbility, 
		FName TaskInstanceName, 
		FVector WorldDirection, 
		float Strength, 
		float Duration, 
		bool bIsAdditive, 
		UCurveFloat* StrengthOverTime,
		ERootMotionFinishVelocityMode VelocityOnFinishMode,
		FVector SetVelocityOnFinish,
		float ClampVelocityOnFinish,
		bool bEnableGravity
	);

	/** Tick function for this task, if bTickingTask == true */
	UE_API virtual void TickTask(float DeltaTime) override;

	UE_API virtual void PreDestroyFromReplication() override;
	UE_API virtual void OnDestroy(bool AbilityIsEnding) override;

protected:

	UE_API virtual void SharedInitAndApply() override;

protected:

	UPROPERTY(Replicated)
	FVector WorldDirection;

	UPROPERTY(Replicated)
	float Strength;

	UPROPERTY(Replicated)
	float Duration;

	UPROPERTY(Replicated)
	bool bIsAdditive;

	/** 
	 *  Strength of the force over time
	 *  Curve Y is 0 to 1 which is percent of full Strength parameter to apply
	 *  Curve X is 0 to 1 normalized time if this force has a limited duration (Duration > 0), or
	 *          is in units of seconds if this force has unlimited duration (Duration < 0)
	 */
	UPROPERTY(Replicated)
	TObjectPtr<UCurveFloat> StrengthOverTime;

	UPROPERTY(Replicated)
	bool bEnableGravity;

};

#undef UE_API
