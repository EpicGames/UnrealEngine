// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "Abilities/Tasks/AbilityTask_ApplyRootMotion_Base.h"
#include "AbilityTask_ApplyRootMotionJumpForce.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UCharacterMovementComponent;
class UCurveFloat;
class UCurveVector;
class UGameplayTasksComponent;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FApplyRootMotionJumpForceDelegate);

class AActor;

/**
 *	Applies force to character's movement
 */
UCLASS(MinimalAPI)
class UAbilityTask_ApplyRootMotionJumpForce : public UAbilityTask_ApplyRootMotion_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FApplyRootMotionJumpForceDelegate OnFinish;

	UPROPERTY(BlueprintAssignable)
	FApplyRootMotionJumpForceDelegate OnLanded;

	UFUNCTION(BlueprintCallable, Category="Ability|Tasks")
	UE_API void Finish();

	UFUNCTION()
	UE_API void OnLandedCallback(const FHitResult& Hit);

	/** Apply force to character's movement */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_ApplyRootMotionJumpForce* ApplyRootMotionJumpForce(UGameplayAbility* OwningAbility, FName TaskInstanceName, FRotator Rotation, float Distance, float Height, float Duration, float MinimumLandedTriggerTime,
		bool bFinishOnLanded, ERootMotionFinishVelocityMode VelocityOnFinishMode, FVector SetVelocityOnFinish, float ClampVelocityOnFinish, UCurveVector* PathOffsetCurve, UCurveFloat* TimeMappingCurve);

	UE_API virtual void Activate() override;

	/** Tick function for this task, if bTickingTask == true */
	UE_API virtual void TickTask(float DeltaTime) override;

	UE_API virtual void PreDestroyFromReplication() override;
	UE_API virtual void OnDestroy(bool AbilityIsEnding) override;

protected:

	UE_API virtual void SharedInitAndApply() override;

	/**
	* Work-around for OnLanded being called during bClientUpdating in movement replay code
	* Don't want to trigger our Landed logic during a replay, so we wait until next frame
	* If we don't, we end up removing root motion from a replay root motion set instead
	* of the real one
	*/
	UE_API void TriggerLanded();

protected:

	UPROPERTY(Replicated)
	FRotator Rotation;

	UPROPERTY(Replicated)
	float Distance;

	UPROPERTY(Replicated)
	float Height;

	UPROPERTY(Replicated)
	float Duration;

	UPROPERTY(Replicated)
	float MinimumLandedTriggerTime;

	UPROPERTY(Replicated)
	bool bFinishOnLanded;

	UPROPERTY(Replicated)
	TObjectPtr<UCurveVector> PathOffsetCurve;

	/** 
	 *  Maps real time to movement fraction curve to affect the speed of the
	 *  movement through the path
	 *  Curve X is 0 to 1 normalized real time (a fraction of the duration)
	 *  Curve Y is 0 to 1 is what percent of the move should be at a given X
	 *  Default if unset is a 1:1 correspondence
	 */
	UPROPERTY(Replicated)
	TObjectPtr<UCurveFloat> TimeMappingCurve;

	bool bHasLanded;
};

#undef UE_API
