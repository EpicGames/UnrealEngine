// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "Abilities/Tasks/AbilityTask_ApplyRootMotion_Base.h"
#include "Abilities/GameplayAbilityTargetTypes.h"
#include "Delegates/IDelegateInstance.h"
#include "AbilityTask_ApplyRootMotionMoveToActorForce.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UCharacterMovementComponent;
class UCurveFloat;
class UCurveVector;
class UGameplayTasksComponent;
enum class ERootMotionFinishVelocityMode : uint8;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FApplyRootMotionMoveToActorForceDelegate, bool, DestinationReached, bool, TimedOut, FVector, FinalTargetLocation);

class AActor;

UENUM()
enum class ERootMotionMoveToActorTargetOffsetType : uint8
{
	// Align target offset vector from target to source, ignoring height difference
	AlignFromTargetToSource = 0,
	// Align from target actor location to target actor forward
	AlignToTargetForward,
	// Align in world space
	AlignToWorldSpace
};

/**
 *	Applies force to character's movement
 */
UCLASS(MinimalAPI)
class UAbilityTask_ApplyRootMotionMoveToActorForce : public UAbilityTask_ApplyRootMotion_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FApplyRootMotionMoveToActorForceDelegate OnFinished;

	/** Apply force to character's movement to move to a target actor */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_ApplyRootMotionMoveToActorForce* ApplyRootMotionMoveToActorForce(UGameplayAbility* OwningAbility, FName TaskInstanceName, AActor* TargetActor, FVector TargetLocationOffset, ERootMotionMoveToActorTargetOffsetType OffsetAlignment, float Duration, UCurveFloat* TargetLerpSpeedHorizontal, UCurveFloat* TargetLerpSpeedVertical, bool bSetNewMovementMode, EMovementMode MovementMode, bool bRestrictSpeedToExpected, UCurveVector* PathOffsetCurve, UCurveFloat* TimeMappingCurve, ERootMotionFinishVelocityMode VelocityOnFinishMode, FVector SetVelocityOnFinish, float ClampVelocityOnFinish, bool bDisableDestinationReachedInterrupt, float ReachedDestinationDistance = 50.0f);

	/** Apply force to character's movement to move to a target component with relative location*/
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_ApplyRootMotionMoveToActorForce* ApplyRootMotionMoveToComponentForce(UGameplayAbility* OwningAbility, FName TaskInstanceName, USceneComponent* TargetComponent, FVector TargetComponentRelativeLocation, FVector TargetLocationOffset, ERootMotionMoveToActorTargetOffsetType OffsetAlignment, float Duration, UCurveFloat* TargetLerpSpeedHorizontal, UCurveFloat* TargetLerpSpeedVertical, bool bSetNewMovementMode, EMovementMode MovementMode, bool bRestrictSpeedToExpected, UCurveVector* PathOffsetCurve, UCurveFloat* TimeMappingCurve, ERootMotionFinishVelocityMode VelocityOnFinishMode, FVector SetVelocityOnFinish, float ClampVelocityOnFinish, bool bDisableDestinationReachedInterrupt, float ReachedDestinationDistance = 50.0f);
	
	/** Apply force to character's movement using an index into targetData instead of using an actor directly. */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_ApplyRootMotionMoveToActorForce* ApplyRootMotionMoveToTargetDataActorForce(UGameplayAbility* OwningAbility, FName TaskInstanceName, FGameplayAbilityTargetDataHandle TargetDataHandle, int32 TargetDataIndex, int32 TargetActorIndex, FVector TargetLocationOffset, ERootMotionMoveToActorTargetOffsetType OffsetAlignment, float Duration, UCurveFloat* TargetLerpSpeedHorizontal, UCurveFloat* TargetLerpSpeedVertical, bool bSetNewMovementMode, EMovementMode MovementMode, bool bRestrictSpeedToExpected, UCurveVector* PathOffsetCurve, UCurveFloat* TimeMappingCurve, ERootMotionFinishVelocityMode VelocityOnFinishMode, FVector SetVelocityOnFinish, float ClampVelocityOnFinish, bool bDisableDestinationReachedInterrupt, float ReachedDestinationDistance = 50.0f);

	/** Tick function for this task, if bTickingTask == true */
	UE_API virtual void TickTask(float DeltaTime) override;

	UE_API virtual void PreDestroyFromReplication() override;
	UE_API virtual void OnDestroy(bool AbilityIsEnding) override;

	UFUNCTION()
	UE_API void OnTargetActorSwapped(class AActor* OriginalTarget, class AActor* NewTarget);

protected:

	UE_API virtual void SharedInitAndApply() override;

	UE_API bool UpdateTargetLocation(float DeltaTime);

	UE_API void SetRootMotionTargetLocation(FVector NewTargetLocation);

	UE_API FVector CalculateTargetOffset() const;

	UFUNCTION()
	UE_API void OnRep_TargetLocation();

protected:

	FDelegateHandle TargetActorSwapHandle;

	UPROPERTY(Replicated)
	FVector StartLocation;

	UPROPERTY(ReplicatedUsing=OnRep_TargetLocation)
	FVector TargetLocation;

	UPROPERTY(Replicated)
	TObjectPtr<AActor> TargetActor;

	UPROPERTY(Replicated)
	TObjectPtr<USceneComponent> TargetComponent = nullptr;

	UPROPERTY(Replicated)
	FVector TargetComponentRelativeLocation = FVector::ZeroVector;

	UPROPERTY(Replicated)
	FVector TargetLocationOffset;

	UPROPERTY(Replicated)
	ERootMotionMoveToActorTargetOffsetType OffsetAlignment;

	UPROPERTY(Replicated)
	float Duration;

	/** By default, this force ends when the destination is reached. Using this parameter you can disable it so it will not 
	 *  "early out" and get interrupted by reaching the destination and instead go to its full duration. */
	UPROPERTY(Replicated)
	bool bDisableDestinationReachedInterrupt;

	/** This is the distance where, at or less that this distance to the destination, the user is considered to be at the destination. */
	UPROPERTY(Replicated)
	float ReachedDestinationDistance;

	UPROPERTY(Replicated)
	bool bSetNewMovementMode;

	UPROPERTY(Replicated)
	TEnumAsByte<EMovementMode> NewMovementMode;

	/** If enabled, we limit velocity to the initial expected velocity to go distance to the target over Duration.
	 *  This prevents cases of getting really high velocity the last few frames of the root motion if you were being blocked by
	 *  collision. Disabled means we do everything we can to velocity during the move to get to the TargetLocation. */
	UPROPERTY(Replicated)
	bool bRestrictSpeedToExpected;

	UPROPERTY(Replicated)
	TObjectPtr<UCurveVector> PathOffsetCurve;

	/** 
	 *  Maps real time to movement fraction curve to affect the speed of the
	 *  movement through the path
	 *  Curve X is 0 to 1 normalized real time (a fraction of the duration)
	 *  Curve Y is 0 to 1 is what percent of the move should be at a given X
	 *  Default if unset is a 1:1 correspondence
	 */
	UPROPERTY(Replicated)
	TObjectPtr<UCurveFloat> TimeMappingCurve;

	UPROPERTY(Replicated)
	TObjectPtr<UCurveFloat> TargetLerpSpeedHorizontalCurve;

	UPROPERTY(Replicated)
	TObjectPtr<UCurveFloat> TargetLerpSpeedVerticalCurve;

	EMovementMode PreviousMovementMode;
	uint8 PreviousCustomMode;
};

#undef UE_API
