// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "Abilities/Tasks/AbilityTask_ApplyRootMotion_Base.h"
#include "AbilityTask_ApplyRootMotionMoveToForce.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UCharacterMovementComponent;
class UCurveVector;
class UGameplayTasksComponent;
enum class ERootMotionFinishVelocityMode : uint8;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FApplyRootMotionMoveToForceDelegate);

class AActor;

/**
 *	Applies force to character's movement
 */
UCLASS(MinimalAPI)
class UAbilityTask_ApplyRootMotionMoveToForce : public UAbilityTask_ApplyRootMotion_Base
{
	GENERATED_BODY()
	
public:
	UPROPERTY(BlueprintAssignable)
	FApplyRootMotionMoveToForceDelegate OnTimedOut;

	UPROPERTY(BlueprintAssignable)
	FApplyRootMotionMoveToForceDelegate OnTimedOutAndDestinationReached;

	/** Apply force to character's movement */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_ApplyRootMotionMoveToForce* ApplyRootMotionMoveToForce(UGameplayAbility* OwningAbility, FName TaskInstanceName, FVector TargetLocation, float Duration, bool bSetNewMovementMode, EMovementMode MovementMode, bool bRestrictSpeedToExpected, UCurveVector* PathOffsetCurve, ERootMotionFinishVelocityMode VelocityOnFinishMode, FVector SetVelocityOnFinish, float ClampVelocityOnFinish);

	/** Tick function for this task, if bTickingTask == true */
	UE_API virtual void TickTask(float DeltaTime) override;

	UE_API virtual void PreDestroyFromReplication() override;
	UE_API virtual void OnDestroy(bool AbilityIsEnding) override;

protected:
	UE_API virtual void SharedInitAndApply() override;

	UPROPERTY(Replicated)
	FVector StartLocation;

	UPROPERTY(Replicated)
	FVector TargetLocation;

	UPROPERTY(Replicated)
	float Duration;

	UPROPERTY(Replicated)
	bool bSetNewMovementMode = false;

	UPROPERTY(Replicated)
	TEnumAsByte<EMovementMode> NewMovementMode = EMovementMode::MOVE_Walking;

	/** If enabled, we limit velocity to the initial expected velocity to go distance to the target over Duration.
	 *  This prevents cases of getting really high velocity the last few frames of the root motion if you were being blocked by
	 *  collision. Disabled means we do everything we can to velocity during the move to get to the TargetLocation. */
	UPROPERTY(Replicated)
	bool bRestrictSpeedToExpected = false;

	UPROPERTY(Replicated)
	TObjectPtr<UCurveVector> PathOffsetCurve = nullptr;

	EMovementMode PreviousMovementMode = EMovementMode::MOVE_None;
	uint8 PreviousCustomMovementMode = 0;
};

#undef UE_API
