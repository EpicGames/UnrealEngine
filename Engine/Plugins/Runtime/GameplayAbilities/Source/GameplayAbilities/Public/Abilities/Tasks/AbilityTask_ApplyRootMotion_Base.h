// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_ApplyRootMotion_Base.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UCharacterMovementComponent;
enum class ERootMotionFinishVelocityMode : uint8;

/** This delegate can be used to support target swapping on abilities.  e.g. If a decoy is created and you want root motion to switch the destination to the decoy */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTargetActorSwapped, AActor*, AActor*);

/** Base class for ability tasks that apply root motion */
UCLASS(MinimalAPI)
class UAbilityTask_ApplyRootMotion_Base : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UE_API virtual void InitSimulatedTask(UGameplayTasksComponent& InGameplayTasksComponent) override;

	//..See notes on delegate definition FOnTargetActorSwapped.
	static UE_API FOnTargetActorSwapped OnTargetActorSwapped;

protected:

	virtual void SharedInitAndApply() {};
	UE_API virtual bool HasTimedOut() const;

	UPROPERTY(Replicated)
	FName ForceName;

	/** What to do with character's Velocity when root motion finishes */
	UPROPERTY(Replicated)
	ERootMotionFinishVelocityMode FinishVelocityMode;

	/** If FinishVelocityMode mode is "SetVelocity", character velocity is set to this value when root motion finishes */
	UPROPERTY(Replicated)
	FVector FinishSetVelocity;

	/** If FinishVelocityMode mode is "ClampVelocity", character velocity is clamped to this value when root motion finishes */
	UPROPERTY(Replicated)
	float FinishClampVelocity;

	UPROPERTY()
	TWeakObjectPtr<UCharacterMovementComponent> MovementComponent; 
	
	uint16 RootMotionSourceID;

	bool bIsFinished;

	float StartTime;
	float EndTime;
};

#undef UE_API
