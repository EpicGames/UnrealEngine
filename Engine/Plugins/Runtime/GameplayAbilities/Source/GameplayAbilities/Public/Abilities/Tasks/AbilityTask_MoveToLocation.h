// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_MoveToLocation.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UCurveFloat;
class UCurveVector;
class UGameplayTasksComponent;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FMoveToLocationDelegate);

class AActor;
class UPrimitiveComponent;
class UGameplayTasksComponent;

/**
 *	TODO:
 *		-Implement replicated time so that this can work as a simulated task for Join In Progress clients.
 */


/**
 *	Move to a location, ignoring clipping, over a given length of time. Ends when the TargetLocation is reached.
 *	This will RESET your character's current movement mode! If you wish to maintain PHYS_Flying or PHYS_Custom, you must
 *	reset it on completion.!
 */
UCLASS(MinimalAPI)
class UAbilityTask_MoveToLocation : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FMoveToLocationDelegate		OnTargetLocationReached;

	UE_API virtual void InitSimulatedTask(UGameplayTasksComponent& InGameplayTasksComponent) override;

	/** Move to the specified location, using the vector curve (range 0 - 1) if specified, otherwise the float curve (range 0 - 1) or fallback to linear interpolation */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_MoveToLocation* MoveToLocation(UGameplayAbility* OwningAbility, FName TaskInstanceName, FVector Location, float Duration, UCurveFloat* OptionalInterpolationCurve, UCurveVector* OptionalVectorInterpolationCurve);

	UE_API virtual void Activate() override;

	/** Tick function for this task, if bTickingTask == true */
	UE_API virtual void TickTask(float DeltaTime) override;

	UE_API virtual void OnDestroy(bool AbilityIsEnding) override;

protected:

	bool bIsFinished;

	UPROPERTY(Replicated)
	FVector StartLocation;

	//FVector 
	
	UPROPERTY(Replicated)
	FVector TargetLocation;

	UPROPERTY(Replicated)
	float DurationOfMovement;


	float TimeMoveStarted;

	float TimeMoveWillEnd;

	UPROPERTY(Replicated)
	TObjectPtr<UCurveFloat> LerpCurve;

	UPROPERTY(Replicated)
	TObjectPtr<UCurveVector> LerpCurveVector;

	
};

#undef UE_API
