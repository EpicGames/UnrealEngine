// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_NetworkSyncPoint.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FNetworkSyncDelegate);

UENUM()
enum class EAbilityTaskNetSyncType : uint8
{
	/** Both client and server wait until the other signals */
	BothWait,

	/** Only server will wait for the client signal. Client will signal and immediately continue without waiting to hear from Server. */
	OnlyServerWait,

	/** Only client will wait for the server signal. Server will signal and immediately continue without waiting to hear from Client. */
	OnlyClientWait	
};

/** Task for providing a generic sync point for client server (one can wait for a signal from the other) */
UCLASS(MinimalAPI)
class UAbilityTask_NetworkSyncPoint : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FNetworkSyncDelegate	OnSync;

	UFUNCTION()
	UE_API void OnSignalCallback();

	UE_API virtual void Activate() override;

	/**
	 *	
	 *	Synchronize execution flow between Client and Server. Depending on SyncType, the Client and Server will wait for the other to reach this node or another WaitNetSync node in the ability before continuing execution.  
	 *	
	 *	BothWait - Both Client and Server will wait until the other reaches the node. (Whoever gets their first, waits for the other before continueing).
	 *	OnlyServerWait - Only server will wait for the client signal. Client will signal and immediately continue without waiting to hear from Server.
	 *	OnlyClientWait - Only client will wait for the server signal. Server will signal and immediately continue without waiting to hear from Client.
	 *	
	 *	Note that this is "ability instance wide". These sync points never affect sync points in other abilities. 
	 *	
	 *	In most cases you will have both client and server execution paths connected to the same WaitNetSync node. However it is possible to use separate nodes
	 *	for cleanliness of the graph. The "signal" is "ability instance wide".
	 *	
	 */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_NetworkSyncPoint* WaitNetSync(UGameplayAbility* OwningAbility, EAbilityTaskNetSyncType SyncType);

protected:

	UE_API void SyncFinished();

	/** The event we replicate */
	EAbilityGenericReplicatedEvent::Type	ReplicatedEventToListenFor;

	EAbilityTaskNetSyncType SyncType;
};

#undef UE_API
