// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimInstance.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_PlayAnimAndWait.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FPlayAnimWaitSimpleDelegate);

class UAnimSequence;

UCLASS(MinimalAPI)
class UAbilityTask_PlayAnimAndWait : public UAbilityTask
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintAssignable)
	FPlayAnimWaitSimpleDelegate	OnCompleted;

	UPROPERTY(BlueprintAssignable)
	FPlayAnimWaitSimpleDelegate	OnBlendOut;

	UPROPERTY(BlueprintAssignable)
	FPlayAnimWaitSimpleDelegate	OnBlendIn;

	UPROPERTY(BlueprintAssignable)
	FPlayAnimWaitSimpleDelegate	OnInterrupted;

	UPROPERTY(BlueprintAssignable)
	FPlayAnimWaitSimpleDelegate	OnCancelled;

	UFUNCTION()
	UE_API void OnMontageBlendedIn(UAnimMontage* Montage);

	UFUNCTION()
	UE_API void OnMontageBlendingOut(UAnimMontage* Montage, bool bInterrupted);

	UFUNCTION()
	UE_API void OnMontageInterrupted();

	UFUNCTION()
	UE_API void OnMontageEnded(UAnimMontage* Montage, bool bInterrupted);

	/** 
	 * Start playing an animation montage on the avatar actor and wait for it to finish
	 * If StopWhenAbilityEnds is true, this montage will be aborted if the ability ends normally. It is always stopped when the ability is explicitly cancelled.
	 * On normal execution, OnBlendOut is called when the montage is blending out, and OnCompleted when it is completely done playing
	 * OnInterrupted is called if another montage overwrites this, and OnCancelled is called if the ability or task is cancelled
	 *
	 * @param TaskInstanceName Set to override the name of this task, for later querying
	 * @param MontageToPlay The montage to play on the character
	 * @param Rate Change to play the montage faster or slower
	 * @param StartSection If not empty, named montage section to start from
	 * @param bStopWhenAbilityEnds If true, this montage will be aborted if the ability ends normally. It is always stopped when the ability is explicitly cancelled
	 * @param AnimRootMotionTranslationScale Change to modify size of root motion or set to 0 to block it entirely
	 * @param StartTimeSeconds Starting time offset in montage, this will be overridden by StartSection if that is also set
	 */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (DisplayName="Play Anim And Wait",
		HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_PlayAnimAndWait* CreatePlayAnimAndWaitProxy(UGameplayAbility* OwningAbility,
		FName TaskInstanceName, UAnimSequence* AnimSequence, FName SlotName = NAME_None, float BlendInTime = 0.25f, float BlendOutTime = 0.25f, float InPlayRate = 1.0f, float StartTimeSeconds = 0.0f, bool bStopWhenAbilityEnds = true, float AnimRootMotionTranslationScale = 1.0f, float InPlayCount = 1.0f);

	UE_API virtual void Activate() override;

	/** Called when the ability is asked to cancel from an outside node. What this means depends on the individual task. By default, this does nothing other than ending the task. */
	UE_API virtual void ExternalCancel() override;

	UE_API virtual FString GetDebugString() const override;

protected:

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	/** Checks if the ability is playing a montage and stops that montage, returns true if a montage was stopped, false if not. */
	UE_API bool StopPlayingMontage();

	FOnMontageBlendingOutStarted BlendingOutDelegate;
	FOnMontageBlendedInEnded BlendedInDelegate;
	FOnMontageEnded MontageEndedDelegate;
	FDelegateHandle InterruptedHandle;

	UPROPERTY()
	TObjectPtr<UAnimSequence> AnimSequenceToPlay;

	FName SlotName = NAME_None;
	float BlendInTime = 0.25f;
	float BlendOutTime = 0.25f;
	float PlayRate = 1.0f;
	float PlayCount = 1.0f;
	float AnimRootMotionTranslationScale = 1.0f;
	float StartTimeSeconds = 0.0f;
	bool bStopWhenAbilityEnds = true;
};

#undef UE_API
