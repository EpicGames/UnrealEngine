// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimInstance.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_PlayMontageAndWait.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FMontageWaitSimpleDelegate);

/** Ability task to simply play a montage. Many games will want to make a modified version of this task that looks for game-specific events */
UCLASS(MinimalAPI)
class UAbilityTask_PlayMontageAndWait : public UAbilityTask
{
	GENERATED_BODY()
public:

	UPROPERTY(BlueprintAssignable)
	FMontageWaitSimpleDelegate	OnCompleted;

	UPROPERTY(BlueprintAssignable)
	FMontageWaitSimpleDelegate	OnBlendedIn;

	UPROPERTY(BlueprintAssignable)
	FMontageWaitSimpleDelegate	OnBlendOut;

	UPROPERTY(BlueprintAssignable)
	FMontageWaitSimpleDelegate	OnInterrupted;

	UPROPERTY(BlueprintAssignable)
	FMontageWaitSimpleDelegate	OnCancelled;

	UFUNCTION()
	UE_API void OnMontageBlendedIn(UAnimMontage* Montage);

	UFUNCTION()
	UE_API void OnMontageBlendingOut(UAnimMontage* Montage, bool bInterrupted);

	/** Callback function for when the owning Gameplay Ability is cancelled */
	UFUNCTION()
	UE_API void OnGameplayAbilityCancelled();

	UFUNCTION()
	UE_API void OnMontageEnded(UAnimMontage* Montage, bool bInterrupted);

	/** 
	 * Start playing an animation montage on the avatar actor and wait for it to finish
	 * If StopWhenAbilityEnds is true, this montage will be aborted if the ability ends normally. It is always stopped when the ability is explicitly cancelled.
	 * On normal execution, OnBlendOut is called when the montage is blending out, and OnCompleted when it is completely done playing
	 * OnInterrupted is called if another montage overwrites this, and OnCancelled is called if the ability or task is cancelled
	 *
	 * @param TaskInstanceName Set to override the name of this task, for later querying
	 * @param MontageToPlay The montage to play on the character
	 * @param Rate Change to play the montage faster or slower
	 * @param StartSection If not empty, named montage section to start from
	 * @param bStopWhenAbilityEnds If true, this montage will be aborted if the ability ends normally. It is always stopped when the ability is explicitly cancelled
	 * @param AnimRootMotionTranslationScale Change to modify size of root motion or set to 0 to block it entirely
	 * @param StartTimeSeconds Starting time offset in montage, this will be overridden by StartSection if that is also set
	 * @param bAllowInterruptAfterBlendOut If true, you can receive OnInterrupted after an OnBlendOut started (otherwise OnInterrupted will not fire when interrupted, but you will not get OnComplete).
	 */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (DisplayName="PlayMontageAndWait",
		HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_PlayMontageAndWait* CreatePlayMontageAndWaitProxy(UGameplayAbility* OwningAbility,
		FName TaskInstanceName, UAnimMontage* MontageToPlay, float Rate = 1.f, FName StartSection = NAME_None, bool bStopWhenAbilityEnds = true, float AnimRootMotionTranslationScale = 1.f, float StartTimeSeconds = 0.f, bool bAllowInterruptAfterBlendOut = false);

	UE_API virtual void Activate() override;

	/** Called when the ability is asked to cancel from an outside node. What this means depends on the individual task. By default, this does nothing other than ending the task. */
	UE_API virtual void ExternalCancel() override;

	UE_API virtual FString GetDebugString() const override;

protected:

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	/** Checks if the ability is playing a montage and stops that montage, returns true if a montage was stopped, false if not. */
	UE_API bool StopPlayingMontage();

	FOnMontageBlendedInEnded BlendedInDelegate;
	FOnMontageBlendingOutStarted BlendingOutDelegate;
	FOnMontageEnded MontageEndedDelegate;
	FDelegateHandle InterruptedHandle;

	UPROPERTY()
	TObjectPtr<UAnimMontage> MontageToPlay;

	UPROPERTY()
	float Rate;

	UPROPERTY()
	FName StartSection;

	UPROPERTY()
	float AnimRootMotionTranslationScale;

	UPROPERTY()
	float StartTimeSeconds;

	UPROPERTY()
	bool bStopWhenAbilityEnds;

	UPROPERTY()
	bool bAllowInterruptAfterBlendOut;
};

#undef UE_API
