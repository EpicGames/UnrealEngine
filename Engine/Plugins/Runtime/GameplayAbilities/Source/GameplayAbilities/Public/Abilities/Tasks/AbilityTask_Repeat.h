// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_Repeat.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FRepeatedActionDelegate, int32, ActionNumber);

/**
 *	Repeat a task a certain number of times at a given interval.
 */
UCLASS(MinimalAPI)
class UAbilityTask_Repeat : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FRepeatedActionDelegate		OnPerformAction;

	UPROPERTY(BlueprintAssignable)
	FRepeatedActionDelegate		OnFinished;

	UE_API virtual FString GetDebugString() const override;

	UE_API void PerformAction();

	/** Start a task that repeats an action or set of actions. */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_Repeat* RepeatAction(UGameplayAbility* OwningAbility, float TimeBetweenActions, int32 TotalActionCount);

	UE_API virtual void Activate() override;

protected:
	int32 ActionPerformancesDesired;
	int32 ActionCounter;
	float TimeBetweenActions;

	/** Handle for efficient management of PerformAction timer */
	FTimerHandle TimerHandle_PerformAction;

	UE_API virtual void OnDestroy(bool AbilityIsEnding) override;
};

#undef UE_API
