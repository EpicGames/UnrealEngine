// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "Abilities/GameplayAbilityTargetTypes.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_SpawnActor.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FSpawnActorDelegate, AActor*, SpawnedActor);

/**
 *	Convenience task for spawning actors on the network authority. If not the net authority, we will not spawn and Success will not be called.
 *	The nice thing this adds is the ability to modify expose on spawn properties while also implicitly checking network role before spawning.
 *	
 *	Though this task doesn't do much - games can implement similiar tasks that carry out game specific rules. For example a 'SpawnProjectile'
 *	task that limits the available classes to the games projectile class, and that does game specific stuff on spawn (for example, determining
 *	firing position from a weapon attachment - logic that we don't necessarily want in ability blueprints).
 *	
 *	Long term we can also use this task as a sync point. If the executing client could wait execution until the server creates and replicates the 
 *	actor down to it. We could potentially also use this to do predictive actor spawning / reconciliation.
 */

UCLASS(MinimalAPI)
class UAbilityTask_SpawnActor: public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FSpawnActorDelegate	Success;

	/** Called when we can't spawn: on clients or potentially on server if they fail to spawn (rare) */
	UPROPERTY(BlueprintAssignable)
	FSpawnActorDelegate	DidNotSpawn;
	
	/** Spawn new Actor on the network authority (server) */
	UFUNCTION(BlueprintCallable, meta=(HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true"), Category="Ability|Tasks")
	static UE_API UAbilityTask_SpawnActor* SpawnActor(UGameplayAbility* OwningAbility, FGameplayAbilityTargetDataHandle TargetData, TSubclassOf<AActor> Class);

	UFUNCTION(BlueprintCallable, meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true"), Category = "Abilities")
	UE_API bool BeginSpawningActor(UGameplayAbility* OwningAbility, FGameplayAbilityTargetDataHandle TargetData, TSubclassOf<AActor> Class, AActor*& SpawnedActor);

	UFUNCTION(BlueprintCallable, meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true"), Category = "Abilities")
	UE_API void FinishSpawningActor(UGameplayAbility* OwningAbility, FGameplayAbilityTargetDataHandle TargetData, AActor* SpawnedActor);

protected:
	FGameplayAbilityTargetDataHandle CachedTargetDataHandle;
};

#undef UE_API
