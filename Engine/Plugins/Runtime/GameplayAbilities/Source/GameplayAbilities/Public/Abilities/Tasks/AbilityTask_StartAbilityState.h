// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_StartAbilityState.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FAbilityStateDelegate);

/**
 * An ability state is simply an ability task that provides a way to handle the ability being interrupted.
 * You can use ability states to do state-specific cleanup if the ability ends or was interrupted at a certain point during it's execution.
 *
 * An ability state will always result in either 'OnStateEnded' or 'OnStateInterrupted' being called.
 *
 * 'OnStateEnded' will be called if:
 * - The ability itself ends via AGameplayAbility::EndAbility
 * - The ability state is manually ended via AGameplayAbility::EndAbilityState
 * - Another ability state is started will 'bEndCurrentState' set to true
 *
 * 'OnStateInterrupted' will be called if:
 * - The ability itself is cancelled via AGameplayAbility::CancelAbility
 */
UCLASS(MinimalAPI)
class UAbilityTask_StartAbilityState : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	/** Invoked if 'EndAbilityState' is called or if 'EndAbility' is called and this state is active. */
	UPROPERTY(BlueprintAssignable)
	FAbilityStateDelegate OnStateEnded;

	/** Invoked if the ability was interrupted and this state is active. */
	UPROPERTY(BlueprintAssignable)
	FAbilityStateDelegate OnStateInterrupted;

	UE_API virtual void Activate() override;

	UE_API virtual void ExternalCancel() override;

	UE_API virtual FString GetDebugString() const override;

	/**
	 * Starts a new ability state.
	 *
	 * @param StateName The name of the state.
	 * @param bEndCurrentState If true, all other active ability states will be ended.
	 */
	UFUNCTION(BlueprintCallable, Meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true", HideSpawnParms = "Instigator"), Category = "Ability|Tasks")
	static UE_API UAbilityTask_StartAbilityState* StartAbilityState(UGameplayAbility* OwningAbility, FName StateName, bool bEndCurrentState = true);

private:

	FDelegateHandle EndStateHandle;
	FDelegateHandle InterruptStateHandle;

	bool bWasEnded;
	bool bWasInterrupted;

	bool bEndCurrentState;

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	void OnEndState(FName StateNameToEnd);
	void OnInterruptState();
};

#undef UE_API
