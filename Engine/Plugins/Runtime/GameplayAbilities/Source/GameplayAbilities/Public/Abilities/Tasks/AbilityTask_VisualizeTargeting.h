// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Abilities/GameplayAbilityTargetActor.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_VisualizeTargeting.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FVisualizeTargetingDelegate);

UCLASS(MinimalAPI)
class UAbilityTask_VisualizeTargeting: public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FVisualizeTargetingDelegate TimeElapsed;

	UE_API void OnTimeElapsed();

	/** Spawns target actor and uses it for visualization. */
	UFUNCTION(BlueprintCallable, meta=(HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true", HideSpawnParms="Instigator"), Category="Ability|Tasks")
	static UE_API UAbilityTask_VisualizeTargeting* VisualizeTargeting(UGameplayAbility* OwningAbility, TSubclassOf<AGameplayAbilityTargetActor> Class, FName TaskInstanceName, float Duration = -1.0f);

	/** Visualize target using a specified target actor. */
	UFUNCTION(BlueprintCallable, meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true"), Category = "Ability|Tasks")
	static UE_API UAbilityTask_VisualizeTargeting* VisualizeTargetingUsingActor(UGameplayAbility* OwningAbility, AGameplayAbilityTargetActor* TargetActor, FName TaskInstanceName, float Duration = -1.0f);

	UE_API virtual void Activate() override;

	UFUNCTION(BlueprintCallable, meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true"), Category = "Abilities")
	UE_API bool BeginSpawningActor(UGameplayAbility* OwningAbility, TSubclassOf<AGameplayAbilityTargetActor> Class, AGameplayAbilityTargetActor*& SpawnedActor);

	UFUNCTION(BlueprintCallable, meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true"), Category = "Abilities")
	UE_API void FinishSpawningActor(UGameplayAbility* OwningAbility, AGameplayAbilityTargetActor* SpawnedActor);

protected:

	UE_API void SetDuration(const float Duration);

	UE_API bool ShouldSpawnTargetActor() const;
	UE_API void InitializeTargetActor(AGameplayAbilityTargetActor* SpawnedActor) const;
	UE_API void FinalizeTargetActor(AGameplayAbilityTargetActor* SpawnedActor) const;

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

protected:

	TSubclassOf<AGameplayAbilityTargetActor> TargetClass;

	/** The TargetActor that we spawned */
	TWeakObjectPtr<AGameplayAbilityTargetActor>	TargetActor;

	/** Handle for efficient management of OnTimeElapsed timer */
	FTimerHandle TimerHandle_OnTimeElapsed;
};


/**
*	Requirements for using Begin/Finish SpawningActor functionality:
*		-Have a parameters named 'Class' in your Proxy factor function (E.g., WaitTargetdata)
*		-Have a function named BeginSpawningActor w/ the same Class parameter
*			-This function should spawn the actor with SpawnActorDeferred and return true/false if it spawned something.
*		-Have a function named FinishSpawningActor w/ an AActor* of the class you spawned
*			-This function *must* call ExecuteConstruction + PostActorConstruction
*/

#undef UE_API
