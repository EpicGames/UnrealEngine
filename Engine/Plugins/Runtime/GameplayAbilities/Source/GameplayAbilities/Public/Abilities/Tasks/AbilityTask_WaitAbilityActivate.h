// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTagContainer.h"
#include "GameplayEffectTypes.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitAbilityActivate.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FWaitAbilityActivateDelegate, UGameplayAbility*, ActivatedAbility);

class AActor;

/**
 *	Waits for the actor to activate another ability
 */
UCLASS(MinimalAPI)
class UAbilityTask_WaitAbilityActivate : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitAbilityActivateDelegate	OnActivate;

	UE_API virtual void Activate() override;

	UFUNCTION()
	UE_API void OnAbilityActivate(UGameplayAbility *ActivatedAbility);

	/** Wait until a new ability (of the same or different type) is activated. Only input based abilities will be counted unless IncludeTriggeredAbilities is true. */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitAbilityActivate* WaitForAbilityActivate(UGameplayAbility* OwningAbility, FGameplayTag WithTag, FGameplayTag WithoutTag, bool IncludeTriggeredAbilities=false, bool TriggerOnce=true);

	/** Wait until a new ability (of the same or different type) is activated. Only input based abilities will be counted unless IncludeTriggeredAbilities is true. Uses a tag requirements structure to filter abilities. */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitAbilityActivate* WaitForAbilityActivateWithTagRequirements(UGameplayAbility* OwningAbility, FGameplayTagRequirements TagRequirements, bool IncludeTriggeredAbilities = false, bool TriggerOnce = true);

	/** Wait until a new ability (of the same or different type) is activated. Only input based abilities will be counted unless IncludeTriggeredAbilities is true. */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitAbilityActivate* WaitForAbilityActivate_Query(UGameplayAbility* OwningAbility, FGameplayTagQuery Query, bool IncludeTriggeredAbilities=false, bool TriggerOnce=true);

	FGameplayTag WithTag;
	FGameplayTag WithoutTag;
	bool IncludeTriggeredAbilities;
	bool TriggerOnce;
	FGameplayTagRequirements TagRequirements;

	FGameplayTagQuery Query;

protected:

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	FDelegateHandle OnAbilityActivateDelegateHandle;
};

#undef UE_API
