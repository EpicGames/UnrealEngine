// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTagContainer.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitAbilityCommit.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FWaitAbilityCommitDelegate, UGameplayAbility*, ActivatedAbility);

/**
 *	Waits for the actor to activate another ability
 */
UCLASS(MinimalAPI)
class UAbilityTask_WaitAbilityCommit : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitAbilityCommitDelegate	OnCommit;

	UE_API virtual void Activate() override;

	UFUNCTION()
	UE_API void OnAbilityCommit(UGameplayAbility *ActivatedAbility);

	/** Wait until a new ability (of the same or different type) is commited. */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE", DisplayName = "Wait For New Ability Commit"))
	static UE_API UAbilityTask_WaitAbilityCommit* WaitForAbilityCommit(UGameplayAbility* OwningAbility, FGameplayTag WithTag, FGameplayTag WithoutTage, bool TriggerOnce=true);

	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE", DisplayName = "Wait For New Ability Commit Query"))
	static UE_API UAbilityTask_WaitAbilityCommit* WaitForAbilityCommit_Query(UGameplayAbility* OwningAbility, FGameplayTagQuery Query, bool TriggerOnce=true);

	FGameplayTag WithTag;
	FGameplayTag WithoutTag;
	bool TriggerOnce;


	FGameplayTagQuery Query;

protected:

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	FDelegateHandle OnAbilityCommitDelegateHandle;
};

#undef UE_API
