// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AttributeSet.h"
#include "GameplayTagContainer.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitAttributeChange.generated.h"

#define UE_API GAMEPLAYABILITIES_API

struct FGameplayEffectModCallbackData;

UENUM()
namespace EWaitAttributeChangeComparison
{
	enum Type : int
	{
		None,
		GreaterThan,
		LessThan,
		GreaterThanOrEqualTo,
		LessThanOrEqualTo,
		NotEqualTo,
		ExactlyEqualTo,
		MAX UMETA(Hidden)
	};
}

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FWaitAttributeChangeDelegate);

/**
 *	Waits for the actor to activate another ability
 */
UCLASS(MinimalAPI)
class UAbilityTask_WaitAttributeChange : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitAttributeChangeDelegate	OnChange;

	UE_API virtual void Activate() override;

	UE_API void OnAttributeChange(const FOnAttributeChangeData& CallbackData);

	/** Wait until an attribute changes. */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitAttributeChange* WaitForAttributeChange(UGameplayAbility* OwningAbility, FGameplayAttribute Attribute, FGameplayTag WithSrcTag, FGameplayTag WithoutSrcTag, bool TriggerOnce=true, AActor* OptionalExternalOwner = nullptr);

	/** Wait until an attribute changes to pass a given test. */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitAttributeChange* WaitForAttributeChangeWithComparison(UGameplayAbility* OwningAbility, FGameplayAttribute InAttribute, FGameplayTag InWithTag, FGameplayTag InWithoutTag, TEnumAsByte<EWaitAttributeChangeComparison::Type> InComparisonType, float InComparisonValue, bool TriggerOnce=true, AActor* OptionalExternalOwner = nullptr);

	FGameplayTag WithTag;
	FGameplayTag WithoutTag;
	FGameplayAttribute	Attribute;
	TEnumAsByte<EWaitAttributeChangeComparison::Type> ComparisonType;
	float ComparisonValue;
	bool bTriggerOnce;
	FDelegateHandle OnAttributeChangeDelegateHandle;

protected:

	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> ExternalOwner;

	UE_API UAbilitySystemComponent* GetFocusedASC();

	UE_API virtual void OnDestroy(bool AbilityEnded) override;
};

#undef UE_API
