// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "AttributeSet.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "Abilities/Tasks/AbilityTask_WaitAttributeChange.h"
#include "AbilityTask_WaitAttributeChangeRatioThreshold.generated.h"

#define UE_API GAMEPLAYABILITIES_API

struct FGameplayEffectModCallbackData;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FWaitAttributeChangeRatioThresholdDelegate, bool, bMatchesComparison, float, CurrentRatio);

/**
 *	Waits for the ratio between two attributes to match a threshold
 */
UCLASS(MinimalAPI)
class UAbilityTask_WaitAttributeChangeRatioThreshold : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitAttributeChangeRatioThresholdDelegate OnChange;

	UE_API virtual void Activate() override;

	UE_API void OnNumeratorAttributeChange(const FOnAttributeChangeData& CallbackData);
	UE_API void OnDenominatorAttributeChange(const FOnAttributeChangeData& CallbackData);

	/** Wait on attribute ratio change meeting a comparison threshold. */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitAttributeChangeRatioThreshold* WaitForAttributeChangeRatioThreshold(UGameplayAbility* OwningAbility, FGameplayAttribute AttributeNumerator, FGameplayAttribute AttributeDenominator, TEnumAsByte<EWaitAttributeChangeComparison::Type> ComparisonType, float ComparisonValue, bool bTriggerOnce, AActor* OptionalExternalOwner = nullptr);

	FGameplayAttribute AttributeNumerator;
	FGameplayAttribute AttributeDenominator;
	TEnumAsByte<EWaitAttributeChangeComparison::Type> ComparisonType;
	float ComparisonValue;
	bool bTriggerOnce;
	FDelegateHandle OnNumeratorAttributeChangeDelegateHandle;
	FDelegateHandle OnDenominatorAttributeChangeDelegateHandle;

protected:

	float LastAttributeNumeratorValue;
	float LastAttributeDenominatorValue;
	bool bMatchedComparisonLastAttributeChange;

	/** Timer used when either numerator or denominator attribute is changed to delay checking of ratio to avoid false positives (MaxHealth changed before Health updates accordingly) */
	FTimerHandle CheckAttributeTimer;

	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> ExternalOwner;

	UE_API UAbilitySystemComponent* GetFocusedASC();

	UE_API void OnAttributeChange();
	UE_API void OnRatioChange();

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	UE_API bool DoesValuePassComparison(float ValueNumerator, float ValueDenominator) const;
};

#undef UE_API
