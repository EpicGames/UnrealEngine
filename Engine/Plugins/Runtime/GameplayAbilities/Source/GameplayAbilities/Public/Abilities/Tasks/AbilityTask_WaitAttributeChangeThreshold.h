// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AttributeSet.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "Abilities/Tasks/AbilityTask_WaitAttributeChange.h"
#include "AbilityTask_WaitAttributeChangeThreshold.generated.h"

#define UE_API GAMEPLAYABILITIES_API

struct FGameplayEffectModCallbackData;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FWaitAttributeChangeThresholdDelegate, bool, bMatchesComparison, float, CurrentValue);

/**
 *	Waits for an attribute to match a threshold
 */
UCLASS(MinimalAPI)
class UAbilityTask_WaitAttributeChangeThreshold : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitAttributeChangeThresholdDelegate OnChange;

	UE_API virtual void Activate() override;

	UE_API void OnAttributeChange(const FOnAttributeChangeData& CallbackData);

	/** Wait on attribute change meeting a comparison threshold. */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitAttributeChangeThreshold* WaitForAttributeChangeThreshold(UGameplayAbility* OwningAbility, FGameplayAttribute Attribute, TEnumAsByte<EWaitAttributeChangeComparison::Type> ComparisonType, float ComparisonValue, bool bTriggerOnce, AActor* OptionalExternalOwner = nullptr);

	FGameplayAttribute Attribute;
	TEnumAsByte<EWaitAttributeChangeComparison::Type> ComparisonType;
	float ComparisonValue;
	bool bTriggerOnce;
	FDelegateHandle OnAttributeChangeDelegateHandle;

protected:

	bool bMatchedComparisonLastAttributeChange;

	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> ExternalOwner;

	UE_API UAbilitySystemComponent* GetFocusedASC();

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	UE_API bool DoesValuePassComparison(float Value) const;
};

#undef UE_API
