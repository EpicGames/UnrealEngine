// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitCancel.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FWaitCancelDelegate);

UCLASS(MinimalAPI)
class UAbilityTask_WaitCancel : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitCancelDelegate	OnCancel;
	
	UFUNCTION()
	UE_API void OnCancelCallback();

	UFUNCTION()
	UE_API void OnLocalCancelCallback();

	UFUNCTION(BlueprintCallable, meta=(HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true", DisplayName="Wait for Cancel Input"), Category="Ability|Tasks")
	static UE_API UAbilityTask_WaitCancel* WaitCancel(UGameplayAbility* OwningAbility);	

	UE_API virtual void Activate() override;

protected:

	UE_API virtual void OnDestroy(bool AbilityEnding) override;

	bool RegisteredCallbacks;
	
};

#undef UE_API
