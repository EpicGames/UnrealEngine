// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitConfirm.generated.h"

#define UE_API GAMEPLAYABILITIES_API

UCLASS(MinimalAPI)
class UAbilityTask_WaitConfirm : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FGenericGameplayTaskDelegate	OnConfirm;

	UFUNCTION()
	UE_API void OnConfirmCallback(UGameplayAbility* InAbility);

	UE_API virtual void Activate() override;

	/** Wait until the server confirms the use of this ability. This is used to gate predictive portions of the ability */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitConfirm* WaitConfirm(UGameplayAbility* OwningAbility);

protected:

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	bool RegisteredCallback;

	FDelegateHandle OnConfirmCallbackDelegateHandle;
};

#undef UE_API
