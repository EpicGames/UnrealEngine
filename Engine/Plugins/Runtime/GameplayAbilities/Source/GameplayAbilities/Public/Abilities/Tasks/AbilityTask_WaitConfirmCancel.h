// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitConfirmCancel.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FWaitConfirmCancelDelegate);

// Fixme: this name is conflicting with AbilityTask_WaitConfirm
// UAbilityTask_WaitConfirmCancel = Wait for Targeting confirm/cancel
// UAbilityTask_WaitConfirm = Wait for server to confirm ability activation

UCLASS(MinimalAPI)
class UAbilityTask_WaitConfirmCancel : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitConfirmCancelDelegate	OnConfirm;

	UPROPERTY(BlueprintAssignable)
	FWaitConfirmCancelDelegate	OnCancel;
	
	UFUNCTION()
	UE_API void OnConfirmCallback();

	UFUNCTION()
	UE_API void OnCancelCallback();

	UFUNCTION()
	UE_API void OnLocalConfirmCallback();

	UFUNCTION()
	UE_API void OnLocalCancelCallback();

	UFUNCTION(BlueprintCallable, meta=(HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true", DisplayName="Wait for Confirm Input"), Category="Ability|Tasks")
	static UE_API UAbilityTask_WaitConfirmCancel* WaitConfirmCancel(UGameplayAbility* OwningAbility);	

	UE_API virtual void Activate() override;

protected:

	UE_API virtual void OnDestroy(bool AbilityEnding) override;

	bool RegisteredCallbacks;
	
};

#undef UE_API
