// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitDelay.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FWaitDelayDelegate);

UCLASS(MinimalAPI)
class UAbilityTask_WaitDelay : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitDelayDelegate	OnFinish;

	UE_API virtual void Activate() override;

	/** Return debug string describing task */
	UE_API virtual FString GetDebugString() const override;

	/** Wait specified time. This is functionally the same as a standard Delay node. */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitDelay* WaitDelay(UGameplayAbility* OwningAbility, float Time);

private:

	void OnTimeFinish();

	float Time;
	float TimeStarted;
};

#undef UE_API
