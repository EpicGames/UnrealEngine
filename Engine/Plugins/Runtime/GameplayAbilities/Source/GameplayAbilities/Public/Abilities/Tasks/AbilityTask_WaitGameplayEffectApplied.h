// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayEffectTypes.h"
#include "Abilities/GameplayAbilityTargetDataFilter.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitGameplayEffectApplied.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

UCLASS(MinimalAPI)
class UAbilityTask_WaitGameplayEffectApplied : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UFUNCTION()
	UE_API void OnApplyGameplayEffectCallback(UAbilitySystemComponent* Target, const FGameplayEffectSpec& SpecApplied, FActiveGameplayEffectHandle ActiveHandle);

	UE_API virtual void Activate() override;

	// Actor filter for the source actor applying this effect that must be passed to trigger this task
	FGameplayTargetDataFilterHandle Filter;
	// Tag requirements for source actor that applies the GameplayEffect
	FGameplayTagRequirements SourceTagRequirements;
	// Tag requirements for target actor to which the GameplayEffect is applied
	FGameplayTagRequirements TargetTagRequirements;
	// Tag requirements for asset tags ('Tags this Effect Has') on the GameplayEffect being applied
	FGameplayTagRequirements AssetTagRequirements;
	// Tag requirements for granted tags ('Grant Tags to Target Actor') that the GameplayEffect applies to the target actor
	FGameplayTagRequirements GrantedTagRequirements;

	// Tag query that must be passed for source actor that applies the GameplayEffect
	FGameplayTagQuery SourceTagQuery;
	// Tag query that must be passed for target actor to which the GameplayEffect is applied
	FGameplayTagQuery TargetTagQuery;
	// Tag query that must be passed for asset tags ('Tags this Effect Has') on the GameplayEffect being applied
	FGameplayTagQuery AssetTagQuery;
	// Tag query that must be passed for granted tags ('Grant Tags to Target Actor') that the GameplayEffect applies to the target actor
	FGameplayTagQuery GrantedTagQuery;

	bool TriggerOnce;
	bool ListenForPeriodicEffects;

	UE_API void SetExternalActor(AActor* InActor);

protected:

	UE_API UAbilitySystemComponent* GetASC();

	virtual void BroadcastDelegate(AActor* Avatar, FGameplayEffectSpecHandle SpecHandle, FActiveGameplayEffectHandle ActiveHandle) { }
	virtual void RegisterDelegate() { }
	virtual void RemoveDelegate() { }

	FString GetBroadcastingEffectStackString() const;

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	bool RegisteredCallback;
	bool UseExternalOwner;

	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> ExternalOwner;

	// If we are in the process of broadcasting, we do not allow a GameplayEffect class to trigger application of another instance of the same class.
	TArray<TSubclassOf<class UGameplayEffect>> BroadcastingEffectStack;
};

#undef UE_API
