// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayEffectTypes.h"
#include "Abilities/GameplayAbilityTargetDataFilter.h"
#include "Abilities/Tasks/AbilityTask_WaitGameplayEffectApplied.h"
#include "AbilityTask_WaitGameplayEffectApplied_Target.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FGameplayEffectAppliedTargetDelegate, AActor*, Target, FGameplayEffectSpecHandle, SpecHandle,  FActiveGameplayEffectHandle, ActiveHandle );

UCLASS(MinimalAPI)
class UAbilityTask_WaitGameplayEffectApplied_Target : public UAbilityTask_WaitGameplayEffectApplied
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FGameplayEffectAppliedTargetDelegate OnApplied;

	/** 
	 * Wait until the owner (or External Owner) applies a GameplayEffect to a Target (the target may be the owner too!).
	 * 
	 * @param OwningAbility The ability that executes this task
	 * @param SourceFilter Actor filter for the source actor applying this effect that must be passed to trigger this task
	 * @param SourceTagRequirements Gameplay tag requirements for the source actor applying this effect
	 * @param TargetTagRequirements Gameplay tag requirements for the target actor receiving this effect
	 * @param AssetTagRequirements Requirements for the asset tags of the applied effect: 'Tags this Effect Has' on the GameplayEffect class.
	 * @param GrantedTagRequirements Requirements for the tags granted by the applied effect: 'Grant Tags to Target Actor' on the GameplayEffect class.
	 * @param TriggerOnce If TriggerOnce is true, this task will only return one time. Otherwise it will return everytime a GE is applied that meets the requirements over the life of the ability.
	 * @param OptionalExternalOwner Can be used to run this task on someone else (not the owner of the ability). By default you can leave this empty.
	 * @param ListenForPeriodicEffects Whether to also fire on periodic ticks of the applied effect
	 */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitGameplayEffectApplied_Target* WaitGameplayEffectAppliedToTarget(UGameplayAbility* OwningAbility, const FGameplayTargetDataFilterHandle TargetFilter, FGameplayTagRequirements SourceTagRequirements, FGameplayTagRequirements TargetTagRequirements, FGameplayTagRequirements AssetTagRequirements, FGameplayTagRequirements GrantedTagRequirements, bool TriggerOnce=false, AActor* OptionalExternalOwner=nullptr, bool ListenForPeriodicEffects=false);

	/** 
	 * Wait until the owner (or External Owner) applies a GameplayEffect to a Target (the target may be the owner too!).
	 * This version uses FGameplayTagQuery (more power) instead of FGameplayTagRequirements (faster).
	 * 
	 * @param OwningAbility The ability that executes this task
	 * @param SourceFilter Actor filter for the source actor applying this effect that must be passed to trigger this task
	 * @param SourceTagQuery Gameplay tag requirements for the source actor applying this effect
	 * @param TargetTagQuery Gameplay tag requirements for the target actor receiving this effect
	 * @param AssetTagQuery Requirements for the asset tags of the applied effect: 'Tags this Effect Has' on the GameplayEffect class
	 * @param GrantedTagQuery Requirements for the tags granted by the applied effect: 'Grant Tags to Target Actor' on the GameplayEffect class
	 * @param TriggerOnce If TriggerOnce is true, this task will only return one time. Otherwise it will return everytime a GE is applied that meets the requirements over the life of the ability.
	 * @param OptionalExternalOwner Can be used to run this task on someone else (not the owner of the ability). By default you can leave this empty.
	 * @param ListenForPeriodicEffects Whether to also fire on periodic ticks of the applied effect
	 */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitGameplayEffectApplied_Target* WaitGameplayEffectAppliedToTarget_Query(UGameplayAbility* OwningAbility, const FGameplayTargetDataFilterHandle SourceFilter, FGameplayTagQuery SourceTagQuery, FGameplayTagQuery TargetTagQuery, FGameplayTagQuery AssetTagQuery, FGameplayTagQuery GrantedTagQuery, bool TriggerOnce=false, AActor* OptionalExternalOwner=nullptr, bool ListenForPeriodicEffect=false );
	
protected:

	UE_API virtual void BroadcastDelegate(AActor* Avatar, FGameplayEffectSpecHandle SpecHandle, FActiveGameplayEffectHandle ActiveHandle) override;
	UE_API virtual void RegisterDelegate() override;
	UE_API virtual void RemoveDelegate() override;

private:
	FDelegateHandle OnApplyGameplayEffectCallbackDelegateHandle;
	FDelegateHandle OnPeriodicGameplayEffectExecuteCallbackDelegateHandle;
};

#undef UE_API
