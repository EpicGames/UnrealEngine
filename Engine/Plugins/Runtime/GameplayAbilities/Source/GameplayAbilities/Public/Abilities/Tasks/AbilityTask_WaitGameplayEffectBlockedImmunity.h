// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayEffectTypes.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitGameplayEffectBlockedImmunity.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FGameplayEffectBlockedDelegate, FGameplayEffectSpecHandle, BlockedSpec, FActiveGameplayEffectHandle, ImmunityGameplayEffectHandle);

UCLASS(MinimalAPI)
class UAbilityTask_WaitGameplayEffectBlockedImmunity : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FGameplayEffectBlockedDelegate Blocked;

	/** Listens for GE immunity. By default this means "this hero blocked a GE due to immunity". Setting OptionalExternalTarget will listen for a GE being blocked on an external target. Note this only works on the server. */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitGameplayEffectBlockedImmunity* WaitGameplayEffectBlockedByImmunity(UGameplayAbility* OwningAbility, FGameplayTagRequirements SourceTagRequirements, FGameplayTagRequirements TargetTagRequirements, AActor* OptionalExternalTarget=nullptr, bool OnlyTriggerOnce=false);

	UE_API virtual void Activate() override;

	UE_API virtual void ImmunityCallback(const FGameplayEffectSpec& BlockedSpec, const FActiveGameplayEffect* ImmunityGE);
	
	FGameplayTagRequirements SourceTagRequirements;
	FGameplayTagRequirements TargetTagRequirements;

	bool TriggerOnce;
	bool ListenForPeriodicEffects;

	UE_API void SetExternalActor(AActor* InActor);

protected:

	UE_API UAbilitySystemComponent* GetASC();
		
	UE_API void RegisterDelegate();
	UE_API void RemoveDelegate();

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	bool RegisteredCallback;
	bool UseExternalOwner;

	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> ExternalOwner;

	FDelegateHandle DelegateHandle;
};

#undef UE_API
