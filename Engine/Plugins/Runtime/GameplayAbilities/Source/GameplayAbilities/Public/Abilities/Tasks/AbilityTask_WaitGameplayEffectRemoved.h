// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayEffectTypes.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitGameplayEffectRemoved.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FWaitGameplayEffectRemovedDelegate, const FGameplayEffectRemovalInfo &, GameplayEffectRemovalInfo);

class AActor;
class UPrimitiveComponent;

/**
 *	Waits for the actor to activate another ability
 */
UCLASS(MinimalAPI)
class UAbilityTask_WaitGameplayEffectRemoved : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitGameplayEffectRemovedDelegate	OnRemoved;

	UPROPERTY(BlueprintAssignable)
	FWaitGameplayEffectRemovedDelegate	InvalidHandle;

	UE_API virtual void Activate() override;

	UFUNCTION()
	UE_API void OnGameplayEffectRemoved(const FGameplayEffectRemovalInfo& InGameplayEffectRemovalInfo);

	/** Wait until the specified gameplay effect is removed. */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitGameplayEffectRemoved* WaitForGameplayEffectRemoved(UGameplayAbility* OwningAbility, FActiveGameplayEffectHandle Handle);

	FActiveGameplayEffectHandle Handle;

protected:

	UE_API virtual void OnDestroy(bool AbilityIsEnding) override;
	bool Registered;

	FDelegateHandle OnGameplayEffectRemovedDelegateHandle;
	FDelegateHandle OnGameplayEffectRemovedDelegateHandle_Deprecated;
};

#undef UE_API
