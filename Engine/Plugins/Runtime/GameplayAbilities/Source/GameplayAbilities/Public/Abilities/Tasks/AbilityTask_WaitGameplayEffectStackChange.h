// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayEffectTypes.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitGameplayEffectStackChange.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FWaitGameplayEffectStackChangeDelegate, FActiveGameplayEffectHandle, Handle, int32, NewCount, int32, OldCount);

class AActor;
class UPrimitiveComponent;

/**
 *	Waits for the actor to activate another ability
 */
UCLASS(MinimalAPI)
class UAbilityTask_WaitGameplayEffectStackChange : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitGameplayEffectStackChangeDelegate	OnChange;

	UPROPERTY(BlueprintAssignable)
	FWaitGameplayEffectStackChangeDelegate	InvalidHandle;

	UE_API virtual void Activate() override;

	UFUNCTION()
	UE_API void OnGameplayEffectStackChange(FActiveGameplayEffectHandle Handle, int32 NewCount, int32 OldCount);

	/** Wait until the specified gameplay effect is removed. */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitGameplayEffectStackChange* WaitForGameplayEffectStackChange(UGameplayAbility* OwningAbility, FActiveGameplayEffectHandle Handle);

	FActiveGameplayEffectHandle Handle;

protected:

	UE_API virtual void OnDestroy(bool AbilityIsEnding) override;
	bool Registered;

	FDelegateHandle OnGameplayEffectStackChangeDelegateHandle;
};

#undef UE_API
