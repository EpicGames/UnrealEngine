// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTagContainer.h"
#include "Abilities/GameplayAbilityTypes.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitGameplayEvent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FWaitGameplayEventDelegate, FGameplayEventData, Payload);

UCLASS(MinimalAPI)
class UAbilityTask_WaitGameplayEvent : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitGameplayEventDelegate	EventReceived;

	/**
	 * Wait until the specified gameplay tag event is triggered. By default this will look at the owner of this ability. OptionalExternalTarget can be set to make this look at another actor's tags for changes
	 * It will keep listening as long as OnlyTriggerOnce = false
	 * If OnlyMatchExact = false it will trigger for nested tags
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitGameplayEvent* WaitGameplayEvent(UGameplayAbility* OwningAbility, UPARAM(meta=(GameplayTagFilter="GameplayEventTagsCategory"))FGameplayTag EventTag, AActor* OptionalExternalTarget=nullptr, bool OnlyTriggerOnce=false, bool OnlyMatchExact = true);

	UE_API void SetExternalTarget(AActor* Actor);

	UE_API UAbilitySystemComponent* GetTargetASC();

	UE_API virtual void Activate() override;

	UE_API virtual void GameplayEventCallback(const FGameplayEventData* Payload);
	UE_API virtual void GameplayEventContainerCallback(FGameplayTag MatchingTag, const FGameplayEventData* Payload);

	UE_API void OnDestroy(bool AbilityEnding) override;

	FGameplayTag Tag;

	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> OptionalExternalTarget;

	bool UseExternalTarget;	
	bool OnlyTriggerOnce;
	bool OnlyMatchExact;

	FDelegateHandle MyHandle;
};

#undef UE_API
