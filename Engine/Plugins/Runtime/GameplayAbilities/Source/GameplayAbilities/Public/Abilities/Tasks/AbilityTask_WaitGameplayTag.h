// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTagContainer.h"
#include "Abilities/Tasks/AbilityTask_WaitGameplayTagBase.h"
#include "AbilityTask_WaitGameplayTag.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FWaitGameplayTagDelegate);

UCLASS(MinimalAPI)
class UAbilityTask_WaitGameplayTagAdded : public UAbilityTask_WaitGameplayTag
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitGameplayTagDelegate	Added;

	/**
	 * 	Wait until the specified gameplay tag is Added. By default this will look at the owner of this ability. OptionalExternalTarget can be set to make this look at another actor's tags for changes. 
	 *  If the tag is already present when this task is started, it will immediately broadcast the Added event. It will keep listening as long as OnlyTriggerOnce = false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitGameplayTagAdded* WaitGameplayTagAdd(UGameplayAbility* OwningAbility, FGameplayTag Tag, AActor* InOptionalExternalTarget=nullptr, bool OnlyTriggerOnce=false);

	UE_API virtual void Activate() override;

	UE_API virtual void GameplayTagCallback(const FGameplayTag Tag, int32 NewCount) override;
};

UCLASS(MinimalAPI)
class UAbilityTask_WaitGameplayTagRemoved : public UAbilityTask_WaitGameplayTag
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitGameplayTagDelegate	Removed;

	/**
	 * 	Wait until the specified gameplay tag is Removed. By default this will look at the owner of this ability. OptionalExternalTarget can be set to make this look at another actor's tags for changes. 
	 *  If the tag is not present when this task is started, it will immediately broadcast the Removed event. It will keep listening as long as OnlyTriggerOnce = false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitGameplayTagRemoved* WaitGameplayTagRemove(UGameplayAbility* OwningAbility, FGameplayTag Tag, AActor* InOptionalExternalTarget=nullptr, bool OnlyTriggerOnce=false);

	UE_API virtual void Activate() override;

	UE_API virtual void GameplayTagCallback(const FGameplayTag Tag, int32 NewCount) override;
};

#undef UE_API
