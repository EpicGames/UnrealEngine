// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTagContainer.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitGameplayTagBase.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

UCLASS(MinimalAPI)
class UAbilityTask_WaitGameplayTag : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UE_API virtual void Activate() override;

	UFUNCTION()
	UE_API virtual void GameplayTagCallback(const FGameplayTag Tag, int32 NewCount);
	
	UE_API void SetExternalTarget(AActor* Actor);

	FGameplayTag	Tag;

protected:

	UE_API UAbilitySystemComponent* GetTargetASC();

	UE_API virtual void OnDestroy(bool AbilityIsEnding) override;
	
	bool RegisteredCallback;
	
	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> OptionalExternalTarget;

	bool UseExternalTarget;	
	bool OnlyTriggerOnce;

	FDelegateHandle DelegateHandle;
};

#undef UE_API
