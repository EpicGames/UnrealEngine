// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Abilities/Tasks/AbilityTask.h"
#include "GameplayTagContainer.h"
#include "AbilityTask_WaitGameplayTagCountChanged.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FWaitGameplayTagCountDelegate, int32, TagCount);

UCLASS(MinimalAPI)
class UAbilityTask_WaitGameplayTagCountChanged : public UAbilityTask
{
	GENERATED_BODY()

protected:
	UPROPERTY(BlueprintAssignable)
	FWaitGameplayTagCountDelegate TagCountChanged;

	/**
	 * 	Wait until the specified gameplay tag count has changed. By default this will look at the owner of this ability. OptionalExternalTarget can be set to make this look at another actor's tags for changes. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitGameplayTagCountChanged* WaitGameplayTagCountChange(UGameplayAbility* OwningAbility, FGameplayTag Tag, AActor* InOptionalExternalTarget = nullptr);

	UE_API virtual void Activate() override;
	UE_API virtual void GameplayTagCallback(const FGameplayTag Tag, int32 NewCount);

	FGameplayTag Tag;

	UE_API UAbilitySystemComponent* GetTargetASC();
	UE_API virtual void OnDestroy(bool bAbilityIsEnding) override;

	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> OptionalExternalTarget;

	FDelegateHandle GameplayTagCountChangedHandle;
};

#undef UE_API
