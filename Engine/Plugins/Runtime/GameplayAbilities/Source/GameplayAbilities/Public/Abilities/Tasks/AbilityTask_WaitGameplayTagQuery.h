// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Abilities/Tasks/AbilityTask.h"

#include "AbilityTask_WaitGameplayTagQuery.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FWaitGameplayTagQueryDelegate);

/** This enum defines the condition in which the wait gameplay tag query node will trigger. */
UENUM()
enum class EWaitGameplayTagQueryTriggerCondition : uint8
{
	WhenTrue = 0,
	WhenFalse,
};

/** This class defines a node to wait on a gameplay tag query. */
UCLASS(MinimalAPI)
class UAbilityTask_WaitGameplayTagQuery : public UAbilityTask
{
	GENERATED_BODY()
	
public:
	/** Activates this AbilityTask. */
	UE_API virtual void Activate() override;
	
	/** Sets the external target that this node whould use for checking tags. */
	UE_API void SetExternalTarget(const AActor* Actor);

	/**
	 * 	Wait until the given gameplay tag query has become true or false, based on TriggerCondition. 
	 *  By default this will look at the owner of this ability. OptionalExternalTarget can be set to
	 *  make this look at another actor's tags for changes.  If the the tag query already satisfies 
	 *  the TriggerCondition when this task is started, it will immediately broadcast the Triggered 
	 *  event. It will keep listening as long as bOnlyTriggerOnce = false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitGameplayTagQuery* WaitGameplayTagQuery(UGameplayAbility* OwningAbility, 
																   const FGameplayTagQuery TagQuery, 
																   const AActor* InOptionalExternalTarget = nullptr, 
																   const EWaitGameplayTagQueryTriggerCondition TriggerCondition = EWaitGameplayTagQueryTriggerCondition::WhenTrue, 
																   const bool bOnlyTriggerOnce = false);

protected:

	/** This will update the tags in the TargetTags container to reflect that tags that are on the target ASC. */
	UFUNCTION()
	UE_API void UpdateTargetTags(const FGameplayTag Tag, int32 NewCount);

	/** This will evaluate the TargetTags using the given TagQuery, executing the Trigger delegate if needed. */
	UE_API void EvaluateTagQuery();

	/** This gets the ASC to use to listen to tag changed events. */
	UE_API UAbilitySystemComponent* GetTargetASC();

	/** This will handle cleaning up any registered delegates. */
	UE_API virtual void OnDestroy(bool AbilityIsEnding) override;

	/** This is the tag query to evaluate for triggering this node. */
	FGameplayTagQuery TagQuery;

	/** Indicates if the callbacks for tag changes have been registered. */
	bool bRegisteredCallbacks = false;

	/** This was the last result of evaluating the TagQuery. */
	bool bQueryState = false;

	/** This is the tag container of the targets the target ASC currently has. 
		NOTE: This will only contain the tags referenced in the TagQuery. */
	FGameplayTagContainer TargetTags;

	/** This is the handles to the tag changed delegate for each gameplay tag in the TagQuery. */
	TMap<FGameplayTag, FDelegateHandle> TagHandleMap;

	/** This delegate will be triggered when the trigger condition has been met. */
	UPROPERTY(BlueprintAssignable)
	FWaitGameplayTagQueryDelegate Triggered;
	
	/** This is the optional external target to use when getting the ASC to get tags from. */
	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> OptionalExternalTarget = nullptr;

	/** This indicates when to Trigger the Triggered output for this node. */
	EWaitGameplayTagQueryTriggerCondition TriggerCondition = EWaitGameplayTagQueryTriggerCondition::WhenTrue;

	/** This indicates if we should use the external target. */
	bool bUseExternalTarget = false;	

	/** This indicates if this node should only trigger once, or any number of times. */
	bool bOnlyTriggerOnce = false;
};

#undef UE_API
