// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitInputPress.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FInputPressDelegate, float, TimeWaited);

/**
 *	Waits until the input is pressed from activating an ability. This should be true immediately upon starting the ability, since the key was pressed to activate it.
 *	We expect server to execute this task in parallel and keep its own time. We do not keep track of 
 */
UCLASS(MinimalAPI)
class UAbilityTask_WaitInputPress : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FInputPressDelegate		OnPress;

	UFUNCTION()
	UE_API void OnPressCallback();

	UE_API virtual void Activate() override;

	/** Wait until the user presses the input button for this ability's activation. Returns time this node spent waiting for the press. Will return 0 if input was already down. */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitInputPress* WaitInputPress(UGameplayAbility* OwningAbility, bool bTestAlreadyPressed=false);

protected:

	float StartTime;
	bool bTestInitialState;
	FDelegateHandle DelegateHandle;
};

#undef UE_API
