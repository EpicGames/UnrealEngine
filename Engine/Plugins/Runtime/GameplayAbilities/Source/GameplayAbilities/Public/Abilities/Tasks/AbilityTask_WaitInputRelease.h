// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitInputRelease.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FInputReleaseDelegate, float, TimeHeld);

/**
 *	Waits until the input is released from activating an ability. Clients will replicate a 'release input' event to the server, but not the exact time it was held locally.
 *	We expect server to execute this task in parallel and keep its own time.
 */
UCLASS(MinimalAPI)
class UAbilityTask_WaitInputRelease : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FInputReleaseDelegate	OnRelease;

	UFUNCTION()
	UE_API void OnReleaseCallback();

	UE_API virtual void Activate() override;

	/** Wait until the user releases the input button for this ability's activation. Returns time from hitting this node, till release. Will return 0 if input was already released. */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitInputRelease* WaitInputRelease(UGameplayAbility* OwningAbility, bool bTestAlreadyReleased=false);

protected:

	float StartTime;
	bool bTestInitialState;
	FDelegateHandle DelegateHandle;
};

#undef UE_API
