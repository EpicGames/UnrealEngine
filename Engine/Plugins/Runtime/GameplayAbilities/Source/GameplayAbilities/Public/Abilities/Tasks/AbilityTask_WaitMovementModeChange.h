// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitMovementModeChange.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class ACharacter;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMovementModeChangedDelegate, EMovementMode, NewMovementMode);

class ACharacter;

UCLASS(MinimalAPI)
class UAbilityTask_WaitMovementModeChange : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(BlueprintAssignable)
	FMovementModeChangedDelegate	OnChange;

	UFUNCTION()
	UE_API void OnMovementModeChange(ACharacter * Character, EMovementMode PrevMovementMode, uint8 PreviousCustomMode);

	EMovementMode	RequiredMode;

	/** Wait until movement mode changes (E.g., landing) */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE", DisplayName="WaitMovementModeChange"))
	static UE_API UAbilityTask_WaitMovementModeChange* CreateWaitMovementModeChange(UGameplayAbility* OwningAbility, EMovementMode NewMode);

	UE_API virtual void Activate() override;
private:

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	TWeakObjectPtr<ACharacter>	MyCharacter;
};

#undef UE_API
