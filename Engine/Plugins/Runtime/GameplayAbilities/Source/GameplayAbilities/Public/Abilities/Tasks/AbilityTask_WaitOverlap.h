// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitOverlap.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FWaitOverlapDelegate, const FGameplayAbilityTargetDataHandle&, TargetData);

class AActor;
class UPrimitiveComponent;

/**
 *	Fixme: this is still incomplete and probablyh not what most games want for melee systems.
 *		-Only actually activates on Blocking hits
 *		-Uses first PrimitiveComponent instead of being able to specify arbitrary component.
 */
UCLASS(MinimalAPI)
class UAbilityTask_WaitOverlap : public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitOverlapDelegate	OnOverlap;

	UFUNCTION()
	UE_API void OnHitCallback(UPrimitiveComponent* HitComp, AActor* OtherActor, UPrimitiveComponent* OtherComp, FVector NormalImpulse, const FHitResult& Hit);

	UE_API virtual void Activate() override;

	/** Wait until an overlap occurs. This will need to be better fleshed out so we can specify game specific collision requirements */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitOverlap* WaitForOverlap(UGameplayAbility* OwningAbility);

private:

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	UPrimitiveComponent* GetComponent();
	
};

#undef UE_API
