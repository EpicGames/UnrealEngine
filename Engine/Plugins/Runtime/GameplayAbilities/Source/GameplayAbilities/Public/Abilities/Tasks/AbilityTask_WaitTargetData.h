// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameplayTagContainer.h"
#include "Abilities/GameplayAbilityTargetActor.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitTargetData.generated.h"

#define UE_API GAMEPLAYABILITIES_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FWaitTargetDataDelegate, const FGameplayAbilityTargetDataHandle&, Data);

/**
 * Wait for targeting actor (spawned from parameter) to provide data. Can be set not to end upon outputting data. Can be ended by task name.
 *
 * WARNING: These actors are spawned once per ability activation and in their default form are not very efficient
 * For most games you will need to subclass and heavily modify this actor, or you will want to implement similar functions in a game-specific actor or blueprint to avoid actor spawn costs
 * This task is not well tested by internal games, but it is a useful class to look at to learn how target replication occurs
 */
UCLASS(MinimalAPI)
class UAbilityTask_WaitTargetData: public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FWaitTargetDataDelegate	ValidData;

	UPROPERTY(BlueprintAssignable)
	FWaitTargetDataDelegate	Cancelled;

	UFUNCTION()
	UE_API virtual void OnTargetDataReplicatedCallback(const FGameplayAbilityTargetDataHandle& Data, FGameplayTag ActivationTag);

	UFUNCTION()
	UE_API virtual void OnTargetDataReplicatedCancelledCallback();

	UFUNCTION()
	UE_API virtual void OnTargetDataReadyCallback(const FGameplayAbilityTargetDataHandle& Data);

	UFUNCTION()
	UE_API virtual void OnTargetDataCancelledCallback(const FGameplayAbilityTargetDataHandle& Data);

	/** Spawns target actor and waits for it to return valid data or to be canceled. */
	UFUNCTION(BlueprintCallable, meta=(HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true", HideSpawnParms="Instigator"), Category="Ability|Tasks")
	static UE_API UAbilityTask_WaitTargetData* WaitTargetData(UGameplayAbility* OwningAbility, FName TaskInstanceName, TEnumAsByte<EGameplayTargetingConfirmation::Type> ConfirmationType, TSubclassOf<AGameplayAbilityTargetActor> Class);

	/** Uses specified target actor and waits for it to return valid data or to be canceled. */
	UFUNCTION(BlueprintCallable, meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true", HideSpawnParms = "Instigator"), Category = "Ability|Tasks")
	static UE_API UAbilityTask_WaitTargetData* WaitTargetDataUsingActor(UGameplayAbility* OwningAbility, FName TaskInstanceName, TEnumAsByte<EGameplayTargetingConfirmation::Type> ConfirmationType, AGameplayAbilityTargetActor* TargetActor);

	UE_API virtual void Activate() override;

	UFUNCTION(BlueprintCallable, meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true"), Category = "Abilities")
	UE_API virtual bool BeginSpawningActor(UGameplayAbility* OwningAbility, TSubclassOf<AGameplayAbilityTargetActor> Class, AGameplayAbilityTargetActor*& SpawnedActor);

	UFUNCTION(BlueprintCallable, meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "true"), Category = "Abilities")
	UE_API virtual void FinishSpawningActor(UGameplayAbility* OwningAbility, AGameplayAbilityTargetActor* SpawnedActor);

	/** Called when the ability is asked to confirm from an outside node. What this means depends on the individual task. By default, this does nothing other than ending if bEndTask is true. */
	UE_API virtual void ExternalConfirm(bool bEndTask) override;

	/** Called when the ability is asked to cancel from an outside node. What this means depends on the individual task. By default, this does nothing other than ending the task. */
	UE_API virtual void ExternalCancel() override;

protected:

	UE_API virtual bool ShouldSpawnTargetActor() const;
	UE_API virtual void InitializeTargetActor(AGameplayAbilityTargetActor* SpawnedActor) const;
	UE_API virtual void FinalizeTargetActor(AGameplayAbilityTargetActor* SpawnedActor) const;

	UE_API virtual void RegisterTargetDataCallbacks();

	UE_API virtual void OnDestroy(bool AbilityEnded) override;

	UE_API virtual bool ShouldReplicateDataToServer() const;

protected:

	UPROPERTY()
	TSubclassOf<AGameplayAbilityTargetActor> TargetClass;

	/** The TargetActor that we spawned */
	UPROPERTY()
	TObjectPtr<AGameplayAbilityTargetActor> TargetActor;

	TEnumAsByte<EGameplayTargetingConfirmation::Type> ConfirmationType;

	FDelegateHandle OnTargetDataReplicatedCallbackDelegateHandle;
};


/**
*	Requirements for using Begin/Finish SpawningActor functionality:
*		-Have a parameters named 'Class' in your Proxy factor function (E.g., WaitTargetdata)
*		-Have a function named BeginSpawningActor w/ the same Class parameter
*			-This function should spawn the actor with SpawnActorDeferred and return true/false if it spawned something.
*		-Have a function named FinishSpawningActor w/ an AActor* of the class you spawned
*			-This function *must* call ExecuteConstruction + PostActorConstruction
*/

#undef UE_API
