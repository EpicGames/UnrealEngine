// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "AbilityTask_WaitVelocityChange.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UMovementComponent;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FWaitVelocityChangeDelegate);

UCLASS(MinimalAPI)
class UAbilityTask_WaitVelocityChange: public UAbilityTask
{
	GENERATED_UCLASS_BODY()

	/** Delegate called when velocity requirements are met */
	UPROPERTY(BlueprintAssignable)
	FWaitVelocityChangeDelegate OnVelocityChage;

	UE_API virtual void TickTask(float DeltaTime) override;

	/** Wait for the actor's movement component velocity to be of minimum magnitude when projected along given direction */
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (DisplayName="WaitVelocityChange",HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_WaitVelocityChange* CreateWaitVelocityChange(UGameplayAbility* OwningAbility, FVector Direction, float MinimumMagnitude);
		
	UE_API virtual void Activate() override;

protected:

	UPROPERTY()
	TWeakObjectPtr<UMovementComponent>	CachedMovementComponent;

	float	MinimumMagnitude;
	FVector Direction;
};

#undef UE_API
