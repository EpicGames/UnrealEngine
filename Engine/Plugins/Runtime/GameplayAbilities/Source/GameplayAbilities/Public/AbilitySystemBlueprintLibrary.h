// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "UObject/ScriptInterface.h"
#include "GameplayTagContainer.h"
#include "AttributeSet.h"
#include "GameplayEffectTypes.h"
#include "Abilities/GameplayAbilityTargetTypes.h"
#include "GameplayCueInterface.h"
#include "Abilities/GameplayAbilityTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Abilities/GameplayAbilityTargetDataFilter.h"
#include "ScalableFloat.h"
#include "AbilitySystemBlueprintLibrary.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;
class UGameplayEffect;
class UGameplayEffectUIData;

/** Called when a gameplay tag bound to a event wrapper via one of the BindEventWrapper<to gameplay tag(s)> methods on the AbilitySystemLibrary changes. */
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnGameplayTagChangedEventWrapperSignature, const FGameplayTag&, Tag, int32, TagCount);

/** Holds tracking data for gameplay tag change event wrappers that have been bound. */
struct FGameplayTagChangedEventWrapperSpec
{
	FGameplayTagChangedEventWrapperSpec(
		UAbilitySystemComponent* AbilitySystemComponent,
		FOnGameplayTagChangedEventWrapperSignature InGameplayTagChangedEventWrapperDelegate, 
		EGameplayTagEventType::Type InTagListeningPolicy);
	~FGameplayTagChangedEventWrapperSpec();

	/** The AbilitySystemComponent this spec is bound to. */
	TWeakObjectPtr<UAbilitySystemComponent> AbilitySystemComponentWk;

	/** The event wrapper delegate cached off, to be executed when the gameplaytag(s) we care about are changed. */
	FOnGameplayTagChangedEventWrapperSignature GameplayTagChangedEventWrapperDelegate;

	/** Specifies what kinds of gameplay tag changes we will execute for. */
	EGameplayTagEventType::Type TagListeningPolicy;

	/** Map of the respective gameplay tags to the delegate handle the ASC gave us to use for unbinding later. */
	TMap<FGameplayTag, FDelegateHandle> DelegateBindings;
};

/** Handle to a event wrapper listening for gameplay tag change(s) via one of the BindEventWrapper<to gameplay tag(s)> methods on the AbilitySystemLibrary */
USTRUCT(BlueprintType)
struct FGameplayTagChangedEventWrapperSpecHandle
{
	GENERATED_BODY()

	FGameplayTagChangedEventWrapperSpecHandle();
	FGameplayTagChangedEventWrapperSpecHandle(FGameplayTagChangedEventWrapperSpec* DataPtr);

	/** Internal pointer to binding spec */
	TSharedPtr<FGameplayTagChangedEventWrapperSpec>	Data;

	bool operator==(FGameplayTagChangedEventWrapperSpecHandle const& Other) const;
	bool operator!=(FGameplayTagChangedEventWrapperSpecHandle const& Other) const;
};

/** Blueprint library for ability system. Many of these functions are useful to call from native as well */
UCLASS(meta=(ScriptName="AbilitySystemLibrary"), MinimalAPI)
class UAbilitySystemBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/** Tries to find an ability system component on the actor, will use AbilitySystemInterface or fall back to a component search */
	UFUNCTION(BlueprintPure, Category = Ability, Meta=(DefaultToSelf = "Actor"))
	static UE_API UAbilitySystemComponent* GetAbilitySystemComponent(AActor* Actor);

	/**
	 * This function can be used to trigger an ability on the actor in question with useful payload data.
	 * NOTE: GetAbilitySystemComponent is called on the actor to find a good component, and if the component isn't
	 * found, the event will not be sent.
	 */
	UFUNCTION(BlueprintCallable, Category = Ability, Meta = (Tooltip = "This function can be used to trigger an ability on the actor in question with useful payload data."))
	static UE_API void SendGameplayEventToActor(AActor* Actor, UPARAM(meta=(GameplayTagFilter="GameplayEventTagsCategory")) FGameplayTag EventTag, FGameplayEventData Payload);
	
	// -------------------------------------------------------------------------------
	//		Gameplay Tag
	// -------------------------------------------------------------------------------

	/**
	 * Binds to changes in the given Tag on the given ASC's owned tags.
	 * Cache off the returned handle and call one of the 'Unbind'...'EventWrapper' fns and pass in the handle when you are finished with the binding.
	 * @param Tag                               FGameplayTag to listen for changes on
	 * @param BindingObject                     UObject owning the Event (usually self)
	 * @param GameplayTagChangedEventWrapperDelegate Tag Changed Event to trigger
	 * @param bExecuteImmediatelyIfTagApplied   If true, the bound event will immediately execute if we already have the tag.
	 * @param TagListeningPolicy                Whether we are listening for any tag change or just applied/removed.
	 * @return                                  FGameplayTagChangedEventWrapperSpecHandle Handle by which this binding request can be unbound.
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay Tags", meta = (AdvancedDisplay = "bExecuteImmediatelyIfTagApplied, TagListeningPolicy"))
	static UE_API FGameplayTagChangedEventWrapperSpecHandle BindEventWrapperToGameplayTagChanged(
		UAbilitySystemComponent* AbilitySystemComponent,
		FGameplayTag Tag, 
		FOnGameplayTagChangedEventWrapperSignature GameplayTagChangedEventWrapperDelegate, 
		bool bExecuteImmediatelyIfTagApplied = true,
		EGameplayTagEventType::Type TagListeningPolicy = EGameplayTagEventType::NewOrRemoved);

	/**
	 * Binds to changes in the given Tags on the given ASC's owned tags.
	 * Cache off the returned handle and call one of the 'Unbind'...'EventWrapper' fns and pass in the handle when you are finished with the binding.
	 * @param Tags                              TArray of FGameplayTag to listen for changes on
	 * @param BindingObject                     UObject owning the Event (usually self)
	 * @param GameplayTagChangedEventWrapperDelegate Tag Changed Event to trigger
	 * @param bExecuteImmediatelyIfTagApplied   If true, the bound event will immediately execute if we already have the tag.
	 * @param TagListeningPolicy                Whether we are listening for any tag change or just applied/removed.
	 * @return                                  FGameplayTagChangedEventWrapperSpecHandle Handle by which this binding request can be unbound.
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay Tags", meta = (AdvancedDisplay = "bExecuteImmediatelyIfTagApplied, TagListeningPolicy"))
	static UE_API FGameplayTagChangedEventWrapperSpecHandle BindEventWrapperToAnyOfGameplayTagsChanged(
		UAbilitySystemComponent* AbilitySystemComponent,
		const TArray<FGameplayTag>& Tags,
		FOnGameplayTagChangedEventWrapperSignature GameplayTagChangedEventWrapperDelegate, 
		bool bExecuteImmediatelyIfTagApplied = true,
		EGameplayTagEventType::Type TagListeningPolicy = EGameplayTagEventType::NewOrRemoved);

	/**
	 * Binds to changes in the given TagContainer on the given ASC's owned tags.
	 * Cache off the returned handle and call one of the 'Unbind'...'EventWrapper' fns and pass in the handle when you are finished with the binding.
	 * @param TagContainer                      FGameplayTagContainer of tags to listen for changes on
	 * @param BindingObject                     UObject owning the Event (usually self)
	 * @param GameplayTagChangedEventWrapperDelegate Tag Changed Event to trigger
	 * @param bExecuteImmediatelyIfTagApplied   If true, the bound event will immediately execute if we already have the tag.
	 * @param TagListeningPolicy                Whether we are listening for any tag change or just applied/removed.
	 * @return                                  FGameplayTagChangedEventWrapperSpecHandle Handle by which this binding request can be unbound.
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay Tags", meta = (AdvancedDisplay = "bExecuteImmediatelyIfTagApplied, TagListeningPolicy"))
	static UE_API FGameplayTagChangedEventWrapperSpecHandle BindEventWrapperToAnyOfGameplayTagContainerChanged(
		UAbilitySystemComponent* AbilitySystemComponent,
		const FGameplayTagContainer TagContainer, 
		FOnGameplayTagChangedEventWrapperSignature GameplayTagChangedEventWrapperDelegate, 
		bool bExecuteImmediatelyIfTagApplied = true,
		EGameplayTagEventType::Type TagListeningPolicy = EGameplayTagEventType::NewOrRemoved);

	/**
	 * Unbinds the event wrapper tag change event bound via a BindEventWrapper<to gameplay tag(s)> method that is tied to the given Handle.
	 * (expected to unbind 1 or none)
	 * @param Handle     FGameplayTagChangedEventWrapperSpecHandle Handle provided when binding to a delegate
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay Tags")
	static UE_API void UnbindAllGameplayTagChangedEventWrappersForHandle(FGameplayTagChangedEventWrapperSpecHandle Handle);

	/**
	 * Unbinds the event wrapper tag change event bound via a BindEventWrapper<to gameplay tag(s)> method that is tied to the given Handle, for the specific tag.
	 *  and were bound to the given Tag.
	 * (expected to unbind 1 or none, only makes sense to call if the original binding was for listening to multiple tags.)
	 * @param Tag        FGameplayTag to unbind from
	 * @param Handle     int Handle provided when binding to a delegate
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay Tags")
	static UE_API void UnbindGameplayTagChangedEventWrapperForHandle(FGameplayTag Tag, FGameplayTagChangedEventWrapperSpecHandle Handle);

protected:

	// Helper fn to process gameplay tag changed event wrappers.
	static UE_API void ProcessGameplayTagChangedEventWrapper(const FGameplayTag GameplayTag, int32 GameplayTagCount, FOnGameplayTagChangedEventWrapperSignature GameplayTagChangedEventWrapperDelegate);

public:

	// -------------------------------------------------------------------------------
	//		Attribute
	// -------------------------------------------------------------------------------

	/** Returns true if the attribute actually exists */
	UFUNCTION(BlueprintPure, Category = "Ability|Attribute")
	static UE_API bool IsValid(FGameplayAttribute Attribute);

	/** Returns the value of Attribute from the ability system component belonging to Actor. */
	UFUNCTION(BlueprintPure, Category = "Ability|Attribute")
	static UE_API float GetFloatAttribute(const AActor* Actor, FGameplayAttribute Attribute, bool& bSuccessfullyFoundAttribute);

	/** Returns the value of Attribute from the ability system component AbilitySystem. */
	UFUNCTION(BlueprintPure, Category = "Ability|Attribute")
	static UE_API float GetFloatAttributeFromAbilitySystemComponent(const UAbilitySystemComponent* AbilitySystem, FGameplayAttribute Attribute, bool& bSuccessfullyFoundAttribute);

	/** Returns the base value of Attribute from the ability system component belonging to Actor. */
	UFUNCTION(BlueprintPure, Category = "Ability|Attribute")
	static UE_API float GetFloatAttributeBase(const AActor* Actor, FGameplayAttribute Attribute, bool& bSuccessfullyFoundAttribute);

	/** Returns the base value of Attribute from the ability system component AbilitySystemComponent. */
	UFUNCTION(BlueprintPure, Category = "Ability|Attribute")
	static UE_API float GetFloatAttributeBaseFromAbilitySystemComponent(const UAbilitySystemComponent* AbilitySystemComponent, FGameplayAttribute Attribute, bool& bSuccessfullyFoundAttribute);

	/** Returns the value of Attribute from the ability system component AbilitySystem after evaluating it with source and target tags. bSuccess indicates the success or failure of this operation. */
	UFUNCTION(BlueprintPure, Category = "Ability|Attribute")
	static UE_API float EvaluateAttributeValueWithTags(UAbilitySystemComponent* AbilitySystem, FGameplayAttribute Attribute, const FGameplayTagContainer& SourceTags, const FGameplayTagContainer& TargetTags, bool& bSuccess);

	/** Returns the value of Attribute from the ability system component AbilitySystem after evaluating it with source and target tags using the passed in base value instead of the real base value. bSuccess indicates the success or failure of this operation. */
	UFUNCTION(BlueprintPure, Category = "Ability|Attribute")
	static UE_API float EvaluateAttributeValueWithTagsAndBase(UAbilitySystemComponent* AbilitySystem, FGameplayAttribute Attribute, const FGameplayTagContainer& SourceTags, const FGameplayTagContainer& TargetTags, float BaseValue, bool& bSuccess);

	/** Simple equality operator for gameplay attributes */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (Gameplay Attribute)", CompactNodeTitle = "==", Keywords = "== equal"), Category="Ability|Attribute")
	static UE_API bool EqualEqual_GameplayAttributeGameplayAttribute(FGameplayAttribute AttributeA, FGameplayAttribute AttributeB);

	/** Simple inequality operator for gameplay attributes */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal (Gameplay Attribute)", CompactNodeTitle = "!=", Keywords = "!= not equal"), Category="Ability|Attribute")
	static UE_API bool NotEqual_GameplayAttributeGameplayAttribute(FGameplayAttribute AttributeA, FGameplayAttribute AttributeB);

	/** Returns FString representation of a gameplay attribute's set class and name, in the form of AttrSetName.AttrName (or just AttrName if not part of a set).  */
	UFUNCTION(BlueprintPure, Category="Ability|Attribute")
	static UE_API FString GetDebugStringFromGameplayAttribute(const FGameplayAttribute& Attribute);

	// -------------------------------------------------------------------------------
	//		TargetData
	// -------------------------------------------------------------------------------

	/** Copies targets from HandleToAdd to TargetHandle */
	UFUNCTION(BlueprintCallable, Category = "Ability|TargetData")
	static UE_API FGameplayAbilityTargetDataHandle AppendTargetDataHandle(FGameplayAbilityTargetDataHandle TargetHandle, const FGameplayAbilityTargetDataHandle& HandleToAdd);

	/** Creates a target data with a source and destination location */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API FGameplayAbilityTargetDataHandle	AbilityTargetDataFromLocations(const FGameplayAbilityTargetingLocationInfo& SourceLocation, const FGameplayAbilityTargetingLocationInfo& TargetLocation);

	/** Creates a target data with a single hit result */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API FGameplayAbilityTargetDataHandle	AbilityTargetDataFromHitResult(const FHitResult& HitResult);

	/** Returns number of target data objects, not necessarily number of distinct targets */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API int32 GetDataCountFromTargetData(const FGameplayAbilityTargetDataHandle& TargetData);

	/** Creates single actor target data */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API FGameplayAbilityTargetDataHandle	AbilityTargetDataFromActor(AActor* Actor);

	/** Creates actor array target data */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API FGameplayAbilityTargetDataHandle	AbilityTargetDataFromActorArray(const TArray<AActor*>& ActorArray, bool OneTargetPerHandle);

	/** Create a new target data handle with filtration performed on the data */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API FGameplayAbilityTargetDataHandle	FilterTargetData(const FGameplayAbilityTargetDataHandle& TargetDataHandle, FGameplayTargetDataFilterHandle ActorFilterClass);

	/** Create a handle for filtering target data, filling out all fields */
	UFUNCTION(BlueprintPure, Category = "Filter")
	static UE_API FGameplayTargetDataFilterHandle MakeFilterHandle(FGameplayTargetDataFilter Filter, AActor* FilterActor);

	/** Create a spec handle, filling out all fields */
	UE_DEPRECATED(5.5, "Use MakeSpecHandleByClass. It's safer as InGameplayEffect needs to be a CDO")
	UFUNCTION(BlueprintPure, Category = "Spec", meta=(DeprecatedFunction, DeprecatedMessage="Use Make Spec Handle (By Class)"))
	static UE_API FGameplayEffectSpecHandle MakeSpecHandle(UGameplayEffect* InGameplayEffect, AActor* InInstigator, AActor* InEffectCauser, float InLevel = 1.0f);

	/** Create a spec handle, filling out all fields */
	UFUNCTION(BlueprintCallable, Category = "Spec", meta=(DisplayName="Make Spec Handle (By Class)"))
	static UE_API FGameplayEffectSpecHandle MakeSpecHandleByClass(TSubclassOf<UGameplayEffect> GameplayEffect, AActor* Instigator, AActor* EffectCauser, float Level = 1.0f);

	/** Create a spec handle, cloning another */
	UFUNCTION(BlueprintPure, Category = "Spec")
	static UE_API FGameplayEffectSpecHandle CloneSpecHandle(AActor* InNewInstigator, AActor* InEffectCauser, FGameplayEffectSpecHandle GameplayEffectSpecHandle_Clone);

	/** Returns all actors targeted, for a given index */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API TArray<AActor*> GetActorsFromTargetData(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index);

	/** Returns all actors targeted */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API TArray<AActor*> GetAllActorsFromTargetData(const FGameplayAbilityTargetDataHandle& TargetData);

	/** Returns true if the given TargetData has the actor passed in targeted */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API bool DoesTargetDataContainActor(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index, AActor* Actor);

	/** Returns true if the given TargetData has at least 1 actor targeted */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API bool TargetDataHasActor(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index);

	/** Returns true if the target data has a hit result */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API bool TargetDataHasHitResult(const FGameplayAbilityTargetDataHandle& HitResult, int32 Index);

	/** Returns the hit result for a given index if it exists */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API FHitResult GetHitResultFromTargetData(const FGameplayAbilityTargetDataHandle& HitResult, int32 Index);

	/** Returns true if the target data has an origin */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API bool TargetDataHasOrigin(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index);

	/** Returns the origin for a given index if it exists */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API FTransform GetTargetDataOrigin(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index);

	/** Returns true if the target data has an end point */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API bool TargetDataHasEndPoint(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index);

	/** Returns the end point for a given index if it exists */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API FVector GetTargetDataEndPoint(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index);

	/** Returns the end point transform for a given index if it exists */
	UFUNCTION(BlueprintPure, Category = "Ability|TargetData")
	static UE_API FTransform GetTargetDataEndPointTransform(const FGameplayAbilityTargetDataHandle& TargetData, int32 Index);

	// -------------------------------------------------------------------------------
	//		GameplayEffectContext
	// -------------------------------------------------------------------------------

	/** Returns true if this context has ever been initialized */
	UFUNCTION(BlueprintPure, Category = "Ability|EffectContext", Meta = (DisplayName = "IsValid"))
	static UE_API bool EffectContextIsValid(FGameplayEffectContextHandle EffectContext);

	/** Returns true if the ability system component that instigated this is locally controlled */
	UFUNCTION(BlueprintPure, Category = "Ability|EffectContext", Meta = (DisplayName = "IsInstigatorLocallyControlled"))
	static UE_API bool EffectContextIsInstigatorLocallyControlled(FGameplayEffectContextHandle EffectContext);

	/** Extracts a hit result from the effect context if it is set */
	UFUNCTION(BlueprintPure, Category = "Ability|EffectContext", Meta = (DisplayName = "GetHitResult"))
	static UE_API FHitResult EffectContextGetHitResult(FGameplayEffectContextHandle EffectContext);

	/** Returns true if there is a valid hit result inside the effect context */
	UFUNCTION(BlueprintPure, Category = "Ability|EffectContext", Meta = (DisplayName = "HasHitResult"))
	static UE_API bool EffectContextHasHitResult(FGameplayEffectContextHandle EffectContext);

	/** Adds a hit result to the effect context */
	UFUNCTION(BlueprintCallable, Category = "Ability|EffectContext", Meta = (DisplayName = "AddHitResult"))
	static UE_API void EffectContextAddHitResult(FGameplayEffectContextHandle EffectContext, FHitResult HitResult, bool bReset);

	/** Gets the location the effect originated from */
	UFUNCTION(BlueprintPure, Category = "Ability|EffectContext", Meta = (DisplayName = "GetOrigin"))
	static UE_API FVector EffectContextGetOrigin(FGameplayEffectContextHandle EffectContext);

	/** Sets the location the effect originated from */
	UFUNCTION(BlueprintCallable, Category = "Ability|EffectContext", Meta = (DisplayName = "SetOrigin"))
	static UE_API void EffectContextSetOrigin(FGameplayEffectContextHandle EffectContext, FVector Origin);

	/** Gets the instigating actor (that holds the ability system component) of the EffectContext */
	UFUNCTION(BlueprintPure, Category = "Ability|EffectContext", Meta = (DisplayName = "GetInstigatorActor"))
	static UE_API AActor* EffectContextGetInstigatorActor(FGameplayEffectContextHandle EffectContext);

	/** Gets the original instigator actor that started the chain of events to cause this effect */
	UFUNCTION(BlueprintPure, Category = "Ability|EffectContext", Meta = (DisplayName = "GetOriginalInstigatorActor"))
	static UE_API AActor* EffectContextGetOriginalInstigatorActor(FGameplayEffectContextHandle EffectContext);

	/** Gets the physical actor that caused the effect, possibly a projectile or weapon */
	UFUNCTION(BlueprintPure, Category = "Ability|EffectContext", Meta = (DisplayName = "GetEffectCauser"))
	static UE_API AActor* EffectContextGetEffectCauser(FGameplayEffectContextHandle EffectContext);

	/** Gets the source object of the effect. */
	UFUNCTION(BlueprintPure, Category = "Ability|EffectContext", Meta = (DisplayName = "GetSourceObject"))
	static UE_API UObject* EffectContextGetSourceObject(FGameplayEffectContextHandle EffectContext);

	// -------------------------------------------------------------------------------
	//		GameplayCue
	// -------------------------------------------------------------------------------

	/** Returns true if the ability system component that spawned this cue is locally controlled */
	UFUNCTION(BlueprintPure, Category="Ability|GameplayCue")
	static UE_API bool IsInstigatorLocallyControlled(FGameplayCueParameters Parameters);

	/** Returns true if the ability system component that spawned this cue is locally controlled and a player */
	UFUNCTION(BlueprintPure, Category="Ability|GameplayCue")
	static UE_API bool IsInstigatorLocallyControlledPlayer(FGameplayCueParameters Parameters);

	/** Returns number of actors stored in the Effect Context used by this cue */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayCue")
	static UE_API int32 GetActorCount(FGameplayCueParameters Parameters);

	/** Returns actor stored in the Effect Context used by this cue */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayCue")
	static UE_API AActor* GetActorByIndex(FGameplayCueParameters Parameters, int32 Index);

	/** Returns a hit result stored in the effect context if valid */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayCue")
	static UE_API FHitResult GetHitResult(FGameplayCueParameters Parameters);

	/** Checks if the effect context has a hit reslt stored inside */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayCue")
	static UE_API bool HasHitResult(FGameplayCueParameters Parameters);

	/** Forwards the gameplay cue to another gameplay cue interface object */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayCue")
	static UE_API void ForwardGameplayCueToTarget(TScriptInterface<IGameplayCueInterface> TargetCueInterface, EGameplayCueEvent::Type EventType, FGameplayCueParameters Parameters);

	/** Gets the instigating actor (that holds the ability system component) of the GameplayCue */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayCue")
	static UE_API AActor* GetInstigatorActor(FGameplayCueParameters Parameters);

	/** Gets instigating world location */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayCue")
	static UE_API FTransform GetInstigatorTransform(FGameplayCueParameters Parameters);

	/** Gets instigating world location */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayCue")
	static UE_API FVector GetOrigin(FGameplayCueParameters Parameters);

	/** Gets the best end location and normal for this gameplay cue. If there is hit result data, it will return this. Otherwise it will return the target actor's location/rotation. If none of this is available, it will return false. */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayCue")
	static UE_API bool GetGameplayCueEndLocationAndNormal(AActor* TargetActor, FGameplayCueParameters Parameters, FVector& Location, FVector& Normal);

	/** Gets the best normalized effect direction for this gameplay cue. This is useful for effects that require the direction of an enemy attack. Returns true if a valid direction could be calculated. */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayCue")
	static UE_API bool GetGameplayCueDirection(AActor* TargetActor, FGameplayCueParameters Parameters, FVector& Direction);

	/** Returns true if the aggregated source and target tags from the effect spec meets the tag requirements */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayCue")
	static UE_API bool DoesGameplayCueMeetTagRequirements(FGameplayCueParameters Parameters, const FGameplayTagRequirements& SourceTagReqs, const FGameplayTagRequirements& TargetTagReqs);

	/** Native make, to avoid having to deal with quantized vector types */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayCue", meta = (NativeMakeFunc, AdvancedDisplay=5, Location="0,0,0", Normal= "0,0,0", GameplayEffectLevel = "1", AbilityLevel = "1"))
	static UE_API FGameplayCueParameters MakeGameplayCueParameters(float NormalizedMagnitude, float RawMagnitude, FGameplayEffectContextHandle EffectContext, FGameplayTag MatchedTagName, FGameplayTag OriginalTag, FGameplayTagContainer AggregatedSourceTags, FGameplayTagContainer AggregatedTargetTags, FVector Location, FVector Normal, AActor* Instigator, AActor* EffectCauser, UObject* SourceObject, UPhysicalMaterial* PhysicalMaterial, int32 GameplayEffectLevel, int32 AbilityLevel, USceneComponent* TargetAttachComponent, bool bReplicateLocationWhenUsingMinimalRepProxy);

	/** Native break, to avoid having to deal with quantized vector types */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayCue", meta = (NativeBreakFunc, AdvancedDisplay=6))
	static UE_API void BreakGameplayCueParameters(const struct FGameplayCueParameters& Parameters, float& NormalizedMagnitude, float& RawMagnitude, FGameplayEffectContextHandle& EffectContext, FGameplayTag& MatchedTagName, FGameplayTag& OriginalTag, FGameplayTagContainer& AggregatedSourceTags, FGameplayTagContainer& AggregatedTargetTags, FVector& Location, FVector& Normal, AActor*& Instigator, AActor*& EffectCauser, UObject*& SourceObject, UPhysicalMaterial*& PhysicalMaterial, int32& GameplayEffectLevel, int32& AbilityLevel, USceneComponent*& TargetAttachComponent, bool& bReplicateLocationWhenUsingMinimalRepProxy);

	// -------------------------------------------------------------------------------
	//		GameplayEffectSpec
	// -------------------------------------------------------------------------------

	/** Sets a raw name Set By Caller magnitude value, the tag version should normally be used */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API FGameplayEffectSpecHandle AssignSetByCallerMagnitude(FGameplayEffectSpecHandle SpecHandle, FName DataName, float Magnitude);

	/** Sets a gameplay tag Set By Caller magnitude value */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect", meta = (GameplayTagFilter = "SetByCaller"))
	static UE_API FGameplayEffectSpecHandle AssignTagSetByCallerMagnitude(FGameplayEffectSpecHandle SpecHandle, FGameplayTag DataTag, float Magnitude);

	/** Manually sets the duration on a specific effect */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API FGameplayEffectSpecHandle SetDuration(FGameplayEffectSpecHandle SpecHandle, float Duration);

	/** This instance of the effect will now grant NewGameplayTag to the object that this effect is applied to */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API FGameplayEffectSpecHandle AddGrantedTag(FGameplayEffectSpecHandle SpecHandle, FGameplayTag NewGameplayTag);

	/** This instance of the effect will now grant NewGameplayTags to the object that this effect is applied to */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API FGameplayEffectSpecHandle AddGrantedTags(FGameplayEffectSpecHandle SpecHandle, FGameplayTagContainer NewGameplayTags);

	/** Return all tags granted by this GameplayEffectSpec: both from the GE asset and dynamic granted tags. */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API FGameplayTagContainer GetGrantedTags(FGameplayEffectSpecHandle SpecHandle);

	/** Adds NewGameplayTag to this instance of the effect */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API FGameplayEffectSpecHandle AddAssetTag(FGameplayEffectSpecHandle SpecHandle, FGameplayTag NewGameplayTag);

	/** Adds NewGameplayTags to this instance of the effect */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API FGameplayEffectSpecHandle AddAssetTags(FGameplayEffectSpecHandle SpecHandle, FGameplayTagContainer NewGameplayTags);

	/** Return all asset tags of this GameplayEffectSpec: both from the GE asset and dynamic asset tags. */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API FGameplayTagContainer GetAssetTags(FGameplayEffectSpecHandle SpecHandle);

	/** Adds LinkedGameplayEffectSpec to SpecHandles. LinkedGameplayEffectSpec will be applied when/if SpecHandle is applied successfully. LinkedGameplayEffectSpec will not be modified here. Returns the ORIGINAL SpecHandle (legacy decision) */
	UE_DEPRECATED(5.3, "Linked GameplayEffects aren't replicated.  The new UAdditionalGameplayEffectsComponent renders this functionality obsolete (and since it's configured in the Asset, it is properly synced).")
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect", meta = (DeprecatedFunction, DeprecatedMessage="Linked GameplayEffects aren't replicated.  Configure the GameplayEffect asset with a suitable GameplayEffectComponent."))
	static UE_API FGameplayEffectSpecHandle AddLinkedGameplayEffectSpec(FGameplayEffectSpecHandle SpecHandle, FGameplayEffectSpecHandle LinkedGameplayEffectSpec);

	/** Adds LinkedGameplayEffect to SpecHandles. LinkedGameplayEffectSpec will be applied when/if SpecHandle is applied successfully. This will initialize the LinkedGameplayEffect's Spec for you. Returns to NEW linked spec in case you want to add more to it. */
	UE_DEPRECATED(5.3, "Linked GameplayEffects aren't replicated.  The new UAdditionalGameplayEffectsComponent renders this functionality obsolete (and since it's configured in the Asset, it is properly synced).")
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect", meta = (DeprecatedFunction, DeprecatedMessage = "Linked GameplayEffects aren't replicated.  Configure the GameplayEffect asset with a suitable GameplayEffectComponent."))
	static UE_API FGameplayEffectSpecHandle AddLinkedGameplayEffect(FGameplayEffectSpecHandle SpecHandle, TSubclassOf<UGameplayEffect> LinkedGameplayEffect);

	/** Sets the GameplayEffectSpec's StackCount to the specified amount (prior to applying) */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API FGameplayEffectSpecHandle SetStackCount(FGameplayEffectSpecHandle SpecHandle, int32 StackCount);

	/** Sets the GameplayEffectSpec's StackCount to the max stack count defined in the GameplayEffect definition */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API FGameplayEffectSpecHandle SetStackCountToMax(FGameplayEffectSpecHandle SpecHandle);

	/** Gets the GameplayEffectSpec's effect context handle */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API FGameplayEffectContextHandle GetEffectContext(FGameplayEffectSpecHandle SpecHandle);

	/** Returns handles for all Linked GE Specs that SpecHandle may apply. Useful if you want to append additional information to them. */
	UE_DEPRECATED(5.3, "Linked GameplayEffects aren't replicated.  The new UAdditionalGameplayEffectsComponent renders this functionality obsolete (and since it's configured in the Asset, it is properly synced).")
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayEffect", meta = (DeprecatedFunction, DeprecatedMessage = "Linked GameplayEffects aren't replicated.  Configure the GameplayEffect asset with a suitable GameplayEffectComponent."))
	static UE_API TArray<FGameplayEffectSpecHandle> GetAllLinkedGameplayEffectSpecHandles(FGameplayEffectSpecHandle SpecHandle);

	/** Manually adds a set of tags to a given actor, and optionally replicates them. */
	UFUNCTION(BlueprintCallable, Category="Ability|GameplayEffect")
	static UE_API bool AddLooseGameplayTags(AActor* Actor, const FGameplayTagContainer& GameplayTags, bool bShouldReplicate=false);

	/** Manually removes a set of tags from a given actor, with optional replication. */
	UFUNCTION(BlueprintCallable, Category="Ability|GameplayEffect")
	static UE_API bool RemoveLooseGameplayTags(AActor* Actor, const FGameplayTagContainer& GameplayTags, bool bShouldReplicate=false);

	/** Manually adds a set of tags to a given actor */
	UFUNCTION(BlueprintCallable, Category="Ability|GameplayEffect")
	static UE_API bool AddGameplayTags(AActor* Actor, const FGameplayTagContainer& GameplayTags, EGameplayTagReplicationState ReplicationRule = EGameplayTagReplicationState::None);

	/** Manually removes a set of tags from a given actor */
	UFUNCTION(BlueprintCallable, Category="Ability|GameplayEffect")
	static UE_API bool RemoveGameplayTags(AActor* Actor, const FGameplayTagContainer& GameplayTags, EGameplayTagReplicationState ReplicationRule = EGameplayTagReplicationState::None);

	/** Get the GameplayEffect definition from a GameplayEffectSpecHandle. */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API const UGameplayEffect* GetGameplayEffectFromSpecHandle(FGameplayEffectSpecHandle Handle);

	/** Returns whether the GameplayEffectSpecHandle represents a non-instant GameplayEffect. */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API bool IsDurationGameplayEffectSpecHandle(FGameplayEffectSpecHandle Handle);

	/** Returns the Duration Policy of a GameplayEffectSpec. */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API EGameplayEffectDurationType GetDurationPolicyFromGameplayEffectSpecHandle(FGameplayEffectSpecHandle Handle);

	// -------------------------------------------------------------------------------
	//		GameplayEffectSpec
	// -------------------------------------------------------------------------------

	/** Gets the magnitude of change for an attribute on an APPLIED GameplayEffectSpec. */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API float GetModifiedAttributeMagnitude(FGameplayEffectSpecHandle SpecHandle, FGameplayAttribute Attribute);

	/** Helper function that may be useful to call from native as well */
	static UE_API float GetModifiedAttributeMagnitude(const FGameplayEffectSpec& SpecHandle, FGameplayAttribute Attribute);

	// -------------------------------------------------------------------------------
	//		FActiveGameplayEffectHandle
	// -------------------------------------------------------------------------------

	/** Returns whether the active gameplay effect handle has a valid handle value, regardless of whether the effect expired. */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static bool IsActiveGameplayEffectHandleValid(FActiveGameplayEffectHandle Handle);

	/** Returns whether the active gameplay effect handle represents a gameplay effect that is valid and still active. */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static bool IsActiveGameplayEffectHandleActive(FActiveGameplayEffectHandle Handle);

	/** Returns AbilitySystemComponent from an ActiveGameplayEffectHandle. */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static class UAbilitySystemComponent* GetAbilitySystemComponentFromActiveGameplayEffectHandle(FActiveGameplayEffectHandle Handle);

	/** Returns current stack count of an active Gameplay Effect. Will return 0 if the GameplayEffect is no longer valid. */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API int32 GetActiveGameplayEffectStackCount(FActiveGameplayEffectHandle ActiveHandle);

	/** Returns stack limit count of an active Gameplay Effect. Will return 0 if the GameplayEffect is no longer valid. */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API int32 GetActiveGameplayEffectStackLimitCount(FActiveGameplayEffectHandle ActiveHandle);

	/** Returns the start time (time which the GE was added) for a given GameplayEffect */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API float GetActiveGameplayEffectStartTime(FActiveGameplayEffectHandle ActiveHandle);

	/** Returns the expected end time (when we think the GE will expire) for a given GameplayEffect (note someone could remove or change it before that happens!) */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API float GetActiveGameplayEffectExpectedEndTime(FActiveGameplayEffectHandle ActiveHandle);

	/** Returns the total duration for a given GameplayEffect */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	static UE_API float GetActiveGameplayEffectTotalDuration(FActiveGameplayEffectHandle ActiveHandle);

	/** Returns the total duration for a given GameplayEffect, basically ExpectedEndTime - Current Time */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect", meta = (WorldContext = "WorldContextObject"))
	static UE_API float GetActiveGameplayEffectRemainingDuration(UObject* WorldContextObject, FActiveGameplayEffectHandle ActiveHandle);

	/** Returns a debug string for display */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayEffect", Meta = (DisplayName = "Get Active GameplayEffect Debug String "))
	static UE_API FString GetActiveGameplayEffectDebugString(FActiveGameplayEffectHandle ActiveHandle);

	/** Returns the UI data for a gameplay effect class (if any) */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect", Meta = (DisplayName = "Get GameplayEffect UI Data", DeterminesOutputType="DataType"))
	static UE_API const UGameplayEffectUIData* GetGameplayEffectUIData(TSubclassOf<UGameplayEffect> EffectClass, TSubclassOf<UGameplayEffectUIData> DataType);

	/** Equality operator for two Active Gameplay Effect Handles */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayEffect", meta = (DisplayName = "Equal (Active Gameplay Effect Handle)", CompactNodeTitle = "==", ScriptOperator = "=="))
	static UE_API bool EqualEqual_ActiveGameplayEffectHandle(const FActiveGameplayEffectHandle& A, const FActiveGameplayEffectHandle& B);

	/** Inequality operator for two Active Gameplay Effect Handles */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayEffect", meta = (DisplayName = "Not Equal (Active Gameplay Effect Handle)", CompactNodeTitle = "!=", ScriptOperator = "!="))
	static UE_API bool NotEqual_ActiveGameplayEffectHandle(const FActiveGameplayEffectHandle& A, const FActiveGameplayEffectHandle& B);

	/**
	 * Returns the Gameplay Effect CDO from an active handle.
	 * This reference should be considered read only,
	 * but you can use it to read additional Gameplay Effect info, such as icon, description, etc. 
	 */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayEffect")
	static UE_API const UGameplayEffect* GetGameplayEffectFromActiveEffectHandle(const FActiveGameplayEffectHandle& ActiveHandle);

	// -------------------------------------------------------------------------------
	//		Gameplay Effect
	// -------------------------------------------------------------------------------

	/** Returns all tags that the Gameplay Effect *has* (that denote the GE Asset itself) and *does not* grant to any Actor. */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayEffect", meta = (DisplayName = "Get Asset Tags"))
	static UE_API const FGameplayTagContainer& GetGameplayEffectAssetTags(TSubclassOf<UGameplayEffect> EffectClass);

	/** Returns all tags that the Gameplay Effect grants to the target Actor */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayEffect", meta = (DisplayName = "Get Granted Tags"))
	static UE_API const FGameplayTagContainer& GetGameplayEffectGrantedTags(TSubclassOf<UGameplayEffect> EffectClass);

	// -------------------------------------------------------------------------------
	//		GameplayAbility
	// -------------------------------------------------------------------------------

	/**
	 * Provides the Gameplay Ability object associated with an Ability Spec Handle
	 * This can be either an instanced ability, or in the case of shared abilities, the Class Default Object
	 * 
	 * @param AbilitySpec The Gameplay Ability Spec you want to get the object from
	 * @param bIsInstance Set to true if this is an instanced ability instead of a shared CDO
	 * 
	 * @return Pointer to the Gameplay Ability object
	 */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayAbility")
	static UE_API const UGameplayAbility* GetGameplayAbilityFromSpecHandle(UAbilitySystemComponent* AbilitySystem, const FGameplayAbilitySpecHandle& AbilitySpecHandle, bool& bIsInstance);

	/** Returns true if the passed-in Gameplay Ability instance is active (activated and not yet ended). */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayAbility", meta=(DisplayName="Is Active",DefaultToSelf=GameplayAbility))
	static UE_API bool IsGameplayAbilityActive(const UGameplayAbility* GameplayAbility);

	/** Returns true if the ASC has any abilities that meet a given predicate.
	 * @param AbilitySystemComponent         The ability system component we are checking abilities on.
	 * @param Predicate                      The predicate to run on each ability
	 * @param bOnlyRunPredicateOnAbilityCDOs If true, we will only run the predicate on the ability specs' Ability CDO instead of on its instance(s).
	 * @return                               Returns true if any abilities passed the predicate.
	 */
	static UE_API bool HasAnyAbilitiesByPredicate(
		const UAbilitySystemComponent* AbilitySystemComponent, 
		const TFunctionRef<bool(const UGameplayAbility& AbilityRef)> Predicate, 
		bool bOnlyRunPredicateOnAbilityCDOs = false);

	/** Returns true if the ASC has any abilities that have a specific asset tag.*/
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayAbility")
	static UE_API bool HasAnyAbilitiesWithAssetTag(const UAbilitySystemComponent* AbilitySystemComponent, FGameplayTag AssetTag);

	/** Equality operator for two Gameplay Ability Spec Handles */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayAbility", meta = (DisplayName = "Equal (Gameplay Ability Spec Handle)", CompactNodeTitle = "==", ScriptOperator = "=="))
	static UE_API bool EqualEqual_GameplayAbilitySpecHandle(const FGameplayAbilitySpecHandle& A, const FGameplayAbilitySpecHandle& B);

	/** Inequality operator for two Gameplay Ability Spec Handles */
	UFUNCTION(BlueprintPure, Category = "Ability|GameplayAbility", meta = (DisplayName = "Not Equal (Gameplay Ability Spec Handle)", CompactNodeTitle = "!=", ScriptOperator = "!="))
	static UE_API bool NotEqual_GameplayAbilitySpecHandle(const FGameplayAbilitySpecHandle& A, const FGameplayAbilitySpecHandle& B);

	/**
	 * Converts the given Scalable float to the value at the given level.
	 * 
	 * @param Input	The scalable float to get the value from 
	 * @param Level The Level of which to get the value at
	 * @return The single-precision float value at the given level from the scalable float
	 */
	UFUNCTION(BlueprintPure, Category="ScalableFloat", meta=(DisplayName="Get Value At Level (single-precision)", BlueprintAutocast, IgnoreTypePromotion))
	static UE_API float Conv_ScalableFloatToFloat(const FScalableFloat& Input, float Level = 0.0f);

	/**
	 * Converts the given Scalable float to the value at the given level.
	 * 
	 * @param Input	The scalable float to get the value from 
	 * @param Level The Level of which to get the value at
	 * @return The double-precision float value at the given level from the scalable float
	 */
	UFUNCTION(BlueprintPure, Category="ScalableFloat", meta=(DisplayName="Get Value At Level (double-precision)", BlueprintAutocast, IgnoreTypePromotion))
	static UE_API double Conv_ScalableFloatToDouble(const FScalableFloat& Input, float Level = 0.0f);
};

#undef UE_API
