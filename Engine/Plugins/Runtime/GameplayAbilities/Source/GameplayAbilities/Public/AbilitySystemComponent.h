// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AbilitySystemBlueprintLibrary.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/NetSerialization.h"
#include "Engine/EngineTypes.h"
#include "Engine/TimerHandle.h"
#include "GameplayTagContainer.h"
#include "AttributeSet.h"
#include "EngineDefines.h"
#include "GameplayPrediction.h"
#include "GameplayCueInterface.h"
#include "GameplayTagAssetInterface.h"
#include "GameplayAbilitySpec.h"
#include "GameplayEffect.h"
#include "GameplayTasksComponent.h"
#include "Abilities/GameplayAbilityRepAnimMontage.h"
#include "Abilities/GameplayAbilityTargetTypes.h"
#include "Abilities/GameplayAbility.h"
#include "AbilitySystemReplicationProxyInterface.h"
#include "AbilitySystemComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class AGameplayAbilityTargetActor;
class AHUD;
class FDebugDisplayInfo;
class UAnimMontage;
class UAnimSequenceBase;
class UCanvas;
class UInputComponent;

/** 
 *	UAbilitySystemComponent	
 *
 *	A component to easily interface with the 3 aspects of the AbilitySystem:
 *	
 *	GameplayAbilities:
 *		-Provides a way to give/assign abilities that can be used (by a player or AI for example)
 *		-Provides management of instanced abilities (something must hold onto them)
 *		-Provides replication functionality
 *			-Ability state must always be replicated on the UGameplayAbility itself, but UAbilitySystemComponent provides RPC replication
 *			for the actual activation of abilities
 *			
 *	GameplayEffects:
 *		-Provides an FActiveGameplayEffectsContainer for holding active GameplayEffects
 *		-Provides methods for applying GameplayEffects to a target or to self
 *		-Provides wrappers for querying information in FActiveGameplayEffectsContainers (duration, magnitude, etc)
 *		-Provides methods for clearing/remove GameplayEffects
 *		
 *	GameplayAttributes
 *		-Provides methods for allocating and initializing attribute sets
 *		-Provides methods for getting AttributeSets
 *  
 */

/** Called when a targeting actor rejects target confirmation */
DECLARE_MULTICAST_DELEGATE_OneParam(FTargetingRejectedConfirmation, int32);

/** Called when ability fails to activate, passes along the failed ability and a tag explaining why */
DECLARE_MULTICAST_DELEGATE_TwoParams(FAbilityFailedDelegate, const UGameplayAbility*, const FGameplayTagContainer&);

/** Called when ability ends */
DECLARE_MULTICAST_DELEGATE_OneParam(FAbilityEnded, UGameplayAbility*);

/** Notify interested parties that ability spec has been modified */
DECLARE_MULTICAST_DELEGATE_OneParam(FAbilitySpecDirtied, const FGameplayAbilitySpec&);

/** Notifies when GameplayEffectSpec is blocked by an ActiveGameplayEffect due to immunity  */
DECLARE_MULTICAST_DELEGATE_TwoParams(FImmunityBlockGE, const FGameplayEffectSpec& /*BlockedSpec*/, const FActiveGameplayEffect* /*ImmunityGameplayEffect*/);

/** We allow a list of delegates to decide if the application of a Gameplay Effect can be blocked. If it's blocked, it will call the ImmunityBlockGE above */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FGameplayEffectApplicationQuery, const FActiveGameplayEffectsContainer& /*ActiveGEContainer*/, const FGameplayEffectSpec& /*GESpecToConsider*/);

/** How gameplay effects will be replicated to clients */
UENUM()
enum class EGameplayEffectReplicationMode : uint8
{
	/** Only replicate minimal gameplay effect info. Note: this does not work for Owned AbilitySystemComponents (Use Mixed instead). */
	Minimal,
	/** Only replicate minimal gameplay effect info to simulated proxies but full info to owners and autonomous proxies */
	Mixed,
	/** Replicate full gameplay info to all */
	Full,
};

/** When performing actions (such as gathering activatable abilities), how do we deal with Pending items (e.g. abilities not yet added or removed) */
enum class EConsiderPending : uint8
{
	/** Don't consider any Pending actions (such as Pending Abilities Added or Removed) */
	None = 0,

	/** Consider Pending Adds when performing the action */
	PendingAdd = (1 << 0),

	/** Consider Pending Removes when performing the action */
	PendingRemove = (1 << 1),

	All = PendingAdd | PendingRemove
};
ENUM_CLASS_FLAGS(EConsiderPending)

/** The core ActorComponent for interfacing with the GameplayAbilities System */
UCLASS(ClassGroup=AbilitySystem, hidecategories=(Object,LOD,Lighting,Transform,Sockets,TextureStreaming), editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UAbilitySystemComponent : public UGameplayTasksComponent, public IGameplayTagAssetInterface, public IAbilitySystemReplicationProxyInterface
{
	GENERATED_UCLASS_BODY()

	/** Used to register callbacks to ability-key input */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FAbilityAbilityKey, /*UGameplayAbility*, Ability, */int32, InputID);

	/** Used to register callbacks to confirm/cancel input */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FAbilityConfirmOrCancel);

	/** Delegate for when an effect is applied */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnGameplayEffectAppliedDelegate, UAbilitySystemComponent*, const FGameplayEffectSpec&, FActiveGameplayEffectHandle);

	// ----------------------------------------------------------------------------------------------------------------
	//	Attributes
	// ----------------------------------------------------------------------------------------------------------------

	/** Finds existing AttributeSet */
	template <class T >
	const T*	GetSet() const
	{
		return (T*)GetAttributeSubobject(T::StaticClass());
	}

	/** Finds existing AttributeSet. Asserts if it isn't there. */
	template <class T >
	const T*	GetSetChecked() const
	{
		return (T*)GetAttributeSubobjectChecked(T::StaticClass());
	}

	/** Finds existing AttributeSet for the given actor */
	template <class T >
	static const T* GetSetOnActor(AActor* InActor)
	{
		if (UAbilitySystemComponent* ASC = UAbilitySystemBlueprintLibrary::GetAbilitySystemComponent(InActor))
		{
			return ASC->GetSet<T>();
		}

		return nullptr;
	}

	/** Adds a new AttributeSet (initialized to default values) */
	template <class T >
	const T*  AddSet()
	{
		return (T*)GetOrCreateAttributeSubobject(T::StaticClass());
	}

	/** 
	 * Manually add a new attribute set that is a subobject of this ability system component.
	 * All subobjects of this component are automatically added during initialization.
	 */
	template <class T>
	const T* AddAttributeSetSubobject(T* Subobject)
	{
		AddSpawnedAttribute(Subobject);
		return Subobject;
	}

	/**
	 * Does this ability system component have this attribute?
	 * 
	 * @param Attribute	Handle of the gameplay effect to retrieve target tags from
	 * 
	 * @return true if Attribute is valid and this ability system component contains an attribute set that contains Attribute. Returns false otherwise.
	 */
	UE_API bool HasAttributeSetForAttribute(FGameplayAttribute Attribute) const;

	/** Initializes starting attributes from a data table. Not well supported, a gameplay effect with curve table references may be a better solution */
	UE_API const UAttributeSet* InitStats(TSubclassOf<class UAttributeSet> Attributes, const UDataTable* DataTable);

	UFUNCTION(BlueprintCallable, Category="Skills", meta=(DisplayName="InitStats", ScriptName="InitStats"))
	UE_API void K2_InitStats(TSubclassOf<class UAttributeSet> Attributes, const UDataTable* DataTable);
		
	/** Returns a list of all attributes for this abilty system component */
	UFUNCTION(BlueprintPure, Category="Gameplay Attributes")
	UE_API void GetAllAttributes(TArray<FGameplayAttribute>& OutAttributes);

	/**
	 * Returns a reference to the Attribute Set instance, if one exists in this component
	 *
	 * @param AttributeSetClass The type of attribute set to look for
	 * @param bFound Set to true if an instance of the Attribute Set exists
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Gameplay Attributes", meta=(DeterminesOutputType = AttributeSetClass))
	UE_API const UAttributeSet* GetAttributeSet(TSubclassOf<UAttributeSet> AttributeSetClass) const;

	/**
	 * Returns the current value of the given gameplay attribute, or zero if the attribute is not found.
	 * NOTE: This doesn't take predicted gameplay effect modifiers into consideration, so the value may not be accurate on clients at all times.
	 *
	 * @param Attribute The gameplay attribute to query
	 * @param bFound Set to true if the attribute exists in this component
	 */
	UFUNCTION(BlueprintPure, Category = "Gameplay Attributes")
	UE_API float GetGameplayAttributeValue(FGameplayAttribute Attribute, bool& bFound) const;

	UPROPERTY(EditAnywhere, Category="AttributeTest")
	TArray<FAttributeDefaults>	DefaultStartingData;

	/** Remove all current AttributeSets and register the ones in the passed array. Note that it's better to call Add/Remove directly when possible. */
	UE_API void SetSpawnedAttributes(const TArray<UAttributeSet*>& NewAttributeSet);

	/** Access the spawned attributes list when you don't intend to modify the list. */
	UE_API const TArray<UAttributeSet*>& GetSpawnedAttributes() const;

	/** Add a new attribute set */
	UE_API void AddSpawnedAttribute(UAttributeSet* Attribute);

	/** Remove an existing attribute set */
	UE_API void RemoveSpawnedAttribute(UAttributeSet* Attribute);

	/** Remove all attribute sets */
	UE_API void RemoveAllSpawnedAttributes();


	/** The linked Anim Instance that this component will play montages in. Use NAME_None for the main anim instance. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Skills")
	FName AffectedAnimInstanceTag; 


	/** Sets the base value of an attribute. Existing active modifiers are NOT cleared and will act upon the new base value. */
	UE_API void SetNumericAttributeBase(const FGameplayAttribute &Attribute, float NewBaseValue);

	/** Gets the base value of an attribute. That is, the value of the attribute with no stateful modifiers */
	UE_API float GetNumericAttributeBase(const FGameplayAttribute &Attribute) const;

	/**
	 *	Applies an in-place mod to the given attribute. This correctly update the attribute's aggregator, updates the attribute set property,
	 *	and invokes the OnDirty callbacks.
	 *	This does not invoke Pre/PostGameplayEffectExecute calls on the attribute set. This does no tag checking, application requirements, immunity, etc.
	 *	No GameplayEffectSpec is created or is applied!
	 *	This should only be used in cases where applying a real GameplayEffectSpec is too slow or not possible.
	 */
	UE_API void ApplyModToAttribute(const FGameplayAttribute &Attribute, TEnumAsByte<EGameplayModOp::Type> ModifierOp, float ModifierMagnitude);

	/**
	 *  Applies an inplace mod to the given attribute. Unlike ApplyModToAttribute this function will run on the client or server.
	 *  This may result in problems related to prediction and will not roll back properly.
	 */
	UE_API void ApplyModToAttributeUnsafe(const FGameplayAttribute &Attribute, TEnumAsByte<EGameplayModOp::Type> ModifierOp, float ModifierMagnitude);

	/** Returns current (final) value of an attribute */
	UE_API float GetNumericAttribute(const FGameplayAttribute &Attribute) const;
	UE_API float GetNumericAttributeChecked(const FGameplayAttribute &Attribute) const;

	/** Returns an attribute value, after applying tag filters */
	UE_API float GetFilteredAttributeValue(const FGameplayAttribute& Attribute, const FGameplayTagRequirements& SourceTags, const FGameplayTagContainer& TargetTags, const TArray<FActiveGameplayEffectHandle>& HandlesToIgnore = TArray<FActiveGameplayEffectHandle>());

	// ----------------------------------------------------------------------------------------------------------------
	//	Replication
	// ----------------------------------------------------------------------------------------------------------------

	/** Forces avatar actor to update it's replication. Useful for things like needing to replication for movement / locations reasons. */
	UE_API virtual void ForceAvatarReplication();

	/** When true, we will not replicate active gameplay effects for this ability system component, so attributes and tags */
	UE_API virtual void SetReplicationMode(EGameplayEffectReplicationMode NewReplicationMode);

	/** How gameplay effects are replicated */
	EGameplayEffectReplicationMode ReplicationMode;

	/** Who to route replication through if ReplicationProxyEnabled (if this returns null, when ReplicationProxyEnabled, we wont replicate)  */
	UE_API virtual IAbilitySystemReplicationProxyInterface* GetReplicationInterface();

	/** Current prediction key, set with FScopedPredictionWindow */
	FPredictionKey	ScopedPredictionKey;

	/** Returns the prediction key that should be used for any actions */
	FPredictionKey GetPredictionKeyForNewAction() const
	{
		return ScopedPredictionKey.IsValidForMorePrediction() ? ScopedPredictionKey : FPredictionKey();
	}

	/** Do we have a valid prediction key to do more predictive actions with */
	bool CanPredict() const
	{
		return ScopedPredictionKey.IsValidForMorePrediction();
	}

	/** Returns true if this is running on the server or has a valid prediction key */
	UE_API bool HasAuthorityOrPredictionKey(const FGameplayAbilityActivationInfo* ActivationInfo) const;

	/** Returns true if this component's actor has authority */
	UE_API virtual bool IsOwnerActorAuthoritative() const;

	/** Returns true if this component should record montage replication info. */
	UE_API virtual bool ShouldRecordMontageReplication() const;

	/** Replicate that an ability has ended/canceled, to the client or server as appropriate */
	UE_API virtual void ReplicateEndOrCancelAbility(FGameplayAbilitySpecHandle Handle, FGameplayAbilityActivationInfo ActivationInfo, UGameplayAbility* Ability, bool bWasCanceled);

	/** Force cancels the ability and does not replicate this to the other side. This should be called when the ability is cancelled by the other side */
	UE_API virtual void ForceCancelAbilityDueToReplication(UGameplayAbility* Instance);

	/** A pending activation that cannot be activated yet, will be rechecked at a later point */
	struct FPendingAbilityInfo
	{
		bool operator==(const FPendingAbilityInfo& Other) const
		{
			// Don't compare event data, not valid to have multiple activations in flight with same key and handle but different event data
			return PredictionKey == Other.PredictionKey	&& Handle == Other.Handle;
		}

		/** Properties of the ability that needs to be activated */
		FGameplayAbilitySpecHandle Handle;
		FPredictionKey	PredictionKey;
		FGameplayEventData TriggerEventData;

		/** True if this ability was activated remotely and needs to follow up, false if the ability hasn't been activated at all yet */
		bool bPartiallyActivated;

		FPendingAbilityInfo()
			: bPartiallyActivated(false)
		{}
	};

	/** This is a list of GameplayAbilities that were activated on the server and can't yet execute on the client. It will try to execute these at a later point */
	TArray<FPendingAbilityInfo> PendingServerActivatedAbilities;

	// ----------------------------------------------------------------------------------------------------------------
	//	GameplayEffects: Primary outward facing API for other systems
	// ----------------------------------------------------------------------------------------------------------------

	/** Applies a previously created gameplay effect spec to a target */
	UFUNCTION(BlueprintCallable, Category = GameplayEffects, meta = (DisplayName = "ApplyGameplayEffectSpecToTarget", ScriptName = "ApplyGameplayEffectSpecToTarget"))
	UE_API FActiveGameplayEffectHandle BP_ApplyGameplayEffectSpecToTarget(const FGameplayEffectSpecHandle& SpecHandle, UAbilitySystemComponent* Target);

	UE_API virtual FActiveGameplayEffectHandle ApplyGameplayEffectSpecToTarget(const FGameplayEffectSpec& GameplayEffect, UAbilitySystemComponent *Target, FPredictionKey PredictionKey=FPredictionKey());

	/** Applies a previously created gameplay effect spec to this component */
	UFUNCTION(BlueprintCallable, Category = GameplayEffects, meta = (DisplayName = "ApplyGameplayEffectSpecToSelf", ScriptName = "ApplyGameplayEffectSpecToSelf"))
	UE_API FActiveGameplayEffectHandle BP_ApplyGameplayEffectSpecToSelf(const FGameplayEffectSpecHandle& SpecHandle);

	UE_API virtual FActiveGameplayEffectHandle ApplyGameplayEffectSpecToSelf(const FGameplayEffectSpec& GameplayEffect, FPredictionKey PredictionKey = FPredictionKey());

	/** Gets the FActiveGameplayEffect based on the passed in Handle */
	UE_API const UGameplayEffect* GetGameplayEffectDefForHandle(FActiveGameplayEffectHandle Handle);

	/** 
	* Removes the specified GameplayEffect by Handle.
	* @param Handle								The handle of the gameplay effect to remove
	* @param StacksToRemove						Number of stacks to remove, -1 means remove all
	* 
	* @return Whether the effect was successfully removed.
	*/
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = GameplayEffects)
	UE_API virtual bool RemoveActiveGameplayEffect(FActiveGameplayEffectHandle Handle, int32 StacksToRemove=-1);

	/** 
	 * Remove active gameplay effects whose backing definition are the specified gameplay effect class
	 * 
	 * @param GameplayEffect					Class of gameplay effect to remove; Does nothing if left null
	 * @param InstigatorAbilitySystemComponent	If specified, will only remove gameplay effects applied from this instigator ability system component
	 * @param StacksToRemove					Number of stacks to remove, -1 means remove all
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = GameplayEffects)
	UE_API virtual void RemoveActiveGameplayEffectBySourceEffect(TSubclassOf<UGameplayEffect> GameplayEffect, UAbilitySystemComponent* InstigatorAbilitySystemComponent, int32 StacksToRemove = -1);

	/** Get an outgoing GameplayEffectSpec that is ready to be applied to other things. */
	UFUNCTION(BlueprintCallable, Category = GameplayEffects)
	UE_API virtual FGameplayEffectSpecHandle MakeOutgoingSpec(TSubclassOf<UGameplayEffect> GameplayEffectClass, float Level, FGameplayEffectContextHandle Context) const;

	/** Create an EffectContext for the owner of this AbilitySystemComponent */
	UFUNCTION(BlueprintCallable, Category = GameplayEffects)
	UE_API virtual FGameplayEffectContextHandle MakeEffectContext() const;

	/**
	 * Get the count of the specified source effect on the ability system component. For non-stacking effects, this is the sum of all active instances.
	 * For stacking effects, this is the sum of all valid stack counts. If an instigator is specified, only effects from that instigator are counted.
	 * 
	 * @param SourceGameplayEffect					Effect to get the count of
	 * @param OptionalInstigatorFilterComponent		If specified, only count effects applied by this ability system component
	 * 
	 * @return Count of the specified source effect
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category=GameplayEffects)
	UE_API int32 GetGameplayEffectCount(TSubclassOf<UGameplayEffect> SourceGameplayEffect, UAbilitySystemComponent* OptionalInstigatorFilterComponent, bool bEnforceOnGoingCheck = true) const;

	/**
	 * Get the count of the specified source effect on the ability system component. For non-stacking effects, this is the sum of all active instances.
	 * For stacking effects, this is the sum of all valid stack counts. If an instigator is specified, only effects from that instigator are counted.
	 * 
	 * @param SoftSourceGameplayEffect				Effect to get the count of. If this is not currently loaded, the count is 0
	 * @param OptionalInstigatorFilterComponent		If specified, only count effects applied by this ability system component
	 * 
	 * @return Count of the specified source effect
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = GameplayEffects)
	UE_API int32 GetGameplayEffectCount_IfLoaded(TSoftClassPtr<UGameplayEffect> SoftSourceGameplayEffect, UAbilitySystemComponent* OptionalInstigatorFilterComponent, bool bEnforceOnGoingCheck = true) const;

	/** Returns the sum of StackCount of all gameplay effects that pass query */
	UE_API int32 GetAggregatedStackCount(const FGameplayEffectQuery& Query) const;

	/** This only exists so it can be hooked up to a multicast delegate */
	UE_DEPRECATED_FORGAME(5.5, "This shouldn't be public. Use RemoveActiveGameplayEffect (which won't allow non-authority to remove) or expose RemoveActiveGameplayEffect_AllowClientRemoval if your intent is to break that contract.")
	UE_API void RemoveActiveGameplayEffect_NoReturn(FActiveGameplayEffectHandle Handle, int32 StacksToRemove = -1);

	/** Called for predictively added gameplay cue. Needs to remove tag count and possible invoke OnRemove event if misprediction */
	UE_API virtual void OnPredictiveGameplayCueCatchup(FGameplayTag Tag);

	/** Returns the total duration of a gameplay effect */
	UE_API float GetGameplayEffectDuration(FActiveGameplayEffectHandle Handle) const;

	/** Called whenever the server time replicates via the game state to keep our cooldown timers in sync with the server */
	UE_API virtual void RecomputeGameplayEffectStartTimes(const float WorldTime, const float ServerWorldTime);

	/** Return start time and total duration of a gameplay effect */
	UE_API void GetGameplayEffectStartTimeAndDuration(FActiveGameplayEffectHandle Handle, float& StartEffectTime, float& Duration) const;

	/** Dynamically update the set-by-caller magnitude for an active gameplay effect */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = GameplayEffects)
	UE_API virtual void UpdateActiveGameplayEffectSetByCallerMagnitude(FActiveGameplayEffectHandle ActiveHandle, UPARAM(meta=(Categories = "SetByCaller"))FGameplayTag SetByCallerTag, float NewValue);

	/** Dynamically update multiple set-by-caller magnitudes for an active gameplay effect */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = GameplayEffects)
	UE_API virtual void UpdateActiveGameplayEffectSetByCallerMagnitudes(FActiveGameplayEffectHandle ActiveHandle, const TMap<FGameplayTag, float>& NewSetByCallerValues);

	/** Updates the level of an already applied gameplay effect. The intention is that this is 'seemless' and doesnt behave like removing/reapplying */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = GameplayEffects)
	UE_API virtual void SetActiveGameplayEffectLevel(FActiveGameplayEffectHandle ActiveHandle, int32 NewLevel);

	/** Updates the level of an already applied gameplay effect. The intention is that this is 'seemless' and doesnt behave like removing/reapplying */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = GameplayEffects)
	UE_API virtual void SetActiveGameplayEffectLevelUsingQuery(FGameplayEffectQuery Query, int32 NewLevel);

	/**
	 * (Un-)Inhibit an Active Gameplay Effect so it may be disabled (and perform some disabling actions, such as removing tags).
	 * An inhibited Active Gameplay Effect will lay dormant for re-enabling on some condition (usually tags).  When it's uninhibited, it will reapply some of its functionality.
	 * NOTE:  The passed-in ActiveGEHandle is no longer valid.  Use the returned FActiveGameplayEffectHandle to determine if the Active GE Handle is still active.
	 */
	UE_API virtual FActiveGameplayEffectHandle SetActiveGameplayEffectInhibit(FActiveGameplayEffectHandle&& ActiveGEHandle, bool bInhibit, bool bInvokePredictedEffects);

	/**
	 * Raw accessor to ask the magnitude of a gameplay effect, not necessarily always correct. How should outside code (UI, etc) ask things like 'how much is this gameplay effect modifying my damage by'
	 * (most likely we want to catch this on the backend - when damage is applied we can get a full dump/history of how the number got to where it is. But still we may need polling methods like below (how much would my damage be)
	 */
	UFUNCTION(BlueprintCallable, Category = GameplayEffects)
	UE_API float GetGameplayEffectMagnitude(FActiveGameplayEffectHandle Handle, FGameplayAttribute Attribute) const;

	/** Returns current stack count of an already applied GE */
	UE_API int32 GetCurrentStackCount(FActiveGameplayEffectHandle Handle) const;

	/** Returns current stack count of an already applied GE, but given the ability spec handle that was granted by the GE */
	UE_API int32 GetCurrentStackCount(FGameplayAbilitySpecHandle Handle) const;

	/** Returns debug string describing active gameplay effect */
	UE_API FString GetActiveGEDebugString(FActiveGameplayEffectHandle Handle) const;

	/** Gets the GE Handle of the GE that granted the passed in Ability */
	UE_API FActiveGameplayEffectHandle FindActiveGameplayEffectHandle(FGameplayAbilitySpecHandle Handle) const;

	/** Returns const pointer to the actual active gameplay effect structure */
	UE_API const FActiveGameplayEffect* GetActiveGameplayEffect(const FActiveGameplayEffectHandle Handle) const;

	/** Returns all active gameplay effects on this ASC */
	UE_API const FActiveGameplayEffectsContainer& GetActiveGameplayEffects() const;

	/** Returns a const pointer to the gameplay effect CDO associated with an active handle. */
	UE_API const UGameplayEffect* GetGameplayEffectCDO(const FActiveGameplayEffectHandle Handle) const;

	/**
	 * Get the source tags from the gameplay spec represented by the specified handle, if possible
	 * 
	 * @param Handle	Handle of the gameplay effect to retrieve source tags from
	 * 
	 * @return Source tags from the gameplay spec represented by the handle, if possible
	 */
	UE_API const FGameplayTagContainer* GetGameplayEffectSourceTagsFromHandle(FActiveGameplayEffectHandle Handle) const;

	/**
	 * Get the target tags from the gameplay spec represented by the specified handle, if possible
	 * 
	 * @param Handle	Handle of the gameplay effect to retrieve target tags from
	 * 
	 * @return Target tags from the gameplay spec represented by the handle, if possible
	 */
	UE_API const FGameplayTagContainer* GetGameplayEffectTargetTagsFromHandle(FActiveGameplayEffectHandle Handle) const;

	/**
	 * Populate the specified capture spec with the data necessary to capture an attribute from the component
	 * 
	 * @param OutCaptureSpec	[OUT] Capture spec to populate with captured data
	 */
	UE_API void CaptureAttributeForGameplayEffect(OUT FGameplayEffectAttributeCaptureSpec& OutCaptureSpec);
	
	// ----------------------------------------------------------------------------------------------------------------
	//  Callbacks / Notifies
	//  (these need to be at the UObject level so we can safely bind, rather than binding to raw at the ActiveGameplayEffect/Container level which is unsafe if the AbilitySystemComponent were killed).
	// ----------------------------------------------------------------------------------------------------------------

	/** Called when a specific attribute aggregator value changes, gameplay effects refresh their values when this happens */
	UE_API void OnAttributeAggregatorDirty(FAggregator* Aggregator, FGameplayAttribute Attribute, bool FromRecursiveCall=false);

	/** Called when attribute magnitudes change, to forward information to dependent gameplay effects */
	UE_API void OnMagnitudeDependencyChange(FActiveGameplayEffectHandle Handle, const FAggregator* ChangedAggregator);

	/** This ASC has successfully applied a GE to something (potentially itself) */
	UE_API void OnGameplayEffectAppliedToTarget(UAbilitySystemComponent* Target, const FGameplayEffectSpec& SpecApplied, FActiveGameplayEffectHandle ActiveHandle);
	UE_API void OnGameplayEffectAppliedToSelf(UAbilitySystemComponent* Source, const FGameplayEffectSpec& SpecApplied, FActiveGameplayEffectHandle ActiveHandle);
	UE_API void OnPeriodicGameplayEffectExecuteOnTarget(UAbilitySystemComponent* Target, const FGameplayEffectSpec& SpecExecuted, FActiveGameplayEffectHandle ActiveHandle);
	UE_API void OnPeriodicGameplayEffectExecuteOnSelf(UAbilitySystemComponent* Source, const FGameplayEffectSpec& SpecExecuted, FActiveGameplayEffectHandle ActiveHandle);

	/** Called when the duration of a gameplay effect has changed */
	UE_API virtual void OnGameplayEffectDurationChange(struct FActiveGameplayEffect& ActiveEffect);

	/** Called on server whenever a GE is applied to self. This includes instant and duration based GEs. */
	FOnGameplayEffectAppliedDelegate OnGameplayEffectAppliedDelegateToSelf;

	/** Called on server whenever a GE is applied to someone else. This includes instant and duration based GEs. */
	FOnGameplayEffectAppliedDelegate OnGameplayEffectAppliedDelegateToTarget;

	/** Called on both client and server whenever a duration based GE is added (E.g., instant GEs do not trigger this). */
	FOnGameplayEffectAppliedDelegate OnActiveGameplayEffectAddedDelegateToSelf;

	/** Called on server whenever a periodic GE executes on self */
	FOnGameplayEffectAppliedDelegate OnPeriodicGameplayEffectExecuteDelegateOnSelf;

	/** Called on server whenever a periodic GE executes on target */
	FOnGameplayEffectAppliedDelegate OnPeriodicGameplayEffectExecuteDelegateOnTarget;

	/** Immunity notification support */
	FImmunityBlockGE OnImmunityBlockGameplayEffectDelegate;

	/** Register for when an attribute value changes, should be replaced by GetGameplayAttributeValueChangeDelegate */
	UE_API FOnGameplayAttributeChange& RegisterGameplayAttributeEvent(FGameplayAttribute Attribute);

	/** Register for when an attribute value changes */
	UE_API FOnGameplayAttributeValueChange& GetGameplayAttributeValueChangeDelegate(FGameplayAttribute Attribute);

	/** A generic callback anytime an ability is activated (started) */
	FGenericAbilityDelegate AbilityActivatedCallbacks;

	/** Callback anytime an ability is ended */
	FAbilityEnded AbilityEndedCallbacks;

	/** Callback anytime an ability is ended, with extra information */
	FGameplayAbilityEndedDelegate OnAbilityEnded;

	/** A generic callback anytime an ability is committed (cost/cooldown applied) */
	FGenericAbilityDelegate AbilityCommittedCallbacks;

	/** Called with a failure reason when an ability fails to activate */
	FAbilityFailedDelegate AbilityFailedCallbacks;

	/** Called when an ability spec's internals have changed */
	FAbilitySpecDirtied AbilitySpecDirtiedCallbacks;

	/** We allow users to setup a series of functions that must be true in order to allow a GameplayEffect to be applied */
	TArray<FGameplayEffectApplicationQuery> GameplayEffectApplicationQueries;

	/** Call notify callbacks above */
	UE_API virtual void NotifyAbilityCommit(UGameplayAbility* Ability);
	UE_API virtual void NotifyAbilityActivated(const FGameplayAbilitySpecHandle Handle, UGameplayAbility* Ability);
	UE_API virtual void NotifyAbilityFailed(const FGameplayAbilitySpecHandle Handle, UGameplayAbility* Ability, const FGameplayTagContainer& FailureReason);

	/** Called when any gameplay effects are removed */
	UE_API FOnGivenActiveGameplayEffectRemoved& OnAnyGameplayEffectRemovedDelegate();

	/** Returns delegate structure that allows binding to several gameplay effect changes */
	UE_API FActiveGameplayEffectEvents* GetActiveEffectEventSet(FActiveGameplayEffectHandle Handle);
	UE_API FOnActiveGameplayEffectRemoved_Info* OnGameplayEffectRemoved_InfoDelegate(FActiveGameplayEffectHandle Handle);
	UE_API FOnActiveGameplayEffectStackChange* OnGameplayEffectStackChangeDelegate(FActiveGameplayEffectHandle Handle);
	UE_API FOnActiveGameplayEffectTimeChange* OnGameplayEffectTimeChangeDelegate(FActiveGameplayEffectHandle Handle);
	UE_API FOnActiveGameplayEffectInhibitionChanged* OnGameplayEffectInhibitionChangedDelegate(FActiveGameplayEffectHandle Handle);

	// ----------------------------------------------------------------------------------------------------------------
	//  Gameplay tag operations
	//  Implements IGameplayTagAssetInterface using the TagCountContainer
	// ----------------------------------------------------------------------------------------------------------------
	inline bool HasMatchingGameplayTag(FGameplayTag TagToCheck) const override
	{
		return GameplayTagCountContainer.HasMatchingGameplayTag(TagToCheck);
	}

	inline bool HasAllMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const override
	{
		return GameplayTagCountContainer.HasAllMatchingGameplayTags(TagContainer);
	}

	inline bool HasAnyMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const override
	{
		return GameplayTagCountContainer.HasAnyMatchingGameplayTags(TagContainer);
	}

	inline void GetOwnedGameplayTags(FGameplayTagContainer& TagContainer) const override
	{
		TagContainer.Reset();
		TagContainer.AppendTags(GetOwnedGameplayTags());
	}

	[[nodiscard]] inline const FGameplayTagContainer& GetOwnedGameplayTags() const
	{
		return GameplayTagCountContainer.GetExplicitGameplayTags();
	}

	/** Checks whether the query matches the owned GameplayTags */
	inline bool MatchesGameplayTagQuery(const FGameplayTagQuery& TagQuery) const
	{
		return TagQuery.Matches(GameplayTagCountContainer.GetExplicitGameplayTags());
	}

	/** Returns the number of instances of a given tag */
	inline int32 GetTagCount(FGameplayTag TagToCheck) const
	{
		return GameplayTagCountContainer.GetTagCount(TagToCheck);
	}

	/** Forcibly sets the number of instances of a given tag */
	inline void SetTagMapCount(const FGameplayTag& Tag, int32 NewCount, EGameplayTagReplicationState TagRepState = EGameplayTagReplicationState::None)
	{
		const int32 CurrentCount = GameplayTagCountContainer.GetExplicitTagCount(Tag);
		UpdateTagMapSingle_Internal(Tag, NewCount - CurrentCount, TagRepState);
	}

	/** Update the number of instances of a given tag and calls callback */
	inline void UpdateTagMap(const FGameplayTag& BaseTag, int32 CountDelta, EGameplayTagReplicationState TagRepState)
	{
		UpdateTagMapSingle_Internal(BaseTag, CountDelta, TagRepState);
	}

	/** Update the number of instances of a given tag and calls callback */
	inline void UpdateTagMap(const FGameplayTagContainer& Container, int32 CountDelta, EGameplayTagReplicationState TagRepState)
	{
		if (!Container.IsEmpty())
		{
			UpdateTagMap_Internal(Container, CountDelta, TagRepState);
		}
	}

	/** Fills TagContainer with BlockedAbilityTags */
	inline void GetBlockedAbilityTags(FGameplayTagContainer& TagContainer) const
	{
		TagContainer.AppendTags(GetBlockedAbilityTags());
	}

	[[nodiscard]] inline const FGameplayTagContainer GetBlockedAbilityTags() const
	{
		return BlockedAbilityTags.GetExplicitGameplayTags();
	}

	/** 	 
	 *  Allows GameCode to add loose gameplaytags which are not backed by a GameplayEffect.
	 *	It is up to the calling GameCode to make sure these tags are added on clients/server where necessary
	 */
	inline void AddLooseGameplayTag(const FGameplayTag& GameplayTag, int32 Count=1, EGameplayTagReplicationState TagRepState = EGameplayTagReplicationState::None)
	{
		UpdateTagMap(GameplayTag, Count, TagRepState);
	}

	inline void AddLooseGameplayTags(const FGameplayTagContainer& GameplayTags, int32 Count = 1, EGameplayTagReplicationState TagRepState = EGameplayTagReplicationState::None)
	{
		UpdateTagMap(GameplayTags, Count, TagRepState);
	}

	inline void RemoveLooseGameplayTag(const FGameplayTag& GameplayTag, int32 Count = 1, EGameplayTagReplicationState TagRepState = EGameplayTagReplicationState::None)
	{
		UpdateTagMap(GameplayTag, -Count, TagRepState);
	}

	inline void RemoveLooseGameplayTags(const FGameplayTagContainer& GameplayTags, int32 Count = 1, EGameplayTagReplicationState TagRepState = EGameplayTagReplicationState::None)
	{
		UpdateTagMap(GameplayTags, -Count, TagRepState);
	}

	inline void SetLooseGameplayTagCount(const FGameplayTag& GameplayTag, int32 NewCount, EGameplayTagReplicationState TagRepState = EGameplayTagReplicationState::None)
	{
		SetTagMapCount(GameplayTag, NewCount, TagRepState);
	}

	/**
	 * Returns the current count of the given gameplay tag.
	 * This includes both loose tags, and tags granted by gameplay effects and abilities.
	 * This function can be called on the client, but it may not display the most current count on the server.
	 *
	 * @param GameplayTag The gameplay tag to query
	 */
	UFUNCTION(BlueprintPure, Category = "Gameplay Tags")
	UE_API int32 GetGameplayTagCount(FGameplayTag GameplayTag) const;

	/** 	 
	 * Minimally replicated tags are replicated tags that come from GEs when in bMinimalReplication mode. 
	 * (The GEs do not replicate, but the tags they grant do replicate via these functions)
	 */
	inline void AddMinimalReplicationGameplayTag(const FGameplayTag& GameplayTag)
	{
		UpdateTagMap(GameplayTag, 1, EGameplayTagReplicationState::TagOnly);
		GetMinimalReplicationTags_Mutable().AddTag(GameplayTag);
	}

	inline void AddMinimalReplicationGameplayTags(const FGameplayTagContainer& GameplayTags)
	{
		UpdateTagMap(GameplayTags, 1, EGameplayTagReplicationState::TagOnly);
		GetMinimalReplicationTags_Mutable().AddTags(GameplayTags);
	}

	inline void RemoveMinimalReplicationGameplayTag(const FGameplayTag& GameplayTag)
	{
		UpdateTagMap(GameplayTag, -1, EGameplayTagReplicationState::TagOnly);
		GetMinimalReplicationTags_Mutable().RemoveTag(GameplayTag);
	}

	inline void RemoveMinimalReplicationGameplayTags(const FGameplayTagContainer& GameplayTags)
	{
		UpdateTagMap(GameplayTags, -1, EGameplayTagReplicationState::TagOnly);
		GetMinimalReplicationTags_Mutable().RemoveTags(GameplayTags);
	}

	/** Allow events to be registered for specific gameplay tags being added or removed */
	UE_API FOnGameplayEffectTagCountChanged& RegisterGameplayTagEvent(FGameplayTag Tag, EGameplayTagEventType::Type EventType=EGameplayTagEventType::NewOrRemoved);

	/** Unregister previously added events */
	UE_API bool UnregisterGameplayTagEvent(FDelegateHandle DelegateHandle, FGameplayTag Tag, EGameplayTagEventType::Type EventType=EGameplayTagEventType::NewOrRemoved);

	/** Register a tag event and immediately call it */
	UE_API FDelegateHandle RegisterAndCallGameplayTagEvent(FGameplayTag Tag, FOnGameplayEffectTagCountChanged::FDelegate Delegate, EGameplayTagEventType::Type EventType=EGameplayTagEventType::NewOrRemoved);

	/** Returns multicast delegate that is invoked whenever a tag is added or removed (but not if just count is increased. Only for 'new' and 'removed' events) */
	UE_API FOnGameplayEffectTagCountChanged& RegisterGenericGameplayTagEvent();

	/** Executes a gameplay event. Returns the number of successful ability activations triggered by the event */
	UE_API virtual int32 HandleGameplayEvent(FGameplayTag EventTag, const FGameplayEventData* Payload);

	/** Adds a new delegate to call when gameplay events happen. It will only be called if it matches any tags in passed filter container */
	UE_API FDelegateHandle AddGameplayEventTagContainerDelegate(const FGameplayTagContainer& TagFilter, const FGameplayEventTagMulticastDelegate::FDelegate& Delegate);

	/** Remotes previously registered delegate */
	UE_API void RemoveGameplayEventTagContainerDelegate(const FGameplayTagContainer& TagFilter, FDelegateHandle DelegateHandle);

	/** Callbacks bound to Gameplay tags, these only activate if the exact tag is used. To handle tag hierarchies use AddGameplayEventContainerDelegate */
	TMap<FGameplayTag, FGameplayEventMulticastDelegate> GenericGameplayEventCallbacks;

	// ----------------------------------------------------------------------------------------------------------------
	//  System Attributes
	// ----------------------------------------------------------------------------------------------------------------

	/** Internal Attribute that modifies the duration of gameplay effects created by this component */
	UPROPERTY(meta=(SystemGameplayAttribute="true"))
	float OutgoingDuration;

	/** Internal Attribute that modifies the duration of gameplay effects applied to this component */
	UPROPERTY(meta = (SystemGameplayAttribute = "true"))
	float IncomingDuration;

	static UE_API FProperty* GetOutgoingDurationProperty();
	static UE_API FProperty* GetIncomingDurationProperty();

	static UE_API const FGameplayEffectAttributeCaptureDefinition& GetOutgoingDurationCapture();
	static UE_API const FGameplayEffectAttributeCaptureDefinition& GetIncomingDurationCapture();

	// ----------------------------------------------------------------------------------------------------------------
	//  Additional Helper Functions
	// ----------------------------------------------------------------------------------------------------------------

	/** Apply a gameplay effect to passed in target */
	UFUNCTION(BlueprintCallable, Category = GameplayEffects, meta=(DisplayName = "ApplyGameplayEffectToTarget", ScriptName = "ApplyGameplayEffectToTarget"))
	UE_API FActiveGameplayEffectHandle BP_ApplyGameplayEffectToTarget(TSubclassOf<UGameplayEffect> GameplayEffectClass, UAbilitySystemComponent *Target, float Level, FGameplayEffectContextHandle Context);
	UE_API FActiveGameplayEffectHandle ApplyGameplayEffectToTarget(UGameplayEffect *GameplayEffect, UAbilitySystemComponent *Target, float Level = UGameplayEffect::INVALID_LEVEL, FGameplayEffectContextHandle Context = FGameplayEffectContextHandle(), FPredictionKey PredictionKey = FPredictionKey());

	/** Apply a gameplay effect to self */
	UFUNCTION(BlueprintCallable, Category = GameplayEffects, meta=(DisplayName = "ApplyGameplayEffectToSelf", ScriptName = "ApplyGameplayEffectToSelf"))
	UE_API FActiveGameplayEffectHandle BP_ApplyGameplayEffectToSelf(TSubclassOf<UGameplayEffect> GameplayEffectClass, float Level, FGameplayEffectContextHandle EffectContext);
	UE_API FActiveGameplayEffectHandle ApplyGameplayEffectToSelf(const UGameplayEffect *GameplayEffect, float Level, const FGameplayEffectContextHandle& EffectContext, FPredictionKey PredictionKey = FPredictionKey());

	/** Returns the number of gameplay effects that are currently active on this ability system component */
	int32 GetNumActiveGameplayEffects() const
	{
		return ActiveGameplayEffects.GetNumGameplayEffects();
	}

	/** Makes a copy of all the active effects on this ability component */
	void GetAllActiveGameplayEffectSpecs(TArray<FGameplayEffectSpec>& OutSpecCopies) const
	{
		ActiveGameplayEffects.GetAllActiveGameplayEffectSpecs(OutSpecCopies);
	}

	/** Call from OnRep functions to set the attribute base value on the client */
	void SetBaseAttributeValueFromReplication(const FGameplayAttribute& Attribute, float NewValue, float OldValue)
	{
		ActiveGameplayEffects.SetBaseAttributeValueFromReplication(Attribute, FGameplayAttributeData(NewValue), FGameplayAttributeData(OldValue) );
	}

	/** Call from OnRep functions to set the attribute base value on the client */
	void SetBaseAttributeValueFromReplication(const FGameplayAttribute& Attribute, const FGameplayAttributeData& NewValue, const FGameplayAttributeData& OldValue)
	{
		ActiveGameplayEffects.SetBaseAttributeValueFromReplication(Attribute, NewValue, OldValue);
	}

	/** Tests if all modifiers in this GameplayEffect will leave the attribute > 0.f */
	bool CanApplyAttributeModifiers(const UGameplayEffect *GameplayEffect, float Level, const FGameplayEffectContextHandle& EffectContext)
	{
		return ActiveGameplayEffects.CanApplyAttributeModifiers(GameplayEffect, Level, EffectContext);
	}

	/** Gets time remaining for all effects that match query */
	UE_API TArray<float> GetActiveEffectsTimeRemaining(const FGameplayEffectQuery& Query) const;

	/** Gets total duration for all effects that match query */
	UE_API TArray<float> GetActiveEffectsDuration(const FGameplayEffectQuery& Query) const;

	/** Gets both time remaining and total duration  for all effects that match query */
	UE_API TArray<TPair<float,float>> GetActiveEffectsTimeRemainingAndDuration(const FGameplayEffectQuery& Query) const;

	/** Returns list of active effects, for a query */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "GameplayEffects", meta=(DisplayName = "Get Active Gameplay Effects for Query"))
	UE_API TArray<FActiveGameplayEffectHandle> GetActiveEffects(const FGameplayEffectQuery& Query) const;

	/** Returns list of active effects that have all of the passed in tags */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "GameplayEffects")
	UE_API TArray<FActiveGameplayEffectHandle> GetActiveEffectsWithAllTags(FGameplayTagContainer Tags) const;

	/** This will give the world time that all effects matching this query will be finished. If multiple effects match, it returns the one that returns last */
	UE_API float GetActiveEffectsEndTime(const FGameplayEffectQuery& Query) const;
	UE_API float GetActiveEffectsEndTimeWithInstigators(const FGameplayEffectQuery& Query, TArray<AActor*>& Instigators) const;

	/** Returns end time and total duration */
	UE_API bool GetActiveEffectsEndTimeAndDuration(const FGameplayEffectQuery& Query, float& EndTime, float& Duration) const;

	/** Modify the start time of a gameplay effect, to deal with timers being out of sync originally */
	UE_API virtual void ModifyActiveEffectStartTime(FActiveGameplayEffectHandle Handle, float StartTimeDiff);

	/** Removes all active effects that contain any of the tags in Tags */
	UFUNCTION(BlueprintCallable, Category = GameplayEffects)
	UE_API int32 RemoveActiveEffectsWithTags(FGameplayTagContainer Tags);

	/** Removes all active effects with captured source tags that contain any of the tags in Tags */
	UFUNCTION(BlueprintCallable, Category = GameplayEffects)
	UE_API int32 RemoveActiveEffectsWithSourceTags(FGameplayTagContainer Tags);

	/** Removes all active effects that apply any of the tags in Tags */
	UFUNCTION(BlueprintCallable, Category = GameplayEffects)
	UE_API int32 RemoveActiveEffectsWithAppliedTags(FGameplayTagContainer Tags);

	/** Removes all active effects that grant any of the tags in Tags */
	UFUNCTION(BlueprintCallable, Category = GameplayEffects)
	UE_API int32 RemoveActiveEffectsWithGrantedTags(FGameplayTagContainer Tags);

	/** Removes all active effects that match given query. StacksToRemove=-1 will remove all stacks. */
	UE_API virtual int32 RemoveActiveEffects(const FGameplayEffectQuery& Query, int32 StacksToRemove = -1);

	// ----------------------------------------------------------------------------------------------------------------
	//	GameplayCues
	// ----------------------------------------------------------------------------------------------------------------
	
	// Do not call these functions directly, call the wrappers on GameplayCueManager instead
	UFUNCTION(NetMulticast, unreliable)
	UE_API void NetMulticast_InvokeGameplayCueExecuted_FromSpec(const FGameplayEffectSpecForRPC Spec, FPredictionKey PredictionKey) override;

	UFUNCTION(NetMulticast, unreliable)
	UE_API void NetMulticast_InvokeGameplayCueExecuted(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext) override;

	UFUNCTION(NetMulticast, unreliable)
	UE_API void NetMulticast_InvokeGameplayCuesExecuted(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext) override;

	UFUNCTION(NetMulticast, unreliable)
	UE_API void NetMulticast_InvokeGameplayCueExecuted_WithParams(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters) override;

	UFUNCTION(NetMulticast, unreliable)
	UE_API void NetMulticast_InvokeGameplayCuesExecuted_WithParams(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters) override;

	UFUNCTION(NetMulticast, unreliable)
	UE_API void NetMulticast_InvokeGameplayCueAdded(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext) override;

	UFUNCTION(NetMulticast, unreliable)
	UE_API void NetMulticast_InvokeGameplayCueAdded_WithParams(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters Parameters) override;
	
	UFUNCTION(NetMulticast, unreliable)
	UE_API void NetMulticast_InvokeGameplayCueAddedAndWhileActive_FromSpec(const FGameplayEffectSpecForRPC& Spec, FPredictionKey PredictionKey) override;

	UFUNCTION(NetMulticast, unreliable)
	UE_API void NetMulticast_InvokeGameplayCueAddedAndWhileActive_WithParams(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters) override;

	UFUNCTION(NetMulticast, unreliable)
	UE_API void NetMulticast_InvokeGameplayCuesAddedAndWhileActive_WithParams(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters) override;

	/** GameplayCues can also come on their own. These take an optional effect context to pass through hit result, etc */
	UE_API void ExecuteGameplayCue(const FGameplayTag GameplayCueTag, FGameplayEffectContextHandle EffectContext = FGameplayEffectContextHandle());
	UE_API void ExecuteGameplayCue(const FGameplayTag GameplayCueTag, const FGameplayCueParameters& GameplayCueParameters);

	/** Add a persistent gameplay cue */
	UE_API void AddGameplayCue(const FGameplayTag GameplayCueTag, FGameplayEffectContextHandle EffectContext = FGameplayEffectContextHandle());
	UE_API void AddGameplayCue(const FGameplayTag GameplayCueTag, const FGameplayCueParameters& GameplayCueParameters);

	/** Add gameplaycue for minimal replication mode. Should only be called in paths that would replicate gameplaycues in other ways (through GE for example) if not in minimal replication mode */
	UE_API void AddGameplayCue_MinimalReplication(const FGameplayTag GameplayCueTag, FGameplayEffectContextHandle EffectContext = FGameplayEffectContextHandle());

	/** Remove a persistent gameplay cue */
	UE_API void RemoveGameplayCue(const FGameplayTag GameplayCueTag);
	
	/** Remove gameplaycue for minimal replication mode. Should only be called in paths that would replicate gameplaycues in other ways (through GE for example) if not in minimal replication mode */
	UE_API void RemoveGameplayCue_MinimalReplication(const FGameplayTag GameplayCueTag);
	
	/** Removes any GameplayCue added on its own, i.e. not as part of a GameplayEffect. */
	UE_API void RemoveAllGameplayCues();

	/** Handles gameplay cue events from external sources */
	UE_API void InvokeGameplayCueEvent(const FGameplayEffectSpecForRPC& Spec, EGameplayCueEvent::Type EventType);
	UE_API void InvokeGameplayCueEvent(const FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, FGameplayEffectContextHandle EffectContext = FGameplayEffectContextHandle());
	UE_API void InvokeGameplayCueEvent(const FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& GameplayCueParameters);

	/** Allows polling to see if a GameplayCue is active. We expect most GameplayCue handling to be event based, but some cases we may need to check if a GameplayCue is active (Animation Blueprint for example) */
	UFUNCTION(BlueprintCallable, Category="GameplayCue", meta=(GameplayTagFilter="GameplayCue"))
	bool IsGameplayCueActive(const FGameplayTag GameplayCueTag) const
	{
		return HasMatchingGameplayTag(GameplayCueTag);
	}

	/** Will initialize gameplay cue parameters with this ASC's Owner (Instigator) and AvatarActor (EffectCauser) */
	UE_API virtual void InitDefaultGameplayCueParameters(FGameplayCueParameters& Parameters);

	/** Are we ready to invoke gameplaycues yet? */
	UE_API virtual bool IsReadyForGameplayCues();

	/** Handle GameplayCues that may have been deferred while doing the NetDeltaSerialize and waiting for the avatar actor to get loaded */
	UE_API virtual void HandleDeferredGameplayCues(const FActiveGameplayEffectsContainer* GameplayEffectsContainer);

	/**
	 *	GameplayAbilities
	 *	
	 *	The role of the AbilitySystemComponent with respect to Abilities is to provide:
	 *		-Management of ability instances (whether per actor or per execution instance).
	 *			-Someone *has* to keep track of these instances.
	 *			-Non instanced abilities *could* be executed without any ability stuff in AbilitySystemComponent.
	 *				They should be able to operate on an GameplayAbilityActorInfo + GameplayAbility.
	 *		
	 *	As convenience it may provide some other features:
	 *		-Some basic input binding (whether instanced or non instanced abilities).
	 *		-Concepts like "this component has these abilities
	 *	
	 */

	/*
	 * Grants an Ability.
	 * This will be ignored if the actor is not authoritative.
	 * Returns handle that can be used in TryActivateAbility, etc.
	 * 
	 * @param AbilitySpec FGameplayAbilitySpec containing information about the ability class, level and input ID to bind it to.
	 */
	UE_API FGameplayAbilitySpecHandle GiveAbility(const FGameplayAbilitySpec& AbilitySpec);

	/*
	 * Grants an ability and attempts to activate it exactly one time, which will cause it to be removed.
	 * Only valid on the server, and the ability's Net Execution Policy cannot be set to Local or Local Predicted
	 * 
	 * @param AbilitySpec FGameplayAbilitySpec containing information about the ability class, level and input ID to bind it to.
	 * @param GameplayEventData Optional activation event data. If provided, Activate Ability From Event will be called instead of ActivateAbility, passing the Event Data
	 */
	UE_API FGameplayAbilitySpecHandle GiveAbilityAndActivateOnce(FGameplayAbilitySpec& AbilitySpec, const FGameplayEventData* GameplayEventData = nullptr);

	/**
	 * Grants a Gameplay Ability and returns its handle.
	 * This will be ignored if the actor is not authoritative.
	 *
	 * @param AbilityClass Type of ability to grant
	 * @param Level Level to grant the ability at
	 * @param InputID Input ID value to bind ability activation to.
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "Gameplay Abilities", meta = (DisplayName = "Give Ability", ScriptName = "GiveAbility"))
	UE_API FGameplayAbilitySpecHandle K2_GiveAbility(TSubclassOf<UGameplayAbility> AbilityClass, int32 Level = 0, int32 InputID = -1);

	/**
	 * Grants a Gameplay Ability, activates it once, and removes it.
	 * This will be ignored if the actor is not authoritative.
	 *
	 * @param AbilityClass Type of ability to grant
	 * @param Level Level to grant the ability at
	 * @param InputID Input ID value to bind ability activation to.
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "Gameplay Abilities", meta = (DisplayName = "Give Ability And Activate Once", ScriptName = "GiveAbilityAndActivateOnce"))
	UE_API FGameplayAbilitySpecHandle K2_GiveAbilityAndActivateOnce(TSubclassOf<UGameplayAbility> AbilityClass, int32 Level = 0, int32 InputID = -1);

	/** Wipes all 'given' abilities. This will be ignored if the actor is not authoritative. */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category="Gameplay Abilities")
	UE_API void ClearAllAbilities();

	/**
	 * Clears all abilities bound to a given Input ID
	 * This will be ignored if the actor is not authoritative
	 *
	 * @param InputID The numeric Input ID of the abilities to remove
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "Gameplay Abilities")
	UE_API void ClearAllAbilitiesWithInputID(int32 InputID = 0);

	/** 
	 * Removes the specified ability.
	 * This will be ignored if the actor is not authoritative.
	 * 
	 * @param Handle Ability Spec Handle of the ability we want to remove
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "Gameplay Abilities")
	UE_API void ClearAbility(const FGameplayAbilitySpecHandle& Handle);
	
	/** Sets an ability spec to remove when its finished. If the spec is not currently active, it terminates it immediately. Also clears InputID of the Spec. */
	UE_API void SetRemoveAbilityOnEnd(FGameplayAbilitySpecHandle AbilitySpecHandle);

	/** 
	 * Gets all Activatable Gameplay Abilities that match all tags in GameplayTagContainer AND for which
	 * DoesAbilitySatisfyTagRequirements() is true.  The latter requirement allows this function to find the correct
	 * ability without requiring advanced knowledge.  For example, if there are two "Melee" abilities, one of which
	 * requires a weapon and one of which requires being unarmed, then those abilities can use Blocking and Required
	 * tags to determine when they can fire.  Using the Satisfying Tags requirements simplifies a lot of usage cases.
	 * For example, Behavior Trees can use various decorators to test an ability fetched using this mechanism as well
	 * as the Task to execute the ability without needing to know that there even is more than one such ability.
	 */
	UE_API void GetActivatableGameplayAbilitySpecsByAllMatchingTags(const FGameplayTagContainer& GameplayTagContainer, TArray < struct FGameplayAbilitySpec* >& MatchingGameplayAbilities, bool bOnlyAbilitiesThatSatisfyTagRequirements = true) const;

	/** 
	 * Attempts to activate every gameplay ability that matches the given tag and DoesAbilitySatisfyTagRequirements().
	 * Returns true if anything attempts to activate. Can activate more than one ability and the ability may fail later.
	 * If bAllowRemoteActivation is true, it will remotely activate local/server abilities, if false it will only try to locally activate abilities.
	 */
	UFUNCTION(BlueprintCallable, Category = "Abilities")
	UE_API bool TryActivateAbilitiesByTag(const FGameplayTagContainer& GameplayTagContainer, bool bAllowRemoteActivation = true);

	/**
	 * Attempts to activate the ability that is passed in. This will check costs and requirements before doing so.
	 * Returns true if it thinks it activated, but it may return false positives due to failure later in activation.
	 * If bAllowRemoteActivation is true, it will remotely activate local/server abilities, if false it will only try to locally activate the ability
	 */
	UFUNCTION(BlueprintCallable, Category = "Abilities")
	UE_API bool TryActivateAbilityByClass(TSubclassOf<UGameplayAbility> InAbilityToActivate, bool bAllowRemoteActivation = true);

	/** 
	 * Attempts to activate the given ability, will check costs and requirements before doing so.
	 * Returns true if it thinks it activated, but it may return false positives due to failure later in activation.
	 * If bAllowRemoteActivation is true, it will remotely activate local/server abilities, if false it will only try to locally activate the ability
	 */
	UFUNCTION(BlueprintCallable, Category = "Abilities")
	UE_API bool TryActivateAbility(FGameplayAbilitySpecHandle AbilityToActivate, bool bAllowRemoteActivation = true);

	UE_API bool HasActivatableTriggeredAbility(FGameplayTag Tag);

	/** Triggers an ability from a gameplay event, will only trigger on local/server depending on execution flags */
	UE_API bool TriggerAbilityFromGameplayEvent(FGameplayAbilitySpecHandle AbilityToTrigger, FGameplayAbilityActorInfo* ActorInfo, FGameplayTag Tag, const FGameplayEventData* Payload, UAbilitySystemComponent& Component);

	// ----------------------------------------------------------------------------------------------------------------
	// Ability Cancelling/Interrupts
	// ----------------------------------------------------------------------------------------------------------------

	/** Cancels the specified ability CDO. */
	UE_API void CancelAbility(UGameplayAbility* Ability);	

	/** Cancels the ability indicated by passed in spec handle. If handle is not found among reactivated abilities nothing happens. */
	UE_API void CancelAbilityHandle(const FGameplayAbilitySpecHandle& AbilityHandle);

	/** Cancel all abilities with the specified tags. Will not cancel the Ignore instance */
	UE_API void CancelAbilities(const FGameplayTagContainer* WithTags=nullptr, const FGameplayTagContainer* WithoutTags=nullptr, UGameplayAbility* Ignore=nullptr);

	/** Cancels all abilities regardless of tags. Will not cancel the ignore instance */
	UE_API void CancelAllAbilities(UGameplayAbility* Ignore=nullptr);

	/** Cancels all abilities and kills any remaining instanced abilities */
	UE_API virtual void DestroyActiveState();

	/** 
	 * Called from ability activation or native code, will apply the correct ability blocking tags and cancel existing abilities. Subclasses can override the behavior 
	 * 
	 * @param AbilityTags The tags of the ability that has block and cancel flags
	 * @param RequestingAbility The gameplay ability requesting the change, can be NULL for native events
	 * @param bEnableBlockTags If true will enable the block tags, if false will disable the block tags
	 * @param BlockTags What tags to block
	 * @param bExecuteCancelTags If true will cancel abilities matching tags
	 * @param CancelTags what tags to cancel
	 */
	UE_API virtual void ApplyAbilityBlockAndCancelTags(const FGameplayTagContainer& AbilityTags, UGameplayAbility* RequestingAbility, bool bEnableBlockTags, const FGameplayTagContainer& BlockTags, bool bExecuteCancelTags, const FGameplayTagContainer& CancelTags);

	/** Called when an ability is cancellable or not. Doesn't do anything by default, can be overridden to tie into gameplay events */
	virtual void HandleChangeAbilityCanBeCanceled(const FGameplayTagContainer& AbilityTags, UGameplayAbility* RequestingAbility, bool bCanBeCanceled) {}

	/** Returns true if any passed in tags are blocked */
	UFUNCTION(BlueprintCallable, Category="Abilities")
	UE_API virtual bool AreAbilityTagsBlocked(const FGameplayTagContainer& Tags) const;

	/** Block or cancel blocking for specific ability tags */
	UE_API void BlockAbilitiesWithTags(const FGameplayTagContainer& Tags);
	UE_API void UnBlockAbilitiesWithTags(const FGameplayTagContainer& Tags);

	/** Checks if the ability system is currently blocking InputID. Returns true if InputID is blocked, false otherwise.  */
	UE_API bool IsAbilityInputBlocked(int32 InputID) const;

	/** Block or cancel blocking for specific input IDs */
	UE_API void BlockAbilityByInputID(int32 InputID);
	UE_API void UnBlockAbilityByInputID(int32 InputID);

	// ----------------------------------------------------------------------------------------------------------------
	// Functions meant to be called from GameplayAbility and subclasses, but not meant for general use
	// ----------------------------------------------------------------------------------------------------------------

	/** Returns the list of all activatable abilities. Read-only. */
	const TArray<FGameplayAbilitySpec>& GetActivatableAbilities() const
	{
		return ActivatableAbilities.Items;
	}

	/** Returns the list of all activatable abilities. */
	TArray<FGameplayAbilitySpec>& GetActivatableAbilities()
	{
		return ActivatableAbilities.Items;
	}

	/** Returns local world time that an ability was activated. Valid on authority (server) and autonomous proxy (controlling client).  */
	float GetAbilityLastActivatedTime() const { return AbilityLastActivatedTime; }

	/** Returns an ability spec from a handle. If modifying call MarkAbilitySpecDirty. Treat the return value as ephemeral as the pointer will potentially be invalidated on any subsequent call into AbilitySystemComponent. */
	UE_API FGameplayAbilitySpec* FindAbilitySpecFromHandle(FGameplayAbilitySpecHandle Handle, EConsiderPending ConsiderPending = EConsiderPending::PendingRemove) const;
	
	/**
	* Returns all ability spec handles granted from a GE handle. Only the server may call this function.
	* @param ScopeLock - The lock to communicate to the caller that the return value is only valid for as long as this lock is in scope
	* @param Handle - The handle of the Active Gameplay Effect which granted the abilities we are looking for
	* @param ConsiderPending - Are we returning AbilitySpecs that are pending for addition/removal?
	*/
	UE_API TArray<const FGameplayAbilitySpec*> FindAbilitySpecsFromGEHandle(const FScopedAbilityListLock& ScopeLock, FActiveGameplayEffectHandle Handle, EConsiderPending ConsiderPending = EConsiderPending::PendingRemove) const;

	/** Returns an ability spec corresponding to given ability class. If modifying call MarkAbilitySpecDirty */
	UE_API FGameplayAbilitySpec* FindAbilitySpecFromClass(TSubclassOf<UGameplayAbility> InAbilityClass) const;

	/** Returns an ability spec from a handle. If modifying call MarkAbilitySpecDirty */
	UE_API FGameplayAbilitySpec* FindAbilitySpecFromInputID(int32 InputID) const;

	/**
	 * Returns all abilities with the given InputID
	 *
	 * @param InputID The Input ID to match
	 * @param OutAbilitySpecs Array of pointers to matching specs
	 */
	UE_API virtual void FindAllAbilitySpecsFromInputID(int32 InputID, TArray<const FGameplayAbilitySpec*>& OutAbilitySpecs) const;

	/**
	 * Build a simple FGameplayAbilitySpec from class, level and optional Input ID
	 */
	UE_API virtual FGameplayAbilitySpec BuildAbilitySpecFromClass(TSubclassOf<UGameplayAbility> AbilityClass, int32 Level = 0, int32 InputID = -1);

	/**
	 * Returns an array with all granted ability handles
	 * NOTE: currently this doesn't include abilities that are mid-activation
	 * 
	 * @param OutAbilityHandles This array will be filled with the granted Ability Spec Handles
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Gameplay Abilities")
	UE_API void GetAllAbilities(TArray<FGameplayAbilitySpecHandle>& OutAbilityHandles) const;

	/**
	 * Returns an array with all abilities that match the provided tags
	 *
	 * @param OutAbilityHandles This array will be filled with matching Ability Spec Handles
	 * @param Tags Gameplay Tags to match
	 * @param bExactMatch If true, tags must be matched exactly. Otherwise, abilities matching any of the tags will be returned
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Gameplay Abilities")
	UE_API void FindAllAbilitiesWithTags(TArray<FGameplayAbilitySpecHandle>& OutAbilityHandles, FGameplayTagContainer Tags, bool bExactMatch = true) const;

	/**
	 * Returns an array with all abilities that match the provided Gameplay Tag Query
	 *
	 * @param OutAbilityHandles This array will be filled with matching Ability Spec Handles
	 * @param Query Gameplay Tag Query to match
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Gameplay Abilities")
	UE_API void FindAllAbilitiesMatchingQuery(TArray<FGameplayAbilitySpecHandle>& OutAbilityHandles, FGameplayTagQuery Query) const;

	/**
	 * Returns an array with all abilities bound to an Input ID value
	 *
	 * @param OutAbilityHandles This array will be filled with matching Ability Spec Handles
	 * @param InputID The Input ID to match
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Gameplay Abilities")
	UE_API void FindAllAbilitiesWithInputID(TArray<FGameplayAbilitySpecHandle>& OutAbilityHandles, int32 InputID = 0) const;

	/** Retrieves the EffectContext of the GameplayEffect of the active GameplayEffect. */
	UE_API FGameplayEffectContextHandle GetEffectContextFromActiveGEHandle(FActiveGameplayEffectHandle Handle);

	/** Call to mark that an ability spec has been modified */
	UE_API void MarkAbilitySpecDirty(FGameplayAbilitySpec& Spec, bool WasAddOrRemove=false);

	/** Attempts to activate the given ability, will only work if called from the correct client/server context */
	UE_API bool InternalTryActivateAbility(FGameplayAbilitySpecHandle AbilityToActivate, FPredictionKey InPredictionKey = FPredictionKey(), UGameplayAbility ** OutInstancedAbility = nullptr, FOnGameplayAbilityEnded::FDelegate* OnGameplayAbilityEndedDelegate = nullptr, const FGameplayEventData* TriggerEventData = nullptr);

	/** Failure tags used by InternalTryActivateAbility (E.g., this stores the  FailureTags of the last call to InternalTryActivateAbility */
	FGameplayTagContainer InternalTryActivateAbilityFailureTags;

	/** Called from the ability to let the component know it is ended */
	UE_API virtual void NotifyAbilityEnded(FGameplayAbilitySpecHandle Handle, UGameplayAbility* Ability, bool bWasCancelled);

	UE_API void ClearAbilityReplicatedDataCache(FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActivationInfo& ActivationInfo);

	/** Called from FScopedAbilityListLock */
	UE_API void IncrementAbilityListLock();
	UE_API void DecrementAbilityListLock();

	// ----------------------------------------------------------------------------------------------------------------
	// Debugging
	// ----------------------------------------------------------------------------------------------------------------

	struct FAbilitySystemComponentDebugInfo
	{
		FAbilitySystemComponentDebugInfo()
		{
			FMemory::Memzero(*this);
		}

		class UCanvas* Canvas;

		bool bPrintToLog;

		bool bShowAttributes;
		bool bShowGameplayEffects;;
		bool bShowAbilities;

		float XPos;
		float YPos;
		float OriginalX;
		float OriginalY;
		float MaxY;
		float NewColumnYPadding;
		float YL;

		bool Accumulate;
		TArray<FString>	Strings;

		int32 GameFlags; // arbitrary flags for games to set/read in Debug_Internal
	};

	static UE_API void OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

	UE_API virtual void DisplayDebug(class UCanvas* Canvas, const class FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos);
	UE_API virtual void PrintDebug();

	UE_API void AccumulateScreenPos(FAbilitySystemComponentDebugInfo& Info);
	UE_API virtual void Debug_Internal(struct FAbilitySystemComponentDebugInfo& Info);
	UE_API void DebugLine(struct FAbilitySystemComponentDebugInfo& Info, FString Str, float XOffset, float YOffset, int32 MinTextRowsToAdvance = 0);
	UE_API FString CleanupName(FString Str);

	/** Print a debug list of all gameplay effects */
	UE_API void PrintAllGameplayEffects() const;

	/** Ask the server to send ability system debug information back to the client, via ClientPrintDebug_Response  */
	UFUNCTION(Server, reliable, WithValidation)
	UE_API void ServerPrintDebug_Request();

	/** Same as ServerPrintDebug_Request but this includes the client debug strings so that the server can embed them in replays */
	UFUNCTION(Server, reliable, WithValidation)
	UE_API void ServerPrintDebug_RequestWithStrings(const TArray<FString>& Strings);

	/** Virtual function games can override to do their own stuff when either ServerPrintDebug function runs on the server */
	UE_API virtual void OnServerPrintDebug_Request();

	/** Determines whether to call ServerPrintDebug_Request or ServerPrintDebug_RequestWithStrings.   */
	UE_API virtual bool ShouldSendClientDebugStringsToServer() const;

	UFUNCTION(Client, reliable)
	UE_API void ClientPrintDebug_Response(const TArray<FString>& Strings, int32 GameFlags);
	UE_API virtual void OnClientPrintDebug_Response(const TArray<FString>& Strings, int32 GameFlags);

#if ENABLE_VISUAL_LOG
	UE_API void ClearDebugInstantEffects();
	
	UE_API virtual void GrabDebugSnapshot(FVisualLogEntry* Snapshot) const override;
#endif // ENABLE_VISUAL_LOG

private:
	UPROPERTY(ReplicatedUsing=OnRep_ClientDebugString)
	TArray<FString>	ClientDebugStrings;

	UPROPERTY(ReplicatedUsing = OnRep_ServerDebugString)
	TArray<FString>	ServerDebugStrings;

public:
	UE_API void SetClientDebugStrings(TArray<FString>&& NewClientDebugStrings);
	UE_API TArray<FString>& GetClientDebugStrings_Mutable();
	UE_API const TArray<FString>& GetClientDebugStrings() const;

	UE_API void SetServerDebugStrings(TArray<FString>&& NewServerDebugStrings);
	UE_API TArray<FString>& GetServerDebugStrings_Mutable();
	UE_API const TArray<FString>& GetServerDebugStrings() const;

	UFUNCTION()
	UE_API virtual void OnRep_ClientDebugString();

	UFUNCTION()
	UE_API virtual void OnRep_ServerDebugString();

	// ----------------------------------------------------------------------------------------------------------------
	// Batching client->server RPCs
	// This is a WIP feature to batch up client->server communication. It is opt in and not complete. It only batches the below functions. Other Server RPCs are not safe to call during a batch window. Only opt in if you know what you are doing!
	// ----------------------------------------------------------------------------------------------------------------	

	UE_API void CallServerTryActivateAbility(FGameplayAbilitySpecHandle AbilityToActivate, bool InputPressed, FPredictionKey PredictionKey);
	UE_API void CallServerSetReplicatedTargetData(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, const FGameplayAbilityTargetDataHandle& ReplicatedTargetDataHandle, FGameplayTag ApplicationTag, FPredictionKey CurrentPredictionKey);
	UE_API void CallServerEndAbility(FGameplayAbilitySpecHandle AbilityToEnd, FGameplayAbilityActivationInfo ActivationInfo, FPredictionKey PredictionKey);

	virtual bool ShouldDoServerAbilityRPCBatch() const { return false; }
	UE_API virtual void BeginServerAbilityRPCBatch(FGameplayAbilitySpecHandle AbilityHandle);
	UE_API virtual void EndServerAbilityRPCBatch(FGameplayAbilitySpecHandle AbilityHandle);

	/** Accumulated client side data that is batched out to server on EndServerAbilityRPCBatch */
	TArray<FServerAbilityRPCBatch, TInlineAllocator<1> > LocalServerAbilityRPCBatchData;

	UFUNCTION(Server, reliable, WithValidation)
	UE_API void	ServerAbilityRPCBatch(FServerAbilityRPCBatch BatchInfo);

	// Overridable function for sub classes
	UE_API virtual void ServerAbilityRPCBatch_Internal(FServerAbilityRPCBatch& BatchInfo);

	// ----------------------------------------------------------------------------------------------------------------
	//	Input handling/targeting
	// ----------------------------------------------------------------------------------------------------------------	

	/**
	 * This is meant to be used to inhibit activating an ability from an input perspective. (E.g., the menu is pulled up, another game mechanism is consuming all input, etc)
	 * This should only be called on locally owned players.
	 * This should not be used to game mechanics like silences or disables. Those should be done through gameplay effects.
	 */
	UFUNCTION(BlueprintCallable, Category="Abilities")
	UE_API bool GetUserAbilityActivationInhibited() const;
	
	/** Disable or Enable a local user from being able to activate abilities. This should only be used for input/UI etc related inhibition. Do not use for game mechanics. */
	UFUNCTION(BlueprintCallable, Category="Abilities")
	UE_API virtual void SetUserAbilityActivationInhibited(bool NewInhibit);

	/** Rather activation is currently inhibited */
	UPROPERTY()
	bool UserAbilityActivationInhibited;

	/** When enabled GameplayCue RPCs will be routed through the AvatarActor's IAbilitySystemReplicationProxyInterface rather than this component */
	UPROPERTY()
	bool ReplicationProxyEnabled;

	/** Suppress all ability granting through GEs on this component */
	UPROPERTY()
	bool bSuppressGrantAbility;

	/** Suppress all GameplayCues on this component */
	UPROPERTY()
	bool bSuppressGameplayCues;

	/** List of currently active targeting actors */
	UPROPERTY()
	TArray<TObjectPtr<AGameplayAbilityTargetActor>> SpawnedTargetActors;

	/** Bind to an input component with some default action names */
	UE_API virtual void BindToInputComponent(UInputComponent* InputComponent);

	/** Bind to an input component with customized bindings */
	UE_API virtual void BindAbilityActivationToInputComponent(UInputComponent* InputComponent, FGameplayAbilityInputBinds BindInfo);

	/** Initializes BlockedAbilityBindings variable */
	UE_API virtual void SetBlockAbilityBindingsArray(FGameplayAbilityInputBinds BindInfo);

	/** Called to handle ability bind input */
	UE_API virtual void AbilityLocalInputPressed(int32 InputID);
	UE_API virtual void AbilityLocalInputReleased(int32 InputID);

	/*
	 * Sends a local player Input Pressed event with the provided Input ID, notifying any bound abilities
	 *
	 * @param InputID The Input ID to match
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay Abilities")
	UE_API void PressInputID(int32 InputID);

	/**
	 * Sends a local player Input Released event with the provided Input ID, notifying any bound abilities
	 * @param InputID The Input ID to match
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay Abilities")
	UE_API void ReleaseInputID(int32 InputID);

	/** Handle confirm/cancel for target actors */
	UE_API virtual void LocalInputConfirm();
	UE_API virtual void LocalInputCancel();

	/**
	 * Sends a local player Input Confirm event, notifying abilities
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay Abilities")
	UE_API void InputConfirm();

	/**
	 * Sends a local player Input Cancel event, notifying abilities
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay Abilities")
	UE_API void InputCancel();
	
	/** InputID for binding GenericConfirm/Cancel events */
	int32 GenericConfirmInputID;
	int32 GenericCancelInputID;

	bool IsGenericConfirmInputBound(int32 InputID) const	{ return ((InputID == GenericConfirmInputID) && GenericLocalConfirmCallbacks.IsBound()); }
	bool IsGenericCancelInputBound(int32 InputID) const		{ return ((InputID == GenericCancelInputID) && GenericLocalCancelCallbacks.IsBound()); }

	/** Generic local callback for generic ConfirmEvent that any ability can listen to */
	FAbilityConfirmOrCancel	GenericLocalConfirmCallbacks;

	/** Generic local callback for generic CancelEvent that any ability can listen to */
	FAbilityConfirmOrCancel	GenericLocalCancelCallbacks;

	/** Any active targeting actors will be told to stop and return current targeting data */
	UFUNCTION(BlueprintCallable, Category = "Abilities")
	UE_API virtual void TargetConfirm();

	/** Any active targeting actors will be stopped and canceled, not returning any targeting data */
	UFUNCTION(BlueprintCallable, Category = "Abilities")
	UE_API virtual void TargetCancel();

	// ----------------------------------------------------------------------------------------------------------------
	//	AnimMontage Support
	// ----------------------------------------------------------------------------------------------------------------	

	/** Plays a montage and handles replication and prediction based on passed in ability/activation info */
	UE_API virtual float PlayMontage(UGameplayAbility* AnimatingAbility, FGameplayAbilityActivationInfo ActivationInfo, UAnimMontage* Montage, float InPlayRate, FName StartSectionName = NAME_None, float StartTimeSeconds = 0.0f);

	UE_API virtual UAnimMontage* PlaySlotAnimationAsDynamicMontage(UGameplayAbility* AnimatingAbility, FGameplayAbilityActivationInfo ActivationInfo, UAnimSequenceBase* AnimAsset, FName SlotName, float BlendInTime, float BlendOutTime, float InPlayRate = 1.f, float StartTimeSeconds = 0.0f);

	UE_API virtual UAnimMontage* PlaySlotAnimationAsDynamicMontage_WithFractionalLoops(UGameplayAbility* AnimatingAbility, FGameplayAbilityActivationInfo ActivationInfo, UAnimSequenceBase* AnimAsset, FName SlotName, float BlendInTime, float BlendOutTime, float InPlayRate = 1.f, float StartTimeSeconds = 0.0f, float PlayCount = 1.0f);

	/** Plays a montage without updating replication/prediction structures. Used by simulated proxies when replication tells them to play a montage. */
	UE_API virtual float PlayMontageSimulated(UAnimMontage* Montage, float InPlayRate, FName StartSectionName = NAME_None);
	
	UE_API virtual UAnimMontage* PlaySlotAnimationAsDynamicMontageSimulated(UAnimSequenceBase* AnimAsset, FName SlotName, float BlendInTime, float BlendOutTime, float InPlayRate = 1.f);

	UE_API virtual UAnimMontage* PlaySlotAnimationAsDynamicMontage_WithFractionalLoopsSimulated(UAnimSequenceBase* AnimAsset, FName SlotName, float BlendInTime, float BlendOutTime, float InPlayRate = 1.f, float PlayCount = 1.0f);

	/** Stops whatever montage is currently playing. Expectation is caller should only be stopping it if they are the current animating ability (or have good reason not to check) */
	UE_API virtual void CurrentMontageStop(float OverrideBlendOutTime = -1.0f);

	/** Stops current montage if it's the one given as the Montage param */
	UE_API virtual void StopMontageIfCurrent(const UAnimMontage& Montage, float OverrideBlendOutTime = -1.0f);

	/** Clear the animating ability that is passed in, if it's still currently animating */
	UE_API virtual void ClearAnimatingAbility(UGameplayAbility* Ability);

	/** Jumps current montage to given section. Expectation is caller should only be stopping it if they are the current animating ability (or have good reason not to check) */
	UE_API virtual void CurrentMontageJumpToSection(FName SectionName);

	/** Sets current montages next section name. Expectation is caller should only be stopping it if they are the current animating ability (or have good reason not to check) */
	UE_API virtual void CurrentMontageSetNextSectionName(FName FromSectionName, FName ToSectionName);

	/** Returns the next section name for the current montage. */
	UE_API FName CurrentMontageGetNextSectionName() const;

	/** Sets current montage's play rate */
	UE_API virtual void CurrentMontageSetPlayRate(float InPlayRate);

	/** Returns true if the passed in ability is the current animating ability */
	UE_API bool IsAnimatingAbility(UGameplayAbility* Ability) const;

	/** Returns the current animating ability */
	UE_API UGameplayAbility* GetAnimatingAbility() const;

	/** Returns montage that is currently playing */
	UE_API UAnimMontage* GetCurrentMontage() const;

	/** Get SectionID of currently playing AnimMontage */
	UE_API int32 GetCurrentMontageSectionID() const;

	/** Get SectionName of currently playing AnimMontage */
	UE_API FName GetCurrentMontageSectionName() const;

	/** Get length in time of current section */
	UE_API float GetCurrentMontageSectionLength() const;

	/** Returns amount of time left in current section */
	UE_API float GetCurrentMontageSectionTimeLeft() const;

	/** Method to set the replication method for the position in the montage */
	UE_API void SetMontageRepAnimPositionMethod(ERepAnimPositionMethod InMethod);

	// ----------------------------------------------------------------------------------------------------------------
	//	Actor interaction
	// ----------------------------------------------------------------------------------------------------------------	

private:

	/** The actor that owns this component logically */
	UPROPERTY(ReplicatedUsing = OnRep_OwningActor)
	TObjectPtr<AActor> OwnerActor;

	/** The actor that is the physical representation used for abilities. Can be NULL */
	UPROPERTY(ReplicatedUsing = OnRep_OwningActor)
	TObjectPtr<AActor> AvatarActor;

public:

	UE_API void SetOwnerActor(AActor* NewOwnerActor);
	AActor* GetOwnerActor() const { return OwnerActor; }

	UE_API void SetAvatarActor_Direct(AActor* NewAvatarActor);
	AActor* GetAvatarActor_Direct() const { return AvatarActor; }
	
	UFUNCTION()
	UE_API void OnRep_OwningActor();

	UFUNCTION()
	UE_API void OnAvatarActorDestroyed(AActor* InActor);

	UFUNCTION()
	UE_API void OnOwnerActorDestroyed(AActor* InActor);

	UFUNCTION()
	UE_API void OnSpawnedAttributesEndPlayed(AActor* InActor, EEndPlayReason::Type EndPlayReason);

	/** Cached off data about the owning actor that abilities will need to frequently access (movement component, mesh component, anim instance, etc) */
	TSharedPtr<FGameplayAbilityActorInfo>	AbilityActorInfo;

	/**
	 *	Initialized the Abilities' ActorInfo - the structure that holds information about who we are acting on and who controls us.
	 *      OwnerActor is the actor that logically owns this component.
	 *		AvatarActor is what physical actor in the world we are acting on. Usually a Pawn but it could be a Tower, Building, Turret, etc, may be the same as Owner
	 */
	UE_API virtual void InitAbilityActorInfo(AActor* InOwnerActor, AActor* InAvatarActor);

	/** Returns avatar actor to be used for a specific task, normally GetAvatarActor */
	UE_API virtual AActor* GetGameplayTaskAvatar(const UGameplayTask* Task) const override;

	/** Returns the avatar actor for this component */
	UE_API AActor* GetAvatarActor() const;

	/** Changes the avatar actor, leaves the owner actor the same */
	UE_API void SetAvatarActor(AActor* InAvatarActor);

	/** called when the ASC's AbilityActorInfo has a PlayerController set. */
	virtual void OnPlayerControllerSet() { }

	/**
	* This is called when the actor that is initialized to this system dies, this will clear that actor from this system and FGameplayAbilityActorInfo
	*/
	UE_API virtual void ClearActorInfo();

	/**
	 *	This will refresh the Ability's ActorInfo structure based on the current ActorInfo. That is, AvatarActor will be the same but we will look for new
	 *	AnimInstance, MovementComponent, PlayerController, etc.
	 */	
	UE_API void RefreshAbilityActorInfo();

	// ----------------------------------------------------------------------------------------------------------------
	//	Synchronization RPCs
	//  While these appear to be state, these are actually synchronization events w/ some payload data
	// ----------------------------------------------------------------------------------------------------------------	
	
	/** Replicates the Generic Replicated Event to the server. */
	UFUNCTION(Server, reliable, WithValidation)
	UE_API void ServerSetReplicatedEvent(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, FPredictionKey CurrentPredictionKey);

	/** Replicates the Generic Replicated Event to the server with payload. */
	UFUNCTION(Server, reliable, WithValidation)
	UE_API void ServerSetReplicatedEventWithPayload(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, FPredictionKey CurrentPredictionKey, FVector_NetQuantize100 VectorPayload);

	/** Replicates the Generic Replicated Event to the client. */
	UFUNCTION(Client, reliable)
	UE_API void ClientSetReplicatedEvent(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey);

	/** Calls local callbacks that are registered with the given Generic Replicated Event */
	UE_API bool InvokeReplicatedEvent(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, FPredictionKey CurrentPredictionKey = FPredictionKey());

	/** Calls local callbacks that are registered with the given Generic Replicated Event */
	UE_API bool InvokeReplicatedEventWithPayload(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, FPredictionKey CurrentPredictionKey, FVector_NetQuantize100 VectorPayload);
	
	/** Replicates targeting data to the server */
	UFUNCTION(Server, reliable, WithValidation)
	UE_API void ServerSetReplicatedTargetData(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, const FGameplayAbilityTargetDataHandle& ReplicatedTargetDataHandle, FGameplayTag ApplicationTag, FPredictionKey CurrentPredictionKey);

	/** Replicates to the server that targeting has been cancelled */
	UFUNCTION(Server, reliable, WithValidation)
	UE_API void ServerSetReplicatedTargetDataCancelled(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, FPredictionKey CurrentPredictionKey);

	/** Sets the current target data and calls applicable callbacks */
	UE_API virtual void ConfirmAbilityTargetData(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, const FGameplayAbilityTargetDataHandle& TargetData, const FGameplayTag& ApplicationTag);

	/** Cancels the ability target data and calls callbacks */
	UE_API virtual void CancelAbilityTargetData(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey);

	/** Deletes all cached ability client data (Was: ConsumeAbilityTargetData)*/
	UE_API void ConsumeAllReplicatedData(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey);
	/** Consumes cached TargetData from client (only TargetData) */
	UE_API void ConsumeClientReplicatedTargetData(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey);

	/** Consumes the given Generic Replicated Event (unsets it). */
	UE_API void ConsumeGenericReplicatedEvent(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey);

	/** Gets replicated data of the given Generic Replicated Event. */
	UE_API FAbilityReplicatedData GetReplicatedDataOfGenericReplicatedEvent(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey);
	
	/** Calls any Replicated delegates that have been sent (TargetData or Generic Replicated Events). Note this can be dangerous if multiple places in an ability register events and then call this function. */
	UE_API void CallAllReplicatedDelegatesIfSet(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey);

	/** Calls the TargetData Confirm/Cancel events if they have been sent. */
	UE_API bool CallReplicatedTargetDataDelegatesIfSet(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey);

	/** Calls a given Generic Replicated Event delegate if the event has already been sent */
	UE_API bool CallReplicatedEventDelegateIfSet(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey);

	/** Calls passed in delegate if the Client Event has already been sent. If not, it adds the delegate to our multicast callback that will fire when it does. */
	UE_API bool CallOrAddReplicatedDelegate(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey, FSimpleMulticastDelegate::FDelegate Delegate);

	/** Returns TargetDataSet delegate for a given Ability/PredictionKey pair */
	UE_API FAbilityTargetDataSetDelegate& AbilityTargetDataSetDelegate(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey);

	/** Returns TargetData Cancelled delegate for a given Ability/PredictionKey pair */
	UE_API FSimpleMulticastDelegate& AbilityTargetDataCancelledDelegate(FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey);

	/** Returns Generic Replicated Event for a given Ability/PredictionKey pair */
	UE_API FSimpleMulticastDelegate& AbilityReplicatedEventDelegate(EAbilityGenericReplicatedEvent::Type EventType, FGameplayAbilitySpecHandle AbilityHandle, FPredictionKey AbilityOriginalPredictionKey);

	/** Direct Input state replication. These will be called if bReplicateInputDirectly is true on the ability and is generally not a good thing to use. (Instead, prefer to use Generic Replicated Events). */
	UFUNCTION(Server, reliable, WithValidation)
	UE_API void ServerSetInputPressed(FGameplayAbilitySpecHandle AbilityHandle);

	UFUNCTION(Server, reliable, WithValidation)
	UE_API void ServerSetInputReleased(FGameplayAbilitySpecHandle AbilityHandle);

	/** Called on local player always. Called on server only if bReplicateInputDirectly is set on the GameplayAbility. */
	UE_API virtual void AbilitySpecInputPressed(FGameplayAbilitySpec& Spec);

	/** Called on local player always. Called on server only if bReplicateInputDirectly is set on the GameplayAbility. */
	UE_API virtual void AbilitySpecInputReleased(FGameplayAbilitySpec& Spec);

	// ----------------------------------------------------------------------------------------------------------------
	//  Component overrides
	// ----------------------------------------------------------------------------------------------------------------

	UE_API virtual void InitializeComponent() override;
	UE_API virtual void UninitializeComponent() override;
	UE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	UE_API virtual bool GetShouldTick() const override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	UE_API virtual bool ReplicateSubobjects(class UActorChannel *Channel, class FOutBunch *Bunch, FReplicationFlags *RepFlags) override;
	/** Force owning actor to update it's replication, to make sure that gameplay cues get sent down quickly. Override to change how aggressive this is */
	UE_API virtual void ForceReplication() override;
	UE_API virtual void PreNetReceive() override;
	UE_API virtual void PostNetReceive() override;
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void ReadyForReplication() override;
	UE_API virtual void BeginPlay() override;
	UE_API virtual void Serialize(FArchive& Ar) override;

protected:
	UE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	UE_API virtual void GetReplicatedCustomConditionState(FCustomPropertyConditionState& OutActiveState) const override;

	UE_API void UpdateActiveGameplayEffectsReplicationCondition();
	UE_API void UpdateMinimalReplicationGameplayCuesCondition();

	/**
	 *	The abilities we can activate. 
	 *		-This will include CDOs for non instanced abilities and per-execution instanced abilities. 
	 *		-Actor-instanced abilities will be the actual instance (not CDO)
	 *		
	 *	This array is not vital for things to work. It is a convenience thing for 'giving abilities to the actor'. But abilities could also work on things
	 *	without an AbilitySystemComponent. For example an ability could be written to execute on a StaticMeshActor. As long as the ability doesn't require 
	 *	instancing or anything else that the AbilitySystemComponent would provide, then it doesn't need the component to function.
	 */

	UPROPERTY(ReplicatedUsing = OnRep_ActivateAbilities, BlueprintReadOnly, Transient, Category = "Abilities")
	FGameplayAbilitySpecContainer ActivatableAbilities;

	/** Maps from an ability spec to the target data. Used to track replicated data and callbacks */
	FGameplayAbilityReplicatedDataContainer AbilityTargetDataMap;

	/** List of gameplay tag container filters, and the delegates they call */
	TArray<TPair<FGameplayTagContainer, FGameplayEventTagMulticastDelegate>> GameplayEventTagContainerDelegates;

private:
	/** Full list of all instance-per-execution gameplay abilities associated with this component */
	UPROPERTY()
	TArray<TObjectPtr<UGameplayAbility>>	AllReplicatedInstancedAbilities;

protected:
	/** Full list of all instance-per-execution gameplay abilities associated with this component */
	const TArray<UGameplayAbility*>& GetReplicatedInstancedAbilities() const { return AllReplicatedInstancedAbilities; }

	/** Add a gameplay ability associated to this component */
	void AddReplicatedInstancedAbility(UGameplayAbility* GameplayAbility);

	/** Remove a gameplay ability associated to this component */
	UE_API void RemoveReplicatedInstancedAbility(UGameplayAbility* GameplayAbility);

	/** Unregister all the gameplay abilities of this component */
	UE_API void RemoveAllReplicatedInstancedAbilities();

	/** Will be called from GiveAbility or from OnRep. Initializes events (triggers and inputs) with the given ability */
	UE_API virtual void OnGiveAbility(FGameplayAbilitySpec& AbilitySpec);

	/** Will be called from RemoveAbility or from OnRep. Unbinds inputs with the given ability */
	UE_API virtual void OnRemoveAbility(FGameplayAbilitySpec& AbilitySpec);

	/** Called from ClearAbility, ClearAllAbilities or OnRep. Clears any triggers that should no longer exist. */
	UE_API void CheckForClearedAbilities();

	/** Cancel a specific ability spec */
	UE_API virtual void CancelAbilitySpec(FGameplayAbilitySpec& Spec, UGameplayAbility* Ignore);

	/** Creates a new instance of an ability, storing it in the spec */
	UE_API virtual UGameplayAbility* CreateNewInstanceOfAbility(FGameplayAbilitySpec& Spec, const UGameplayAbility* Ability);

	/** Indicates how many levels of ABILITY_SCOPE_LOCK() we are in. The ability list may not be modified while AbilityScopeLockCount > 0. */
	int32 AbilityScopeLockCount;
	/** Abilities that will be removed when exiting the current ability scope lock. */
	TArray<FGameplayAbilitySpecHandle, TInlineAllocator<2> > AbilityPendingRemoves;
	/** Abilities that will be added when exiting the current ability scope lock. */
	TArray<FGameplayAbilitySpec, TInlineAllocator<2> > AbilityPendingAdds;
	/** Whether all abilities should be removed when exiting the current ability scope lock. Will be prioritized over pending adds. */
	bool bAbilityPendingClearAll;

	/** Local World time of the last ability activation. This is used for AFK/idle detection */
	float AbilityLastActivatedTime;

	UFUNCTION()
	UE_API virtual void OnRep_ActivateAbilities();

	UFUNCTION(Server, reliable, WithValidation)
	UE_API void	ServerTryActivateAbility(FGameplayAbilitySpecHandle AbilityToActivate, bool InputPressed, FPredictionKey PredictionKey);

	UFUNCTION(Server, reliable, WithValidation)
	UE_API void	ServerTryActivateAbilityWithEventData(FGameplayAbilitySpecHandle AbilityToActivate, bool InputPressed, FPredictionKey PredictionKey, FGameplayEventData TriggerEventData);

	UFUNCTION(Client, reliable)
	UE_API void	ClientTryActivateAbility(FGameplayAbilitySpecHandle AbilityToActivate);

	/** Called by ServerEndAbility and ClientEndAbility; avoids code duplication. */
	UE_API void	RemoteEndOrCancelAbility(FGameplayAbilitySpecHandle AbilityToEnd, FGameplayAbilityActivationInfo ActivationInfo, bool bWasCanceled);

	UFUNCTION(Server, reliable, WithValidation)
	UE_API void	ServerEndAbility(FGameplayAbilitySpecHandle AbilityToEnd, FGameplayAbilityActivationInfo ActivationInfo, FPredictionKey PredictionKey);

	UFUNCTION(Client, reliable)
	UE_API void	ClientEndAbility(FGameplayAbilitySpecHandle AbilityToEnd, FGameplayAbilityActivationInfo ActivationInfo);

	UFUNCTION(Server, reliable, WithValidation)
	UE_API void    ServerCancelAbility(FGameplayAbilitySpecHandle AbilityToCancel, FGameplayAbilityActivationInfo ActivationInfo);

	UFUNCTION(Client, reliable)
	UE_API void    ClientCancelAbility(FGameplayAbilitySpecHandle AbilityToCancel, FGameplayAbilityActivationInfo ActivationInfo);

	UFUNCTION(Client, Reliable)
	UE_API void	ClientActivateAbilityFailed(FGameplayAbilitySpecHandle AbilityToActivate, int16 PredictionKey);
	int32	ClientActivateAbilityFailedCountRecent;
	float	ClientActivateAbilityFailedStartTime;

	
	UE_API void	OnClientActivateAbilityCaughtUp(FGameplayAbilitySpecHandle AbilityToActivate, FPredictionKey::KeyType PredictionKey);

	UFUNCTION(Client, Reliable)
	UE_API void	ClientActivateAbilitySucceed(FGameplayAbilitySpecHandle AbilityToActivate, FPredictionKey PredictionKey);

	UFUNCTION(Client, Reliable)
	UE_API void	ClientActivateAbilitySucceedWithEventData(FGameplayAbilitySpecHandle AbilityToActivate, FPredictionKey PredictionKey, FGameplayEventData TriggerEventData);

	/** Implementation of ServerTryActivateAbility */
	UE_API virtual void InternalServerTryActivateAbility(FGameplayAbilitySpecHandle AbilityToActivate, bool InputPressed, const FPredictionKey& PredictionKey, const FGameplayEventData* TriggerEventData);

	/** Called when a prediction key that played a montage is rejected */
	UE_API void OnPredictiveMontageRejected(UAnimMontage* PredictiveMontage);

	/** Copy LocalAnimMontageInfo into RepAnimMontageInfo */
	UE_API void AnimMontage_UpdateReplicatedData();
	UE_API void AnimMontage_UpdateReplicatedData(FGameplayAbilityRepAnimMontage& OutRepAnimMontageInfo);

	/** Copy over playing flags for duplicate animation data */
	UE_API void AnimMontage_UpdateForcedPlayFlags(FGameplayAbilityRepAnimMontage& OutRepAnimMontageInfo);

	UE_DEPRECATED(4.26, "This will be made private in future engine versions. Use SetRepAnimMontageInfo, GetRepAnimMontageInfo, or GetRepAnimMontageInfo_Mutable instead.")
	/** Data structure for replicating montage info to simulated clients */
	UPROPERTY(ReplicatedUsing=OnRep_ReplicatedAnimMontage)
	FGameplayAbilityRepAnimMontage RepAnimMontageInfo;

	UE_API void SetRepAnimMontageInfo(const FGameplayAbilityRepAnimMontage& NewRepAnimMontageInfo);
	UE_API FGameplayAbilityRepAnimMontage& GetRepAnimMontageInfo_Mutable();
	UE_API const FGameplayAbilityRepAnimMontage& GetRepAnimMontageInfo() const;

	/** Cached value of rather this is a simulated actor */
	UPROPERTY()
	bool bCachedIsNetSimulated;

	/** Set if montage rep happens while we don't have the animinstance associated with us yet */
	UPROPERTY()
	bool bPendingMontageRep;

	/** Data structure for montages that were instigated locally (everything if server, predictive if client. replicated if simulated proxy) */
	UPROPERTY()
	FGameplayAbilityLocalAnimMontage LocalAnimMontageInfo;

	UFUNCTION()
	UE_API virtual void OnRep_ReplicatedAnimMontage();

	/** Returns true if we are ready to handle replicated montage information */
	UE_API virtual bool IsReadyForReplicatedMontage();

	/** RPC function called from CurrentMontageSetNextSectionName, replicates to other clients */
	UFUNCTION(reliable, server, WithValidation)
	UE_API void ServerCurrentMontageSetNextSectionName(UAnimSequenceBase* ClientAnimation, float ClientPosition, FName SectionName, FName NextSectionName);

	/** RPC function called from CurrentMontageJumpToSection, replicates to other clients */
	UFUNCTION(reliable, server, WithValidation)
	UE_API void ServerCurrentMontageJumpToSectionName(UAnimSequenceBase* ClientAnimation, FName SectionName);

	/** RPC function called from CurrentMontageSetPlayRate, replicates to other clients */
	UFUNCTION(reliable, server, WithValidation)
	UE_API void ServerCurrentMontageSetPlayRate(UAnimSequenceBase* ClientAnimation, float InPlayRate);

	/** Abilities that are triggered from a gameplay event */
	TMap<FGameplayTag, TArray<FGameplayAbilitySpecHandle > > GameplayEventTriggeredAbilities;

	/** Abilities that are triggered from a tag being added to the owner */
	TMap<FGameplayTag, TArray<FGameplayAbilitySpecHandle > > OwnedTagTriggeredAbilities;

	/** Callback that is called when an owned tag bound to an ability changes */
	UE_API virtual void MonitoredTagChanged(const FGameplayTag Tag, int32 NewCount);

	/** Returns true if the specified ability should be activated from an event in this network mode */
	UE_API bool HasNetworkAuthorityToActivateTriggeredAbility(const FGameplayAbilitySpec &Spec) const;

	// Internal gameplay cue functions
	UE_API virtual void AddGameplayCue_Internal(const FGameplayTag GameplayCueTag, FGameplayEffectContextHandle& EffectContext, FActiveGameplayCueContainer& GameplayCueContainer);
	UE_API virtual void AddGameplayCue_Internal(const FGameplayTag GameplayCueTag, const FGameplayCueParameters& GameplayCueParameters, FActiveGameplayCueContainer& GameplayCueContainer);
	UE_API virtual void RemoveGameplayCue_Internal(const FGameplayTag GameplayCueTag, FActiveGameplayCueContainer& GameplayCueContainer);

	/** Actually pushes the final attribute value to the attribute set's property. Should not be called by outside code since this does not go through the attribute aggregator system. */
	UE_API void SetNumericAttribute_Internal(const FGameplayAttribute &Attribute, float& NewFloatValue);

	UE_API bool HasNetworkAuthorityToApplyGameplayEffect(FPredictionKey PredictionKey) const;

	UE_API void ExecutePeriodicEffect(FActiveGameplayEffectHandle	Handle);

	UE_API void ExecuteGameplayEffect(FGameplayEffectSpec &Spec, FPredictionKey PredictionKey);

	UE_API void CheckDurationExpired(FActiveGameplayEffectHandle Handle);
		
	UE_API TArray<TObjectPtr<UGameplayTask>>&	GetAbilityActiveTasks(UGameplayAbility* Ability);

	/** A version of RemoveActiveGameplayEffect that allows us to remove it even when we don't have authority */
	UE_API void RemoveActiveGameplayEffect_AllowClientRemoval(FActiveGameplayEffectHandle Handle, int32 StacksToRemove = -1);

	UE_API void OnCaughtUpActiveGameplayEffect(FActiveGameplayEffectHandle Handle, int32 StacksToRemove = -1);

	UE_API void OnRejectedActiveGameplayEffect(FActiveGameplayEffectHandle Handle, int32 StacksToRemove = -1);

	UE_API void OnPredictiveGameplayEffectStackCaughtUp(FActiveGameplayEffectHandle Handle);

	/** Contains all of the gameplay effects that are currently active on this component */
	UPROPERTY(Replicated)
	FActiveGameplayEffectsContainer ActiveGameplayEffects;

	/** List of all active gameplay cues (executed outside of Gameplay Effects) */
	UPROPERTY(Replicated)
	FActiveGameplayCueContainer ActiveGameplayCues;

	/** Replicated gameplaycues when in minimal replication mode. These are cues that would come normally come from ActiveGameplayEffects (but since we do not replicate AGE in minimal mode, they must be replicated through here) */
	UPROPERTY(Replicated)
	FActiveGameplayCueContainer MinimalReplicationGameplayCues;

	/** Abilities with these tags are not able to be activated */
	FGameplayTagCountContainer BlockedAbilityTags;

	UE_API void SetBlockedAbilityBindings(const TArray<uint8>& NewBlockedAbilityBindings);
	UE_API TArray<uint8>& GetBlockedAbilityBindings_Mutable();
	UE_API const TArray<uint8>& GetBlockedAbilityBindings() const;

	UE_API void DebugCyclicAggregatorBroadcasts(struct FAggregator* Aggregator);
	
	/** Acceleration map for all gameplay tags (OwnedGameplayTags from GEs and explicit GameplayCueTags) */
	UPROPERTY(Replicated)
	FGameplayTagCountContainer GameplayTagCountContainer;

	UE_API void SetMinimalReplicationTags(const FMinimalReplicationTagCountMap& NewMinimalReplicationTags);
	UE_API FMinimalReplicationTagCountMap& GetMinimalReplicationTags_Mutable();
	UE_API const FMinimalReplicationTagCountMap& GetMinimalReplicationTags() const;

	UE_API FMinimalReplicationTagCountMap& GetReplicatedLooseTags_Mutable();
	UE_API const FMinimalReplicationTagCountMap& GetReplicatedLooseTags() const;

	UE_API void ResetTagMap();

	UE_API void NotifyTagMap_StackCountChange(const FGameplayTagContainer& Container);

	virtual void OnTagUpdated(const FGameplayTag& Tag, bool TagExists) {};
	
	UE_API const UAttributeSet*	GetAttributeSubobject(const TSubclassOf<UAttributeSet> AttributeClass) const;
	UE_API const UAttributeSet*	GetAttributeSubobjectChecked(const TSubclassOf<UAttributeSet> AttributeClass) const;
	UE_API const UAttributeSet*	GetOrCreateAttributeSubobject(TSubclassOf<UAttributeSet> AttributeClass);

	UE_API void UpdateTagMapSingle_Internal(const FGameplayTag& Tag, int32 CountDelta, EGameplayTagReplicationState TagRepState);
	UE_API void UpdateTagMap_Internal(const FGameplayTagContainer& Container, int32 CountDelta, EGameplayTagReplicationState TagRepState);

	friend struct FActiveGameplayEffect;
	friend struct FActiveGameplayEffectAction;
	friend struct FActiveGameplayEffectsContainer;
	friend struct FActiveGameplayCue;
	friend struct FActiveGameplayCueContainer;
	friend struct FGameplayAbilitySpec;
	friend struct FGameplayAbilitySpecContainer;
	friend struct FAggregator;
	friend struct FActiveGameplayEffectAction_Add;
	friend struct FGameplayEffectSpec;
	friend class AAbilitySystemDebugHUD;
	friend class UAbilitySystemGlobals;

private:
	UE_API TArray<TObjectPtr<UAttributeSet>>& GetSpawnedAttributes_Mutable();

	// Needs to be called when modifying the SpawnedAttributes array for changes to be replicated
	UE_API void SetSpawnedAttributesListDirty();

	/** Tracks abilities that are blocked based on input binding. An ability is blocked if BlockedAbilityBindings[InputID] > 0 */
	UPROPERTY(Transient, Replicated)
	TArray<uint8> BlockedAbilityBindings;

	UE_DEPRECATED(5.7, "This will be removed in favor of using TagOnly replication state for on the replicated Tag Count Container")
	UPROPERTY(Replicated)
	FMinimalReplicationTagCountMap MinimalReplicationTags;

	void RegisterAbilityTriggers(const FGameplayAbilitySpec& Spec, TConstArrayView<FAbilityTriggerData> AbilityTriggers);
	void UnregisterAbilityTriggers(const FGameplayAbilitySpec& Spec, TConstArrayView<FAbilityTriggerData> AbilityTriggers);

private:

	/** List of attribute sets */
	UPROPERTY(Replicated, ReplicatedUsing = OnRep_SpawnedAttributes, Transient)
	TArray<TObjectPtr<UAttributeSet>>	SpawnedAttributes;

	UFUNCTION()
	UE_API void OnRep_SpawnedAttributes(const TArray<UAttributeSet*>& PreviousSpawnedAttributes);

	FDelegateHandle MonitoredTagChangedDelegateHandle;
	FTimerHandle    OnRep_ActivateAbilitiesTimerHandle;

	/** Container used for replicating loose gameplay tags */
	UE_DEPRECATED(5.7, "This will be removed in favor of using CountToOwner replication state for on the replicated Tag Count Container")
	UPROPERTY(Replicated)
	FMinimalReplicationTagCountMap ReplicatedLooseTags;

	uint8 bDestroyActiveStateInitiated : 1;
public:

	/** Caches the flags that indicate whether this component has network authority. */
	UE_API void CacheIsNetSimulated();

	/** PredictionKeys, see more info in GameplayPrediction.h. This has to come *last* in all replicated properties on the AbilitySystemComponent to ensure OnRep/callback order. */
	UPROPERTY(Replicated, Transient)
	FReplicatedPredictionKeyMap ReplicatedPredictionKeyMap;

protected:

	struct FAbilityListLockActiveChange
	{
		FAbilityListLockActiveChange(UAbilitySystemComponent& InAbilitySystemComp,
									 TArray<FGameplayAbilitySpec, TInlineAllocator<2> >& PendingAdds,
									 TArray<FGameplayAbilitySpecHandle, TInlineAllocator<2> >& PendingRemoves) :
			AbilitySystemComp(InAbilitySystemComp),
			Adds(MoveTemp(PendingAdds)),
			Removes(MoveTemp(PendingRemoves))
		{
			AbilitySystemComp.AbilityListLockActiveChanges.Add(this);
		}

		~FAbilityListLockActiveChange()
		{
			AbilitySystemComp.AbilityListLockActiveChanges.Remove(this);
		}

		UAbilitySystemComponent& AbilitySystemComp;
		TArray<FGameplayAbilitySpec, TInlineAllocator<2> > Adds;
		TArray<FGameplayAbilitySpecHandle, TInlineAllocator<2> > Removes;
	};

	TArray<FAbilityListLockActiveChange*> AbilityListLockActiveChanges;

private:
	UE_API float PlayMontageInternal(UGameplayAbility* AnimatingAbility, FGameplayAbilityActivationInfo ActivationInfo, UAnimMontage* Montage, float InPlayRate, FName StartSectionName, float StartTimeSeconds, TFunctionRef<void(FGameplayAbilityRepAnimMontage&)> MutateRepAnimMontageFunction);

protected:
	/** Called before activation for LocalPredicted Ability to execute CharacterMovementComponent->FlushServerMoves 
	 * When overriding make sure to include Super::PrePredictionActivation(Ability) */
	UE_API virtual void PrePredictionActivation(const UGameplayAbility* Ability);
	
};

#undef UE_API
