// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/HUD.h"
#include "GameplayTagContainer.h"
#include "AbilitySystemDebugHUD.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class APlayerController;
class UAbilitySystemComponent;
class UCanvas;
class UFont;

namespace EAlignHorizontal
{
	enum Type
	{
		Left,
		Center,
		Right,
	};
}

namespace EAlignVertical
{
	enum Type
	{
		Top,
		Center,
		Bottom,
	};
}

/**
 * An extension for the Ability System debug HUD
 * Extensions should be used for displaying information on screen for each Ability System Actor
 */
UCLASS(Abstract, MinimalAPI)
class UAbilitySystemDebugHUDExtension : public UObject
{
	GENERATED_BODY()

public:

	/** Returns whether this extension should be rendered for each actor */
	virtual bool IsEnabled() const { return false; }

	/**
	 * Builds an array of strings to render for Ability System actor
	 * 
	 * @param Actor	The avatar actor for the ability system component
	 * @param Comp	The Ability System Component being debugged
	 * @param OutDebugStrings	Array to fill with debug strings
	 */
	virtual void GetDebugStrings(const AActor* Actor, const UAbilitySystemComponent* Comp, OUT TArray<FString>& OutDebugStrings) const { }
};

/**
 * Ability System Debug HUD extension for drawing all owned tags by an actor
 */
UCLASS()
class UAbilitySystemDebugHUDExtension_Tags final : public UAbilitySystemDebugHUDExtension
{
	GENERATED_BODY()

	TSet<FString> TagsToDisplay;

	bool bEnabled;

public:

	static void ToggleExtension(const TArray<FString>& Args, UWorld* World);

	//~ Begin UAbilitySystemDebugHUDExtension interface
	bool IsEnabled() const override;
	void GetDebugStrings(const AActor* Actor, const UAbilitySystemComponent* Comp, OUT TArray<FString>& OutDebugStrings) const override;
	//~ End UAbilitySystemDebugHUDExtension interface

};

/**
 * Ability System Debug HUD extension for drawing attributes of an actor
 */
UCLASS()
class UAbilitySystemDebugHUDExtension_Attributes final : public UAbilitySystemDebugHUDExtension
{
	GENERATED_BODY()

	TSet<FString> AttributesToDisplay;

	bool bIncludeModifiers;
public:
	static void ToggleExtension(const TArray<FString>& Args, UWorld* World);
	
	static void ToggleIncludeModifiers();

	static void ClearDisplayedAttributes(const TArray<FString>& Args, UWorld* World);

	//~ Begin UAbilitySystemDebugHUDExtension interface
	bool IsEnabled() const override;
	void GetDebugStrings(const AActor* Actor, const UAbilitySystemComponent* Comp, OUT TArray<FString>& OutDebugStrings) const override;
	//~ End UAbilitySystemDebugHUDExtension interface

};

/**
 * Ability System Debug HUD extension for drawing all blocked ability tags on an actor
 */
UCLASS()
class UAbilitySystemDebugHUDExtension_BlockedAbilityTags final : public UAbilitySystemDebugHUDExtension
{
	GENERATED_BODY()

	TSet<FString> TagsToDisplay;

	bool bEnabled;

public:

	static void ToggleExtension(const TArray<FString>& Args, UWorld* World);

	//~ Begin UAbilitySystemDebugHUDExtension interface
	bool IsEnabled() const override;
	void GetDebugStrings(const AActor* Actor, const UAbilitySystemComponent* Comp, OUT TArray<FString>& OutDebugStrings) const override;
	//~ End UAbilitySystemDebugHUDExtension interface

};

UCLASS()
class AAbilitySystemDebugHUD : public AHUD
{
	GENERATED_UCLASS_BODY()

public:

	static void ToggleBasicHUD(const TArray<FString>& Args, UWorld* World);


	/**
	 * Notifies the AbilitySystemDebugHUD that an extension has been enabled/disabled
	 * This will create the HUD if it did not exist and there are any extensions enabled
	 * or destroy the HUD if it did exist and there are no extensions enabled
	 */
	static GAMEPLAYABILITIES_API void NotifyExtensionEnableChanged(UWorld* InWorld);

	/** 
	 * Replaces the AbilitySystemDebugHUD with a user class
	 * NOTE: Only one AbilitySystemDebugHUD class can be active at a time.
	 */
	static GAMEPLAYABILITIES_API void RegisterHUDClass(TSubclassOf<AAbilitySystemDebugHUD> InHUDClass);
	/** main HUD update loop */
	void DrawDebugHUD(UCanvas* Canvas, APlayerController* PC);

protected:

	/** Returns whether debug information should be drawn for a given actor */
	virtual bool ShouldDisplayDebugForActor(UCanvas* InCanvas, const AActor* Actor, const FVector& CameraPosition, const FVector& CameraDir) const;

private:
	static void CreateHUD(UWorld* World);

	void DrawWithBackground(UFont* InFont, const FString& Text, const FColor& TextColor, EAlignHorizontal::Type HAlign, float& OffsetX, EAlignVertical::Type VAlign, float& OffsetY, float Alpha = 1.f);

	void DisplayDebugStrings(UCanvas* InCanvas, const AActor* Actor, const TArray<FString>& DebugStrings, const FVector& CameraPosition, const FVector& CameraDir, float& VerticalOffset) const;

	void DrawDebugAbilitySystemComponent(UAbilitySystemComponent* Component);

	void DrawAbilityDebugInfo(UCanvas* Canvas, APlayerController* PC) const;

private:

	static bool bEnableBasicHUD;

	static TSubclassOf<AAbilitySystemDebugHUD> HUDClass;
	
	static FDelegateHandle DrawDebugDelegateHandle;

};

#undef UE_API
