// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "GameplayTagContainer.h"
#include "GameplayEffectTypes.h"
#include "GameplayAbilitiesModule.h"
#include "AbilitySystemGlobals.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;
class UGameplayCueManager;
class UGameplayTagReponseTable;
struct FGameplayAbilityActorInfo;
struct FGameplayEffectSpec;
struct FGameplayEffectSpecForRPC;

/** Called when ability fails to activate, passes along the failed ability and a tag explaining why */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAbilitySystemAssetOpenedDelegate, FString , int );
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAbilitySystemAssetFoundDelegate, FString, int);

//  Container for safely replicating  script struct references (constrained to a specified parent struct)
USTRUCT()
struct FNetSerializeScriptStructCache
{
	GENERATED_BODY()

	void InitForType(UScriptStruct* InScriptStruct);

	void SetInitDirty()
	{
		bIsInitDirty = true;
	}

	void RemoveInPackages(TConstArrayView<UPackage*>);

	// Serializes reference to given script struct (must be in the cache)
	bool NetSerialize(FArchive& Ar, UScriptStruct*& Struct);

	UPROPERTY()
	TArray<TObjectPtr<UScriptStruct>> ScriptStructs;

	UPROPERTY()
	TObjectPtr<UScriptStruct> BaseStructType;

	bool bIsInitDirty = false;
};


/** Holds global data for the ability system. Configuration is done via the Developer Settings, Project -> Gameplay Abilities Settings  */
UCLASS(config = Game, MinimalAPI)
class UAbilitySystemGlobals : public UObject
{
	friend class UGameplayAbilitiesDeveloperSettings;

	GENERATED_UCLASS_BODY()

	/** Gets the single instance of the globals object, will create it as necessary */
	static UAbilitySystemGlobals& Get()
	{
		return *IGameplayAbilitiesModule::Get().GetAbilitySystemGlobals();
	}
	/** Will be called once on first use to load global data tables and tags (see FGameplayAbilitiesModule::GetAbilitySystemGlobals) */
	UE_API virtual void InitGlobalData();

	/** Returns true if InitGlobalData has been called */
	UE_API bool IsAbilitySystemGlobalsInitialized() const;
	
	// Returns true if we should use debug target from the HUD
	UE_API bool ShouldUseDebugTargetFromHud();
	
	/** Returns the globally registered curve table */
	UE_API UCurveTable* GetGlobalCurveTable();

	/** Returns the data table defining attribute metadata (NOTE: Currently not in use) */
	UE_API UDataTable* GetGlobalAttributeMetaDataTable();

	/** Returns data used to initialize attributes to their default values */
	UE_API FAttributeSetInitter* GetAttributeSetInitter() const;

	/** Searches the passed in actor for an ability system component, will use IAbilitySystemInterface or fall back to a component search */
	static UE_API UAbilitySystemComponent* GetAbilitySystemComponentFromActor(const AActor* Actor, bool LookForComponent=true);

	/** Should allocate a project specific AbilityActorInfo struct. Caller is responsible for deallocation */
	UE_API virtual FGameplayAbilityActorInfo* AllocAbilityActorInfo() const;

	/** Should allocate a project specific GameplayEffectContext struct. Caller is responsible for deallocation */
	UE_API virtual FGameplayEffectContext* AllocGameplayEffectContext() const;

	/** Global callback that can handle game-specific code that needs to run before applying a gameplay effect spec */
	UE_API virtual void GlobalPreGameplayEffectSpecApply(FGameplayEffectSpec& Spec, UAbilitySystemComponent* AbilitySystemComponent);

	/** Override to handle global state when gameplay effects are being applied */
	virtual void PushCurrentAppliedGE(const FGameplayEffectSpec* Spec, UAbilitySystemComponent* AbilitySystemComponent) { }
	virtual void SetCurrentAppliedGE(const FGameplayEffectSpec* Spec) { }
	virtual void PopCurrentAppliedGE() { }

	/** Returns true if the ability system should try to predict gameplay effects applied to non local targets */
	UE_API bool ShouldPredictTargetGameplayEffects() const;

	/** Returns true if tags granted to owners from ability activations should be replicated */
	UE_API bool ShouldReplicateActivationOwnedTags() const;

	/** Searches the passed in class to look for a UFunction implementing the gameplay cue tag, sets MatchedTag to the exact tag found */
	UE_API UFunction* GetGameplayCueFunction(const FGameplayTag &Tag, UClass* Class, FName &MatchedTag);

	/** Returns the gameplay cue manager singleton object, creating if necessary */
	UE_API virtual UGameplayCueManager* GetGameplayCueManager();

	/** Returns the gameplay tag response object, creating if necessary */
	UE_API UGameplayTagReponseTable* GetGameplayTagResponseTable();

	/** Sets a default gameplay cue tag using the asset's name. Returns true if it changed the tag. */
	static UE_API bool DeriveGameplayCueTagFromAssetName(FString AssetName, FGameplayTag& GameplayCueTag, FName& GameplayCueName);

	/** Sets a default gameplay cue tag using the asset's class*/
	template<class T>
	static void DeriveGameplayCueTagFromClass(T* CDO)
	{
#if WITH_EDITOR
		UClass* ParentClass = CDO->GetClass()->GetSuperClass();
		if (T* ParentCDO = Cast<T>(ParentClass->GetDefaultObject()))
		{
			if (ParentCDO->GameplayCueTag.IsValid() && (ParentCDO->GameplayCueTag == CDO->GameplayCueTag))
			{
				// Parente has a valid tag. But maybe there is a better one for this class to use.
				// Reset our GameplayCueTag and see if we find one.
				FGameplayTag ParentTag = ParentCDO->GameplayCueTag;
				CDO->GameplayCueTag = FGameplayTag();
				if (UAbilitySystemGlobals::DeriveGameplayCueTagFromAssetName(CDO->GetName(), CDO->GameplayCueTag, CDO->GameplayCueName) == false)
				{
					// We did not find one, so parent tag it is.
					CDO->GameplayCueTag = ParentTag;
				}
				return;
			}
		}
		UAbilitySystemGlobals::DeriveGameplayCueTagFromAssetName(CDO->GetName(), CDO->GameplayCueTag, CDO->GameplayCueName);
#endif
	}

#if WITH_EDITOR
	// Allows projects to override PostEditChangeProeprty on GEs without having to subclass Gameplayeffect. Intended for validation/auto populating based on changed data.
	virtual void GameplayEffectPostEditChangeProperty(class UGameplayEffect* GE, FPropertyChangedEvent& PropertyChangedEvent) { }
#endif

	/** The class to instantiate as the globals object. Defaults to this class but can be overridden */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->AbilitySystemGlobalsClassName to access this variable in code")
	UPROPERTY(config)
	FSoftClassPath AbilitySystemGlobalsClassName;

	void AutomationTestOnly_SetGlobalAttributeDataTable(UDataTable *InTable)
	{
		GlobalAttributeMetaDataTable = InTable;
	}

	// Cheat functions

	/** Returns true if ability cooldowns are ignored, returns false otherwise. Always returns false in shipping builds. */
	UE_API bool ShouldIgnoreCooldowns() const;

	/** Returns true if ability costs are ignored, returns false otherwise. Always returns false in shipping builds. */
	UE_API bool ShouldIgnoreCosts() const;

	/** Called when debug strings are available, to write them to the display */
	DECLARE_MULTICAST_DELEGATE(FOnClientServerDebugAvailable);
	FOnClientServerDebugAvailable OnClientServerDebugAvailable;

	/** Global place to accumulate debug strings for ability system component. Used when we fill up client side debug string immediately, and then wait for server to send server strings */
	TArray<FString>	AbilitySystemDebugStrings;

	/** Set to true if you want the "ShowDebug AbilitySystem" cheat to use the hud's debug target instead of the ability system's debug target. */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->bUseDebugTargetFromHud to access this variable in code")
	bool bUseDebugTargetFromHud;

	/** Helper functions for applying global scaling to various ability system tasks. This isn't meant to be a shipping feature, but to help with debugging and interation via cvar AbilitySystem.GlobalAbilityScale */
	static UE_API void NonShipping_ApplyGlobalAbilityScaler_Rate(float& Rate);
	static UE_API void NonShipping_ApplyGlobalAbilityScaler_Duration(float& Duration);

	// Global Tags

	/** TryActivate failed due to being dead */
	UE_DEPRECATED(5.5, "This variable is not used in the codebase")
	UPROPERTY()
	FGameplayTag ActivateFailIsDeadTag; 
	
	UE_DEPRECATED(5.5, "This variable is not used in the codebase")
	UPROPERTY(config)
	FName ActivateFailIsDeadName;

	/** TryActivate failed due to being on cooldown */
	UPROPERTY()
	FGameplayTag ActivateFailCooldownTag; 
	
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings (it will map to ActivateFailCooldownTag)")
	UPROPERTY(config)
	FName ActivateFailCooldownName;

	/** TryActivate failed due to not being able to spend costs */
	UPROPERTY()
	FGameplayTag ActivateFailCostTag; 
	
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings (it will map to ActivateFailCostTag)")
	UPROPERTY(config)
	FName ActivateFailCostName;

	/** TryActivate failed due to being blocked by other abilities */
	UPROPERTY()
	FGameplayTag ActivateFailTagsBlockedTag; 
	
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings (it will map to ActivateFailTagsBlockedTag)")
	UPROPERTY(config)
	FName ActivateFailTagsBlockedName;

	/** TryActivate failed due to missing required tags */
	UPROPERTY()
	FGameplayTag ActivateFailTagsMissingTag; 
	
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings (it will map to ActivateFailTagsMissingTag)")
	UPROPERTY(config)
	FName ActivateFailTagsMissingName;

	/** Failed to activate due to invalid networking settings, this is designer error */
	UPROPERTY()
	FGameplayTag ActivateFailNetworkingTag; 
	
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings (it will map to ActivateFailNetworkingTag)")
	UPROPERTY(config)
	FName ActivateFailNetworkingName;

	/** How many bits to use for "number of tags" in FMinimalReplicationTagCountMap::NetSerialize.  */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->MinimalReplicationTagCountBits to access this variable in code")
	UPROPERTY(config)
	int32	MinimalReplicationTagCountBits;

	/** Initialize global tags by reading from config using the names and creating tags for use at runtime */
	UE_API virtual void InitGlobalTags();

	UE_API void InitTargetDataScriptStructCache();

	/** Initialize GameplayCue Parameters */
	UE_API virtual void InitGameplayCueParameters(FGameplayCueParameters& CueParameters, const FGameplayEffectSpecForRPC &Spec);
	UE_API virtual void InitGameplayCueParameters_GESpec(FGameplayCueParameters& CueParameters, const FGameplayEffectSpec &Spec);
	UE_API virtual void InitGameplayCueParameters(FGameplayCueParameters& CueParameters, const FGameplayEffectContextHandle& EffectContext);

	/**
	 * Trigger async loading of the gameplay cue object libraries. By default, the manager will do this on creation,
	 * but that behavior can be changed by a derived class overriding ShouldAsyncLoadObjectLibrariesAtStart and returning false.
	 * In that case, this function must be called to begin the load
	 */
	UE_API virtual void StartAsyncLoadingObjectLibraries();

	/** Simple accessor to whether gameplay modifier evaluation channels should be allowed or not */
	UE_API bool ShouldAllowGameplayModEvaluationChannels() const;

	/**
	 * Returns whether the specified gameplay mod evaluation channel is valid for use or not.
	 * Considers whether channel usage is allowed at all, as well as if the specified channel has a valid alias for the game.
	 */
	UE_API bool IsGameplayModEvaluationChannelValid(EGameplayModEvaluationChannel Channel) const;

	/** Simple channel-based accessor to the alias name for the specified gameplay mod evaluation channel, if any */
	UE_API const FName& GetGameplayModEvaluationChannelAlias(EGameplayModEvaluationChannel Channel) const;

	/** Simple index-based accessor to the alias name for the specified gameplay mod evaluation channel, if any */
	UE_API const FName& GetGameplayModEvaluationChannelAlias(int32 Index) const;

	/** Path where the engine will load gameplay cue notifies from */
	UE_API virtual TArray<FString> GetGameplayCueNotifyPaths();

	/** Add a path to the GameplayCueNotifyPaths array. */
	UE_API virtual void AddGameplayCueNotifyPath(const FString& InPath);

	/**
	 * Remove the given gameplay cue notify path from the GameplayCueNotifyPaths array.
	 *
	 * @return Number of paths removed.
	 */
	UE_API virtual int32 RemoveGameplayCueNotifyPath(const FString& InPath);

	UPROPERTY()
	FNetSerializeScriptStructCache	TargetDataStructCache;

	UPROPERTY()
	FNetSerializeScriptStructCache	EffectContextStructCache;

	UE_API void AddAttributeDefaultTables(const FName OwnerName, const TArray<FSoftObjectPath>& AttribDefaultTableNames);
	UE_API void RemoveAttributeDefaultTables(const FName OwnerName, const TArray<FSoftObjectPath>& AttribDefaultTableNames);
protected:

	/** Get the SoftObjectPaths for all tables that should be loaded for default Attribute values */
	UE_API virtual TArray<FSoftObjectPath> GetGlobalAttributeSetDefaultsTablePaths() const;

	UE_API virtual void InitAttributeDefaults();
	UE_API virtual void ReloadAttributeDefaults();
	UE_API virtual void AllocAttributeSetInitter();

	/** Whether the game should allow the usage of gameplay mod evaluation channels or not */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->bAllowGameplayModEvaluationChannels to access this variable in code")
	UPROPERTY(config)
	bool bAllowGameplayModEvaluationChannels;

	/** The default mod evaluation channel for the game */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->DefaultGameplayModEvaluationChannel to access this variable in code")
	UPROPERTY(config)
	EGameplayModEvaluationChannel DefaultGameplayModEvaluationChannel;

	/** Game-specified named aliases for gameplay mod evaluation channels; Only those with valid aliases are eligible to be used in a game (except Channel0, which is always valid) */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->GameplayModEvaluationChannelAliases to access this variable in code")
	UPROPERTY(config)
	FName GameplayModEvaluationChannelAliases[static_cast<int32>(EGameplayModEvaluationChannel::Channel_MAX)];

	/** Name of global curve table to use as the default for scalable floats, etc. */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->GlobalCurveTableName to access this variable in code")
	UPROPERTY(config)
	FSoftObjectPath GlobalCurveTableName;

	UPROPERTY()
	TObjectPtr<UCurveTable> GlobalCurveTable;

	/** Holds information about the valid attributes' min and max values and stacking rules */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->GlobalAttributeMetaDataTableName to access this variable in code")
	UPROPERTY(config)
	FSoftObjectPath GlobalAttributeMetaDataTableName;

	UPROPERTY()
	TObjectPtr<UDataTable> GlobalAttributeMetaDataTable;

	/** Holds default values for attribute sets, keyed off of Name/Levels. NOTE: Preserved for backwards compatibility, should use the array version below now */
	UE_DEPRECATED(5.5, "See GetGlobalAttributeSetDefaultsTablePaths() which is an array rather than this version (singular)")
	UPROPERTY(config)
	FSoftObjectPath GlobalAttributeSetDefaultsTableName;

	/** Array of curve table names to use for default values for attribute sets, keyed off of Name/Levels */
	UE_DEPRECATED(5.5, "Set this variable through the Project Settings and use GetGlobalAttributeSetDefaultsTablePaths() to access this variable in code")
	UPROPERTY()
	TArray<FSoftObjectPath> GlobalAttributeSetDefaultsTableNames;

	/** Curve tables containing default values for attribute sets, keyed off of Name/Levels */
	UPROPERTY()
	TArray<TObjectPtr<UCurveTable>> GlobalAttributeDefaultsTables;

	/** Class reference to gameplay cue manager. Use this if you want to just instantiate a class for your gameplay cue manager without having to create an asset. */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->GlobalGameplayCueManagerClass to access this variable in code")
	UPROPERTY(config)
	FSoftObjectPath GlobalGameplayCueManagerClass;

	/** Object reference to gameplay cue manager (E.g., reference to a specific blueprint of your GameplayCueManager class. This is not necessary unless you want to have data or blueprints in your gameplay cue manager. */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->GlobalGameplayCueManagerName to access this variable in code")
	UPROPERTY(config)
	FSoftObjectPath GlobalGameplayCueManagerName;

	/** Look in these paths for GameplayCueNotifies. These are your "always loaded" set. */
	UE_DEPRECATED(5.5, "This will be moved to private. Use GetGameplayCueNotifyPaths, AddGameplayCueNotifyPath, or RemoveGameplayCueNotifyPath")
	UPROPERTY(config)
	TArray<FString>	GameplayCueNotifyPaths;

	/** The class to instantiate as the GameplayTagResponseTable. */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->GameplayTagResponseTableName to access this variable in code")
	UPROPERTY(config)
	FSoftObjectPath GameplayTagResponseTableName;

	UPROPERTY()
	TObjectPtr<UGameplayTagReponseTable> GameplayTagResponseTable;

	bool bInitialized = false;

	/** Set to true if you want clients to try to predict gameplay effects done to targets. If false it will only predict self effects */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->PredictTargetGameplayEffects to access this variable in code")
	UPROPERTY(config)
	bool PredictTargetGameplayEffects;

	/** 
	 * Set to true if you want tags granted to owners from ability activations to be replicated. If false, ActivationOwnedTags are only applied locally. 
	 * This should only be disabled for legacy game code that depends on non-replication of ActivationOwnedTags.
	 */
	UE_DEPRECATED(5.5, "Configure this variable through the Project Settings and use GetDefault<UGameplayAbilitiesDeveloperSettings>()->ReplicateActivationOwnedTags to access this variable in code")
	UPROPERTY(config)
	bool ReplicateActivationOwnedTags;

	/** Manager for all gameplay cues */
	UPROPERTY()
	TObjectPtr<UGameplayCueManager> GlobalGameplayCueManager;

	/** Used to initialize attribute sets */
	TSharedPtr<FAttributeSetInitter> GlobalAttributeSetInitter;

	/** 
	 * Curve table names to use for default values for attribute sets, keyed off of Name/Levels (with owners to allow removal of hard reference by GlobalAttributeDefaultsTables)
	 * Required to allow unloading of plugins
	*/
	TMap<FSoftObjectPath, TArray<FName>> GlobalAttributeSetDefaultsTableNamesWithOwners;

	template <class T>
	T* InternalGetLoadTable(T*& Table, FString TableName);

	void OnAttributeDefaultTableChanged();

#if WITH_EDITOR
	UE_API void OnTableReimported(UObject* InObject);
	UE_API void OnPreBeginPIE(const bool bIsSimulatingInEditor);
#endif

	static UE_API void ResetCachedData();
	UE_API void HandlePreLoadMap(const FWorldContext& WorldContext, const FString& MapName);

#if WITH_EDITORONLY_DATA
	bool RegisteredReimportCallback;
#endif

private:
	UE_API void PerformDeveloperSettingsUpgrade();

	FDelegateHandle ModulesChangedHandle;
	FDelegateHandle ModulesUnloadedHandle;

public:
	//To add functionality for opening assets directly from the game.
	UE_API void Notify_OpenAssetInEditor(FString AssetName, int AssetType);
	FOnAbilitySystemAssetOpenedDelegate AbilityOpenAssetInEditorCallbacks;

	//...for finding assets directly from the game.
	UE_API void Notify_FindAssetInEditor(FString AssetName, int AssetType);
	FOnAbilitySystemAssetFoundDelegate AbilityFindAssetInEditorCallbacks;
};

/** Scope object that indicates when a gameplay effect is being applied */
struct FScopeCurrentGameplayEffectBeingApplied
{
	FScopeCurrentGameplayEffectBeingApplied(const FGameplayEffectSpec* Spec, UAbilitySystemComponent* AbilitySystemComponent)
	{
		UAbilitySystemGlobals::Get().PushCurrentAppliedGE(Spec, AbilitySystemComponent);
	}
	~FScopeCurrentGameplayEffectBeingApplied()
	{
		UAbilitySystemGlobals::Get().PopCurrentAppliedGE();
	}
};

#undef UE_API
