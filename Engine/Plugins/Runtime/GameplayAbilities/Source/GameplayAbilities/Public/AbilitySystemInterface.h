// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "AbilitySystemInterface.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

/** 
 * Interface for native actors to expose access to an ability system component. 
 * This is used by native actors to efficiently return a component and handle more complicated cases. This interface should never be called directly, 
 * instead call GetAbilitySystemComponentFromActor or GetAbilitySystemComponent on the AbilitySystemBlueprintLibrary as that also handles looking up by component type.
 */
UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UAbilitySystemInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IAbilitySystemInterface
{
	GENERATED_IINTERFACE_BODY()

	/** Returns the ability system component to use for this actor. It may live on another actor, such as a Pawn using the PlayerState's component */
	virtual UAbilitySystemComponent* GetAbilitySystemComponent() const = 0;
};

#undef UE_API
