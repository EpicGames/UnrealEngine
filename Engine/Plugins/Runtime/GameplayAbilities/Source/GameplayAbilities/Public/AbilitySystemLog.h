// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "VisualLogger/VisualLogger.h"

// Intended categories:
//	Log - This happened. What gameplay programmers may care about to debug
//	Verbose - This is why this happened. What you may turn on to debug the skill system code.
//  VeryVerbose - This is what didn't happen, and why. Extreme printf debugging
//


GAMEPLAYABILITIES_API DECLARE_LOG_CATEGORY_EXTERN(LogAbilitySystem, Display, All);
GAMEPLAYABILITIES_API DECLARE_LOG_CATEGORY_EXTERN(VLogAbilitySystem, Display, All);
GAMEPLAYABILITIES_API DECLARE_LOG_CATEGORY_EXTERN(LogGameplayEffects, Display, All);

#if NO_LOGGING || !PLATFORM_DESKTOP

// Without logging enabled we pass ability system through to UE_LOG which only handles Fatal verbosity in NO_LOGGING
#define ABILITY_LOG(Verbosity, Format, ...) \
{ \
	UE_LOG(LogAbilitySystem, Verbosity, Format, ##__VA_ARGS__); \
}

#define ABILITY_VLOG(Actor, Verbosity, Format, ...) \
{ \
	UE_LOG(LogAbilitySystem, Verbosity, Format, ##__VA_ARGS__); \
	UE_VLOG(Actor, VLogAbilitySystem, Verbosity, Format, ##__VA_ARGS__); \
	static_assert(false, "ABILITY_VLOG is deprecated because the same messages do not make sense in both contexts. Use both UE_LOG and UE_VLOG"); \
}

#else

#define ABILITY_LOG(Verbosity, Format, ...) \
{ \
	UE_LOG(LogAbilitySystem, Verbosity, Format, ##__VA_ARGS__); \
}

#define ABILITY_VLOG(Actor, Verbosity, Format, ...) \
{ \
	UE_LOG(LogAbilitySystem, Verbosity, Format, ##__VA_ARGS__); \
	UE_VLOG(Actor, VLogAbilitySystem, Verbosity, Format, ##__VA_ARGS__); \
	static_assert(false, "ABILITY_VLOG is deprecated because the same messages do not make sense in both contexts. Use both UE_LOG and UE_VLOG"); \
}

#endif //NO_LOGGING

#if ENABLE_VISUAL_LOG

#define ABILITY_VLOG_ATTRIBUTE_GRAPH(Actor, Verbosity, AttributeName, OldValue, NewValue) \
{ \
	if( FVisualLogger::IsRecording() ) \
	{ \
		static const FName GraphName("Attribute Graph"); \
		const double CurrentTime = Actor->GetWorld() ? Actor->GetWorld()->GetTimeSeconds() : 0.0; \
		const FVector2D OldPt(CurrentTime, OldValue); \
		const FVector2D NewPt(CurrentTime, NewValue); \
		const FName LineName(*AttributeName); \
		UE_VLOG_HISTOGRAM(Actor, VLogAbilitySystem, Log, GraphName, LineName, OldPt); \
		UE_VLOG_HISTOGRAM(OwnerActor, VLogAbilitySystem, Log, GraphName, LineName, NewPt); \
	} \
}

#else

#define ABILITY_VLOG_ATTRIBUTE_GRAPH(Actor, Verbosity, AttributeName, OldValue, NewValue)

#endif //ENABLE_VISUAL_LOG
