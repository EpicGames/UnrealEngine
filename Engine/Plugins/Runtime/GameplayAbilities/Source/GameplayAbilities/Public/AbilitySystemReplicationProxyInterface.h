// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"

#include "AbilitySystemReplicationProxyInterface.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;
struct FGameplayCueParameters;
struct FGameplayEffectContextHandle;
struct FGameplayEffectSpecForRPC;
struct FGameplayTag;
struct FGameplayTagContainer;
struct FPredictionKey;

/** Interface for actors that act like an ability system component for replication. This can be used to group together different components for replication */
UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UAbilitySystemReplicationProxyInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IAbilitySystemReplicationProxyInterface
{
	GENERATED_IINTERFACE_BODY()

	virtual void ForceReplication() = 0;


	// --------------------------------------------------------------------------------------------
	//	Call Proxies: these can be optionally implemented to intercept GameplayCue events. By default, they forward to the multicasts.
	// --------------------------------------------------------------------------------------------

	UE_API virtual void Call_InvokeGameplayCueExecuted_FromSpec(const FGameplayEffectSpecForRPC Spec, FPredictionKey PredictionKey);
	UE_API virtual void Call_InvokeGameplayCueExecuted(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext);
	UE_API virtual void Call_InvokeGameplayCuesExecuted(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext);
	UE_API virtual void Call_InvokeGameplayCueExecuted_WithParams(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters);
	UE_API virtual void Call_InvokeGameplayCuesExecuted_WithParams(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters);
	UE_API virtual void Call_InvokeGameplayCueAdded(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext);
	UE_API virtual void Call_InvokeGameplayCueAdded_WithParams(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters Parameters);
	UE_API virtual void Call_InvokeGameplayCueAddedAndWhileActive_FromSpec(const FGameplayEffectSpecForRPC& Spec, FPredictionKey PredictionKey);
	UE_API virtual void Call_InvokeGameplayCueAddedAndWhileActive_WithParams(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters);
	UE_API virtual void Call_InvokeGameplayCuesAddedAndWhileActive_WithParams(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters);

	// ----------------------------------------------
	//	Multicast Proxies: these should be implemented as unreliable NetMulticast functions
	// ----------------------------------------------
	
	virtual void NetMulticast_InvokeGameplayCueExecuted_FromSpec(const FGameplayEffectSpecForRPC Spec, FPredictionKey PredictionKey) = 0;

	virtual void NetMulticast_InvokeGameplayCueExecuted(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext) = 0;
	
	virtual void NetMulticast_InvokeGameplayCuesExecuted(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext) = 0;
	
	virtual void NetMulticast_InvokeGameplayCueExecuted_WithParams(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters) = 0;
	
	virtual void NetMulticast_InvokeGameplayCuesExecuted_WithParams(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters) = 0;
	
	virtual void NetMulticast_InvokeGameplayCueAdded(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext) = 0;
	
	virtual void NetMulticast_InvokeGameplayCueAdded_WithParams(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters Parameters) = 0;
	
	virtual void NetMulticast_InvokeGameplayCueAddedAndWhileActive_FromSpec(const FGameplayEffectSpecForRPC& Spec, FPredictionKey PredictionKey) = 0;
	
	virtual void NetMulticast_InvokeGameplayCueAddedAndWhileActive_WithParams(const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters) = 0;
	
	virtual void NetMulticast_InvokeGameplayCuesAddedAndWhileActive_WithParams(const FGameplayTagContainer GameplayCueTags, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters) = 0;
};

#undef UE_API
