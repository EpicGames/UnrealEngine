// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("AbilitySystem"), STATGROUP_AbilitySystem, STATCAT_Advanced);

DECLARE_CYCLE_STAT_EXTERN(TEXT("GameplayEffectsHasAllTagsTime"), STAT_GameplayEffectsHasAllTags, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GameplayEffectsHasAnyTagTime"), STAT_GameplayEffectsHasAnyTag, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GameplayEffectsGetOwnedTags"), STAT_GameplayEffectsGetOwnedTags, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GameplayEffectsTick"), STAT_GameplayEffectsTick, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CanApplyAttributeModifiers"), STAT_GameplayEffectsCanApplyAttributeModifiers, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GetActiveEffectsData"), STAT_GameplayEffectsGetActiveEffectsTimeRemaining, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GetActiveEffectsData"), STAT_GameplayEffectsGetActiveEffectsDuration, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GetActiveEffectsData"), STAT_GameplayEffectsGetActiveEffectsTimeRemainingAndDuration, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GetActiveEffectsData"), STAT_GameplayEffectsGetActiveEffects, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GetActiveEffectsData"), STAT_GameplayEffectsGetAllActiveEffectHandles, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SetActiveEffectsData"), STAT_GameplayEffectsModifyActiveEffectStartTime, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GetCooldownTimeRemaining"), STAT_GameplayAbilityGetCooldownTimeRemaining, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GetCooldownTimeRemaining"), STAT_GameplayAbilityGetCooldownTimeRemainingAndDuration, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("RemoveActiveGameplayEffect"), STAT_RemoveActiveGameplayEffect, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("ApplyGameplayEffectSpec"), STAT_ApplyGameplayEffectSpec, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GetGameplayCueFunction"), STAT_GetGameplayCueFunction, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GetOutgoingSpec"), STAT_GetOutgoingSpec, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("InitAttributeSetDefaults"), STAT_InitAttributeSetDefaults, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("TickAbilityTasks"), STAT_TickAbilityTasks, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("FindAbilitySpecFromHandle"), STAT_FindAbilitySpecFromHandle, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Aggregator Evaluate"), STAT_AggregatorEvaluate, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Aggregator BroadcastOnDirty"), STAT_AggregatorBroadcastOnDirty, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Has Application Immunity To Spec"), STAT_HasApplicationImmunityToSpec, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Has Matching GameplayTag"), STAT_HasMatchingGameplayTag, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GameplayCueNotify Static"), STAT_HandleGameplayCueNotifyStatic, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GameplayCueNotify Actor"), STAT_HandleGameplayCueNotifyActor, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("ApplyGameplayEffectToTarget"), STAT_ApplyGameplayEffectToTarget, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("ExecuteActiveEffectsFrom"), STAT_ExecuteActiveEffectsFrom, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("InternalExecuteMod"), STAT_InternalExecuteMod, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PostGameplayEffectExecute"), STAT_PostGameplayEffectExecute, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("ActiveGameplayEffect Added"), STAT_OnActiveGameplayEffectAdded, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("ActiveGameplayEffect Removed"), STAT_OnActiveGameplayEffectRemoved, STATGROUP_AbilitySystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GameplayCueInterface HandleGameplayCue"), STAT_GameplayCueInterface_HandleGameplayCue, STATGROUP_AbilitySystem, );

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("AbilityTask Count"), STAT_AbilitySystem_TaskCount, STATGROUP_AbilitySystem, );