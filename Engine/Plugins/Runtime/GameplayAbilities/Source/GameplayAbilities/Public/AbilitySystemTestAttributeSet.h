// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AttributeSet.h"
#include "AbilitySystemTestAttributeSet.generated.h"

#define UE_API GAMEPLAYABILITIES_API

UCLASS(Blueprintable, BlueprintType, meta=(HideInDetailsView), MinimalAPI)
class UAbilitySystemTestAttributeSet : public UAttributeSet
{
	GENERATED_UCLASS_BODY()

	/**
	 *	NOTE ON MUTABLE:
	 *	This is only done so that UAbilitySystemTestAttributeSet can be initialized directly in GameplayEffectsTest.cpp without doing a const_cast in 100+ places.
	 *	Mutable is not required and should never be used on normal attribute sets.
	 */


	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category = "AttributeTest", meta = (HideFromModifiers))			// You can't make a GameplayEffect modify Health Directly (go through)
	mutable float	MaxHealth;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category = "AttributeTest", meta = (HideFromModifiers))			// You can't make a GameplayEffect modify Health Directly (go through)
	mutable float	Health;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category = "AttributeTest")
	mutable FGameplayAttributeData	Mana;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category = "AttributeTest")
	mutable float	MaxMana;

	/** This Damage is just used for applying negative health mods. Its not a 'persistent' attribute. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AttributeTest")		// You can't make a GameplayEffect 'powered' by Damage (Its transient)
	mutable float	Damage;

	/** This Attribute is the actual spell damage for this actor. It will power spell based GameplayEffects */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category = "AttributeTest")
	mutable float	SpellDamage;

	/** This Attribute is the actual physical damage for this actor. It will power physical based GameplayEffects */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category = "AttributeTest")
	mutable float	PhysicalDamage;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category = "AttributeTest")
	mutable float	CritChance;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category = "AttributeTest")
	mutable float	CritMultiplier;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category = "AttributeTest")
	mutable float	ArmorDamageReduction;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category = "AttributeTest")
	mutable float	DodgeChance;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category = "AttributeTest")
	mutable float	LifeSteal;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category = "AttributeTest")
	mutable float	Strength;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AttributeTest")
	mutable float	StackingAttribute1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AttributeTest")
	mutable float	StackingAttribute2;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AttributeTest")
	mutable float	NoStackAttribute;

	UE_API virtual bool PreGameplayEffectExecute(struct FGameplayEffectModCallbackData &Data) override;
	UE_API virtual void PostGameplayEffectExecute(const struct FGameplayEffectModCallbackData &Data) override;
};

#undef UE_API
