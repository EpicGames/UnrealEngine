// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayCueInterface.h"
#include "GameFramework/DefaultPawn.h"
#include "AbilitySystemInterface.h"
#include "AbilitySystemTestPawn.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

UCLASS(Blueprintable, BlueprintType, notplaceable, MinimalAPI)
class AAbilitySystemTestPawn : public ADefaultPawn, public IGameplayCueInterface, public IAbilitySystemInterface
{
	GENERATED_UCLASS_BODY()

	UE_API virtual void PostInitializeComponents() override;

	UE_API virtual UAbilitySystemComponent* GetAbilitySystemComponent() const override;
	
private:
	/** DefaultPawn collision component */
	UPROPERTY(Category = AbilitySystem, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UAbilitySystemComponent> AbilitySystemComponent;
public:

	//UPROPERTY(EditDefaultsOnly, Category=GameplayEffects)
	//UGameplayAbilitySet * DefaultAbilitySet;

	static UE_API FName AbilitySystemComponentName;

	/** Returns AbilitySystemComponent subobject **/
	class UAbilitySystemComponent* GetAbilitySystemComponent() { return AbilitySystemComponent; }
};

#undef UE_API
