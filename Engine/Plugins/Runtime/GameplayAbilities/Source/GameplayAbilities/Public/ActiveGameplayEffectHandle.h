// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActiveGameplayEffectHandle.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

/**
 * This handle is required for things outside of FActiveGameplayEffectsContainer to refer to a specific active GameplayEffect
 *	For example if a skill needs to create an active effect and then destroy that specific effect that it created, it has to do so
 *	through a handle. a pointer or index into the active list is not sufficient. These are not synchronized between clients and server.
 */
USTRUCT(BlueprintType)
struct FActiveGameplayEffectHandle
{
	GENERATED_BODY()

	/** Construct an invalid handle typically used to indicate failure of a Gameplay Effect to execute */
	FActiveGameplayEffectHandle() = default;

	/**
	 * This constructor has been deprecated since it leaks internal implementation, leaves the Owning ASC undefined,
	 * and it's not always clear what the intent of the caller is.  It's been replaced by two factory functions:
	 * 1. GenerateNewHandle which will give you a valid, properly constructed Handle
	 * 2. GetInstantExecutedHandle which will give you a special handle, deemed to be invalid since it already executed instantly.
	 */
	UE_DEPRECATED(5.8, "Use GenerateNewHandle (for properly constructed, valid handle) or GetInstantExecutedHandle (see comments)")
	FActiveGameplayEffectHandle(int32 InHandle)
		: Handle(InHandle),
		bPassedFiltersAndWasExecuted(true)
	{
	}

	/** Generates a new handle, it will be set to successfully applied. It is up to the caller to associate it with an FActiveGameplayEffect. */
	static UE_API FActiveGameplayEffectHandle GenerateNewHandle(UAbilitySystemComponent* OwningComponent);

	/** Get a handle that represents an Instant (non-duration) Gameplay Effect. We use this as a sentinel value for FActiveGameplayEffectHandle
	 * returned from functions where the underlying GE has already completed execution (instantly) and thus cannot be queried further.
	 * 
	 * Such a handle has the following rules:
	 * IsValid() unintuitively returns false, as it does not represent an Active Gameplay Effect.
	 * WasSuccessfullyApplied() returns true, as it is assumed this represents the execution of the Instant GE.
	 * GetOwningAbilitySystemComponent() returns an undefined value and should not be relied upon.
	 */
	UE_API static FActiveGameplayEffectHandle GetInstantExecutedHandle();

	/** True if this is/was tracking an active (non-instant) gameplay effect. Will continue to return true if the GE was removed. */
	bool IsValid() const
	{
		return Handle != INDEX_NONE;
	}

	/** True if this applied a gameplay effect. This can be true when it is not valid if it applied an instant effect */
	bool WasSuccessfullyApplied() const
	{
		return bPassedFiltersAndWasExecuted;
	}

	/** Returns the ability system component that currently has the ActiveGameplayEffect this handle represents.
	 * Note: If the GameplayEffect has been removed, this will return nullptr regardless of where the ActiveGE was applied. */
	UE_API UAbilitySystemComponent* GetOwningAbilitySystemComponent() const;

	UE_DEPRECATED(5.8, "We no longer use a global TMap to get the Owning ASC. You can remove this function.")
	static void ResetGlobalHandleMap() {}

	UE_DEPRECATED(5.8, "We no longer use a global TMap to get the Owning ASC. You can remove this function.")
	void RemoveFromGlobalMap() {}

	bool operator==(const FActiveGameplayEffectHandle& Other) const
	{
		return Handle == Other.Handle;
	}

	bool operator!=(const FActiveGameplayEffectHandle& Other) const
	{
		return Handle != Other.Handle;
	}

	friend uint32 GetTypeHash(const FActiveGameplayEffectHandle& InHandle)
	{
		return InHandle.Handle;
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("%d"), Handle);
	}

	/** Reset the handle to a default (invalid) state */
	void Invalidate()
	{
		*this = FActiveGameplayEffectHandle{};
	}

private:
	UE_API FActiveGameplayEffectHandle(UAbilitySystemComponent* InOwningASC);

	UPROPERTY()
	TWeakObjectPtr<UAbilitySystemComponent> WeakOwningASC = {};

	UPROPERTY()
	int32 Handle = INDEX_NONE;

	UPROPERTY()
	bool bPassedFiltersAndWasExecuted = false;
};

#undef UE_API
