// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/*================================================================================
ActiveGameplayEffectIterator.h: implementation of ActiveGameplayEffect Iterator.

The is an iterator for FActiveGameplayEffectContainer. The main points are:
-This iterates over both the active gameplay effect array and the pending list of new GEs
-This skips any gameplay effects which are pending remove
-This locks the array while the iterator is in scope, meaning any adds or removes 
 to the active gameplay effect array will have their memory operations deffered.
 (That is, the effects will be added/removed, but not removed from memory until the scope lock
 has been removed).
	
================================================================================*/


template< typename ElementType, typename ContainerType >
class FActiveGameplayEffectIterator
{
public:

	FActiveGameplayEffectIterator(const ContainerType& InContainer, int32 StartIdx=0)
		: index(StartIdx)
		, Current(nullptr)
		, Pending(nullptr)
		, Container(const_cast<ContainerType&>(InContainer))
	{
		Container.IncrementLock();

		if (index >= 0)
		{
			UpdateCurrent();
		}
	}

	~FActiveGameplayEffectIterator()
	{
		Container.DecrementLock();
	}


	void operator++()
	{
		Next();
	}

	ElementType& operator*() const
	{
		check(Current);
		return *Current;
	}

	ElementType* operator->() const
	{
		check(Current);
		return Current;
	}

	explicit operator bool() const
	{
		return (Current != nullptr);
	}

	friend bool operator==(const FActiveGameplayEffectIterator& Lhs, const FActiveGameplayEffectIterator& Rhs)
	{
		return Lhs.Current == Rhs.Current;
	}

	friend bool operator!=(const FActiveGameplayEffectIterator& Lhs, const FActiveGameplayEffectIterator& Rhs)
	{
		return Lhs.Current != Rhs.Current;
	}

private:

	inline ElementType* AdvancePending(ElementType** Next)
	{
		// Advance pending next only if this isnt the tail of the pending gameplay effect list
		return (Next != const_cast<ElementType**>(Container.PendingGameplayEffectNext)) ? *Next : nullptr;
	}

	void Next()
	{
		if (index >= 0 )
		{
			// While iterating through the array, just increment the index
			++index;
		}

		else if (Pending)
		{
			// While iteratin through the linked list, jump to the next one
			Pending = AdvancePending(const_cast<ElementType**>(&Pending->PendingNext));
		}
	
		UpdateCurrent();
	}

	void UpdateCurrent()
	{
		if (index < 0 )
		{
			// We are already iterating the linked list, current element should equal the pending list element.
			Current = Pending;
		}

		else if (index >= Container.GameplayEffects_Internal.Num())
		{
			// Once we get to end of the array, we start iterating on the linked list. 
			Pending = AdvancePending(const_cast<ElementType**>(&Container.PendingGameplayEffectHead));
			Current = Pending;
			index = INDEX_NONE;
		}

		else
		{
			// We are still iterating the array, current is equal to the indexed element.
			Current = &Container.GameplayEffects_Internal[index];
		}
		
		// If the current element is pending remove, go to the next one.
		if (Current && Current->IsPendingRemove)
		{
			Next();
		}
	}

	int32	index;
	ElementType*	Current;
	ElementType*	Pending;
	ContainerType& Container;
};
