// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "GameplayCueInterface.h"
#include "GameplayCueManager.h"
#include "AnimNotify_GameplayCue.generated.h"


/**
 * UAnimNotify_GameplayCue
 *
 *	An animation notify used for instantaneous gameplay cues (Burst/Latent).
 */
UCLASS(editinlinenew, Const, hideCategories = Object, collapseCategories, MinimalAPI, Meta = (DisplayName = "GameplayCue (Burst)"))
class UAnimNotify_GameplayCue : public UAnimNotify
{
	GENERATED_BODY()

public:

	UAnimNotify_GameplayCue();

	virtual void Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) override;

	FString GetNotifyName_Implementation() const override;

#if WITH_EDITOR
	virtual bool CanBePlaced(UAnimSequenceBase* Animation) const override;
#endif // #if WITH_EDITOR

protected:

	// GameplayCue tag to invoke.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = GameplayCue, meta = (Categories = "GameplayCue"))
	FGameplayCueTag GameplayCue;
};


/**
 * UAnimNotify_GameplayCueState
 *
 *	An animation notify state used for duration based gameplay cues (Looping).
 */
UCLASS(editinlinenew, Const, hideCategories = Object, collapseCategories, MinimalAPI, Meta = (DisplayName = "GameplayCue (Looping)"))
class UAnimNotify_GameplayCueState : public UAnimNotifyState
{
	GENERATED_BODY()

public:

	UAnimNotify_GameplayCueState();

	virtual void NotifyBegin(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference) override;
	virtual void NotifyTick(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference) override;
	virtual void NotifyEnd(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) override;

	FString GetNotifyName_Implementation() const override;

#if WITH_EDITOR
	virtual bool CanBePlaced(UAnimSequenceBase* Animation) const override;
#endif // #if WITH_EDITOR

protected:

	// GameplayCue tag to invoke
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = GameplayCue, meta = (Categories = "GameplayCue"))
	FGameplayCueTag GameplayCue;

#if WITH_EDITORONLY_DATA
	FGameplayCueProxyTick PreviewProxyTick;
#endif // #if WITH_EDITORONLY_DATA
};
