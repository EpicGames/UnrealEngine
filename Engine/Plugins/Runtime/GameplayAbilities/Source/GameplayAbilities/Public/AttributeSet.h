// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "UObject/UnrealType.h"
#include "Engine/DataTable.h"
#include "AttributeSet.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;
class UAttributeSet;
class UCurveTable;
struct FGameplayAbilityActorInfo;
struct FAggregator;

/** Place in an AttributeSet to create an attribute that can be accessed using FGameplayAttribute. It is strongly encouraged to use this instead of raw float attributes */
USTRUCT(BlueprintType)
struct FGameplayAttributeData
{
	GENERATED_BODY()
	FGameplayAttributeData()
		: BaseValue(0.f)
		, CurrentValue(0.f)
	{}

	FGameplayAttributeData(float DefaultValue)
		: BaseValue(DefaultValue)
		, CurrentValue(DefaultValue)
	{}

	virtual ~FGameplayAttributeData() = default;

	/** Returns the current value, which includes temporary buffs */
	UE_API float GetCurrentValue() const;

	/** Modifies current value, normally only called by ability system or during initialization */
	UE_API virtual void SetCurrentValue(float NewValue);

	/** Returns the base value which only includes permanent changes */
	UE_API float GetBaseValue() const;

	/** Modifies the permanent base value, normally only called by ability system or during initialization */
	UE_API virtual void SetBaseValue(float NewValue);

protected:
	UPROPERTY(BlueprintReadOnly, Category = "Attribute")
	float BaseValue;

	UPROPERTY(BlueprintReadOnly, Category = "Attribute")
	float CurrentValue;
};

/** Describes a FGameplayAttributeData or float property inside an attribute set. Using this provides editor UI and helper functions */
USTRUCT(BlueprintType)
struct FGameplayAttribute
{
	GENERATED_USTRUCT_BODY()

	FGameplayAttribute()
		: Attribute(nullptr)
		, AttributeOwner(nullptr)
	{
	}

	UE_API FGameplayAttribute(FProperty *NewProperty);

	/** Returns true if our Attribute has a valid resolved Field */
	bool IsValid() const
	{
		return Attribute.Get() != nullptr;
	}

	/** Set up from a FProperty inside a set */
	void SetUProperty(FProperty *NewProperty)
	{
		Attribute = NewProperty;
		if (NewProperty)
		{
			AttributeOwner = Attribute->GetOwnerStruct();
			Attribute->GetName(AttributeName);
		}
		else
		{
			AttributeOwner = nullptr;
			AttributeName.Empty();
		}
	}

	/** Returns raw property */
	FProperty* GetUProperty() const
	{
		return Attribute.Get();
	}

	/** Returns the AttributeSet subclass holding this attribute */
	UClass* GetAttributeSetClass() const
	{
		check(Attribute.Get());
		return CastChecked<UClass>(Attribute->GetOwner<UObject>());
	}

	/** Returns true if this is one of the special attributes defined on the AbilitySystemComponent itself */
	UE_API bool IsSystemAttribute() const;

	/** Returns whether a property is able to represent a Gameplay Attribute. Does not check whether the property is owned by an attribute set. **/
	static UE_API bool IsSupportedProperty(const FProperty* Prop);

	/** Returns true if the variable associated with Property is of type FGameplayAttributeData or one of its subclasses */
	static UE_API bool IsGameplayAttributeDataProperty(const FProperty* Property);

	/** Modifies the current value of an attribute, will not modify base value if that is supported */
	UE_API void SetNumericValueChecked(float& NewValue, class UAttributeSet* Dest) const;

	/** Returns the current value of an attribute */
	UE_API float GetNumericValue(const UAttributeSet* Src) const;
	UE_API float GetNumericValueChecked(const UAttributeSet* Src) const;

	/** Returns the AttributeData, will fail if this is a float attribute */
	UE_API const FGameplayAttributeData* GetGameplayAttributeData(const UAttributeSet* Src) const;
	UE_API const FGameplayAttributeData* GetGameplayAttributeDataChecked(const UAttributeSet* Src) const;
	UE_API FGameplayAttributeData* GetGameplayAttributeData(UAttributeSet* Src) const;
	UE_API FGameplayAttributeData* GetGameplayAttributeDataChecked(UAttributeSet* Src) const;
	
	/** Equality/Inequality operators */
	UE_API bool operator==(const FGameplayAttribute& Other) const;
	UE_API bool operator!=(const FGameplayAttribute& Other) const;

	friend uint32 GetTypeHash( const FGameplayAttribute& InAttribute )
	{
		// FIXME: Use ObjectID or something to get a better, less collision prone hash
		return PointerHash(InAttribute.Attribute.Get());
	}

	/** Returns name of attribute, usually the same as the property */
	FString GetName() const
	{
		return AttributeName.IsEmpty() ? *GetNameSafe(Attribute.Get()) : AttributeName;
	}

#if WITH_EDITORONLY_DATA
	/** Custom serialization */
	UE_API void PostSerialize(const FArchive& Ar);
#endif

	/** Name of the attribute, usually the same as property name */
	UPROPERTY(Category = GameplayAttribute, VisibleAnywhere, BlueprintReadOnly)
	FString AttributeName;

	/** In editor, this will filter out properties with meta tag "HideInDetailsView" or equal to FilterMetaStr. In non editor, it returns all properties */
	static UE_API void GetAllAttributeProperties(TArray<FProperty*>& OutProperties, FString FilterMetaStr=FString(), bool UseEditorOnlyData=true);

private:
	friend class FAttributePropertyDetails;

	UPROPERTY(Category = GameplayAttribute, EditAnywhere)
	TFieldPath<FProperty> Attribute;
	//FProperty*	Attribute;

	UPROPERTY(Category = GameplayAttribute, VisibleAnywhere)
	TObjectPtr<UStruct> AttributeOwner;
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits< FGameplayAttribute > : public TStructOpsTypeTraitsBase2< FGameplayAttribute >
{
	enum
	{
		WithPostSerialize = true,
	};
};
#endif

/**
 * Defines the set of all GameplayAttributes for your game
 * Games should subclass this and add FGameplayAttributeData properties to represent attributes like health, damage, etc
 * AttributeSets are added to the actors as subobjects, and then registered with the AbilitySystemComponent
 * It often desired to have several sets per project that inherit from each other
 * You could make a base health set, then have a player set that inherits from it and adds more attributes
 */
UCLASS(DefaultToInstanced, Blueprintable, MinimalAPI)
class UAttributeSet : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	// Populates (without emptying) a TArray with all FGameplayAttributes from an attribute set class.
	static UE_API void GetAttributesFromSetClass(const TSubclassOf<UAttributeSet>& AttributeSetClass, TArray<FGameplayAttribute>& Attributes);

	/** Override to disable initialization for specific properties */
	virtual bool ShouldInitProperty(bool FirstInit, FProperty* PropertyToInit) const { return true; }

	/**
	 *	Called just before modifying the value of an attribute. AttributeSet can make additional modifications here. Return true to continue, or false to throw out the modification.
	 *	Note this is only called during an 'execute'. E.g., a modification to the 'base value' of an attribute. It is not called during an application of a GameplayEffect, such as a 5 second +10 movement speed buff.
	 */	
	virtual bool PreGameplayEffectExecute(struct FGameplayEffectModCallbackData &Data) { return true; }
	
	/**
	 *	Called just after a GameplayEffect is executed to modify the base value of an attribute. No more changes can be made.
	 *	Note this is only called during an 'execute'. E.g., a modification to the 'base value' of an attribute. It is not called during an application of a GameplayEffect, such as a 5 second +10 movement speed buff.
	 */
	virtual void PostGameplayEffectExecute(const struct FGameplayEffectModCallbackData &Data) { }

	/**
	 *	An "On Aggregator Change" type of event could go here, and that could be called when active gameplay effects are added or removed to an attribute aggregator.
	 *	It is difficult to give all the information in these cases though - aggregators can change for many reasons: being added, being removed, being modified, having a modifier change, immunity, stacking rules, etc.
	 */

	/**
	 *	Called just before any modification happens to an attribute.
	 *	There is no additional context provided here since anything can trigger this. Executed effects, duration based effects, effects being removed, immunity being applied, stacking rules changing, etc.
	 *	This function is meant to enforce things like "Health = Clamp(Health, 0, MaxHealth)" and NOT things like "trigger this extra thing if damage is applied, etc".
	 *	
	 *	NewValue is a mutable reference so you are able to clamp the newly applied value as well.
	 */
	virtual void PreAttributeChange(const FGameplayAttribute& Attribute, float& NewValue) { }

	/** Called just after any modification happens to an attribute. */
	virtual void PostAttributeChange(const FGameplayAttribute& Attribute, float OldValue, float NewValue) { }

	/**
	 *	This is called just before any modification happens to an attribute's base value when an attribute aggregator exists.
	 *	This function should enforce clamping (presuming you wish to clamp the base value along with the final value in PreAttributeChange)
	 *	This function should NOT invoke gameplay related events or callbacks. Do those in PreAttributeChange() which will be called prior to the
	 *	final value of the attribute actually changing.
	 */
	virtual void PreAttributeBaseChange(const FGameplayAttribute& Attribute, float& NewValue) const { }

	/** Called just after any modification happens to an attribute's base value when an attribute aggregator exists. */
	virtual void PostAttributeBaseChange(const FGameplayAttribute& Attribute, float OldValue, float NewValue) const { }

	/** Callback for when an FAggregator is created for an attribute in this set. Allows custom setup of FAggregator::EvaluationMetaData */
	virtual void OnAttributeAggregatorCreated(const FGameplayAttribute& Attribute, FAggregator* NewAggregator) const { }

	/** This signifies the attribute set can be ID'd by name over the network. */
	UE_API void SetNetAddressable();

	/** Initializes attribute data from a meta DataTable */
	UE_API virtual void InitFromMetaDataTable(const UDataTable* DataTable);

	/** Gets information about owning actor */
	UE_API AActor* GetOwningActor() const;
	UE_API UAbilitySystemComponent* GetOwningAbilitySystemComponent() const;
	UE_API UAbilitySystemComponent* GetOwningAbilitySystemComponentChecked() const;
	UE_API FGameplayAbilityActorInfo* GetActorInfo() const;

	/** Print debug information to the log */
	UE_API virtual void PrintDebug();

	// Overrides
	UE_API virtual bool IsNameStableForNetworking() const override;
	UE_API virtual bool IsSupportedForNetworking() const override;
	UE_API virtual void PreNetReceive() override;
	UE_API virtual void PostNetReceive() override;

	/** Register all replication fragments */
	UE_API virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
protected:
	/** Is this attribute set safe to ID over the network by name?  */
	uint32 bNetAddressable : 1;
};

/**
 *	DataTable that allows us to define meta data about attributes. Still a work in progress.
 */
USTRUCT(BlueprintType)
struct FAttributeMetaData : public FTableRowBase
{
	GENERATED_USTRUCT_BODY()

public:

	UE_API FAttributeMetaData();

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Gameplay Attribute")
	float		BaseValue;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Gameplay Attribute")
	float		MinValue;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Gameplay Attribute")
	float		MaxValue;

	UPROPERTY()
	FString		DerivedAttributeInfo;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Gameplay Attribute")
	bool		bCanStack;
};

/**
 *	Helper struct that facilitates initializing attribute set default values from spread sheets (UCurveTable).
 *	Projects are free to initialize their attribute sets however they want. This is just want example that is 
 *	useful in some cases.
 *	
 *	Basic idea is to have a spreadsheet in this form: 
 *	
 *									1	2	3	4	5	6	7	8	9	10	11	12	13	14	15	16	17	18	19	20
 *
 *	Default.Health.MaxHealth		100	200	300	400	500	600	700	800	900	999	999	999	999	999	999	999	999	999	999	999
 *	Default.Health.HealthRegenRate	1	1	1	1	1	1	1	1	1	1	1	1	1	1	1	1	1	1	1	1
 *	Default.Health.AttackRating		10	10	10	10	10	10	10	10	10	10	10	10	10	10	10	10	10	10	10	10
 *	Default.Move.MaxMoveSpeed		500	500	500	500	500	500	500	500	500	500	500	500	500	500	500	500	500	500	500	500
 *	Hero1.Health.MaxHealth			100	100	100	100	100	100	100	100	100	100	100	100	100	100	100	100	100	100	100	100
 *	Hero1.Health.HealthRegenRate	1	1	1	1	1	1	1	1	1	1	1	1	1	1	1	1 	1	1	1	1
 *	Hero1.Health.AttackRating		10	10	10	10	10	10	10	10	10	10	10	10	10	10	10	10	10	10	10	10
 *	Hero1.Move.MaxMoveSpeed			500	500	500	500	500	500	500	500	500	500	500	500	500	500	500	500	500	500	500	500
 *	
 *	
 *	Where rows are in the form: [GroupName].[AttributeSetName].[Attribute]
 *	GroupName			- arbitrary name to identify the "group"
 *	AttributeSetName	- what UAttributeSet the attributes belong to. (Note that this is a simple partial match on the UClass name. "Health" matches "UMyGameHealthSet").
 *	Attribute			- the name of the actual attribute property (matches full name).
 *		
 *	Columns represent "Level". 
 *	
 *	FAttributeSetInitter::PreloadAttributeSetData(UCurveTable*)
 *	This transforms the CurveTable into a more efficient format to read in at run time. Should be called from UAbilitySystemGlobals for example.
 *
 *	FAttributeSetInitter::InitAttributeSetDefaults(UAbilitySystemComponent* AbilitySystemComponent, FName GroupName, int32 Level) const;
 *	This initializes the given AbilitySystemComponent's attribute sets with the specified GroupName and Level. Game code would be expected to call
 *	this when spawning a new Actor, or leveling up an actor, etc.
 *	
 *	Example Game code usage:
 *	
 *	IGameplayAbilitiesModule::Get().GetAbilitySystemGlobals()->GetAttributeSetInitter()->InitAttributeSetDefaults(MyCharacter->AbilitySystemComponent, "Hero1", MyLevel);
 *	
 *	Notes:
 *	-This lets system designers specify arbitrary values for attributes. They can be based on any formula they want.
 *	-Projects with very large level caps may wish to take a simpler "Attributes gained per level" approach.
 *	-Anything initialized in this method should not be directly modified by gameplay effects. E.g., if MaxMoveSpeed scales with level, anything else that 
 *		modifies MaxMoveSpeed should do so with a non-instant GameplayEffect.
 *	-"Default" is currently the hardcoded, fallback GroupName. If InitAttributeSetDefaults is called without a valid GroupName, we will fallback to default.
 *
 */
struct FAttributeSetInitter
{
	virtual ~FAttributeSetInitter() = default;

	virtual void PreloadAttributeSetData(const TArray<UCurveTable*>& CurveData) = 0;
	virtual void InitAttributeSetDefaults(UAbilitySystemComponent* AbilitySystemComponent, FName GroupName, int32 Level, bool bInitialInit) const = 0;
	virtual void ApplyAttributeDefault(UAbilitySystemComponent* AbilitySystemComponent, FGameplayAttribute& InAttribute, FName GroupName, int32 Level) const = 0;
	virtual TArray<float> GetAttributeSetValues(UClass* AttributeSetClass, FProperty* AttributeProperty, FName GroupName) const { return TArray<float>(); }
};

/** Explicit implementation of attribute set initter, relying on the existence and usage of discrete levels for data look-up (that is, CurveTable->Eval is not possible) */
struct FAttributeSetInitterDiscreteLevels : public FAttributeSetInitter
{
	UE_API virtual void PreloadAttributeSetData(const TArray<UCurveTable*>& CurveData) override;

	UE_API virtual void InitAttributeSetDefaults(UAbilitySystemComponent* AbilitySystemComponent, FName GroupName, int32 Level, bool bInitialInit) const override;
	UE_API virtual void ApplyAttributeDefault(UAbilitySystemComponent* AbilitySystemComponent, FGameplayAttribute& InAttribute, FName GroupName, int32 Level) const override;

	UE_API virtual TArray<float> GetAttributeSetValues(UClass* AttributeSetClass, FProperty* AttributeProperty, FName GroupName) const override;

private:
	struct FAttributeDefaultValueList
	{
		void AddPair(FProperty* InProperty, float InValue)
		{
			List.Add(FOffsetValuePair(InProperty, InValue));
		}

		struct FOffsetValuePair
		{
			FOffsetValuePair(FProperty* InProperty, float InValue)
			: Property(InProperty), Value(InValue) { }

			FProperty*	Property;
			float		Value;
		};

		TArray<FOffsetValuePair>	List;
	};

	struct FAttributeSetDefaults
	{
		TMap<TSubclassOf<UAttributeSet>, FAttributeDefaultValueList> DataMap;
	};

	struct FAttributeSetDefaultsCollection
	{
		TArray<FAttributeSetDefaults>		LevelData;
	};

	TMap<FName, FAttributeSetDefaultsCollection>	Defaults;
};

/**
 *	This is a helper macro that can be used in RepNotify functions to handle attributes that will be predictively modified by clients.
 *	
 *	void UMyHealthSet::OnRep_Health(const FGameplayAttributeData& OldValue)
 *	{
 *		GAMEPLAYATTRIBUTE_REPNOTIFY(UMyHealthSet, Health, OldValue);
 *	}
 */

#define GAMEPLAYATTRIBUTE_REPNOTIFY(ClassName, PropertyName, OldValue) \
{ \
	static FProperty* ThisProperty = FindFieldChecked<FProperty>(ClassName::StaticClass(), GET_MEMBER_NAME_CHECKED(ClassName, PropertyName)); \
	GetOwningAbilitySystemComponentChecked()->SetBaseAttributeValueFromReplication(FGameplayAttribute(ThisProperty), PropertyName, OldValue); \
}

/**
 * This defines a set of helper functions for accessing and initializing attributes, to avoid having to manually write these functions.
 * It would create the following functions, for attribute Health
 *
 *	static FGameplayAttribute UMyHealthSet::GetHealthAttribute();
 *	FORCEINLINE float UMyHealthSet::GetHealth() const;
 *	FORCEINLINE void UMyHealthSet::SetHealth(float NewVal);
 *	FORCEINLINE void UMyHealthSet::InitHealth(float NewVal);
 *
 * To use this in your game you can define something like this, and then add game-specific functions as necessary:
 * 
 *	#define ATTRIBUTE_ACCESSORS(ClassName, PropertyName) \
 *	GAMEPLAYATTRIBUTE_PROPERTY_GETTER(ClassName, PropertyName) \
 *	GAMEPLAYATTRIBUTE_VALUE_GETTER(PropertyName) \
 *	GAMEPLAYATTRIBUTE_VALUE_SETTER(PropertyName) \
 *	GAMEPLAYATTRIBUTE_VALUE_INITTER(PropertyName)
 * 
 *	ATTRIBUTE_ACCESSORS(UMyHealthSet, Health)
 */

#define GAMEPLAYATTRIBUTE_PROPERTY_GETTER(ClassName, PropertyName) \
	static FGameplayAttribute Get##PropertyName##Attribute() \
	{ \
		static FProperty* Prop = FindFieldChecked<FProperty>(ClassName::StaticClass(), GET_MEMBER_NAME_CHECKED(ClassName, PropertyName)); \
		return Prop; \
	}

#define GAMEPLAYATTRIBUTE_VALUE_GETTER(PropertyName) \
	inline float Get##PropertyName() const \
	{ \
		return PropertyName.GetCurrentValue(); \
	}

#define GAMEPLAYATTRIBUTE_VALUE_SETTER(PropertyName) \
	inline void Set##PropertyName(float NewVal) \
	{ \
		UAbilitySystemComponent* AbilityComp = GetOwningAbilitySystemComponent(); \
		if (ensure(AbilityComp)) \
		{ \
			AbilityComp->SetNumericAttributeBase(Get##PropertyName##Attribute(), NewVal); \
		}; \
	}

#define GAMEPLAYATTRIBUTE_VALUE_INITTER(PropertyName) \
	inline void Init##PropertyName(float NewVal) \
	{ \
		PropertyName.SetBaseValue(NewVal); \
		PropertyName.SetCurrentValue(NewVal); \
	}

/**
 * Instead of copying the ATTRIBUTE_ACCESSORS macro from the comment above, you can also immediately 
 * use this one if it fulfills your needs. Named differently to not cause name collision in existing 
 * projects that have copied ATTRIBUTE_ACCESSORS already:
 * 
 *  ATTRIBUTE_ACCESSORS_BASIC(UMyHealthSet, Health)
 */
#define ATTRIBUTE_ACCESSORS_BASIC(ClassName, PropertyName) \
	GAMEPLAYATTRIBUTE_PROPERTY_GETTER(ClassName, PropertyName) \
	GAMEPLAYATTRIBUTE_VALUE_GETTER(PropertyName) \
	GAMEPLAYATTRIBUTE_VALUE_SETTER(PropertyName) \
	GAMEPLAYATTRIBUTE_VALUE_INITTER(PropertyName)

#undef UE_API
