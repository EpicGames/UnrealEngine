// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AbilitySystemGlobals.h"
#include "GameplayAbilitiesModule.h"
#include "GameplayCueManager.h"
#include "GameplayEffectTypes.h"
#include "GameplayTagContainer.h"

#include "Engine/DeveloperSettingsBackedByCVars.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"

#include "GameplayAbilitiesDeveloperSettings.generated.h"

class UAbilitySystemComponent;
class UGameplayCueManager;
class UGameplayTagReponseTable;
struct FGameplayAbilityActorInfo;

/**
 * Expose Global Gameplay Ability Settings in an easy to understand Developer Settings interface (usable through the Editor's Project Settings).
 * This the preferred way to configure the config variables previously found in AbilitySystemGlobals.  Projects may still opt to override the
 * AbilitySystemGlobals class with their own customized class to modify Gameplay Ability functionality across their project.
 */
UCLASS(config = Game, defaultconfig, meta = (DisplayName = "Gameplay Abilities Settings"), MinimalAPI)
class UGameplayAbilitiesDeveloperSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:	
	// The global ability system class to use
	UPROPERTY(Config, EditDefaultsOnly, Category=Gameplay, meta=(DisplayName = "Ability System Globals Class", MetaClass="/Script/GameplayAbilities.AbilitySystemGlobals", ConfigRestartRequired=true))
	FSoftClassPath AbilitySystemGlobalsClassName = UAbilitySystemGlobals::StaticClass();

	//Set to true if you want the "ShowDebug AbilitySystem" cheat to use the hud's debug target instead of the ability system's debug target.
	UPROPERTY(Config, EditDefaultsOnly, Category=Gameplay)
	bool bUseDebugTargetFromHud = false;

	/** Array of curve table names to use for default values for attribute sets, keyed off of Name/Levels */
	UPROPERTY(Config, EditDefaultsOnly, Category=Attribute, meta=(DisplayName = "Global Attribute Set Defaults Tables", AllowedClasses="/Script/Engine.CurveTable", ConfigRestartRequired=true))
	TArray<FSoftObjectPath> GlobalAttributeSetDefaultsTableNames;

	/** Holds information about the valid attributes' min and max values and stacking rules */
	UPROPERTY(Config, EditDefaultsOnly, Category=Attribute, meta=(DisplayName = "Global Attribute Meta Data Table", AllowedClasses="/Script/Engine.DataTable", ConfigRestartRequired=true))
	FSoftObjectPath GlobalAttributeMetaDataTableName;

	/** Class reference to gameplay cue manager. Use this if you want to just instantiate a class for your gameplay cue manager without having to create an asset. */
	UPROPERTY(Config, EditDefaultsOnly, Category=GameplayCue, meta=(DisplayName = "Global GameplayCue Manager Class", MetaClass="/Script/GameplayAbilities.GameplayCueManager", ConfigRestartRequired=true))
	FSoftClassPath GlobalGameplayCueManagerClass = UGameplayCueManager::StaticClass();
	
	/** Class reference to gameplay cue manager. Use this if you want to just instantiate a class for your gameplay cue manager without having to create an asset. */
	UPROPERTY(Config, EditDefaultsOnly, Category=GameplayCue, meta=(AllowedClasses="/Script/GameplayAbilities.GameplayCueManager", ConfigRestartRequired=true))
    FSoftObjectPath GlobalGameplayCueManagerName;

	/** Look in these paths for GameplayCueNotifies. These are your "always loaded" set. */
	UPROPERTY(Config, EditDefaultsOnly, Category=GameplayCue, meta = (ConfigRestartRequired=true))
	TArray<FString>	GameplayCueNotifyPaths;

	/** Name of global curve table to use as the default for scalable floats, etc. */
	UPROPERTY(Config, EditDefaultsOnly, Category=Gameplay, meta=(DisplayName = "Global Curve Table", AllowedClasses="/Script/Engine.CurveTable", ConfigRestartRequired=true))
	FSoftObjectPath GlobalCurveTableName;

	/** Set to true if you want clients to try to predict gameplay effects done to targets. If false it will only predict self effects */
	UPROPERTY(Config, EditDefaultsOnly, Category=Gameplay)
	bool PredictTargetGameplayEffects = true;

	/** 
	 * Set to true if you want tags granted to owners from ability activations to be replicated. If false, ActivationOwnedTags are only applied locally. 
	 * This should only be disabled for legacy game code that depends on non-replication of ActivationOwnedTags.
	 */
	UPROPERTY(Config, EditDefaultsOnly, Category=Gameplay)
	bool ReplicateActivationOwnedTags = true;

	/** TryActive failed due to GameplayAbility's CanActivateAbility function (Blueprint or Native) */
	UPROPERTY(Config, EditDefaultsOnly, Category=Gameplay, meta = (ConfigRestartRequired=true))
	FGameplayTag ActivateFailCanActivateAbilityTag;

	/** TryActivate failed due to being on cooldown */
	UPROPERTY(Config, EditDefaultsOnly, Category=Gameplay, meta = (ConfigRestartRequired=true))
	FGameplayTag ActivateFailCooldownTag; 

	/** TryActivate failed due to not being able to spend costs */
	UPROPERTY(Config, EditDefaultsOnly, Category=Gameplay, meta = (ConfigRestartRequired=true))
	FGameplayTag ActivateFailCostTag;

	/** Failed to activate due to invalid networking settings, this is designer error */
	UPROPERTY(Config, EditDefaultsOnly, Category=Gameplay, meta = (ConfigRestartRequired=true))
	FGameplayTag ActivateFailNetworkingTag; 

	/** TryActivate failed due to being blocked by other abilities */
	UPROPERTY(Config, EditDefaultsOnly, Category=Gameplay, meta = (ConfigRestartRequired=true))
	FGameplayTag ActivateFailTagsBlockedTag;
	
	/** TryActivate failed due to missing required tags */
	UPROPERTY(Config, EditDefaultsOnly, Category=Gameplay, meta = (ConfigRestartRequired=true))
	FGameplayTag ActivateFailTagsMissingTag; 

	/** The class to instantiate as the GameplayTagResponseTable. */
	UPROPERTY(Config, EditDefaultsOnly, Category=Gameplay, meta=(DisplayName = "Gameplay Tag Response Table", AllowedClasses="/Script/GameplayAbilities.GameplayTagReponseTable", ConfigRestartRequired=true))
	FSoftObjectPath GameplayTagResponseTableName;
	
	/** Whether the game should allow the usage of gameplay mod evaluation channels or not */
    UPROPERTY(config, EditDefaultsOnly, Category=GameplayEffects, meta = (ConfigRestartRequired=true))
    bool bAllowGameplayModEvaluationChannels = false;
	
    /** The default mod evaluation channel for the game */
    UPROPERTY(config, EditDefaultsOnly, Category=GameplayEffects, meta = (ConfigRestartRequired=true))
    EGameplayModEvaluationChannel DefaultGameplayModEvaluationChannel = EGameplayModEvaluationChannel::Channel0;

    /** Game-specified named aliases for gameplay mod evaluation channels; Only those with valid aliases are eligible to be used in a game (except Channel0, which is always valid) */
    UPROPERTY(config, EditDefaultsOnly, Category=GameplayEffects, meta = (ConfigRestartRequired=true))
    FName GameplayModEvaluationChannelAliases[10];

	/** How many bits to use for "number of tags" in FMinimalReplicationTagCountMap::NetSerialize.  */
	UPROPERTY(Config, EditDefaultsOnly, Category=Advanced)
	int32	MinimalReplicationTagCountBits = 5;

#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif
	virtual void OverrideConfigSection(FString& InOutSectionName) override
	{
		/*
		 * Use the section of the original AbilitySystemGlobals to keep compatibility
		 * We simply mirror the exact properties here so existing projects do not break.
		 */
		InOutSectionName = TEXT("/Script/GameplayAbilities.AbilitySystemGlobals");
	}
};

UCLASS(config = EditorPerProjectUserSettings, meta = (DisplayName = "Gameplay Abilities Editor Settings"), MinimalAPI)
class UGameplayAbilitiesEditorDeveloperSettings : public UDeveloperSettingsBackedByCVars
{
	GENERATED_BODY()

protected:
	UPROPERTY(Config, EditDefaultsOnly, Category=Debug, meta = (ConsoleVariable = "AbilitySystem.IgnoreCooldowns"))
	bool bIgnoreCooldowns = false;
	
	UPROPERTY(Config, EditDefaultsOnly, Category=Debug, meta = (ConsoleVariable = "AbilitySystem.IgnoreCosts"))
	bool bIgnoreCosts = false;

	UPROPERTY(Config, EditDefaultsOnly, Category=Debug, meta = (ConsoleVariable = "AbilitySystem.GlobalAbilityScale"))
	float AbilitySystemGlobalScaler = 1.f;

	UPROPERTY(Config, EditDefaultsOnly, Category=Debug, meta = (ConsoleVariable = "AbilitySystem.DebugDrawMaxDistance"))
	float DebugDrawMaxDistance = 2048.f;
};
