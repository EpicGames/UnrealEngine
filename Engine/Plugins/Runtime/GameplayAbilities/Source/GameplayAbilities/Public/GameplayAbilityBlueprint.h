// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Blueprint.h"
#include "GameplayAbilityBlueprint.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/**
 * A Gameplay Ability Blueprint is essentially a specialized Blueprint whose graphs control a gameplay ability
 * The ability factory should pick this for you automatically
 */

UCLASS(BlueprintType, MinimalAPI)
class UGameplayAbilityBlueprint : public UBlueprint
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR

	// UBlueprint interface
	virtual bool SupportedByDefaultBlueprintFactory() const override
	{
		return false;
	}
	// End of UBlueprint interface

	/** Returns the most base gameplay ability blueprint for a given blueprint (if it is inherited from another ability blueprint, returning null if only native / non-ability BP classes are it's parent) */
	static UE_API UGameplayAbilityBlueprint* FindRootGameplayAbilityBlueprint(UGameplayAbilityBlueprint* DerivedBlueprint);

#endif
};

#undef UE_API
