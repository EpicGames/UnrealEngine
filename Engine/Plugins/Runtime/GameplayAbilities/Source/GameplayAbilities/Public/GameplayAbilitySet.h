// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/DataAsset.h"
#include "Abilities/GameplayAbility.h"
#include "GameplayAbilitySet.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

/**
 *	This is an example input binding enum for GameplayAbilities. Your project may want to create its own.
 *	The MetaData default bind keys (LMB, RMB, Q, E, etc) are a convenience for designers setting up abilities sets
 *	or whatever other data you have that gives an ability with a default key binding. Actual key binding is up to DefaultInput.ini
 *	
 *	E.g., "Ability1" is the command string that is bound to AbilitySystemComponent::ActivateAbility(1). The Meta data only *suggests*
 *	that you are binding "Ability1" to LMB by default in your projects DefaultInput.ini.
 */
UENUM(BlueprintType)
namespace EGameplayAbilityInputBinds
{
	enum Type : int
	{
		Ability1				UMETA(DisplayName = "Ability1 (LMB)"),
		Ability2				UMETA(DisplayName = "Ability2 (RMB)"),
		Ability3				UMETA(DisplayName = "Ability3 (Q)"),
		Ability4				UMETA(DisplayName = "Ability4 (E)"),
		Ability5				UMETA(DisplayName = "Ability5 (R)"),
		Ability6				UMETA(DisplayName = "Ability6 (Shift)"),
		Ability7				UMETA(DisplayName = "Ability7 (Space)"),
		Ability8				UMETA(DisplayName = "Ability8 (B)"),
		Ability9				UMETA(DisplayName = "Ability9 (T)"),
	};
}

/**
 *	Example struct that pairs a enum input command to a GameplayAbilityClass.6
 */
USTRUCT()
struct FGameplayAbilityBindInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = BindInfo)
	TEnumAsByte<EGameplayAbilityInputBinds::Type>	Command = EGameplayAbilityInputBinds::Ability1;;

	UPROPERTY(EditAnywhere, Category = BindInfo)
	TSubclassOf<UGameplayAbility>	GameplayAbilityClass;
};

/**
 *	This is an example DataAsset that could be used for defining a set of abilities to give to an AbilitySystemComponent and bind to an input command.
 *	Your project is free to implement this however it wants!
 *	
 *	
 */
UCLASS(MinimalAPI)
class UGameplayAbilitySet : public UDataAsset
{
	GENERATED_UCLASS_BODY()

	
	UPROPERTY(EditAnywhere, Category = AbilitySet)
	TArray<FGameplayAbilityBindInfo>	Abilities;

	UE_API void GiveAbilities(UAbilitySystemComponent* AbilitySystemComponent) const;
};

#undef UE_API
