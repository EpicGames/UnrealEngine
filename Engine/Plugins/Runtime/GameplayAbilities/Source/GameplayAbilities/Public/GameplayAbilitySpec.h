// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "Engine/NetSerialization.h"
#include "AttributeSet.h"
#include "Abilities/GameplayAbilityTriggerType.h"
#include "GameplayAbilitySpecHandle.h"
#include "GameplayEffectTypes.h"
#include "GameplayPrediction.h"
#include "ScalableFloat.h"
#include "GameplayAbilitySpec.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;
class UGameplayAbility;
struct FGameplayEventData;

/** Describes the status of activating this ability, this is updated as prediction is handled */
UENUM(BlueprintType)
namespace EGameplayAbilityActivationMode
{
	enum Type : int
	{
		/** We are the authority activating this ability */
		Authority,

		/** We are not the authority but aren't predicting yet. This is a mostly invalid state to be in */
		NonAuthority,

		/** We are predicting the activation of this ability */
		Predicting,

		/** We are not the authority, but the authority has confirmed this activation */
		Confirmed,

		/** We tried to activate it, and server told us we couldn't (even though we thought we could) */
		Rejected,
	};
}

/** Describes what happens when a GameplayEffect, that is granting an active ability, is removed from its owner. */
UENUM()
enum class EGameplayEffectGrantedAbilityRemovePolicy : uint8
{
	/** Active abilities are immediately canceled and the ability is removed. */
	CancelAbilityImmediately,
	/** Active abilities are allowed to finish, and then removed. */
	RemoveAbilityOnEnd,
	/** Granted abilities are left lone when the granting GameplayEffect is removed. */
	DoNothing,
};

/** This is data that can be used to create an FGameplayAbilitySpec. Has some data that is only relevant when granted by a GameplayEffect */
USTRUCT(BlueprintType)
struct FGameplayAbilitySpecDef
{
	FGameplayAbilitySpecDef()
		: InputID(INDEX_NONE)
		, RemovalPolicy(EGameplayEffectGrantedAbilityRemovePolicy::CancelAbilityImmediately)
		, SourceObject(nullptr)
	{
		LevelScalableFloat.SetValue(1.f);
	}

	GENERATED_USTRUCT_BODY()

	/** What ability to grant */
	UPROPERTY(EditDefaultsOnly, Category="Ability Definition", NotReplicated)
	TSubclassOf<UGameplayAbility> Ability;
	
	/** What level to grant this ability at */
	UPROPERTY(EditDefaultsOnly, Category="Ability Definition", NotReplicated, DisplayName="Level")
	FScalableFloat LevelScalableFloat; 

	/** Input ID to bind this ability to */
	UPROPERTY(EditDefaultsOnly, Category="Ability Definition", NotReplicated)
	int32 InputID;

	/** What will remove this ability later */
	UPROPERTY(EditDefaultsOnly, Category="Ability Definition", NotReplicated)
	EGameplayEffectGrantedAbilityRemovePolicy RemovalPolicy;

	/** What granted this spec, not replicated or settable in editor */
	UPROPERTY(NotReplicated)
	TWeakObjectPtr<UObject> SourceObject;

	/** SetbyCaller Magnitudes that were passed in to this ability by a GE (GE's that grant abilities) */
	TMap<FGameplayTag, float>	SetByCallerTagMagnitudes;

	/** This handle can be set if the SpecDef is used to create a real FGameplaybilitySpec */
	UPROPERTY()
	FGameplayAbilitySpecHandle	AssignedHandle;

	bool operator==(const FGameplayAbilitySpecDef& Other) const;
	bool operator!=(const FGameplayAbilitySpecDef& Other) const;
};

/**
 *	FGameplayAbilityActivationInfo
 *
 *	Data tied to a specific activation of an ability.
 *		-Tell us whether we are the authority, if we are predicting, confirmed, etc.
 *		-Holds current and previous PredictionKey
 *		-Generally not meant to be subclassed in projects.
 *		-Passed around by value since the struct is small.
 */
USTRUCT(BlueprintType)
struct FGameplayAbilityActivationInfo
{
	GENERATED_USTRUCT_BODY()

	FGameplayAbilityActivationInfo()
		: ActivationMode(EGameplayAbilityActivationMode::Authority)
		, bCanBeEndedByOtherInstance(false)
	{
	}

	UE_API FGameplayAbilityActivationInfo(AActor* InActor);

	FGameplayAbilityActivationInfo(EGameplayAbilityActivationMode::Type InType)
		: ActivationMode(InType)
		, bCanBeEndedByOtherInstance(false)
	{
	}	

	/** Activation status of this ability */
	UPROPERTY(BlueprintReadOnly, Category = "ActorInfo")
	mutable TEnumAsByte<EGameplayAbilityActivationMode::Type>	ActivationMode;

	/** An ability that runs on multiple game instances can be canceled by a remote instance, but only if that remote instance has already confirmed starting it. */
	UPROPERTY()
	uint8 bCanBeEndedByOtherInstance:1;

	/** Called on client when activation is confirmed on server */
	UE_API void SetActivationConfirmed();

	/** Called when activation was rejected by the server */
	UE_API void SetActivationRejected();

	/** Called on client to set this as a predicted ability */
	UE_API void SetPredicting(FPredictionKey PredictionKey);

	/** Called on the server to set the key used by the client to predict this ability */
	UE_API void ServerSetActivationPredictionKey(FPredictionKey PredictionKey);

	/** Returns prediction key, const to avoid being able to modify it after creation */
	const FPredictionKey& GetActivationPredictionKey() const { return PredictionKeyWhenActivated; }

private:

	/** This was the prediction key used to activate this ability. It does not get updated if new prediction keys are generated over the course of the ability */
	UPROPERTY()
	FPredictionKey PredictionKeyWhenActivated;
	
};

/**
 * An activatable ability spec, hosted on the ability system component. This defines both what the ability is (what class, what level, input binding etc)
 * and also holds runtime state that must be kept outside of the ability being instanced/activated.
 */
USTRUCT(BlueprintType)
struct FGameplayAbilitySpec : public FFastArraySerializerItem
{
	GENERATED_USTRUCT_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGameplayAbilitySpec(const FGameplayAbilitySpec&) = default;
	FGameplayAbilitySpec(FGameplayAbilitySpec&&) = default;
	FGameplayAbilitySpec& operator=(const FGameplayAbilitySpec&) = default;
	FGameplayAbilitySpec& operator=(FGameplayAbilitySpec&&) = default;
	~FGameplayAbilitySpec() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FGameplayAbilitySpec()
		: Ability(nullptr), Level(1), InputID(INDEX_NONE), SourceObject(nullptr), ActiveCount(0), InputPressed(false), RemoveAfterActivation(false), PendingRemove(false), bActivateOnce(false)
	{ }

	/** Version that takes an ability class */
	UE_API FGameplayAbilitySpec(TSubclassOf<UGameplayAbility> InAbilityClass, int32 InLevel = 1, int32 InInputID = INDEX_NONE, UObject* InSourceObject = nullptr);

	/** Version that takes an ability CDO, this exists for backward compatibility */
	UE_API FGameplayAbilitySpec(UGameplayAbility* InAbility, int32 InLevel = 1, int32 InInputID = INDEX_NONE, UObject* InSourceObject = nullptr);

	/** Version that takes an existing spec def */
	UE_API FGameplayAbilitySpec(FGameplayAbilitySpecDef& InDef, int32 InGameplayEffectLevel, FActiveGameplayEffectHandle InGameplayEffectHandle = FActiveGameplayEffectHandle());

	/** Handle for outside sources to refer to this spec by */
	UPROPERTY()
	FGameplayAbilitySpecHandle Handle;
	
	/** Ability of the spec (Always the CDO. This should be const but too many things modify it currently) */
	UPROPERTY()
	TObjectPtr<UGameplayAbility> Ability;
	
	/** Level of Ability */
	UPROPERTY()
	int32	Level;

	/** InputID, if bound */
	UPROPERTY()
	int32	InputID;

	/** Object this ability was created from, can be an actor or static object. Useful to bind an ability to a gameplay object */
	UPROPERTY()
	TWeakObjectPtr<UObject> SourceObject;

	/** A count of the number of times this ability has been activated minus the number of times it has been ended. For instanced abilities this will be the number of currently active instances. Can't replicate until prediction accurately handles this.*/
	UPROPERTY(NotReplicated)
	uint8 ActiveCount;

	/** Is input currently pressed. Set to false when input is released */
	UPROPERTY(NotReplicated)
	uint8 InputPressed:1;

	/** If true, this ability should be removed as soon as it finishes executing */
	UPROPERTY(NotReplicated)
	uint8 RemoveAfterActivation:1;

	/** Pending removal due to scope lock */
	UPROPERTY(NotReplicated)
	uint8 PendingRemove:1;

	/** This ability should be activated once when it is granted. */
	UPROPERTY(NotReplicated)
	uint8 bActivateOnce : 1;

	/** Cached GameplayEventData if this ability was pending for add and activate due to scope lock */
	TSharedPtr<FGameplayEventData> GameplayEventData = nullptr;

	/** Activation state of this ability. This is not replicated since it needs to be overwritten locally on clients during prediction. */
	UE_DEPRECATED(5.5, "ActivationInfo on the Spec only applies to NonInstanced abilities (which are now deprecated; instanced abilities have their own per-instance CurrentActivationInfo)")
	UPROPERTY(NotReplicated)
	FGameplayAbilityActivationInfo	ActivationInfo;

	/** Optional ability tags that are replicated.  These tags are also captured as source tags by applied gameplay effects. */
	UE_DEPRECATED(5.5, "Use GetDynamicSpecSourceTags() which better represents what this variable does")
	UPROPERTY()
	FGameplayTagContainer DynamicAbilityTags;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Optional tags that are replicated with this AbilitySpec.  The specified tags are captured as a GESpec's Source tags by GE's created with this ability spec (@see UGameplayAbility::MakeOutgoingGameplayEffectSpec). */
	FGameplayTagContainer& GetDynamicSpecSourceTags() { return DynamicAbilityTags; }
	const FGameplayTagContainer& GetDynamicSpecSourceTags() const { return DynamicAbilityTags; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Additional Ability Triggers that can be used in order to trigger the ability associated with this spec. */
	UPROPERTY()
	TArray<FAbilityTriggerData> DynamicAbilityTriggers;

	/** Non replicating instances of this ability. */
	UPROPERTY(NotReplicated)
	TArray<TObjectPtr<UGameplayAbility>> NonReplicatedInstances;

	/** Replicated instances of this ability.. */
	UPROPERTY()
	TArray<TObjectPtr<UGameplayAbility>> ReplicatedInstances;

	/**
	 * Handle to GE that granted us (usually invalid). FActiveGameplayEffectHandles are not synced across the network and this is valid only on Authority.
	 * If you need FGameplayAbilitySpec -> FActiveGameplayEffectHandle, then use AbilitySystemComponent::FindActiveGameplayEffectHandle.
	 */
	UPROPERTY(NotReplicated)
	FActiveGameplayEffectHandle	GameplayEffectHandle;

	/** Passed on SetByCaller magnitudes if this ability was granted by a GE */
	TMap<FGameplayTag, float> SetByCallerTagMagnitudes;

	/** Returns the primary instance, only valid on InstancedPerActor abilities (returns nullptr otherwise) */
	UE_API UGameplayAbility* GetPrimaryInstance() const;

	/** interface function to see if the ability should replicated the ability spec or not */
	UE_API bool ShouldReplicateAbilitySpec() const;

	/** Returns all instances, which can include InstancedPerExecution abilities */
	TArray<UGameplayAbility*> GetAbilityInstances() const
	{
		TArray<UGameplayAbility*> Abilities;
		Abilities.Append(ReplicatedInstances);
		Abilities.Append(NonReplicatedInstances);
		return Abilities;
	}

	/** Returns true if this ability is active in any way */
	UE_API bool IsActive() const;

	UE_API void PreReplicatedRemove(const struct FGameplayAbilitySpecContainer& InArraySerializer);
	UE_API void PostReplicatedChange(const struct FGameplayAbilitySpecContainer& InArraySerializer);
	UE_API void PostReplicatedAdd(const struct FGameplayAbilitySpecContainer& InArraySerializer);

	UE_API FString GetDebugString();
};

/** Fast serializer wrapper for above struct */
USTRUCT(BlueprintType)
struct FGameplayAbilitySpecContainer : public FFastArraySerializer
{
	GENERATED_USTRUCT_BODY()

	FGameplayAbilitySpecContainer()
		: Owner(nullptr)
	{
	}

	/** List of activatable abilities */
	UPROPERTY()
	TArray<FGameplayAbilitySpec> Items;

	/** Component that owns this list */
	UPROPERTY(NotReplicated)
	TObjectPtr<UAbilitySystemComponent> Owner;

	/** Initializes Owner variable */
	GAMEPLAYABILITIES_API void RegisterWithOwner(UAbilitySystemComponent* Owner);

	GAMEPLAYABILITIES_API bool NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms);

	template< typename Type, typename SerializerType >
	bool ShouldWriteFastArrayItem(const Type& Item, const bool bIsWritingOnClient)
	{
		// if we do not want the FGameplayAbilitySpec to replicated return false;
		if (!Item.ShouldReplicateAbilitySpec())
		{
			return false;
		}

		if (bIsWritingOnClient)
		{
			return Item.ReplicationID != INDEX_NONE;
		}

		return true;
	}
};

template<>
struct TStructOpsTypeTraits< FGameplayAbilitySpecContainer > : public TStructOpsTypeTraitsBase2< FGameplayAbilitySpecContainer >
{
	enum
	{
		WithNetDeltaSerializer = true,
	};
};

/** Used to stop us from removing abilities from an ability system component while we're iterating through the abilities */
struct FScopedAbilityListLock
{
	UE_API FScopedAbilityListLock(UAbilitySystemComponent& InContainer);
	UE_API ~FScopedAbilityListLock();

private:
	UAbilitySystemComponent& AbilitySystemComponent;
};

#define ABILITYLIST_SCOPE_LOCK()	FScopedAbilityListLock ActiveScopeLock(*this);

/** Used to stop us from canceling or ending an ability while we're iterating through its gameplay targets */
struct FScopedTargetListLock
{
	UE_API FScopedTargetListLock(UAbilitySystemComponent& InAbilitySystemComponent, const UGameplayAbility& InAbility);
	UE_API ~FScopedTargetListLock();

private:
	const UGameplayAbility& GameplayAbility;

	// we also need to make sure the ability isn't removed while we're in this lock
	FScopedAbilityListLock AbilityLock;
};

#define TARGETLIST_SCOPE_LOCK(ASC)	FScopedTargetListLock ActiveScopeLock(ASC, *this);

#undef UE_API
