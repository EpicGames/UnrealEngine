// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayAbilitySpecHandle.generated.h"


/**
 *	This file exists in addition so that GameplayEffect.h can use FGameplayAbilitySpec without having to include GameplayAbilityTypes.h which has dependencies on
 *	GameplayEffect.h
 */

/** Handle that points to a specific granted ability. These are globally unique */
USTRUCT(BlueprintType)
struct FGameplayAbilitySpecHandle
{
	GENERATED_USTRUCT_BODY()

	FGameplayAbilitySpecHandle()
		: Handle(INDEX_NONE)
	{
	}

	/** True if GenerateNewHandle was called on this handle */
	bool IsValid() const
	{
		return Handle != INDEX_NONE;
	}

	/** Sets this to a valid handle */
	void GenerateNewHandle();

	bool operator==(const FGameplayAbilitySpecHandle& Other) const
	{
		return Handle == Other.Handle;
	}

	bool operator!=(const FGameplayAbilitySpecHandle& Other) const
	{
		return Handle != Other.Handle;
	}

	/** Operator to expose FGameplayAbilitySpecHandle serialization to custom serialization functions like NetSerialize overrides. */
	friend FArchive& operator<<(FArchive& Ar, FGameplayAbilitySpecHandle& Value)
	{
		static_assert(sizeof(FGameplayAbilitySpecHandle) == 4, "If properties of FGameplayAbilitySpecHandle change, consider updating this operator implementation.");
		Ar << Value.Handle;
		return Ar;
	}

	friend uint32 GetTypeHash(const FGameplayAbilitySpecHandle& SpecHandle)
	{
		return ::GetTypeHash(SpecHandle.Handle);
	}

	FString ToString() const
	{
		return IsValid() ? FString::FromInt(Handle) : TEXT("Invalid");
	}

private:

	UPROPERTY()
	int32 Handle;
};