// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GameplayEffectTypes.h"
#include "GameplayTagContainer.h"
#include "GameplayCueFunctionLibrary.generated.h"

#define UE_API GAMEPLAYABILITIES_API


struct FHitResult;


/**
 * UGameplayCueFunctionLibrary
 *
 *	Helpful utility function for working with gameplay cues.
 */
UCLASS(MinimalAPI)
class UGameplayCueFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	// Builds gameplay cue parameters using data from a hit result.
	UFUNCTION(BlueprintPure, Category = "GameplayCue")
	static UE_API FGameplayCueParameters MakeGameplayCueParametersFromHitResult(const FHitResult& HitResult);

	// Invoke a one time "instant" execute event for a gameplay cue on the target actor.
	// * If the actor has an ability system, the event will fire on authority only and will be replicated.
	// * If the actor does not have an ability system, the event will only be fired locally.
	UFUNCTION(BlueprintCallable, Category = "GameplayCue", Meta = (GameplayTagFilter = "GameplayCue"), DisplayName = "Execute GameplayCue On Actor (Burst)")
	static UE_API void ExecuteGameplayCueOnActor(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters);

	// Invoke the added event for a gameplay cue on the target actor. This should be paired with a RemoveGameplayCueOnActor call.
	// * If the actor has an ability system, the event will fire on authority only and will be replicated.
	// * If the actor does not have an ability system, the event will only be fired locally.
	UFUNCTION(BlueprintCallable, Category = "GameplayCue", Meta = (GameplayTagFilter = "GameplayCue"), DisplayName = "Add GameplayCue On Actor (Looping)")
	static UE_API void AddGameplayCueOnActor(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters);

	// Invoke the removed event for a gameplay cue on the target actor. This should be paired with an AddGameplayCueOnActor call.
	// * If the actor has an ability system, the event will fire on authority only and will be replicated.
	// * If the actor does not have an ability system, the event will only be fired locally.
	UFUNCTION(BlueprintCallable, Category = "GameplayCue", Meta = (GameplayTagFilter = "GameplayCue"), DisplayName = "Remove GameplayCue On Actor (Looping)")
	static UE_API void RemoveGameplayCueOnActor(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters);
};

#undef UE_API
