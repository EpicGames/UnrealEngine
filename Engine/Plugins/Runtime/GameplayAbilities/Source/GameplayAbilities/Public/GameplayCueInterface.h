// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Engine/NetSerialization.h"
#include "UObject/Interface.h"
#include "GameplayTagContainer.h"
#include "GameplayEffectTypes.h"
#include "GameplayPrediction.h"
#include "GameplayCueInterface.generated.h"

#define UE_API GAMEPLAYABILITIES_API

struct FMinimalGameplayCueReplicationProxyForNetSerializer;
namespace UE::Net
{
	class FMinimalGameplayCueReplicationProxyReplicationFragment;
}

/** Interface for actors that wish to handle GameplayCue events from GameplayEffects. Native only because blueprints can't implement interfaces with native functions */
UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UGameplayCueInterface: public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IGameplayCueInterface
{
	GENERATED_IINTERFACE_BODY()

public:

	/** Handle a single gameplay cue */
	UE_API virtual void HandleGameplayCue(UObject* Self, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);

	/** Wrapper that handles multiple cues */
	UE_API virtual void HandleGameplayCues(UObject* Self, const FGameplayTagContainer& GameplayCueTags, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);

	/**
	* Returns true if the object can currently accept gameplay cues associated with the given tag. Returns true by default.
	* Allows objects to opt out of cues in cases such as pending death
	*/
	UE_API virtual bool ShouldAcceptGameplayCue(UObject* Self, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);


	// DEPRECATED - use the UObject* signatures above

	/** Handle a single gameplay cue */
	UE_API virtual void HandleGameplayCue(AActor *Self, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);

	/** Wrapper that handles multiple cues */
	UE_API virtual void HandleGameplayCues(AActor *Self, const FGameplayTagContainer& GameplayCueTags, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);

	/** Returns true if the actor can currently accept gameplay cues associated with the given tag. Returns true by default. Allows actors to opt out of cues in cases such as pending death */
	UE_API virtual bool ShouldAcceptGameplayCue(AActor *Self, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);

	// END DEPRECATED


	/** Return the cue sets used by this object. This is optional and it is possible to leave this list empty. */
	virtual void GetGameplayCueSets(TArray<class UGameplayCueSet*>& OutSets) const {}

	/** Default native handler, called if no tag matches found */
	UE_API virtual void GameplayCueDefaultHandler(EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);

	/** Internal function to map ufunctions directly to gameplaycue tags */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category = GameplayCue, meta = (BlueprintInternalUseOnly = "true"))
	UE_API void BlueprintCustomHandler(EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);

	/** Call from a Cue handler event to continue checking for additional, more generic handlers. Called from the ability system blueprint library */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Ability|GameplayCue")
	UE_API virtual void ForwardGameplayCueToParent();

	/** Calls the UFunction override for a specific gameplay cue */
	static UE_API void DispatchBlueprintCustomHandler(UObject* Object, UFunction* Func, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);

	/** Clears internal cache of what classes implement which functions */
	static UE_API void ClearTagToFunctionMap();

	IGameplayCueInterface() : bForwardToParent(false) {}

private:
	/** If true, keep checking for additional handlers */
	bool bForwardToParent;
};


/**
 *	This is meant to provide another way of using GameplayCues without having to go through GameplayEffects.
 *	E.g., it is convenient if GameplayAbilities can issue replicated GameplayCues without having to create
 *	a GameplayEffect.
 *	
 *	Essentially provides bare necessities to replicate GameplayCue Tags.
 */
struct FActiveGameplayCueContainer;

USTRUCT(BlueprintType)
struct FActiveGameplayCue : public FFastArraySerializerItem
{
	GENERATED_USTRUCT_BODY()

	FActiveGameplayCue()	
	{
		bPredictivelyRemoved = false;
	}

	UPROPERTY()
	FGameplayTag GameplayCueTag;

	UPROPERTY()
	FPredictionKey PredictionKey;

	UPROPERTY()
	FGameplayCueParameters Parameters;

	/** Has this been predictively removed on the client? */
	UPROPERTY(NotReplicated)
	bool bPredictivelyRemoved;

	void PreReplicatedRemove(const struct FActiveGameplayCueContainer &InArray);
	void PostReplicatedAdd(const struct FActiveGameplayCueContainer &InArray);
	void PostReplicatedChange(const struct FActiveGameplayCueContainer &InArray) { }

	FString GetDebugString();
};

USTRUCT(BlueprintType)
struct FActiveGameplayCueContainer : public FFastArraySerializer
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray< FActiveGameplayCue >	GameplayCues;

	UE_API void SetOwner(UAbilitySystemComponent* InOwner);
	UE_API UAbilitySystemComponent* GetOwner() const;

	/** Should this container only replicate in minimal replication mode */
	bool bMinimalReplication;

	UE_API void AddCue(const FGameplayTag& Tag, const FPredictionKey& PredictionKey, const FGameplayCueParameters& Parameters);
	UE_API void RemoveCue(const FGameplayTag& Tag);

	/** Marks as predictively removed so that we dont invoke remove event twice due to onrep */
	UE_API void PredictiveRemove(const FGameplayTag& Tag);

	UE_API void PredictiveAdd(const FGameplayTag& Tag, FPredictionKey& PredictionKey);

	/** Does explicit check for gameplay cue tag */
	UE_API bool HasCue(const FGameplayTag& Tag) const;

	/** Returns true if the instance should be replicated. If false the property is allowed to be disabled for replication. */
	UE_API bool ShouldReplicate() const;

	UE_API bool NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms);

	// Will broadcast the OnRemove event for all currently active cues
	UE_API void RemoveAllCues();

private:

	UE_API int32 GetGameStateTime(const UWorld* World) const;

	UPROPERTY(NotReplicated)
	TObjectPtr<class UAbilitySystemComponent>	Owner = nullptr;
	
	friend struct FActiveGameplayCue;
};

template<>
struct TStructOpsTypeTraits< FActiveGameplayCueContainer > : public TStructOpsTypeTraitsBase2< FActiveGameplayCueContainer >
{
	enum
	{
		WithNetDeltaSerializer = true,
	};
};


/**
 *	Wrapper struct around a gameplaytag with the GameplayCue category. This also allows for a details customization
 */
USTRUCT(BlueprintType)
struct FGameplayCueTag
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (Categories="GameplayCue"), Category="GameplayCue")
	FGameplayTag GameplayCueTag;

	bool IsValid() const
	{
		return GameplayCueTag.IsValid();
	}
};

/** 
 *	An alternative way to replicating gameplay cues. This does not use fast TArray serialization and does not serialize gameplaycue parameters. The parameters are created on the receiving side with default information.
 *	This will be more efficient with server cpu but will take more bandwidth when the array changes.
 *	
 *	To use, put this on your replication proxy actor (such a the pawn). Call SetOwner, PreReplication and RemoveallCues in the appropriate places.
 */
USTRUCT()
struct FMinimalGameplayCueReplicationProxy
{
	GENERATED_BODY()

	UE_API FMinimalGameplayCueReplicationProxy();

	/** Set Owning ASC. This is what the GC callbacks are called on.  */
	UE_API void SetOwner(UAbilitySystemComponent* ASC);

	/** Copies data in from an FActiveGameplayCueContainer (such as the one of the ASC). You must call this manually from PreReplication. */
	UE_API void PreReplication(const FActiveGameplayCueContainer& SourceContainer);

	/** Custom NetSerialization to pack the entire array */
	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** Will broadcast the OnRemove event for all currently active cues */
	UE_API void RemoveAllCues();

	/** If true, we will skip updating the Owner ASC if we replicate on a connection owned by the ASC */
	void SetRequireNonOwningNetConnection(bool b) { bRequireNonOwningNetConnection = b; }

	/** Called to init parameters */
	TFunction<void(FGameplayCueParameters&, UAbilitySystemComponent*)> InitGameplayCueParametersFunc;

	bool operator==(const FMinimalGameplayCueReplicationProxy& Other) const { return LastSourceArrayReplicationKey == Other.LastSourceArrayReplicationKey; }
	bool operator!=(const FMinimalGameplayCueReplicationProxy& Other) const { return !(*this == Other); }

	bool operator==(const FActiveGameplayCueContainer& Other) const { return LastSourceArrayReplicationKey == Other.ArrayReplicationKey; }
	bool operator!=(const FActiveGameplayCueContainer& Other) const { return !(*this == Other); }

private:
	friend FMinimalGameplayCueReplicationProxyForNetSerializer;
	friend UE::Net::FMinimalGameplayCueReplicationProxyReplicationFragment;

	enum { NumInlineTags = 16 };

	TArray< FGameplayTag, TInlineAllocator<NumInlineTags> >	ReplicatedTags;
	TArray< FVector_NetQuantize, TInlineAllocator<NumInlineTags> > ReplicatedLocations;
	TArray< FGameplayTag, TInlineAllocator<NumInlineTags> >	LocalTags;
	TBitArray< TInlineAllocator<NumInlineTags> >			LocalBitMask;

	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> Owner = nullptr;

	int32 LastSourceArrayReplicationKey = -1;

	bool bRequireNonOwningNetConnection = false;
	bool bCachedModifiedOwnerTags = false;
};

template<>
struct TStructOpsTypeTraits< FMinimalGameplayCueReplicationProxy > : public TStructOpsTypeTraitsBase2< FMinimalGameplayCueReplicationProxy >
{
	enum
	{
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
		WithIdenticalViaEquality = true,
	};
};

#undef UE_API
