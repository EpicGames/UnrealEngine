// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "GameplayTagContainer.h"
#include "Engine/DataAsset.h"
#include "GameplayEffectTypes.h"
#include "GameplayPrediction.h"
#include "Engine/World.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/StreamableManager.h"
#include "GameplayCue_Types.h"
#include "GameplayCueTranslator.h"
#include "GameplayCueManager.generated.h"

#define UE_API GAMEPLAYABILITIES_API

#define GAMEPLAYCUE_DEBUG 0

class AGameplayCueNotify_Actor;
class UAbilitySystemComponent;
class UObjectLibrary;

DECLARE_DELEGATE_OneParam(FOnGameplayCueNotifySetLoaded, TArray<FSoftObjectPath>);
DECLARE_DELEGATE_OneParam(FGameplayCueProxyTick, float);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FShouldLoadGCNotifyDelegate, const FAssetData&, FName);

/** Options to specify what parts of gameplay cue execution should be skipped */
enum class EGameplayCueExecutionOptions : int32
{
	// Default options, check everything
	Default = 0,
	// Skip gameplay cue interface check
	IgnoreInterfaces	= 0x00000001,
	// Skip spawning notifies
	IgnoreNotifies		= 0x00000002,
	// Skip tag translation step
	IgnoreTranslation	= 0x00000004,
	// Ignores suppression check, always spawns
	IgnoreSuppression	= 0x00000008,
	// Don't show debug visualizations
	IgnoreDebug			= 0x00000010
};
ENUM_CLASS_FLAGS(EGameplayCueExecutionOptions);

/** An ObjectLibrary for the GameplayCue Notifies. Wraps 2 underlying UObjectLibraries plus options/delegates for how they are loaded */ 
USTRUCT()
struct FGameplayCueObjectLibrary
{
	GENERATED_BODY()
	FGameplayCueObjectLibrary()
		: ActorObjectLibrary(nullptr)
		, StaticObjectLibrary(nullptr)
		, CueSet(nullptr)
		, AsyncPriority(0)
		, bShouldSyncScan(false)
		, bShouldAsyncLoad(false)
		, bShouldSyncLoad(false)
		, bHasBeenInitialized(false)
	{
	}

	/** Paths to search for */
	UPROPERTY()
	TArray<FString> Paths;

	/** Callback for when load finishes */
	FOnGameplayCueNotifySetLoaded OnLoaded;

	/** Callback for "should I add this FAssetData to the set" */
	FShouldLoadGCNotifyDelegate ShouldLoad;

	/** Object library for actor based notifies */
	UPROPERTY()
	TObjectPtr<UObjectLibrary> ActorObjectLibrary;

	/** Object library for object based notifies */
	UPROPERTY()
	TObjectPtr<UObjectLibrary> StaticObjectLibrary;

	/** Set to put the loaded asset data into. If null we will use the global set (RuntimeGameplayCueObjectLibrary.CueSet) */
	UPROPERTY()
	TObjectPtr<UGameplayCueSet> CueSet;

	/** Priority to use if async loading */
	TAsyncLoadPriority AsyncPriority;

	/** Should we force a sync scan on the asset registry in order to discover asset data, or just use what is there */
	UPROPERTY()
	bool bShouldSyncScan;

	/** Should we start async loading everything that we find (that passes ShouldLoad delegate check) */
	UPROPERTY()
	bool bShouldAsyncLoad;

	/** Should we sync load everything that we find (that passes ShouldLoad delegate check) */
	UPROPERTY()
	bool bShouldSyncLoad;

	/** True if this has been initialized with correct data */
	UPROPERTY()
	bool bHasBeenInitialized;
};

/** While a GameplayCueNotify class is being loaded async, GameplayCue events will be deferred until load completion. */
struct FAsyncLoadPendingGameplayCue
{
	FAsyncLoadPendingGameplayCue() : 
		OwningSet(nullptr), GameplayCueTag(), TargetActor(nullptr), EventType(EGameplayCueEvent::Type::OnActive), Parameters() 
	{
	}

	FAsyncLoadPendingGameplayCue(TWeakObjectPtr<UGameplayCueSet> OwningSet, FGameplayTag GameplayCueTag, TWeakObjectPtr<AActor> TargetActor, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters) :
		OwningSet(OwningSet), GameplayCueTag(GameplayCueTag), TargetActor(TargetActor), EventType(EventType), Parameters(Parameters)
	{
	}

	TWeakObjectPtr<UGameplayCueSet> OwningSet;
	FGameplayTag GameplayCueTag;
	TWeakObjectPtr<AActor> TargetActor;
	TEnumAsByte<EGameplayCueEvent::Type> EventType;
	FGameplayCueParameters Parameters;
};

/** Singleton manager object that handles dispatching gameplay cues and spawning GameplayCueNotify actors as needed */
UCLASS(MinimalAPI)
class UGameplayCueManager : public UDataAsset
{
	GENERATED_UCLASS_BODY()

	// -------------------------------------------------------------
	// Wrappers to handle replicating executed cues
	// -------------------------------------------------------------
	UE_API virtual void InvokeGameplayCueExecuted_FromSpec(UAbilitySystemComponent* OwningComponent, const FGameplayEffectSpec& Spec, FPredictionKey PredictionKey);
	UE_API virtual void InvokeGameplayCueExecuted(UAbilitySystemComponent* OwningComponent, const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayEffectContextHandle EffectContext);
	UE_API virtual void InvokeGameplayCueExecuted_WithParams(UAbilitySystemComponent* OwningComponent, const FGameplayTag GameplayCueTag, FPredictionKey PredictionKey, FGameplayCueParameters GameplayCueParameters);

	UE_API virtual void InvokeGameplayCueAddedAndWhileActive_FromSpec(UAbilitySystemComponent* OwningComponent, const FGameplayEffectSpec& Spec, FPredictionKey PredictionKey);

	/** Start or stop a gameplay cue send context. Used by FScopedGameplayCueSendContext above, when all contexts are removed the cues are flushed */
	UE_API void StartGameplayCueSendContext();
	UE_API void EndGameplayCueSendContext();

	/** Send out any pending cues */
	UE_API virtual void FlushPendingCues();

	/** Broadcasted when ::FlushPendingCues runs: useful for custom batching/gameplay cue handling */
	FSimpleMulticastDelegate	OnFlushPendingCues;

	/** Called when manager is first created */
	UE_API virtual void OnCreated();

	/** Called when engine has completely loaded, this is a good time to finalize things */
	UE_API virtual void OnEngineInitComplete();

	/** Process a pending cue, return false if the cue should be rejected. */
	UE_API virtual bool ProcessPendingCueExecute(FGameplayCuePendingExecute& PendingCue);

	/** Returns true if two pending cues match, can be overridden in game */
	UE_API virtual bool DoesPendingCueExecuteMatch(FGameplayCuePendingExecute& PendingCue, FGameplayCuePendingExecute& ExistingCue);

	// -------------------------------------------------------------
	// Handling GameplayCues at runtime:
	// -------------------------------------------------------------

	/** Main entry point for handling a gameplaycue event. These functions will call the 3 functions below to handle gameplay cues */
	UE_API virtual void HandleGameplayCues(AActor* TargetActor, const FGameplayTagContainer& GameplayCueTags, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters, EGameplayCueExecutionOptions Options = EGameplayCueExecutionOptions::Default);
	UE_API virtual void HandleGameplayCue(AActor* TargetActor, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters, EGameplayCueExecutionOptions Options = EGameplayCueExecutionOptions::Default);

	/** 1. returns true to ignore gameplay cues */
	UE_API virtual bool ShouldSuppressGameplayCues(AActor* TargetActor);

	/** 2. Allows Tag to be translated in place to a different Tag. See FGameplayCueTranslorManager */
	UE_API void TranslateGameplayCue(FGameplayTag& Tag, AActor* TargetActor, const FGameplayCueParameters& Parameters);

	/** 3. Actually routes the gameplaycue event to the right place.  */
	UE_API virtual void RouteGameplayCue(AActor* TargetActor, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters, EGameplayCueExecutionOptions Options = EGameplayCueExecutionOptions::Default);

	/** 
	 *  Convenience methods for invoking non-replicated gameplay cue events. 
	 * 
	 *	We want to avoid exposing designers the choice of "is this gameplay cue replicated or non-replicated?".
	 *	We want to make the decision for them in most cases:
	 *	- Abilities will always use replicated GameplayCue events because they are not executed on simulated proxies.
	 *	- Animations always use non-replicated GameplayCue events because they are always executed on simulated proxies.
	 *	
	 *  Sometimes it will be useful to give designers both options: in actor classes where there are many possible use cases.
	 *  Still, we should keep the choice confined to the actor class, and not globally.  E.g., Don't add both choices to the function library
	 *  since they would appear everywhere. Add the choices to the actor class so they only appear there.
	 */
	static UE_API void AddGameplayCue_NonReplicated(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters);
	static UE_API void RemoveGameplayCue_NonReplicated(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters);
	static UE_API void ExecuteGameplayCue_NonReplicated(AActor* Target, const FGameplayTag GameplayCueTag, const FGameplayCueParameters& Parameters);

	// -------------------------------------------------------------

	/** Force any instanced GameplayCueNotifies to stop */
	UE_API virtual void EndGameplayCuesFor(AActor* TargetActor);

	/** Returns the cached instance cue. Creates it if it doesn't exist. */
	UE_API virtual AGameplayCueNotify_Actor* GetInstancedCueActor(AActor* TargetActor, UClass* GameplayCueNotifyActorClass, const FGameplayCueParameters& Parameters);

	/** Notify that this actor is finished and should be destroyed or recycled */
	UE_API virtual void NotifyGameplayCueActorFinished(AGameplayCueNotify_Actor* Actor);

	/** Notify to say the actor is about to be destroyed and the GC manager needs to remove references to it. This should not happen in normal play with recycling enabled, but could happen in replays. */
	UE_API virtual void NotifyGameplayCueActorEndPlay(AGameplayCueNotify_Actor* Actor);

	/** Resets preallocation for a given world */
	UE_API void ResetPreallocation(UWorld* World);

	/** Prespawns a single actor for gameplaycue notify actor classes that need prespawning (should be called by outside gamecode, such as gamestate) */
	UE_API void UpdatePreallocation(UWorld* World);

	UE_API void OnPostWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

	UE_API void OnPreReplayScrub(UWorld* World);

	/** Prints what classes exceeded their preallocation sizes during runtime */
	UE_API void DumpPreallocationStats(const FPreallocationInfo& PreallocationInfo, bool bWarnOnActiveActors = false);

	// -------------------------------------------------------------
	//  Loading GameplayCueNotifies from ObjectLibraries
	//
	//	There are two libraries in the GameplayCueManager:
	//	1. The RunTime object library, which is initialized with the "always loaded" paths, via UAbilitySystemGlobals::GetGameplayCueNotifyPaths()
	//		-GC Notifies in this path are loaded at startup
	//		-Everything loaded will go into the global gameplaycue set, which is where GC events will be routed to by default
	//
	//	2. The Editor object library, which is initialized with the "all valid" paths, via UGameplayCueManager::GetValidGameplayCuePaths()
	//		-Only used in the editor.
	//		-Never loads clases or scans directly itself. 
	//		-Just reflect asset registry to show about "all gameplay cue notifies the game could know about"
	//
	// -------------------------------------------------------------

	/** Returns the Ryntime cueset, which is the global cueset used in runtime, as opposed to the editor cue set which is used only when running the editor */
	UE_API UGameplayCueSet* GetRuntimeCueSet();

	/** Called to setup and initialize the runtime library. The passed in paths will be scanned and added to the global gameplay cue set where appropriate */
	UE_API void InitializeRuntimeObjectLibrary();

	// Will return the Runtime cue set and the EditorCueSet, if the EditorCueSet is available. This is used mainly for handling asset created/deleted in the editor
	UE_API TArray<UGameplayCueSet*> GetGlobalCueSets();

#if WITH_EDITOR
	/** Called from editor to soft load all gameplay cue notifies for the GameplayCueEditor */
	UE_API void InitializeEditorObjectLibrary();

	/** Calling this will make the GC manager periodically refresh the EditorObjectLibrary until the asset registry is finished scanning */
	UE_API void RequestPeriodicUpdateOfEditorObjectLibraryWhileWaitingOnAssetRegistry();

	/** Get filenames of all GC notifies we know about (loaded or not). Useful for cooking */
	UE_API void GetEditorObjectLibraryGameplayCueNotifyFilenames(TArray<FString>& Filenames) const;

	/** Looks in the EditorObjectLibrary for a notify for this tag, if it finds it, it loads it and puts it in the RuntimeObjectLibrary so that it can be previewed in the editor */
	UE_API void LoadNotifyForEditorPreview(FGameplayTag GameplayCueTag);

	UE_API UGameplayCueSet* GetEditorCueSet();

	FSimpleMulticastDelegate OnEditorObjectLibraryUpdated;
	bool EditorObjectLibraryFullyInitialized;

	FTimerHandle EditorPeriodicUpdateHandle;
#endif

protected:

	UE_API virtual bool ShouldSyncScanRuntimeObjectLibraries() const;
	UE_API virtual bool ShouldSyncLoadRuntimeObjectLibraries() const;
	UE_API virtual bool ShouldAsyncLoadRuntimeObjectLibraries() const;
	UE_API virtual bool ShouldDeferScanningRuntimeLibraries() const;

	/** Refreshes the existing, already initialized, object libraries. */
	UE_API void RefreshObjectLibraries();

	/** Internal function to actually init the FGameplayCueObjectLibrary.  Returns StreamableHandle when asset async loading is requested. */
	UE_API virtual TSharedPtr<FStreamableHandle> InitObjectLibrary(FGameplayCueObjectLibrary& Library);

	UE_API virtual TArray<FString> GetAlwaysLoadedGameplayCuePaths();

	/** returns list of valid gameplay cue paths. Subclasses may override this to specify locations that aren't part of the "always loaded" LoadedPaths array */
	virtual TArray<FString>	GetValidGameplayCuePaths() { return GetAlwaysLoadedGameplayCuePaths(); }

	/** Given a TargetActor and a CueClass, find an existing instance of the CueNotify Actor that we can reuse */
	UE_API AGameplayCueNotify_Actor* FindExistingCueOnActor(const AActor& TargetActor, const TSubclassOf<AGameplayCueNotify_Actor>& CueClass, const FGameplayCueParameters& Parameters) const;

	/** Given a CueClass, find an already spawned (but currently unused) recycled instance that exists in FindInWorld so that we may reuse it. Note: This function also compacts the recycled instances, removing stale ones. */
	UE_API AGameplayCueNotify_Actor* FindRecycledCue(const TSubclassOf<AGameplayCueNotify_Actor>& CueClass, const UWorld& FindInWorld);

	UPROPERTY(transient)
	FGameplayCueObjectLibrary RuntimeGameplayCueObjectLibrary;

	UPROPERTY(transient)
	FGameplayCueObjectLibrary EditorGameplayCueObjectLibrary;

	/** Handle to maintain ownership of gameplay cue assets.  
      Note:	Only the latest handle to async load request is cached.
			If projects require multiple concurrent asynchronous loads, handles returned from InitObjectLibrary should be cached as needed. */
	TSharedPtr<FStreamableHandle> GameplayCueAssetHandle;

public:		
	/** Called before loading any gameplay cue notifies from object libraries. Allows subclasses to skip notifies. */
	virtual bool ShouldLoadGameplayCueAssetData(const FAssetData& Data) const { return true; }

	/**
	 * Add a path to the GameplayCueNotifyPaths array.
	 * Re-initalizes RuntimeObjectLibrary based on the new paths
	 *
	 * @param InPath					The path to the directory that should be added to the scan
	 * @param bShouldRescanCueAssets		If true then the runtime object library will be rebuilt.
	 */
	UE_API virtual void AddGameplayCueNotifyPath(const FString& InPath, const bool bShouldRescanCueAssets = true);

	/**
	 * Remove the given gameplay cue notify path from the GameplayCueNotifyPaths array.
	 * Re-initalizes RuntimeObjectLibrary based on the new paths
	 * 
	 * @param InPath					The path to the directory that should be removed from the scan
	 * @param bShouldRescanCueAssets		If true then the runtime object library will be rebuilt.
	 * @return Number of paths removed.
	 */
	UE_API virtual int32 RemoveGameplayCueNotifyPath(const FString& InPath, const bool bShouldRescanCueAssets = true);
	
	UE_API int32 FinishLoadingGameplayCueNotifies();

	FStreamableManager	StreamableManager;
	
	UE_API void PrintGameplayCueNotifyMap();

	UE_API virtual void PrintLoadedGameplayCueNotifyClasses();

	UE_API virtual class UWorld* GetWorld() const override;

	static UE_API UWorld* GetCachedWorldForGameplayCueNotifies();

	DECLARE_EVENT_FiveParams(UGameplayCueManager, FOnRouteGameplayCue, AActor*, FGameplayTag, EGameplayCueEvent::Type, const FGameplayCueParameters&, EGameplayCueExecutionOptions);
	FOnRouteGameplayCue& OnGameplayCueRouted() { return OnRouteGameplayCue; }

#if WITH_EDITOR

	/** Handles updating an object library when a new asset is created */
	UE_API void HandleAssetAdded(UObject *Object);

	/** Handles cleaning up an object library if it matches the passed in object */
	UE_API void HandleAssetDeleted(UObject *Object);

	/** Warns if we move a GameplayCue notify out of the valid search paths */
	UE_API void HandleAssetRenamed(const FAssetData& Data, const FString& String);

	UE_API bool VerifyNotifyAssetIsInValidPath(FString Path);

	bool bAccelerationMapOutdated;

	FOnGameplayCueNotifyChange	OnGameplayCueNotifyAddOrRemove;

	/** Animation Preview Hacks */
	static UE_API class USceneComponent* PreviewComponent;
	static UE_API UWorld* PreviewWorld;
	static UE_API FGameplayCueProxyTick PreviewProxyTick;
#endif

	static UE_API bool IsGameplayCueRecylingEnabled();
	
	virtual bool ShouldAsyncLoadObjectLibrariesAtStart() const { return true; }

	FGameplayCueTranslationManager	TranslationManager;

	// -------------------------------------------------------------
	//  Debugging Help
	// -------------------------------------------------------------

#if GAMEPLAYCUE_DEBUG
	UE_API virtual FGameplayCueDebugInfo* GetDebugInfo(int32 Handle, bool Reset=false);
#endif

	/** If true, this will synchronously load missing gameplay cues */
	UE_API virtual bool ShouldSyncLoadMissingGameplayCues() const;

	/** If true, this will asynchronously load missing gameplay cues, and execute cue when the load finishes */
	UE_API virtual bool ShouldAsyncLoadMissingGameplayCues() const;

	/** Whether a GameplayCueNotify class is currently loading (async load requested by this and streaming callback not received yet). */
	UE_API bool IsAsyncLoadingGameplayCueNotifyClass(const FSoftObjectPath& CueNotifyClass) const;

	/** Handles what to do if a missing cue is requested. If return true, this was loaded and execution should continue */
	UE_API virtual bool HandleMissingGameplayCue(UGameplayCueSet* OwningSet, struct FGameplayCueNotifyData& CueData, AActor* TargetActor, EGameplayCueEvent::Type EventType, FGameplayCueParameters& Parameters);

protected:

#if WITH_EDITOR
	//This handles the case where GameplayCueNotifications have changed between sessions, which is possible in editor.
	UE_API virtual void ReloadObjectLibrary(UWorld* World, const UWorld::InitializationValues IVS);
#endif

	UE_API void BuildCuesToAddToGlobalSet(TConstArrayView<FAssetData> AssetDataList, FName TagPropertyName, TArray<struct FGameplayCueReferencePair>& OutCuesToAdd, TArray<FSoftObjectPath>& OutAssetsToLoad, FShouldLoadGCNotifyDelegate = FShouldLoadGCNotifyDelegate());

	/** The cue manager has a tendency to produce a lot of RPCs. This logs out when we are attempting to fire more RPCs than will actually go off */
	UE_API void CheckForTooManyRPCs(FName FuncName, const FGameplayCuePendingExecute& PendingCue, const FString& CueID, const FGameplayEffectContext* EffectContext);

	UE_API void OnGameplayCueNotifyAsyncLoadComplete(TArray<FSoftObjectPath> StringRef);

	UE_API void OnMissingCueAsyncLoadComplete(FSoftObjectPath LoadedNotifyClass);

	UE_API void CheckForPreallocation(UClass* GCClass);

	/** Hardref to the gameplaycue notify classes we have async loaded*/
	UPROPERTY(transient)
	TArray<TObjectPtr<UClass>> LoadedGameplayCueNotifyClasses;

	/** Classes that we need to preallocate instances for */
	UPROPERTY(transient)
	TArray<TSubclassOf<AGameplayCueNotify_Actor>> GameplayCueClassesForPreallocation;

	/** List of gameplay cue executes that haven't been processed yet */
	UPROPERTY(transient)
	TArray<FGameplayCuePendingExecute> PendingExecuteCues;

	/** Number of active gameplay cue send contexts, when it goes to 0 cues are flushed */
	UPROPERTY(transient)
	int32 GameplayCueSendContextCount;

	/** Cached world we are currently handling cues for. Used for non instanced GC Notifies that need world. */
	static UE_API UWorld* CurrentWorld;

	UE_API FPreallocationInfo& GetPreallocationInfo(const UWorld* World);

	UPROPERTY(transient)
	TArray<FPreallocationInfo>	PreallocationInfoList_Internal;

	/** While a GCN class was not loaded yet, events are requested and async loading is enabled, cache them until the class finishes async loading. */
	TMap<FSoftObjectPath, TArray<FAsyncLoadPendingGameplayCue>> AsyncLoadPendingGameplayCues;

	FOnRouteGameplayCue OnRouteGameplayCue;

private:

	UE_API void AddPendingCueExecuteInternal(FGameplayCuePendingExecute& PendingCue);

};

#undef UE_API
