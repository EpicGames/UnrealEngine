// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Camera/CameraTypes.h"
#include "GameplayEffectTypes.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "GameplayCueNotifyTypes.generated.h"

#define UE_API GAMEPLAYABILITIES_API


class UNiagaraSystem;
class UFXSystemComponent;
class UAudioComponent;
class USoundBase;
class UCameraShakeBase;
class ICameraLensEffectInterface;
class UForceFeedbackEffect;
class UInputDeviceProperty;
class UForceFeedbackAttenuation;
class UForceFeedbackComponent;
class UDecalComponent;
class UMaterialInterface;
class APlayerController;
struct FGameplayCueParameters;
struct FGameplayCueNotify_SpawnContext;


typedef uint64 FGameplayCueNotify_SurfaceMask;
static_assert((EPhysicalSurface::SurfaceType_Max <= (sizeof(FGameplayCueNotify_SurfaceMask) * 8)), "FGameplayCueNotify_SurfaceMask does not have enough bits for EPhysicalSurface");


DECLARE_LOG_CATEGORY_EXTERN(LogGameplayCueNotify, Log, All);

/*
 * The Gameplay Cue Notify structs below cannot be inherited, but they are now exposed and can be aggregated.
 * If you want to extend functionality in your own structs you can include the base struct in your own and use it in
 * your own Notify Cue objects. EX
 *

USTRUCT(BlueprintType)
struct FMyGameplayCueNotify_CustomBurstEffects
{
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	FGameplayCueNotify_BurstEffects BaseBurstEffects;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	float CustomFloat;

	void CustomExecute()
	{
		BaseBurstEffects.ExecuteEffects();
	}
};

 *
 **/

/**
 * EGameplayCueNotify_EffectPlaySpace
 *
 *	Used by some effects (like camera shakes) to specify what coordinate space the they should be applied in.
 */
UENUM(BlueprintType)
enum class EGameplayCueNotify_EffectPlaySpace : uint8
{
	// Play the effect in world space.
	WorldSpace,

	// Play the effect in camera space.
	CameraSpace,
};


/**
 * EGameplayCueNotify_LocallyControlledSource
 *
 *	Specifies what actor to use when determining if the gameplay cue notify is locally controlled.
 */
UENUM(BlueprintType)
enum class EGameplayCueNotify_LocallyControlledSource : uint8
{
	// Use the instigator actor as the source when deciding the locally controlled policy.
	InstigatorActor,

	// Use the target actor as the source when deciding the locally controlled policy.
	TargetActor,
};


/**
 * EGameplayCueNotify_LocallyControlledPolicy
 *
 *	Specifies if the gameplay cue notify should spawn based on it being locally controlled.
 */
UENUM(BlueprintType)
enum class EGameplayCueNotify_LocallyControlledPolicy : uint8
{
	// Always spawns regardless of locally controlled.
	Always,

	// Only spawn if the source actor is locally controlled.
	LocalOnly,

	// Only spawn if the source actor is NOT locally controlled.
	NotLocal,
};


/**
 * FGameplayCueNotify_SpawnCondition
 *
 *	Conditions used to determine if the gameplay cue notify should spawn.
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_SpawnCondition final
{
	GENERATED_BODY()

	GAMEPLAYABILITIES_API FGameplayCueNotify_SpawnCondition();

	GAMEPLAYABILITIES_API bool ShouldSpawn(const FGameplayCueNotify_SpawnContext& SpawnContext) const;

	// Source actor to use when determining if it is locally controlled.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	EGameplayCueNotify_LocallyControlledSource LocallyControlledSource;

	// Locally controlled policy used to determine if the gameplay cue effects should spawn.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	EGameplayCueNotify_LocallyControlledPolicy LocallyControlledPolicy;

	// Random chance to play the effects.  (1.0 = always play,  0.0 = never play)
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (UIMin = "0.0", ClampMin = "0.0", UIMax = "1.0", ClampMax = "1.0"))
	float ChanceToPlay;

	// The gameplay cue effects will only spawn if the surface type is in this list.
	// An empty list means everything is allowed.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	TArray<TEnumAsByte<EPhysicalSurface>> AllowedSurfaceTypes;
	mutable FGameplayCueNotify_SurfaceMask AllowedSurfaceMask;

	// The gameplay cue effects will only spawn if the surface type is NOT in this list.
	// An empty list means nothing will be rejected.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	TArray<TEnumAsByte<EPhysicalSurface>> RejectedSurfaceTypes;
	mutable FGameplayCueNotify_SurfaceMask RejectedSurfaceMask;
};


/**
 * EGameplayCueNotify_AttachPolicy
 *
 *	Specifies if the gameplay cue notify should attach to the target actor.
 */
UENUM(BlueprintType)
enum class EGameplayCueNotify_AttachPolicy : uint8
{
	// Do not attach to the target actor.  The target may still be used to get location and other information.
	DoNotAttach,

	// Attach to the target actor if possible.
	AttachToTarget,
};


/**
 * FGameplayCueNotify_PlacementInfo
 *
 *	Specifies how the gameplay cue notify will be positioned in the world.
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_PlacementInfo final
{
	GENERATED_BODY()

	GAMEPLAYABILITIES_API FGameplayCueNotify_PlacementInfo();

	GAMEPLAYABILITIES_API bool FindSpawnTransform(const FGameplayCueNotify_SpawnContext& SpawnContext, FTransform& OutTransform) const;

	GAMEPLAYABILITIES_API void SetComponentTransform(USceneComponent* Component, const FTransform& Transform) const;
	GAMEPLAYABILITIES_API void TryComponentAttachment(USceneComponent* Component, USceneComponent* AttachComponent) const;

	// Target's socket (or bone) used for location and rotation.  If "None", it uses the target's root.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	FName SocketName;

	// Whether to attach to the target actor or not attach.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	EGameplayCueNotify_AttachPolicy AttachPolicy;

	// How the transform is handled when attached.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	EAttachmentRule AttachmentRule;

	// If enabled, will always spawn using rotation override.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverrideRotation : 1;

	// If enabled, will always spawn using the scale override.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverrideScale : 1;

	// If enabled, will always spawn using rotation override.
	// This will also set the rotation to be absolute, so any changes to the parent's rotation will be ignored after attachment.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverrideRotation"))
	FRotator RotationOverride;

	// If enabled, will always spawn using scale override.
	// This will also set the scale to be absolute, so any changes to the parent's scale will be ignored after attachment.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverrideScale"))
	FVector ScaleOverride;
};


/**
 * FGameplayCueNotify_SpawnContext
 *
 *	Temporary spawn information collected from input parameters.
 */
struct FGameplayCueNotify_SpawnContext final
{
public:

	GAMEPLAYABILITIES_API FGameplayCueNotify_SpawnContext(UWorld* InWorld, AActor* InTargetActor, const FGameplayCueParameters& InCueParameters);

	void SetDefaultSpawnCondition(const FGameplayCueNotify_SpawnCondition* InSpawnCondition) {DefaultSpawnCondition = InSpawnCondition;}
	void SetDefaultPlacementInfo(const FGameplayCueNotify_PlacementInfo* InPlacementInfo) {DefaultPlacementInfo = InPlacementInfo;}

	const FGameplayCueNotify_PlacementInfo& GetPlacementInfo(bool bUseOverride, const FGameplayCueNotify_PlacementInfo& PlacementInfoOverride) const
	{
		return (!bUseOverride && DefaultPlacementInfo) ? *DefaultPlacementInfo : PlacementInfoOverride;
	}

	const FGameplayCueNotify_SpawnCondition& GetSpawnCondition(bool bUseOverride, const FGameplayCueNotify_SpawnCondition& SpawnConditionOverride) const
	{
		return (!bUseOverride && DefaultSpawnCondition) ? *DefaultSpawnCondition : SpawnConditionOverride;
	}

	GAMEPLAYABILITIES_API APlayerController* FindLocalPlayerController(EGameplayCueNotify_LocallyControlledSource Source) const;

protected:

	void InitializeContext();

public:

	UWorld* World;
	AActor* TargetActor;
	const FGameplayCueParameters& CueParameters;
	const FHitResult* HitResult;
	USceneComponent* TargetComponent;
	EPhysicalSurface SurfaceType;

private:

	const FGameplayCueNotify_SpawnCondition* DefaultSpawnCondition;
	const FGameplayCueNotify_PlacementInfo* DefaultPlacementInfo;
};


/**
 * FGameplayCueNotify_SpawnResult
 *
 *	Temporary structure used to track results of spawning components.
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_SpawnResult final
{
	GENERATED_BODY()

	GAMEPLAYABILITIES_API FGameplayCueNotify_SpawnResult();

	GAMEPLAYABILITIES_API void Reset();

	// List of FX components spawned.  There may be null pointers here as it matches the defined order.
	UPROPERTY(BlueprintReadOnly, Transient, Category = GameplayCueNotify)
	TArray<TObjectPtr<UFXSystemComponent>> FxSystemComponents;

	// List of audio components spawned.  There may be null pointers here as it matches the defined order.
	UPROPERTY(BlueprintReadOnly, Transient, Category = GameplayCueNotify)
	TArray<TObjectPtr<UAudioComponent>> AudioComponents;

	// List of camera shakes played.  There will be one camera shake per local player controller if shake is played in world.
	UPROPERTY(BlueprintReadOnly, Transient, Category = GameplayCueNotify)
	TArray<TObjectPtr<UCameraShakeBase>> CameraShakes;

	// List of camera len effects spawned.  There will be one camera lens effect per local player controller if the effect is played in world.
	UPROPERTY(BlueprintReadOnly, Transient, Category = GameplayCueNotify)
	TArray<TScriptInterface<ICameraLensEffectInterface>> CameraLensEffects;

	// Force feedback component that was spawned.  This is only valid when force feedback is set to play in world.
	UPROPERTY(BlueprintReadOnly, Transient, Category = GameplayCueNotify)
	TObjectPtr<UForceFeedbackComponent> ForceFeedbackComponent;

	// Player controller used to play the force feedback effect.  Used to stop the effect later.
	UPROPERTY(Transient)
	TObjectPtr<APlayerController> ForceFeedbackTargetPC;

	// Spawned decal component.  This may be null.
	UPROPERTY(BlueprintReadOnly, Transient, Category = GameplayCueNotify)
	TObjectPtr<UDecalComponent> DecalComponent;
};


/**
 * FGameplayCueNotify_ParticleInfo
 *
 *	Properties that specify how to play a particle effect.
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_ParticleInfo final
{
	GENERATED_BODY()
	
	GAMEPLAYABILITIES_API FGameplayCueNotify_ParticleInfo();

	GAMEPLAYABILITIES_API bool PlayParticleEffect(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& 
	OutSpawnResult) const;

	GAMEPLAYABILITIES_API void ValidateBurstAssets(const UObject* ContainingAsset, const FString& Context, class FDataValidationContext& ValidationContext) const;

	// Condition to check before spawning the particle system.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverrideSpawnCondition"))
	FGameplayCueNotify_SpawnCondition SpawnConditionOverride;

	// Defines how the particle system will be placed.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverridePlacementInfo"))
	FGameplayCueNotify_PlacementInfo PlacementInfoOverride;

	// Niagara FX system to spawn.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	TObjectPtr<UNiagaraSystem> NiagaraSystem;

	// If enabled, use the spawn condition override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverrideSpawnCondition : 1;

	// If enabled, use the placement info override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverridePlacementInfo : 1;

	// If enabled, this particle system will cast a shadow.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	uint32 bCastShadow : 1;
};

/**
 * FGameplayCueNotify_SoundParameterInterfaceInfo
 *
 *	Properties that specify how to interface with the ISoundParameterControllerInterface
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_SoundParameterInterfaceInfo final
{
	GENERATED_BODY()

public:

	GAMEPLAYABILITIES_API FGameplayCueNotify_SoundParameterInterfaceInfo();

	// The name of the stop trigger set via the parameter interface
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	FName StopTriggerName = TEXT("OnStop");
};

/**
 * FGameplayCueNotify_SoundInfo
 *
 *	Properties that specify how to play a sound effect.
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_SoundInfo final
{
	GENERATED_BODY()
	
	GAMEPLAYABILITIES_API FGameplayCueNotify_SoundInfo();

	GAMEPLAYABILITIES_API bool PlaySound(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& OutSpawnResult) 
	const;

	GAMEPLAYABILITIES_API void ValidateBurstAssets(const UObject* ContainingAsset, const FString& Context, class FDataValidationContext& ValidationContext) const;

	// Condition to check before playing the sound.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverrideSpawnCondition"))
	FGameplayCueNotify_SpawnCondition SpawnConditionOverride;

	// Defines how the sound will be placed.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverridePlacementInfo"))
	FGameplayCueNotify_PlacementInfo PlacementInfoOverride;

	// Sound to play.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	TObjectPtr<USoundBase> Sound;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "5.0 - SoundCue is deprecated. Instead use the Sound property. The type is USoundBase not USoundCue."))
	TObjectPtr<USoundBase> SoundCue;

	// How long it should take to fade out.  Only used on looping gameplay cues.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	float LoopingFadeOutDuration;

	// The volume level you want the sound to fade out to over the 'Looping Fade Out Duration' before stopping.
	// This value is irrelevant if the 'Looping Fade Out Duration' is zero.
	// NOTE: If the fade out duration is positive and this value is not lower than the volume the sound is playing at, the sound will never stop!
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	float LoopingFadeVolumeLevel;

	// Defines how to interface with the sound.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bUseSoundParameterInterface"))
	FGameplayCueNotify_SoundParameterInterfaceInfo SoundParameterInterfaceInfo;

	// If enabled, use the spawn condition override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverrideSpawnCondition : 1;

	// If enabled, use the placement info override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverridePlacementInfo : 1;

	// If enabled, use the placement info override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bUseSoundParameterInterface : 1;

};


/**
 * FGameplayCueNotify_CameraShakeInfo
 *
 *	Properties that specify how to play a camera shake.
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_CameraShakeInfo final
{
	GENERATED_BODY()

	GAMEPLAYABILITIES_API FGameplayCueNotify_CameraShakeInfo();

	GAMEPLAYABILITIES_API bool PlayCameraShake(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& 
	OutSpawnResult) const;

	GAMEPLAYABILITIES_API void ValidateBurstAssets(const UObject* ContainingAsset, const FString& Context, class FDataValidationContext& ValidationContext) const;

	// Condition to check before playing the camera shake.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverrideSpawnCondition"))
	FGameplayCueNotify_SpawnCondition SpawnConditionOverride;

	// Defines how the camera shake will be placed.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverridePlacementInfo"))
	FGameplayCueNotify_PlacementInfo PlacementInfoOverride;

	// Camera shake to play.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	TSubclassOf<UCameraShakeBase> CameraShake;

	// Scale applied to the camera shake.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	float ShakeScale;

	// What coordinate space to play the camera shake relative to.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	EGameplayCueNotify_EffectPlaySpace PlaySpace;

	// If enabled, use the spawn condition override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverrideSpawnCondition : 1;

	// If enabled, use the placement info override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverridePlacementInfo : 1;

	// If enabled, the camera shake will be played in the world and affect all players.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	uint32 bPlayInWorld : 1;

	// Players inside this radius get the full intensity camera shake.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bPlayInWorld", UIMin = "0.0", ClampMin = "0.0"))
	float WorldInnerRadius;

	// Players outside this radius do not get the camera shake applied.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bPlayInWorld", UIMin = "0.0", ClampMin = "0.0"))
	float WorldOuterRadius;

	// Exponent that describes the shake intensity falloff curve between the inner and outer radii.  (1.0 is linear)
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bPlayInWorld", UIMin = "0.0", ClampMin = "0.0"))
	float WorldFalloffExponent;
};


/**
 * FGameplayCueNotify_CameraLensEffectInfo
 *
 *	Properties that specify how to play a camera lens effect.
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_CameraLensEffectInfo final
{
	GENERATED_BODY()

	GAMEPLAYABILITIES_API FGameplayCueNotify_CameraLensEffectInfo();

	GAMEPLAYABILITIES_API bool PlayCameraLensEffect(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& 
	OutSpawnResult) const;

	GAMEPLAYABILITIES_API void ValidateBurstAssets(const UObject* ContainingAsset, const FString& Context, class FDataValidationContext& ValidationContext) const;

	// Condition to check before playing the camera lens effect.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverrideSpawnCondition"))
	FGameplayCueNotify_SpawnCondition SpawnConditionOverride;

	// Defines how the camera lens effect will be placed.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverridePlacementInfo"))
	FGameplayCueNotify_PlacementInfo PlacementInfoOverride;

	// Camera lens effect to play.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (MustImplement = "/Script/Engine.CameraLensEffectInterface"))
	TSubclassOf<AActor> CameraLensEffect;

	// If enabled, use the spawn condition override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverrideSpawnCondition : 1;

	// If enabled, use the placement info override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverridePlacementInfo : 1;

	// If enabled, the camera lens effect will be played in the world and affect all players.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	uint32 bPlayInWorld : 1;

	// Players inside this radius get the full intensity camera lens effect.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bPlayInWorld", UIMin = "0.0", ClampMin = "0.0"))
	float WorldInnerRadius;

	// Players outside this radius do not get the camera lens effect applied.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bPlayInWorld", UIMin = "0.0", ClampMin = "0.0"))
	float WorldOuterRadius;
};


/**
 * FGameplayCueNotify_ForceFeedbackInfo
 *
 *	Properties that specify how to play a force feedback effect.
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_ForceFeedbackInfo final
{
	GENERATED_BODY()

	GAMEPLAYABILITIES_API FGameplayCueNotify_ForceFeedbackInfo();

	GAMEPLAYABILITIES_API bool PlayForceFeedback(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& 
	OutSpawnResult) const;
	
	GAMEPLAYABILITIES_API void ValidateBurstAssets(const UObject* ContainingAsset, const FString& Context, class FDataValidationContext& ValidationContext) const;

	// Condition to check before playing the force feedback.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverrideSpawnCondition"))
	FGameplayCueNotify_SpawnCondition SpawnConditionOverride;

	// Defines how the force feedback will be placed.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverridePlacementInfo"))
	FGameplayCueNotify_PlacementInfo PlacementInfoOverride;

	// Force feedback effect to play.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	TObjectPtr<UForceFeedbackEffect> ForceFeedbackEffect;

	// Tag used to identify the force feedback effect so it can later be canceled.
	// Warning: If this is "None" it will stop ALL force feedback effects if it is canceled.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	FName ForceFeedbackTag;

	// If enabled, the force feedback will be set to loop.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	uint32 bIsLooping : 1;

	// If enabled, use the spawn condition override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverrideSpawnCondition : 1;

	// If enabled, use the placement info override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverridePlacementInfo : 1;

	// If enabled, the force feedback will be played in the world and affect all players.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	uint32 bPlayInWorld : 1;

	// Multiplier applied to the force feedback when played in world.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bPlayInWorld"))
	float WorldIntensity;

	// How the force feedback is attenuated over distance when played in world.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bPlayInWorld"))
	TObjectPtr<UForceFeedbackAttenuation> WorldAttenuation;
};


/**
 * FGameplayCueNotify_InputDevicePropertyInfo
 *
 * Properties that specify how to set input device properties during a gameplay cue notify
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_InputDevicePropertyInfo final
{
	GENERATED_BODY()
	
	FGameplayCueNotify_InputDevicePropertyInfo() = default;

	/** Set the device properties on specified on this struct on the Input Device Subsystem. */
	GAMEPLAYABILITIES_API bool SetDeviceProperties(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& 
	OutSpawnResult) const;

	/** Validate that the device properties in this effect are usable */
	GAMEPLAYABILITIES_API void ValidateBurstAssets(const UObject* ContainingAsset, const FString& Context, class FDataValidationContext& ValidationContext) const;
	
	/** Input Device properties to apply */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = GameplayCueNotify)
	TArray<TSubclassOf<UInputDeviceProperty>> DeviceProperties;
};

/**
 * FGameplayCueNotify_DecalInfo
 *
 *	Properties that specify how to spawn a decal.
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_DecalInfo final
{
	GENERATED_BODY()

	GAMEPLAYABILITIES_API FGameplayCueNotify_DecalInfo();

	GAMEPLAYABILITIES_API bool SpawnDecal(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& OutSpawnResult) 
	const;

	// Condition to check before spawning the decal.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverrideSpawnCondition"))
	FGameplayCueNotify_SpawnCondition SpawnConditionOverride;

	// Defines how the decal will be placed.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (EditCondition = "bOverridePlacementInfo"))
	FGameplayCueNotify_PlacementInfo PlacementInfoOverride;

	// Decal material to spawn.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	TObjectPtr<UMaterialInterface> DecalMaterial;

	// Decal size in local space (does not include the component scale).
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, meta = (AllowPreserveRatio = "true"))
	FVector DecalSize;

	// If enabled, use the spawn condition override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverrideSpawnCondition : 1;

	// If enabled, use the placement info override and not the default one.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, Meta = (InlineEditConditionToggle))
	uint32 bOverridePlacementInfo : 1;

	// If enabled, override default decal actor fade out values.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	uint32 bOverrideFadeOut : 1;

	// Sets when the decal actor will start fading out.  Will override setting in class.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, meta = (EditCondition = "bOverrideFadeOut"))
	float FadeOutStartDelay;

	// Sets how long it takes for decal actor to fade out.  Will override setting in class.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify, meta = (EditCondition = "bOverrideFadeOut"))
	float FadeOutDuration;
};


/**
 * FGameplayCueNotify_BurstEffects
 *
 *	Set of effects to spawn for a single event, used by all gameplay cue notify types.
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_BurstEffects final
{
	GENERATED_BODY()

	GAMEPLAYABILITIES_API FGameplayCueNotify_BurstEffects();

	GAMEPLAYABILITIES_API void ExecuteEffects(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& OutSpawnResult) const;

	GAMEPLAYABILITIES_API void ValidateAssociatedAssets(const UObject* ContainingAsset, const FString& Context, class FDataValidationContext& ValidationContext) const;
	
protected:

	// Particle systems to be spawned on gameplay cue execution.  These should never use looping effects!
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	TArray<FGameplayCueNotify_ParticleInfo> BurstParticles;

	// Sound to be played on gameplay cue execution.  These should never use looping effects!
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	TArray<FGameplayCueNotify_SoundInfo> BurstSounds;

	// Camera shake to be played on gameplay cue execution.  This should never use a looping effect!
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	FGameplayCueNotify_CameraShakeInfo BurstCameraShake;

	// Camera lens effect to be played on gameplay cue execution.  This should never use a looping effect!
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	FGameplayCueNotify_CameraLensEffectInfo BurstCameraLensEffect;

	// Force feedback to be played on gameplay cue execution.  This should never use a looping effect!
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	FGameplayCueNotify_ForceFeedbackInfo BurstForceFeedback;

	// Input device properties to be applied on gameplay cue execution
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = GameplayCueNotify)
	FGameplayCueNotify_InputDevicePropertyInfo BurstDevicePropertyEffect;

	// Decal to be spawned on gameplay cue execution.  Actor should have fade out time or override should be set so it will clean up properly.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	FGameplayCueNotify_DecalInfo BurstDecal;
};


/**
 * FGameplayCueNotify_LoopingEffects
 *
 *	Set of looping effects to spawn for looping gameplay cues.
 */
USTRUCT(BlueprintType)
struct FGameplayCueNotify_LoopingEffects final
{
	GENERATED_BODY()

	GAMEPLAYABILITIES_API FGameplayCueNotify_LoopingEffects();

	GAMEPLAYABILITIES_API void StartEffects(const FGameplayCueNotify_SpawnContext& SpawnContext, FGameplayCueNotify_SpawnResult& 
	OutSpawnResult) const;
	GAMEPLAYABILITIES_API void StopEffects(FGameplayCueNotify_SpawnResult& SpawnResult) const;
	
	GAMEPLAYABILITIES_API void ValidateAssociatedAssets(const UObject* ContainingAsset, const FString& Context, class FDataValidationContext& ValidationContext) const;

protected:

	// Particle systems to be spawned on gameplay cue loop start.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	TArray<FGameplayCueNotify_ParticleInfo> LoopingParticles;

	// Sound to be played on gameplay cue loop start.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	TArray<FGameplayCueNotify_SoundInfo> LoopingSounds;

	// Camera shake to be played on gameplay cue loop start.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	FGameplayCueNotify_CameraShakeInfo LoopingCameraShake;

	// Camera lens effect to be played on gameplay cue loop start.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	FGameplayCueNotify_CameraLensEffectInfo LoopingCameraLensEffect;

	// Force feedback to be played on gameplay cue loop start.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayCueNotify)
	FGameplayCueNotify_ForceFeedbackInfo LoopingForceFeedback;

	// Input device properties to be applied on gameplay cue loop start.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = GameplayCueNotify)
	FGameplayCueNotify_InputDevicePropertyInfo LoopingInputDevicePropertyEffect;
};

#undef UE_API
