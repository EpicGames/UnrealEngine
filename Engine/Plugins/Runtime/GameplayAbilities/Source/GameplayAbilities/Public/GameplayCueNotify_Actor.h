// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "GameplayTagContainer.h"
#include "GameplayCue_Types.h"
#include "GameplayCueNotify_Actor.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/**
 *	An instantiated Actor that acts as a handler of a GameplayCue. Since they are instantiated, they can maintain state and tick/update every frame if necessary. 
 */

UCLASS(Blueprintable, meta = (ShowWorldContextPin), hidecategories = (Replication), MinimalAPI)
class AGameplayCueNotify_Actor : public AActor
{
	GENERATED_UCLASS_BODY()

	/** Does this GameplayCueNotify handle this type of GameplayCueEvent? */
	UE_API virtual bool HandlesEvent(EGameplayCueEvent::Type EventType) const;

	UFUNCTION()
	UE_API virtual void OnOwnerDestroyed(AActor* DestroyedActor);

	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

protected:
	UE_API virtual void BeginPlay() override;

	UE_API virtual void K2_DestroyActor() override;
	UE_API virtual void Destroyed() override;

#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif

public:
	UE_API virtual void SetOwner( AActor* NewOwner ) override;

	UE_API virtual void PostInitProperties() override;

	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API virtual void HandleGameplayCue(AActor* MyTarget, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);

	/** Called when the GC is finished. It may be about to go back to the recyle pool, or it may be about to be destroyed. */
	UE_API virtual void GameplayCueFinishedCallback();

	UE_API virtual bool GameplayCuePendingRemove();

	/** Called when returning to the recycled pool. Reset all state so that it can be reused. Return false if this class cannot be recycled. */
	UE_API virtual bool Recycle();

	/** Called when we are about to reuse the GC. Should undo anything done in Recycle like hiding the actor */
	UE_API virtual void ReuseAfterRecycle();

	/** Ends the gameplay cue: either destroying it or recycling it. You must call this manually only if you do not use bAutoDestroyOnRemove/AutoDestroyDelay  */
	UFUNCTION(BlueprintCallable, Category="GameplayCueNotify", DisplayName="End (Recycle) GameplayCue", meta=(ScriptName="EndGameplayCue"))
	UE_API virtual void K2_EndGameplayCue();

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	/** We will auto destroy (recycle) this GameplayCueActor when the OnRemove event fires (after OnRemove is called). */
	UPROPERTY(EditDefaultsOnly, Category = Cleanup)
	bool bAutoDestroyOnRemove;

	/** If bAutoDestroyOnRemove is true, the actor will stay alive for this many seconds before being auto destroyed. */
	UPROPERTY(EditAnywhere, Category = Cleanup)
	float AutoDestroyDelay;

	/** Warn if we have a timeline running when we cleanup this gameplay cue (we will kill the timeline either way)  */
	UPROPERTY(EditAnywhere, Category = Cleanup)
	bool WarnIfTimelineIsStillRunning;

	/** Warn if we have a latent action (delay, etc) running when we cleanup this gameplay cue (we will kill the latent action either way)  */
	UPROPERTY(EditAnywhere, Category = Cleanup)
	bool WarnIfLatentActionIsStillRunning;

	/** Generic Event Graph event that will get called for every event type */
	UFUNCTION(BlueprintImplementableEvent, Category = "GameplayCueNotify", DisplayName = "HandleGameplayCue", meta=(ScriptName = "HandleGameplayCue"))
	UE_API void K2_HandleGameplayCue(AActor* MyTarget, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);

	/** Called when a GameplayCue is executed, this is used for instant effects or periodic ticks */
	UFUNCTION(BlueprintNativeEvent, Category = "GameplayCueNotify")
	UE_API bool OnExecute(AActor* MyTarget, const FGameplayCueParameters& Parameters);

	/** Called when a GameplayCue with duration is first activated, this will only be called if the client witnessed the activation */
	UFUNCTION(BlueprintNativeEvent, Category = "GameplayCueNotify", DisplayName="On Burst", meta=(ScriptName="OnBurst;OnActive"))
	UE_API bool OnActive(AActor* MyTarget, const FGameplayCueParameters& Parameters);

	/** Called when a GameplayCue with duration is first seen as active, even if it wasn't actually just applied (Join in progress, etc) */
	UFUNCTION(BlueprintNativeEvent, Category = "GameplayCueNotify", DisplayName="On Become Relevant", meta=(ScriptName="OnBecomeRelevant;WhileActive"))
	UE_API bool WhileActive(AActor* MyTarget, const FGameplayCueParameters& Parameters);

	/** Called when a GameplayCue with duration is removed */
	UFUNCTION(BlueprintNativeEvent, Category = "GameplayCueNotify", DisplayName="On Cease Relevant", meta=(ScriptName="OnCeaseRelevant;OnRemove"))
	UE_API bool OnRemove(AActor* MyTarget, const FGameplayCueParameters& Parameters);

	/** Tag this notify is activated by */
	UPROPERTY(EditDefaultsOnly, Category=GameplayCue, meta=(Categories="GameplayCue"))
	FGameplayTag	GameplayCueTag;

	/** Mirrors GameplayCueTag in order to be asset registry searchable */
	UPROPERTY(AssetRegistrySearchable)
	FName GameplayCueName;

	/** If true, attach this GameplayCue Actor to the target actor while it is active. Attaching is slightly more expensive than not attaching, so only enable when you need to. */
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue)
	bool bAutoAttachToOwner;

	/** Does this Cue override other cues, or is it called in addition to them? E.g., If this is Damage.Physical.Slash, we wont call Damage.Physical afer we run this cue. */
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue)
	bool IsOverride;

	/**
	 *	Does this cue get a new instance for each instigator? For example if two instigators apply a GC to the same source, do we create two of these GameplayCue Notify actors or just one?
	 *	If the notify is simply playing FX or sounds on the source, it should not need unique instances. If this Notify is attaching a beam from the instigator to the target, it does need a unique instance per instigator.
	 */ 	 
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue)
	bool bUniqueInstancePerInstigator;

	/**
	 *	Does this cue get a new instance for each source object? For example if two source objects apply a GC to the same source, do we create two of these GameplayCue Notify actors or just one?
	 *	If the notify is simply playing FX or sounds on the source, it should not need unique instances. If this Notify is attaching a beam from the source object to the target, it does need a unique instance per instigator.
	 */ 	 
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue)
	bool bUniqueInstancePerSourceObject;

	/**
	 *	Does this cue trigger its On Burst event if it's already been triggered?
	 *  This can occur when the associated tag is triggered by multiple sources and there is no unique instancing.
	 */ 	 
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue, DisplayName="Allow Multiple On Burst Events")
	bool bAllowMultipleOnActiveEvents;

	/**
	 *	Does this cue trigger its On Become Relevant event if it's already been triggered?
	 *  This can occur when the associated tag is triggered by multiple sources and there is no unique instancing.
	 */
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue, DisplayName="Allow Multiple On Become Relevant Events")
	bool bAllowMultipleWhileActiveEvents;

	/** How many instances of the gameplay cue to preallocate */
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue)
	int32 NumPreallocatedInstances;

	// Set when the GC actor is in the recycle queue (E.g., not active in world. This is to prevent rentrancy in the recyle code since multiple paths can lead the GC actor there)
	bool bInRecycleQueue;

	// Keep track of the Instigator so we can decide if this GameplayCueNotify is one we're searching for.
	TWeakObjectPtr<AActor> CueInstigator;

	// Keep track of the SourceObject so we can decide if this GameplayCueNotify is one we're searching for.
	TWeakObjectPtr<const UObject> CueSourceObject;
	
protected:
	FTimerHandle FinishTimerHandle;

	UE_API void ClearOwnerDestroyedDelegate();

	bool bHasHandledOnActiveEvent;
	bool bHasHandledWhileActiveEvent;
	bool bHasHandledOnRemoveEvent;

private:
	UE_API virtual void DeriveGameplayCueTagFromAssetName();

	void AttachToOwnerIfNecessary();

};

#undef UE_API
