// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayCueNotify_Static.h"
#include "GameplayCueNotifyTypes.h"
#include "GameplayCueNotify_Burst.generated.h"

#define UE_API GAMEPLAYABILITIES_API


/**
 * UGameplayCueNotify_Burst
 *
 *	This is a non-instanced gameplay cue notify for effects that are one-offs.
 *	Since it is not instanced, it cannot do latent actions such as delays and time lines.
 */
UCLASS(Blueprintable, Category = "GameplayCueNotify", Meta = (ShowWorldContextPin, DisplayName = "GCN Burst", ShortTooltip = "A one-off GameplayCueNotify that is never spawned into the world."), MinimalAPI)
class UGameplayCueNotify_Burst : public UGameplayCueNotify_Static
{
	GENERATED_BODY()

public:

	UE_API UGameplayCueNotify_Burst();

protected:

	UE_API virtual bool OnExecute_Implementation(AActor* Target, const FGameplayCueParameters& Parameters) const override;

	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnBurst(AActor* Target, const FGameplayCueParameters& Parameters, const FGameplayCueNotify_SpawnResult& SpawnResults) const;

#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // #if WITH_EDITOR

protected:

	// Default condition to check before spawning anything.  Applies for all spawns unless overridden.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GCN Defaults")
	FGameplayCueNotify_SpawnCondition DefaultSpawnCondition;

	// Default placement rules.  Applies for all spawns unless overridden.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GCN Defaults")
	FGameplayCueNotify_PlacementInfo DefaultPlacementInfo;

	// List of effects to spawn on burst.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GCN Effects")
	FGameplayCueNotify_BurstEffects BurstEffects;
};

#undef UE_API
