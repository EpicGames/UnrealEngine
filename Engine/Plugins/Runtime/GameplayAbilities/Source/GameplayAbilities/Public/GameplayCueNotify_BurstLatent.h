// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayCueNotify_Actor.h"
#include "GameplayCueNotifyTypes.h"
#include "GameplayCueNotify_BurstLatent.generated.h"

#define UE_API GAMEPLAYABILITIES_API


/**
 * AGameplayCueNotify_BurstLatent
 *
 *	This is an instanced gameplay cue notify for effects that are one-offs.
 *	Since it is instanced, it can do latent things like time lines or delays.
 */
UCLASS(Blueprintable, notplaceable, Category = "GameplayCueNotify", Meta = (ShowWorldContextPin, DisplayName = "GCN Burst Latent", ShortTooltip = "A one-off GameplayCueNotify that can use latent actions such as timelines."), MinimalAPI)
class AGameplayCueNotify_BurstLatent : public AGameplayCueNotify_Actor
{
	GENERATED_BODY()

public:

	UE_API AGameplayCueNotify_BurstLatent();

protected:

	UE_API virtual bool Recycle() override;

	UE_API virtual bool OnExecute_Implementation(AActor* Target, const FGameplayCueParameters& Parameters) override;

	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnBurst(AActor* Target, const FGameplayCueParameters& Parameters, const FGameplayCueNotify_SpawnResult& SpawnResults);

#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // #if WITH_EDITOR

protected:

	// Default condition to check before spawning anything.  Applies for all spawns unless overridden.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GCN Defaults")
	FGameplayCueNotify_SpawnCondition DefaultSpawnCondition;

	// Default placement rules.  Applies for all spawns unless overridden.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GCN Defaults")
	FGameplayCueNotify_PlacementInfo DefaultPlacementInfo;

	// List of effects to spawn on burst.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GCN Effects")
	FGameplayCueNotify_BurstEffects BurstEffects;

	// Results of spawned burst effects.
	UPROPERTY(BlueprintReadOnly, Category = "GCN Effects")
	FGameplayCueNotify_SpawnResult BurstSpawnResults;
};

#undef UE_API
