// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayCueNotify_Actor.h"
#include "GameplayCueNotifyTypes.h"
#include "GameplayCueNotify_Looping.generated.h"

#define UE_API GAMEPLAYABILITIES_API


/**
 * AGameplayCueNotify_Looping
 *
 *	This is an instanced gameplay cue notify for continuous looping effects.
 *	The game is responsible for defining the start/stop by adding/removing the gameplay cue.
 */
UCLASS(Blueprintable, notplaceable, Category = "GameplayCueNotify", Meta = (ShowWorldContextPin, DisplayName = "GCN Looping", ShortTooltip = "A GameplayCueNotify that has a duration that is driven by the game."), MinimalAPI)
class AGameplayCueNotify_Looping : public AGameplayCueNotify_Actor
{
	GENERATED_BODY()

public:

	UE_API AGameplayCueNotify_Looping();

protected:

	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual bool Recycle() override;

	UE_API virtual bool OnActive_Implementation(AActor* Target, const FGameplayCueParameters& Parameters) override;
	UE_API virtual bool WhileActive_Implementation(AActor* Target, const FGameplayCueParameters& Parameters) override;
	UE_API virtual bool OnExecute_Implementation(AActor* Target, const FGameplayCueParameters& Parameters) override;
	UE_API virtual bool OnRemove_Implementation(AActor* Target, const FGameplayCueParameters& Parameters) override;

	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnApplication(AActor* Target, const FGameplayCueParameters& Parameters, const FGameplayCueNotify_SpawnResult& SpawnResults);

	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnLoopingStart(AActor* Target, const FGameplayCueParameters& Parameters, const FGameplayCueNotify_SpawnResult& SpawnResults);

	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnRecurring(AActor* Target, const FGameplayCueParameters& Parameters, const FGameplayCueNotify_SpawnResult& SpawnResults);

	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnRemoval(AActor* Target, const FGameplayCueParameters& Parameters, const FGameplayCueNotify_SpawnResult& SpawnResults);

	UE_API void RemoveLoopingEffects();

#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // #if WITH_EDITOR

protected:

	// Default condition to check before spawning anything.  Applies for all spawns unless overridden.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GCN Defaults")
	FGameplayCueNotify_SpawnCondition DefaultSpawnCondition;

	// Default placement rules.  Applies for all spawns unless overridden.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GCN Defaults")
	FGameplayCueNotify_PlacementInfo DefaultPlacementInfo;

	// List of effects to spawn on application.  These should not be looping effects!
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GCN Application Effects (On Active)")
	FGameplayCueNotify_BurstEffects ApplicationEffects;

	// Results of spawned application effects.
	UPROPERTY(BlueprintReadOnly, Category = "GCN Application Effects (On Active)")
	FGameplayCueNotify_SpawnResult ApplicationSpawnResults;

	// List of effects to spawn on loop start.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GCN Looping Effects (While Active)")
	FGameplayCueNotify_LoopingEffects LoopingEffects;

	// Results of spawned looping effects.
	UPROPERTY(BlueprintReadOnly, Category = "GCN Looping Effects (While Active)")
	FGameplayCueNotify_SpawnResult LoopingSpawnResults;

	// List of effects to spawn for a recurring gameplay effect (e.g. each time a DOT ticks).  These should not be looping effects!
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GCN Recurring Effects (On Execute)")
	FGameplayCueNotify_BurstEffects RecurringEffects;

	// Results of spawned recurring effects.
	UPROPERTY(BlueprintReadOnly, Category = "GCN Recurring Effects (On Execute)")
	FGameplayCueNotify_SpawnResult RecurringSpawnResults;

	// List of effects to spawn on removal.  These should not be looping effects!
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GCN Removal Effects (On Remove)")
	FGameplayCueNotify_BurstEffects RemovalEffects;

	// Results of spawned removal effects.
	UPROPERTY(BlueprintReadOnly, Category = "GCN Removal Effects (On Remove)")
	FGameplayCueNotify_SpawnResult RemovalSpawnResults;

	bool bLoopingEffectsRemoved;
};

#undef UE_API
