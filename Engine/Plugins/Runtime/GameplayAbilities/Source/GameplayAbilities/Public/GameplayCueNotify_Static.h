// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "GameplayTagContainer.h"
#include "GameplayEffectTypes.h"
#include "GameplayCueNotify_Static.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/**
 *	A non instantiated UObject that acts as a handler for a GameplayCue. These are useful for one-off "burst" effects.
 */

UCLASS(Blueprintable, meta = (ShowWorldContextPin), hidecategories = (Replication), MinimalAPI)
class UGameplayCueNotify_Static : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Does this GameplayCueNotify handle this type of GameplayCueEvent? */
	UE_API virtual bool HandlesEvent(EGameplayCueEvent::Type EventType) const;

	UE_API virtual void OnOwnerDestroyed();

	UE_API virtual void PostInitProperties() override;

	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API virtual void HandleGameplayCue(AActor* MyTarget, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);

	UE_API UWorld* GetWorld() const override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	/** Generic Event Graph event that will get called for every event type */
	UFUNCTION(BlueprintImplementableEvent, Category = "GameplayCueNotify", DisplayName = "HandleGameplayCue", meta=(ScriptName = "HandleGameplayCue"))
	UE_API void K2_HandleGameplayCue(AActor* MyTarget, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters) const;

	/** Called when a GameplayCue is executed, this is used for instant effects or periodic ticks */
	UFUNCTION(BlueprintNativeEvent, BlueprintPure, Category = "GameplayCueNotify")
	UE_API bool OnExecute(AActor* MyTarget, const FGameplayCueParameters& Parameters) const;

	/** Called when a GameplayCue with duration is first activated, this will only be called if the client witnessed the activation */
	UFUNCTION(BlueprintNativeEvent, BlueprintPure, Category = "GameplayCueNotify")
	UE_API bool OnActive(AActor* MyTarget, const FGameplayCueParameters& Parameters) const;

	/** Called when a GameplayCue with duration is first seen as active, even if it wasn't actually just applied (Join in progress, etc) */
	UFUNCTION(BlueprintNativeEvent, BlueprintPure, Category = "GameplayCueNotify")
	UE_API bool WhileActive(AActor* MyTarget, const FGameplayCueParameters& Parameters) const;

	/** Called when a GameplayCue with duration is removed */
	UFUNCTION(BlueprintNativeEvent, BlueprintPure, Category = "GameplayCueNotify")
	UE_API bool OnRemove(AActor* MyTarget, const FGameplayCueParameters& Parameters) const;

	/** Tag this notify is activated by */
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue, meta=(Categories="GameplayCue"))
	FGameplayTag	GameplayCueTag;

	/** Mirrors GameplayCueTag in order to be asset registry searchable */
	UPROPERTY(AssetRegistrySearchable)
	FName GameplayCueName;

	/** Does this Cue override other cues, or is it called in addition to them? E.g., If this is Damage.Physical.Slash, we wont call Damage.Physical afer we run this cue. */
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue)
	bool IsOverride;

private:
	UE_API virtual void DeriveGameplayCueTagFromAssetName();
};

#undef UE_API
