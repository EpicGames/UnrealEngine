// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "GameplayTagContainer.h"
#include "Engine/DataAsset.h"
#include "GameplayEffectTypes.h"
#include "GameplayCueSet.generated.h"

#define UE_API GAMEPLAYABILITIES_API

USTRUCT()
struct FGameplayCueNotifyData
{
	GENERATED_USTRUCT_BODY()

	FGameplayCueNotifyData()
	: LoadedGameplayCueClass(nullptr)
	, ParentDataIdx( INDEX_NONE )
	{
	}

	UPROPERTY(EditAnywhere, Category=GameplayCue)
	FGameplayTag GameplayCueTag;

	UPROPERTY(EditAnywhere, Category=GameplayCue, meta=(AllowedClasses="/Script/GameplayAbilities.GameplayCueNotify_Static, /Script/GameplayAbilities.GameplayCueNotify_Actor"))
	FSoftObjectPath GameplayCueNotifyObj;

	UPROPERTY(transient)
	TObjectPtr<UClass> LoadedGameplayCueClass;

	int32 ParentDataIdx;
};

struct FGameplayCueReferencePair
{
	FGameplayTag GameplayCueTag;
	FSoftObjectPath StringRef;

	FGameplayCueReferencePair(const FGameplayTag& InGameplayCueTag, const FSoftObjectPath& InStringRef)
		: GameplayCueTag(InGameplayCueTag)
		, StringRef(InStringRef)
	{}
};

/**
 *	A set of gameplay cue actors to handle gameplay cue events
 */
UCLASS(MinimalAPI)
class UGameplayCueSet : public UDataAsset
{
	GENERATED_UCLASS_BODY()

	/** Handles the cue event by spawning the cue actor. Returns true if the event was handled. */
	UE_API virtual bool HandleGameplayCue(AActor* TargetActor, FGameplayTag GameplayCueTag, EGameplayCueEvent::Type EventType, const FGameplayCueParameters& Parameters);

	/** Adds a list of cues to the set */
	UE_API virtual void AddCues(const TArray<FGameplayCueReferencePair>& CuesToAdd);

	/** Removes all cues from the set matching any of the supplied tags */
	UE_API virtual void RemoveCuesByTags(const FGameplayTagContainer& TagsToRemove);

	/** Removes all cues from the set matching the supplied string refs */
	UE_API virtual void RemoveCuesByStringRefs(const TArray<FSoftObjectPath>& CuesToRemove);

	/** Nulls reference to the loaded class. Note this doesn't remove the entire cue from the internal data structure, just the hard ref to the loaded class */
	UE_API virtual void RemoveLoadedClass(UClass* Class);

	/** Returns filenames of everything we know about (loaded or not) */
	UE_API virtual void GetFilenames(TArray<FString>& Filenames) const;

	/** Extracts all soft object paths pointing to Cues */
	UE_API virtual void GetSoftObjectPaths(TArray<FSoftObjectPath>& List) const;

#if WITH_EDITOR

	UE_API void CopyCueDataToSetForEditorPreview(FGameplayTag Tag, UGameplayCueSet* DestinationSet);

	/** Updates an existing cue */
	UE_API virtual void UpdateCueByStringRefs(const FSoftObjectPath& CueToRemove, FString NewPath);

#endif

	/** Removes all cues from the set */
	UE_API virtual void Empty();

	UE_API virtual void PrintCues() const;
	
	UPROPERTY(EditAnywhere, Category=CueSet)
	TArray<FGameplayCueNotifyData> GameplayCueData;

	/** Maps GameplayCue Tag to index into above GameplayCues array. */
	TMap<FGameplayTag, int32> GameplayCueDataMap;

	static UE_API FGameplayTag	BaseGameplayCueTag();

protected:
	UE_API virtual bool HandleGameplayCueNotify_Internal(AActor* TargetActor, int32 DataIdx, EGameplayCueEvent::Type EventType, FGameplayCueParameters& Parameters);
	UE_API virtual void BuildAccelerationMap_Internal();
};

#undef UE_API
