// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "GameplayTagContainer.h"
#include "GameplayCueTranslator.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class AActor;
class UGameplayCueTranslator;
class UGameplayTagsManager;
struct FGameplayCueParameters;

/**
 *	Overview of the GameplayCueTranslator system
 *
 *	This system facilitates translating a GameplayCue event from one tag to another at runtime. This can be useful for customization 
 *	or override type of systems that want to handle GameplayCues in different ways for different things or in different contexts.
 *
 *	Some example uses:
 *		1. Your game emits generic events: GameplayCue.Hero.Victory (an event to play a victory sound/animation). Depending on which Hero this is played on,
 *		you may want different sounds. This system can be used to translate the generic GameplayCue.Hero.Victory into GameplayCue.<YourHeroName>.Victory.
 *
 *		2. Your game wants to translate GameplayCue.Impact.Material into GameplayCue.Impact.<Stone/Wood/Water/Etc>, based on the physics material of the surface
 *		that was hit.
 *
 *	Though there are other ways of accomplishing these examples, the main advantages to a translator approach is that you can maintain single, atomic GC Notifies,
 *	rather than A) having monolithic GC Notifies/handlers that "know how to handle every possible variation" (and now have to deal with loading/unloading the ones that are needed)
 *	or B) Storing the override assets on a character blueprint/data asset that the GC Notify/handler would pull from (this hampers work flow because now you need to add override
 *	properties somewhere for each GameplayCue event that wants to use this. You can no longer have a simple GC Notify class that "just plays sounds and FX").
 *
 *	How to use:
 *
 *	C++:
 *	Implement your own UGameplayCueTranslator. See UGameplayCueTranslator_Test as an example. You essentially need to implement two functions:
 *	1. GetTranslationNameSpawns: return a list of possible tag translations (called once at startup to gather information)
 *	2. GameplayCueToTranslationIndex: returns the index into the list you returned in #1 for which translation to apply to a given tag/context.
 *
 *	Editor:
 *	Use GameplayCue editor to add new tags and notifies. The GCEditor has built in functionality to make this easier. It can auto create the tags for you and new GC Notify 
 *	assets that derive off of the base tags/notifies.
 *
 *
 *	Useful commands:
 *	Log LogGameplayCueTranslator Verbose				[enable logging of tag translation]
 *	
 *	GameplayCue.PrintGameplayCueTranslator				[prints translationLUT]
 *	GameplayCue.BuildGameplayCueTranslator				[rebuilds translationLUT, useful for debugging]
 */

// -----------------------------------------------------------------------------
//	Editor only. Data that is only used by the GameplayCue editor tool
// -----------------------------------------------------------------------------
#if WITH_EDITOR
struct FGameplayCueTranslationEditorOnlyData
{
	FGameplayCueTranslationEditorOnlyData()
		:UniqueID(0), Enabled(true)
	{
	}

	FName	EditorDescription;	// For pretty/custom printing
	FString ToolTip;			// additional info for tooltip text (exactly where did this rule come from?)
	int32	UniqueID;			// For filtering overrides by translation rule. Set by the GameplaycueTranslator.
	bool	Enabled;			// Is this actually enabled, or not (and if not, we may still want to draw it in the editor but greyed out). In non editor builds, the translator rule class should "just not return the rule" if its disanled.
};

struct FGameplayCueTranslationEditorInfo
{
	FGameplayTag GameplayTag;	// Will only exist if there is an existing FGameplayTag
	FName	GameplayTagName;	// Will always exist, even if tag hasn't been created
	
	FGameplayCueTranslationEditorOnlyData EditorData;
};
#endif

// -----------------------------------------------------------------------------
//	Run Time
// -----------------------------------------------------------------------------

class UGameplayCueTranslator;

/** Basis for name swaps. This swaps FromName to ToName */
struct FGameplayCueTranslationNameSwap
{
	FName	FromName;
	TArray<FName, TInlineAllocator<4> >	ToNames;

#if WITH_EDITOR
	FGameplayCueTranslationEditorOnlyData EditorData;
#endif
};

/** Simple index/handle for referencing items in FGameplayCueTranslationManager::TranslationLUT  */
USTRUCT()
struct FGameplayCueTranslatorNodeIndex
{
	GENERATED_USTRUCT_BODY()

	FGameplayCueTranslatorNodeIndex() : Index(INDEX_NONE) { }

	FGameplayCueTranslatorNodeIndex(FGameplayTagNetIndex InIndex) : Index(InIndex) { }

	UPROPERTY()
	int32 Index;

	bool IsValid() const { return Index != INDEX_NONE; }

	inline operator int32() const
	{
		return Index;
	}

	inline bool operator==(const FGameplayCueTranslatorNodeIndex& Other) const
	{
		return Other.Index == Index;
	}
};

/** Represents a translation from one FGameplayCueTranslatorNode to many others. You will have one of these links per UGameplayCueTranslator that can translate a node. */
USTRUCT()
struct FGameplayCueTranslationLink
{
	GENERATED_USTRUCT_BODY()

	/** The rule that provides this translation */
	UPROPERTY()
	TObjectPtr<const UGameplayCueTranslator> RulesCDO = nullptr;

	/** Fixed size lookup. The RulesCDO return the index into this which will translate to the new node. */
	TArray< FGameplayCueTranslatorNodeIndex > NodeLookup;
};

/** A node in our translation table/graph. The node represents an actual gameplaytag or a possible gameplay tag, with links to what it can be translated into. */
USTRUCT()
struct FGameplayCueTranslatorNode
{
	GENERATED_USTRUCT_BODY()

	/** Ways we can be translated into another FGameplayCueTranslatorNode */
	UPROPERTY()
	TArray<FGameplayCueTranslationLink>	Links;

	/** Our index into FGameplayCueTranslationManager::TranslationLUT  */
	UPROPERTY()
	FGameplayCueTranslatorNodeIndex CachedIndex;

	/** The FGameplayTag if this tag actually exists. This will always exist at runtime. In editor builds, it may not (GameplayCueEditor) */
	UPROPERTY()
	FGameplayTag CachedGameplayTag;
	
	/** FName of the tag. This will always be valid, whether the tag is in the GameplayTag dictionary or not. */
	UPROPERTY()
	FName CachedGameplayTagName;

	/** Translation that have been used to "get us here" and should not be used further down the chain. To avoid recursion */
	TSet<const UGameplayCueTranslator*>	UsedTranslators;

	/** Helper to create a new link for a given UGameplayCueTranslator  */
	FGameplayCueTranslationLink& FindOrCreateLink(const UGameplayCueTranslator* RuleClass, int32 LookupSize);
};

struct FNameSwapData
{
	// Class that provided the rules
	const UGameplayCueTranslator* ClassCDO;

	// What it gave us
	TArray<FGameplayCueTranslationNameSwap>	NameSwaps;
};

// -----------------------------------------------------------------------------


/** This is the struct that does the actual translation. It lives on the GameplayCueManager and encapsulates all translation logic. */
USTRUCT()
struct FGameplayCueTranslationManager
{
	GENERATED_USTRUCT_BODY()

	/** This is *the* runtime function that translates the tag (if necessary) */
	UE_API void TranslateTag(FGameplayTag& Tag, AActor* TargetActor, const FGameplayCueParameters& Parameters);

	/** Builds all our translation tables. This works backwards by looking at the tags in the dictionary and determining which could be derived from translation rules. This is pretty fast. */
	UE_API void BuildTagTranslationTable();
	UE_API bool BuildTagTranslationTable_r(const FName& TagName, const TArray<FName>& SplitNames);

	/** 
	 *  Builds all *possible* translation tables. This works forwards by looking at existing tags and determining what translated tags could be dervied from them. 
	 *	This WILL populate the TranslationLUT with all possible tags, not just ones that exist in the dictionar. This is not as fast and is onyl called in the editor. 
	 */
	UE_API void BuildTagTranslationTable_Forward();
	UE_API void BuildTagTranslationTable_Forward_r(const FName& TagName, const TArray<FName>& SplitNames);
		
	/** refresh our name swap rules. Called internally by the manager and externally by GC tool */
	UE_API void RefreshNameSwaps();

	UE_API void PrintTranslationTable();
	UE_API void PrintTranslationTable_r(FGameplayCueTranslatorNode& Node, FString IdentStr=FString());

	
#if WITH_EDITOR
	/** Used by the GC editor to enumerate possible translation tags. Never called at runtime. */
	UE_API bool GetTranslatedTags(const FName& ParentTag, TArray<FGameplayCueTranslationEditorInfo>& Children);
	const TArray<FNameSwapData>& GetNameSwapData() const { return AllNameSwaps; }

	/** Searches, slowly, to see if the passed in tag can be translated from something else. Don't call this at runtime. */
	UE_API FGameplayTag SearchSlowForTranslationParent(FGameplayTag Tag);
#endif

private:

	/** The Look Up Table */
	UPROPERTY()
	TArray< FGameplayCueTranslatorNode > TranslationLUT;

	/** Acceleration map from gameplay tag name to index into TranslationLUT  */
	UPROPERTY()
	TMap<FName, FGameplayCueTranslatorNodeIndex> TranslationNameToIndexMap;

	/** Cached reference to tag manager */
	UPROPERTY()
	TObjectPtr<UGameplayTagsManager> TagManager = nullptr;
		
	/** All name swpa rules we have gathered */
	TArray<FNameSwapData> AllNameSwaps;

	UE_API bool TranslateTag_Internal(FGameplayCueTranslatorNode& Node, FGameplayTag& OutTag, const FName& TagName, AActor* TargetActor, const FGameplayCueParameters& Parameters);

	UE_API FGameplayCueTranslatorNodeIndex GetTranslationIndexForTag(const FGameplayTag& Tag, bool CreateIfInvalid=false);

	UE_API FGameplayCueTranslatorNode* GetTranslationNodeForTag(const FGameplayTag& Tag, bool CreateIfInvalid=false);
	UE_API FGameplayCueTranslatorNode* GetTranslationNodeForName(FName Name, bool CreateIfInvalid=false);

	UE_API FGameplayCueTranslatorNodeIndex GetTranslationIndexForName(FName Name, bool CreateIfInvalid=false);

	UE_API void ResetTranslationLUT();

	// Only used in debug printing/stats
	int32 TotalNumTranslations;
	int32 TotalNumTheoreticalTranslations;
};

/**
 *	This is the base class for GameplayCue Translators. This is what games must extend from in order to add their own rules.
 *	These are not instantiated, and are basically just a holder for virtual functions that are called on the CDO.
 *
 *	There are two main things this class provides:
 *		1. A set of translation rules. E.g., "I translate GameplayCue.A.B.C into GameplayCue.X.B.C", or rather "I translate A into X". (GetTranslationNameSpawns)
 *		2. A runtime function to actually do the translation, based on the actors and parameters involved in the gameplay cue event. (GameplayCueToTranslationIndex)
 *
 *
 */

UCLASS(Abstract, MinimalAPI)
class UGameplayCueTranslator: public UObject
{
	GENERATED_BODY()

public:

	// return (via out param) list of tag swaps you will do. This should be deterministic/order matters for later!
	virtual void GetTranslationNameSpawns(TArray<FGameplayCueTranslationNameSwap>& SwapList) const { }

	// runtime function to mapping Tag/Actor/Parameters to a translation index. The returned index here maps to the array was modified in ::GetTranslationNameSpawns
	virtual int32 GameplayCueToTranslationIndex(const FName& TagName, AActor* TargetActor, const FGameplayCueParameters& Parameters) const { return INDEX_NONE; }

	// Sorting priority. Higher number = first chance to translate a tag
	virtual int32 GetPriority() const { return 0; }

	// Whether this translator class should be enabled. Useful for disabling WIP translators
	virtual bool IsEnabled() const { return true; }

	// Whether this translator should be shown in the top level view of the filter tree in the gameplaycue editor. If false, we will only add this as children of top level translators
	virtual bool ShouldShowInTopLevelFilterList() const{ return true; }
};

/**
 *	This is an example translator class.
 */
UCLASS()
class UGameplayCueTranslator_Test : public UGameplayCueTranslator
{
	GENERATED_BODY()

public:

	/**
	 *	This adds the name swaps. We create 3 rules, Hero->Steel, Hero->Rampage, Hero->Kurohane.
	 *	All this says is, "this <UGameplayCueTranslator_Test> can translate tags like this".
	 */
	virtual void GetTranslationNameSpawns(TArray<FGameplayCueTranslationNameSwap>& SwapList) const override
	{
		{
			FGameplayCueTranslationNameSwap Temp;
			Temp.FromName = FName(TEXT("Hero"));
			Temp.ToNames.Add( FName(TEXT("Steel")) );
			SwapList.Add(Temp);
		}
		{
			FGameplayCueTranslationNameSwap Temp;
			Temp.FromName = FName(TEXT("Hero"));
			Temp.ToNames.Add( FName(TEXT("Rampage")) );
			SwapList.Add(Temp);
		}
		{
			FGameplayCueTranslationNameSwap Temp;
			Temp.FromName = FName(TEXT("Hero"));
			Temp.ToNames.Add( FName(TEXT("Kurohane")) );
			SwapList.Add(Temp);
		}
	}

	/**
	 *	This is called at runtime to actually do the swapping. This is a trivial example, we will use hardcoded memory addressess to switch on the 3 possible translations.
	 *  A more realistic approach would be to look at something on the AActor (maybe, cast it to your base actor or interface type) or the GameplayCue parameters.
	 *  Using an acceleration map may be a good idea here: your translator class could have access to a global map that maps actor* -> swap index.
	 *
	 *	Important thing to understand: the int32 returned is the index into the SwapList that we created in ::GetTranslationNameSpawns()!
	 */
	virtual int32 GameplayCueToTranslationIndex(const FName& TagName, AActor* TargetActor, const FGameplayCueParameters& Parameters) const
	{
		// Memory comparison is a cheesy example. Could be a class cast, or a look up into a aactor*->int32 map for speed.

		if (TargetActor == (AActor*)0xAAAA)
		{
			return 0;
		}
		if (TargetActor == (AActor*)0xBBBB)
		{
			return 1;
		}
		if (TargetActor == (AActor*)0xCCCC)
		{
			return 2;
		}

		return INDEX_NONE;
	}

	// We never actually want to use this in production, so return false
	virtual bool IsEnabled() const override { return false; }

	/*
	Example usage to test translation:

	FGameplayTag Original = TagManager->RequestGameplayTag(FName(TEXT("GameplayCue.Announcer.Dialog.Hero.CoreDamage"), false));
	FGameplayCueParameters Parameters;

	{
		FGameplayTag NewTag = Original;
		TranslateTag(NewTag, (AActor*)0xAAAA, Parameters);
		UE_LOGF(LogGameplayCueTranslator, Display, "\nTranslated Tag: %ls", *NewTag.ToString());
	}
	{
		FGameplayTag NewTag = Original;
		TranslateTag(NewTag, (AActor*)0xBBBB, Parameters);
		UE_LOGF(LogGameplayCueTranslator, Display, "\nTranslated Tag: %ls", *NewTag.ToString());
	}
	{
		FGameplayTag NewTag = Original;
		TranslateTag(NewTag, (AActor*)0xCCCC, Parameters);
		UE_LOGF(LogGameplayCueTranslator, Display, "\nTranslated Tag: %ls", *NewTag.ToString());
	}

	*/
};

#undef UE_API
