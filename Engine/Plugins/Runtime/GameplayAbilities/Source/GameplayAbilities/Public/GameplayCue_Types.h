// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTagContainer.h"
#include "GameplayEffectTypes.h"
#include "GameplayPrediction.h"
#include "UObject/ObjectKey.h"
#include "GameplayEffect.h"
#include "GameplayCue_Types.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class AGameplayCueNotify_Actor;
class UAbilitySystemComponent;
class UGameplayCueSet;


/** Describes what type of payload is attached to a cue execution, we only replicate what is needed */
UENUM()
enum class EGameplayCuePayloadType : uint8
{
	/** Uses FGameplayCueParameters */
	CueParameters,
	/** Uses FGameplayEffectSpecForRPC */
	FromSpec,
};


/** Structure to keep track of pending gameplay cues that haven't been applied yet. */
USTRUCT()
struct FGameplayCuePendingExecute
{
	GENERATED_USTRUCT_BODY()

	FGameplayCuePendingExecute() {}

	/** List of tags, we allocate one as there is almost always exactly one tag */
	TArray<FGameplayTag, TInlineAllocator<1> > GameplayCueTags;
	
	/** Prediction key that spawned this cue */
	UPROPERTY()
	FPredictionKey PredictionKey;

	/** What type of payload is attached to this cue */
	UPROPERTY()
	EGameplayCuePayloadType PayloadType = EGameplayCuePayloadType::CueParameters;

	/** What component to send the cue on */
	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> OwningComponent = nullptr;

	/** If this cue is from a spec, here's the copy of that spec */
	UPROPERTY()
	FGameplayEffectSpecForRPC FromSpec;

	/** Store the full cue parameters or just the effect context depending on type */
	UPROPERTY()
	FGameplayCueParameters CueParameters;
};

USTRUCT()
struct FGameplayCueNotifyActorArray
{
	GENERATED_BODY()

	UPROPERTY(transient)
	TArray<TObjectPtr<AGameplayCueNotify_Actor>> Actors;
};

/** Struct for pooling and preallocating gameplaycuenotify_actor classes. This data is per world and used to track what actors are available to recycle and which classes need to preallocate instances of those actors */
USTRUCT()
struct FPreallocationInfo
{
	GENERATED_USTRUCT_BODY()

	/** Raw list of pooled instances. This relies on NotifyGameplayCueActorEndPlay always being called when actor is destroyed */
	UPROPERTY(transient)
	TMap<TObjectPtr<UClass>, FGameplayCueNotifyActorArray> PreallocatedInstances;

	/** List of classes that will be pooled */
	UPROPERTY(transient)
	TArray<TSubclassOf<AGameplayCueNotify_Actor>> ClassesNeedingPreallocation;

	/** World that owns this list */
	FObjectKey OwningWorldKey;
};

/**
 *	FScopedGameplayCueSendContext
 *	Add this around code that sends multiple gameplay cues to allow grouping them into a smalkler number of cues for more efficient networking
 */
struct FScopedGameplayCueSendContext
{
	UE_API FScopedGameplayCueSendContext();
	UE_API ~FScopedGameplayCueSendContext();
};

/** Delegate for when GC notifies are added or removed from manager */
DECLARE_MULTICAST_DELEGATE(FOnGameplayCueNotifyChange);

#undef UE_API
