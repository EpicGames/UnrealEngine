// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayTagContainer.h"
#include "GameplayDebuggerCategory.h"
#include "GameplayPrediction.h"

class AActor;
class APlayerController;
class UAbilitySystemComponent;
class UPackageMap;

class FGameplayDebuggerCategory_Abilities : public FGameplayDebuggerCategory
{
public:
	GAMEPLAYABILITIES_API FGameplayDebuggerCategory_Abilities();

	GAMEPLAYABILITIES_API virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;
	GAMEPLAYABILITIES_API virtual void DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext) override;

	GAMEPLAYABILITIES_API static TSharedRef<FGameplayDebuggerCategory> MakeInstance();

	GAMEPLAYABILITIES_API void OnShowGameplayTagsToggle();
	GAMEPLAYABILITIES_API void OnShowGameplayAbilitiesToggle();
	GAMEPLAYABILITIES_API void OnShowGameplayEffectsToggle();
	GAMEPLAYABILITIES_API void OnShowGameplayAttributesToggle();
		

	// Some GAS features such as Attributes can exist on the server, client, or both.  We can also get 'detached' if both sides have the same values (such as Attributes) that aren't networked.
	// We are reusing this same concept to try and reconcile Gameplay Effects predicted locally, or triggered server-side.
	enum class ENetworkStatus : uint8
	{
		ServerOnly, LocalOnly, Networked, Detached, MAX
	};

	// Unary operator + for quick conversion from enum class to int32
	friend constexpr int32 operator+(const ENetworkStatus& value) { return static_cast<int32>(value); }

protected:

	GAMEPLAYABILITIES_API void DrawGameplayTags(FGameplayDebuggerCanvasContext& CanvasContext, const APlayerController* OwnerPC) const;
	GAMEPLAYABILITIES_API void DrawGameplayAbilities(FGameplayDebuggerCanvasContext& CanvasContext, const APlayerController* OwnerPC) const;
	GAMEPLAYABILITIES_API void DrawGameplayEffects(FGameplayDebuggerCanvasContext& CanvasContext, const APlayerController* OwnerPC) const;
	GAMEPLAYABILITIES_API void DrawGameplayAttributes(FGameplayDebuggerCanvasContext& CanvasContext, const APlayerController* OwnerPC) const;

	struct FRepData
	{
		FGameplayTagContainer OwnedTags;
		TArray<int32> TagCounts;

		struct FGameplayAbilityDebug
		{
			FString Ability;
			FString Source;
			int32 Level = 0;
			bool bIsActive = false;
		};
		TArray<FGameplayAbilityDebug> Abilities;

		struct FGameplayEffectDebug
		{
			int32 ReplicationID = INDEX_NONE; // unique & shared between server/client (or INDEX_NONE if local)
			FString Effect;
			FString Context;
			float Duration = 0.0f;
			float Period = 0.0f;
			int32 Stacks = 0;
			float Level = 0.0f;

			ENetworkStatus NetworkStatus = ENetworkStatus::ServerOnly;
			bool bInhibited = false;
		};
		TArray<FGameplayEffectDebug> GameplayEffects;

		struct FGameplayAttributeDebug
		{
			FString AttributeName;
			float BaseValue = 0.0f;
			float CurrentValue = 0.0f;
			ENetworkStatus NetworkStatus = ENetworkStatus::ServerOnly;
		};
		TArray<FGameplayAttributeDebug> Attributes;

		void Serialize(FArchive& Ar);
	};
	FRepData DataPack;

	GAMEPLAYABILITIES_API bool WrapStringAccordingToViewport(const FString& iStr, FString& oStr, FGameplayDebuggerCanvasContext& CanvasContext, float ViewportWitdh) const;

private:
	TArray<FRepData::FGameplayAttributeDebug> CollectAttributeData(const APlayerController* OwnerPC, const UAbilitySystemComponent* AbilityComp) const;
	TArray<FRepData::FGameplayEffectDebug> CollectEffectsData(const APlayerController* OwnerPC, const UAbilitySystemComponent* AbilityComp) const;

	// Save off the last expected draw size so that we can draw a border around it next frame (and hope we're the same size)
	float LastDrawDataEndSize = 0.0f;

	bool bShowGameplayTags = true;
	bool bShowGameplayAbilities = true;
	bool bShowGameplayEffects = true;
	bool bShowGameplayAttributes = true;
};

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
