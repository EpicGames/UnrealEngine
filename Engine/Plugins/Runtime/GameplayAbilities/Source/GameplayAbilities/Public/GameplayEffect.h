// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "Engine/NetSerialization.h"
#include "Engine/EngineTypes.h"
#include "GameplayTagContainer.h"
#include "Engine/CurveTable.h"
#include "AttributeSet.h"
#include "EngineDefines.h"
#include "GameplayEffectTypes.h"
#include "GameplayEffectAggregator.h"
#include "GameplayPrediction.h"
#include "GameplayTagAssetInterface.h"
#include "GameplayAbilitySpec.h"
#include "ActiveGameplayEffectIterator.h"
#include "UObject/ObjectKey.h"
#include "VisualLogger/VisualLoggerDebugSnapshotInterface.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "GameplayEffect.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/**
 * Gameplay Effects are bundles of functionality that are _applied_ to Actors.  Think of Gameplay Effects like something at _affects_ an Actor.
 * Gameplay Effects are assets, and thus immutable at runtime.  There are small exceptions to this where hacks are done to _create_ a GE at runtime (but once created and configured, the data is not modified).
 * 
 * Gameplay Effects and Lifetime
 *		- A GE can be executed instantly, or not.  If not, it has a duration (which can be infinite).  GE's that have durations are _added_ to the Active Gameplay Effects Container.
 *		- A GE that is instant is said to be _executed_, where it never makes its way into the Active Gameplay Effects Container. (exceptions mentioned below).
 *		- In both cases (instant and duration) the lingo we use is "Applied" and all forms thereof (e.g. Application).  So "CanApplyGameplayEffect" does not take into account if it's Instant or not.
 *		- Periodic effects are executed at every period (so it is both Added and Executed).
 *		- One exception to the above is when we're _predicting_ a Gameplay Effect to happen on the Client (when we're ahead of the Server).  Then we pretend it's a Duration effect and wait for server confirmation.
 * 
 * Gameplay Effect Components
 *	Since Unreal 5.3, we have deprecated the Monolithic UGameplayEffect and instead rely on UGameplayEffectComponents.  Those Components are designed to allow users of the Engine to customize the behavior
 *  of UGameplayEffects based on their project without having to derive their own specialized classes.  It should also be more clear what a UGameplayEffect does when looking at the class, as there are less properties.
 *  UGameplayEffectComponents are implemented as instanced SubObjects.  This comes with some baggage, as the current version of Unreal Engine does not seamlessly support SubObjects out of the box.  There is a fix-up step
 *  required in PostCDOCompiled, with an explanation of what is required to achieve this and the limitations.
 * 
 * Gameplay Effect Specs
 *	Gameplay Effect Specs are the runtime versions of Gameplay Effects.  You can think of them as the instanced data wrapper around the Gameplay Effect (the GE is an asset).  The Blueprint functionality is thus more concerned with
 *	Gameplay Effect Specs rather than Gameplay Effects and that is reflected in the AbilitySystemBlueprintLibrary.
 */

class UAbilitySystemComponent;
class UGameplayEffect;
class UGameplayEffectComponent;
class UGameplayEffectCustomApplicationRequirement;
class UGameplayEffectExecutionCalculation;
class UGameplayEffectTemplate;
class UGameplayModMagnitudeCalculation;
struct FActiveGameplayEffectsContainer;
struct FGameplayEffectModCallbackData;
struct FGameplayEffectSpec;
struct FAggregatorEvaluateParameters;
struct FAggregatorMod;
struct FVisualLogEntry;

/** Enumeration outlining the possible gameplay effect magnitude calculation policies. */
UENUM()
enum class EGameplayEffectMagnitudeCalculation : uint8
{
	/** Use a simple, scalable float for the calculation. */
	ScalableFloat,
	/** Perform a calculation based upon an attribute. */
	AttributeBased,
	/** Perform a custom calculation, capable of capturing and acting on multiple attributes, in either BP or native. */
	CustomCalculationClass,	
	/** This magnitude will be set explicitly by the code/blueprint that creates the spec. */
	SetByCaller,
};

/** Enumeration outlining the possible attribute based float calculation policies. */
UENUM()
enum class EAttributeBasedFloatCalculationType : uint8
{
	/** Use the final evaluated magnitude of the attribute. */
	AttributeMagnitude,
	/** Use the base value of the attribute. */
	AttributeBaseValue,
	/** Use the "bonus" evaluated magnitude of the attribute: Equivalent to (FinalMag - BaseValue). */
	AttributeBonusMagnitude,
	/** Use a calculated magnitude stopping with the evaluation of the specified "Final Channel" */
	AttributeMagnitudeEvaluatedUpToChannel
};

/** The version of the UGameplayEffect. Used for upgrade paths. */
UENUM()
enum class EGameplayEffectVersion : uint8
{
	Monolithic,	// Legacy version from Pre-UE5.3 (before we were versioning)
	Modular53,	// New modular version available in UE5.3
	AbilitiesComponent53,	// Granted Abilities are moved into the Abilities Component

	Current = AbilitiesComponent53
};

struct FGameplayEffectConstants
{
	/** Infinite duration */
	static UE_API const float INFINITE_DURATION;

	/** No duration; Time specifying instant application of an effect */
	static UE_API const float INSTANT_APPLICATION;

	/** Constant specifying that the combat effect has no period and doesn't check for over time application */
	static UE_API const float NO_PERIOD;

	/** No Level/Level not set */
	static UE_API const float INVALID_LEVEL;
};

/** 
 * Struct representing a float whose magnitude is dictated by a backing attribute and a calculation policy, follows basic form of:
 * (Coefficient * (PreMultiplyAdditiveValue + [Eval'd Attribute Value According to Policy])) + PostMultiplyAdditiveValue
 */
USTRUCT()
struct FAttributeBasedFloat
{
	GENERATED_USTRUCT_BODY()

public:

	/** Constructor */
	FAttributeBasedFloat()
		: Coefficient(1.f)
		, PreMultiplyAdditiveValue(0.f)
		, PostMultiplyAdditiveValue(0.f)
		, BackingAttribute()
		, AttributeCalculationType(EAttributeBasedFloatCalculationType::AttributeMagnitude)
		, FinalChannel(EGameplayModEvaluationChannel::Channel0)
	{}

	/**
	 * Calculate and return the magnitude of the float given the specified gameplay effect spec.
	 * 
	 * @note:	This function assumes (and asserts on) the existence of the required captured attribute within the spec.
	 *			It is the responsibility of the caller to verify that the spec is properly setup before calling this function.
	 *			
	 * @param InRelevantSpec	Gameplay effect spec providing the backing attribute capture
	 *	
	 * @return Evaluated magnitude based upon the spec & calculation policy
	 */
	float CalculateMagnitude(const FGameplayEffectSpec& InRelevantSpec) const;

	/** Coefficient to the attribute calculation */
	UPROPERTY(EditDefaultsOnly, Category=AttributeFloat)
	FScalableFloat Coefficient;

	/** Additive value to the attribute calculation, added in before the coefficient applies */
	UPROPERTY(EditDefaultsOnly, Category=AttributeFloat)
	FScalableFloat PreMultiplyAdditiveValue;

	/** Additive value to the attribute calculation, added in after the coefficient applies */
	UPROPERTY(EditDefaultsOnly, Category=AttributeFloat)
	FScalableFloat PostMultiplyAdditiveValue;

	/** Attribute backing the calculation */
	UPROPERTY(EditDefaultsOnly, Category=AttributeFloat)
	FGameplayEffectAttributeCaptureDefinition BackingAttribute;

	/** If a curve table entry is specified, the attribute will be used as a lookup into the curve instead of using the attribute directly. */
	UPROPERTY(EditDefaultsOnly, Category=AttributeFloat)
	FCurveTableRowHandle AttributeCurve;

	/** Calculation policy in regards to the attribute */
	UPROPERTY(EditDefaultsOnly, Category=AttributeFloat)
	EAttributeBasedFloatCalculationType AttributeCalculationType;

	/** Channel to terminate evaluation on when using AttributeEvaluatedUpToChannel calculation type */
	UPROPERTY(EditDefaultsOnly, Category=AttributeFloat)
	EGameplayModEvaluationChannel FinalChannel;

	/** Filter to use on source tags; If specified, only modifiers applied with all of these tags will factor into the calculation */
	UPROPERTY(EditDefaultsOnly, Category=AttributeFloat)
	FGameplayTagContainer SourceTagFilter;

	/** Filter to use on target tags; If specified, only modifiers applied with all of these tags will factor into the calculation */
	UPROPERTY(EditDefaultsOnly, Category=AttributeFloat)
	FGameplayTagContainer TargetTagFilter;

	/** Equality/Inequality operators */
	bool operator==(const FAttributeBasedFloat& Other) const;
	bool operator!=(const FAttributeBasedFloat& Other) const;

#if WITH_EDITOR
	EDataValidationResult IsDataValid(class FDataValidationContext& Context, const FString& PathName) const;
#endif
};

/** Structure to encapsulate magnitudes that are calculated via custom calculation */
USTRUCT()
struct FCustomCalculationBasedFloat
{
	GENERATED_USTRUCT_BODY()

	FCustomCalculationBasedFloat()
		: CalculationClassMagnitude(nullptr)
		, Coefficient(1.f)
		, PreMultiplyAdditiveValue(0.f)
		, PostMultiplyAdditiveValue(0.f)
	{}

public:

	/**
	 * Calculate and return the magnitude of the float given the specified gameplay effect spec.
	 * 
	 * @note:	This function assumes (and asserts on) the existence of the required captured attribute within the spec.
	 *			It is the responsibility of the caller to verify that the spec is properly setup before calling this function.
	 *			
	 * @param InRelevantSpec	Gameplay effect spec providing the backing attribute capture
	 *	
	 * @return Evaluated magnitude based upon the spec & calculation policy
	 */
	float CalculateMagnitude(const FGameplayEffectSpec& InRelevantSpec) const;

	UPROPERTY(EditDefaultsOnly, Category=CustomCalculation, DisplayName="Calculation Class")
	TSubclassOf<UGameplayModMagnitudeCalculation> CalculationClassMagnitude;

	/** Coefficient to the custom calculation */
	UPROPERTY(EditDefaultsOnly, Category=CustomCalculation)
	FScalableFloat Coefficient;

	/** Additive value to the attribute calculation, added in before the coefficient applies */
	UPROPERTY(EditDefaultsOnly, Category=CustomCalculation)
	FScalableFloat PreMultiplyAdditiveValue;

	/** Additive value to the attribute calculation, added in after the coefficient applies */
	UPROPERTY(EditDefaultsOnly, Category=CustomCalculation)
	FScalableFloat PostMultiplyAdditiveValue;

	/** If a curve table entry is specified, the OUTPUT of this custom class magnitude (including the pre and post additive values) lookup into the curve instead of using the attribute directly. */
	UPROPERTY(EditDefaultsOnly, Category=CustomCalculation)
	FCurveTableRowHandle FinalLookupCurve;

	/** Equality/Inequality operators */
	bool operator==(const FCustomCalculationBasedFloat& Other) const;
	bool operator!=(const FCustomCalculationBasedFloat& Other) const;

#if WITH_EDITOR
	EDataValidationResult IsDataValid(class FDataValidationContext& Context, const FString& PathName) const;
#endif
};

/** Struct for holding SetBytCaller data */
USTRUCT()
struct FSetByCallerFloat
{
	GENERATED_USTRUCT_BODY()

	FSetByCallerFloat()
	: DataName(NAME_None)
	{}

	/** The Name the caller (code or blueprint) will use to set this magnitude by. */
	UPROPERTY(VisibleDefaultsOnly, Category=SetByCaller)
	FName	DataName;

	UPROPERTY(EditDefaultsOnly, Category = SetByCaller, meta = (Categories = "SetByCaller"))
	FGameplayTag DataTag;

	/** Equality/Inequality operators */
	bool operator==(const FSetByCallerFloat& Other) const;
	bool operator!=(const FSetByCallerFloat& Other) const;
};

/** Struct representing the magnitude of a gameplay effect modifier, potentially calculated in numerous different ways */
USTRUCT()
struct FGameplayEffectModifierMagnitude
{
	GENERATED_USTRUCT_BODY()

public:

	/** Default Constructor */
	FGameplayEffectModifierMagnitude()
		: MagnitudeCalculationType(EGameplayEffectMagnitudeCalculation::ScalableFloat)
	{
	}

	/** Constructors for setting value in code (for automation tests) */
	FGameplayEffectModifierMagnitude(const FScalableFloat& Value)
		: MagnitudeCalculationType(EGameplayEffectMagnitudeCalculation::ScalableFloat)
		, ScalableFloatMagnitude(Value)
	{
	}
	FGameplayEffectModifierMagnitude(const FAttributeBasedFloat& Value)
		: MagnitudeCalculationType(EGameplayEffectMagnitudeCalculation::AttributeBased)
		, AttributeBasedMagnitude(Value)
	{
	}
	FGameplayEffectModifierMagnitude(const FCustomCalculationBasedFloat& Value)
		: MagnitudeCalculationType(EGameplayEffectMagnitudeCalculation::CustomCalculationClass)
		, CustomMagnitude(Value)
	{
	}
	FGameplayEffectModifierMagnitude(const FSetByCallerFloat& Value)
		: MagnitudeCalculationType(EGameplayEffectMagnitudeCalculation::SetByCaller)
		, SetByCallerMagnitude(Value)
	{
	}
 
	/**
	 * Determines if the magnitude can be properly calculated with the specified gameplay effect spec (could fail if relying on an attribute not present, etc.)
	 * 
	 * @param InRelevantSpec	Gameplay effect spec to check for magnitude calculation
	 * 
	 * @return Whether or not the magnitude can be properly calculated
	 */
	UE_API bool CanCalculateMagnitude(const FGameplayEffectSpec& InRelevantSpec) const;

	/**
	 * Attempts to calculate the magnitude given the provided spec. May fail if necessary information (such as captured attributes) is missing from
	 * the spec.
	 * 
	 * @param InRelevantSpec			Gameplay effect spec to use to calculate the magnitude with
	 * @param OutCalculatedMagnitude	[OUT] Calculated value of the magnitude, will be set to 0.f in the event of failure
	 * 
	 * @return True if the calculation was successful, false if it was not
	 */
	UE_API bool AttemptCalculateMagnitude(const FGameplayEffectSpec& InRelevantSpec, OUT float& OutCalculatedMagnitude, bool WarnIfSetByCallerFail=true, float DefaultSetbyCaller=0.f) const;

	/** Attempts to recalculate the magnitude given a changed aggregator. This will only recalculate if we are a modifier that is linked (non snapshot) to the given aggregator. */
	UE_API bool AttemptRecalculateMagnitudeFromDependentAggregatorChange(const FGameplayEffectSpec& InRelevantSpec, OUT float& OutCalculatedMagnitude, const FAggregator* ChangedAggregator) const;

	/**
	 * Gather all of the attribute capture definitions necessary to compute the magnitude and place them into the provided array
	 * 
	 * @param OutCaptureDefs	[OUT] Array populated with necessary attribute capture definitions
	 */
	UE_API void GetAttributeCaptureDefinitions(OUT TArray<FGameplayEffectAttributeCaptureDefinition>& OutCaptureDefs) const;

	EGameplayEffectMagnitudeCalculation GetMagnitudeCalculationType() const { return MagnitudeCalculationType; }

	/** Returns the magnitude as it was entered in data. Only applies to ScalableFloat or any other type that can return data without context */
	UE_API bool GetStaticMagnitudeIfPossible(float InLevel, float& OutMagnitude, const FString* ContextString = nullptr) const;

	/** Returns the DataName associated with this magnitude if it is set by caller */
	UE_API bool GetSetByCallerDataNameIfPossible(FName& OutDataName) const;

	/** Returns SetByCaller data structure, for inspection purposes */
	const FSetByCallerFloat& GetSetByCallerFloat() const { return SetByCallerMagnitude; }

	/** Returns the custom magnitude calculation class, if any, for this magnitude. Only applies to CustomMagnitudes */
	UE_API TSubclassOf<UGameplayModMagnitudeCalculation> GetCustomMagnitudeCalculationClass() const;

	/** Implementing Serialize to clear references to assets that are not needed */
	UE_API bool Serialize(FArchive& Ar);

	UE_API bool operator==(const FGameplayEffectModifierMagnitude& Other) const;
	UE_API bool operator!=(const FGameplayEffectModifierMagnitude& Other) const;

#if WITH_EDITOR
	UE_API FText GetValueForEditorDisplay() const;
	UE_API EDataValidationResult IsDataValid(class FDataValidationContext& Context, const FString& PathName) const;

	UE_DEPRECATED(5.6, "Use IsDataValid")
	UE_API void ReportErrors(const FString& PathName) const;
#endif

protected:

	/** Type of calculation to perform to derive the magnitude */
	UPROPERTY(EditDefaultsOnly, Category=Magnitude)
	EGameplayEffectMagnitudeCalculation MagnitudeCalculationType;

	/** Magnitude value represented by a scalable float */
	UPROPERTY(EditDefaultsOnly, Category=Magnitude)
	FScalableFloat ScalableFloatMagnitude;

	/** Magnitude value represented by an attribute-based float
	(Coefficient * (PreMultiplyAdditiveValue + [Eval'd Attribute Value According to Policy])) + PostMultiplyAdditiveValue */
	UPROPERTY(EditDefaultsOnly, Category=Magnitude)
	FAttributeBasedFloat AttributeBasedMagnitude;

	/** Magnitude value represented by a custom calculation class */
	UPROPERTY(EditDefaultsOnly, Category=Magnitude)
	FCustomCalculationBasedFloat CustomMagnitude;

	/** Magnitude value represented by a SetByCaller magnitude */
	UPROPERTY(EditDefaultsOnly, Category=Magnitude)
	FSetByCallerFloat SetByCallerMagnitude;

	friend class UGameplayEffect;
	friend class FGameplayEffectModifierMagnitudeDetails;
};

template<>
struct TStructOpsTypeTraits<FGameplayEffectModifierMagnitude> : public TStructOpsTypeTraitsBase2<FGameplayEffectModifierMagnitude>
{
	enum
	{
		WithSerializer = true,
	};
};

/** Enumeration representing the types of scoped modifier aggregator usages available */
UENUM()
enum class EGameplayEffectScopedModifierAggregatorType : uint8
{
	/** Aggregator is backed by an attribute capture */
	CapturedAttributeBacked,

	/** Aggregator is entirely transient (acting as a "temporary variable") and must be identified via gameplay tag */
	Transient
};

/** 
 * Struct representing modifier info used exclusively for "scoped" executions that happen instantaneously. These are
 * folded into a calculation only for the extent of the calculation and never permanently added to an aggregator.
 */
USTRUCT(BlueprintType)
struct FGameplayEffectExecutionScopedModifierInfo
{
	GENERATED_USTRUCT_BODY()

	// Constructors
	FGameplayEffectExecutionScopedModifierInfo()
		: AggregatorType(EGameplayEffectScopedModifierAggregatorType::CapturedAttributeBacked)
		, ModifierOp(EGameplayModOp::Additive)
	{}

	FGameplayEffectExecutionScopedModifierInfo(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef)
		: CapturedAttribute(InCaptureDef)
		, AggregatorType(EGameplayEffectScopedModifierAggregatorType::CapturedAttributeBacked)
		, ModifierOp(EGameplayModOp::Additive)
	{
	}

	FGameplayEffectExecutionScopedModifierInfo(const FGameplayTag& InTransientAggregatorIdentifier)
		: TransientAggregatorIdentifier(InTransientAggregatorIdentifier)
		, AggregatorType(EGameplayEffectScopedModifierAggregatorType::Transient)
		, ModifierOp(EGameplayModOp::Additive)
	{
	}

	/** Backing attribute that the scoped modifier is for */
	UPROPERTY(VisibleDefaultsOnly, Category=Execution)
	FGameplayEffectAttributeCaptureDefinition CapturedAttribute;

	/** Identifier for aggregator if acting as a transient "temporary variable" aggregator */
	UPROPERTY(VisibleDefaultsOnly, Category=Execution)
	FGameplayTag TransientAggregatorIdentifier;

	/** Type of aggregator backing the scoped mod */
	UPROPERTY(VisibleDefaultsOnly, Category=Execution)
	EGameplayEffectScopedModifierAggregatorType AggregatorType;

	/** Modifier operation to perform */
	UPROPERTY(EditDefaultsOnly, Category=Execution)
	TEnumAsByte<EGameplayModOp::Type> ModifierOp;

	/** Magnitude of the scoped modifier */
	UPROPERTY(EditDefaultsOnly, Category=Execution)
	FGameplayEffectModifierMagnitude ModifierMagnitude;

	/** Evaluation channel settings of the scoped modifier */
	UPROPERTY(EditDefaultsOnly, Category=Execution)
	FGameplayModEvaluationChannelSettings EvaluationChannelSettings;

	/** Source tag requirements for the modifier to apply */
	UPROPERTY(EditDefaultsOnly, Category=Execution)
	FGameplayTagRequirements SourceTags;

	/** Target tag requirements for the modifier to apply */
	UPROPERTY(EditDefaultsOnly, Category=Execution)
	FGameplayTagRequirements TargetTags;
};

/** Describes what happens to a Gameplay Effect that was applied on-application by another Gameplay Effect
 *  (via UAdditionalEffectsGameplayEffectComponent) when the applying effect is removed. */
UENUM()
enum class EGameplayEffectGrantedEffectRemovalPolicy : uint8
{
	/** The applied Gameplay Effect manages its own lifetime independently. This is the default behavior. */
	GrantedEffectControlsOwnLifetime,

	/** The applied Gameplay Effect will be removed when the Gameplay Effect that applied it is removed.
	 *  Cannot be used when the granted effect has an Instant duration (instant GEs are never persisted). */
	RemoveGrantedEffectOnEnd,
};

/**
 * Struct for gameplay effects that apply only if another gameplay effect (or execution) was successfully applied.
 */
USTRUCT(BlueprintType)
struct FConditionalGameplayEffect
{
	GENERATED_USTRUCT_BODY()

	UE_API bool CanApply(const FGameplayTagContainer& SourceTags, float SourceLevel) const;

	UE_API FGameplayEffectSpecHandle CreateSpec(FGameplayEffectContextHandle EffectContext, float SourceLevel) const;

	/** gameplay effect that will be applied to the target */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayEffect)
	TSubclassOf<UGameplayEffect> EffectClass;

	/** Tags that the source must have for this GE to apply.  If this is blank, then there are no requirements to apply the EffectClass. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayEffect)
	FGameplayTagContainer RequiredSourceTags;

	/** What happens to this applied effect when the Gameplay Effect that applied it is removed.
	 *  Only relevant when this entry is used in UAdditionalEffectsGameplayEffectComponent::OnApplicationGameplayEffects. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayEffect)
	EGameplayEffectGrantedEffectRemovalPolicy RemovalPolicy = EGameplayEffectGrantedEffectRemovalPolicy::GrantedEffectControlsOwnLifetime;

	/** Number of stacks to remove when RemovalPolicy is RemoveGrantedEffectOnEnd.
	 *  INDEX_NONE (default) removes all stacks. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayEffect, meta = (EditCondition = "RemovalPolicy == EGameplayEffectGrantedEffectRemovalPolicy::RemoveGrantedEffectOnEnd", EditConditionHides))
	int32 StackCountToRemove = INDEX_NONE;

	UE_API bool operator==(const FConditionalGameplayEffect& Other) const;
	UE_API bool operator!=(const FConditionalGameplayEffect& Other) const;
};

/** 
 * Struct representing the definition of a custom execution for a gameplay effect.
 * Custom executions run special logic from an outside class each time the gameplay effect executes.
 */
USTRUCT(BlueprintType)
struct FGameplayEffectExecutionDefinition
{
	GENERATED_USTRUCT_BODY()

	/**
	 * Gathers and populates the specified array with the capture definitions that the execution would like in order
	 * to perform its custom calculation. Up to the individual execution calculation to handle if some of them are missing
	 * or not.
	 * 
	 * @param OutCaptureDefs	[OUT] Capture definitions requested by the execution
	 */
	UE_API void GetAttributeCaptureDefinitions(OUT TArray<FGameplayEffectAttributeCaptureDefinition>& OutCaptureDefs) const;

	/** Custom execution calculation class to run when the gameplay effect executes */
	UPROPERTY(EditDefaultsOnly, Category=Execution)
	TSubclassOf<UGameplayEffectExecutionCalculation> CalculationClass;
	
	/** These tags are passed into the execution as is, and may be used to do conditional logic */
	UPROPERTY(EditDefaultsOnly, Category = Execution)
	FGameplayTagContainer PassedInTags;

	/** Modifiers that are applied "in place" during the execution calculation */
	UPROPERTY(EditDefaultsOnly, Category = Execution)
	TArray<FGameplayEffectExecutionScopedModifierInfo> CalculationModifiers;

	/** Other Gameplay Effects that will be applied to the target of this execution if the execution is successful. Note if no execution class is selected, these will always apply. */
	UPROPERTY(EditDefaultsOnly, Category = Execution)
	TArray<FConditionalGameplayEffect> ConditionalGameplayEffects;
};

/**
 * FGameplayModifierInfo
 *	Tells us "Who/What we" modify
 *	Does not tell us how exactly
 */
USTRUCT(BlueprintType)
struct FGameplayModifierInfo
{
	GENERATED_USTRUCT_BODY()
	
	/** The Attribute we modify or the GE we modify modifies. */
	UPROPERTY(EditDefaultsOnly, Category=GameplayModifier, meta=(FilterMetaTag="HideFromModifiers"))
	FGameplayAttribute Attribute;

	/**
	 * The numeric operation of this modifier: Override, Add, Multiply, etc
	 * When multiple modifiers aggregate together, the equation is:
	 * ((BaseValue + AddBase) * MultiplyAdditive / DivideAdditive * MultiplyCompound) + AddFinal
	 */
	UPROPERTY(EditDefaultsOnly, Category=GameplayModifier)
	TEnumAsByte<EGameplayModOp::Type> ModifierOp = EGameplayModOp::Additive;

	/** Magnitude of the modifier */
	UPROPERTY(EditDefaultsOnly, Category=GameplayModifier)
	FGameplayEffectModifierMagnitude ModifierMagnitude;

	/** Evaluation channel settings of the modifier */
	UPROPERTY(EditDefaultsOnly, Category=GameplayModifier)
	FGameplayModEvaluationChannelSettings EvaluationChannelSettings;

	UPROPERTY(EditDefaultsOnly, Category=GameplayModifier)
	FGameplayTagRequirements	SourceTags;

	UPROPERTY(EditDefaultsOnly, Category=GameplayModifier)
	FGameplayTagRequirements	TargetTags;

	/** Equality/Inequality operators */
	UE_API bool operator==(const FGameplayModifierInfo& Other) const;
	UE_API bool operator!=(const FGameplayModifierInfo& Other) const;
};

/**
 * FGameplayEffectCue
 *	This is a cosmetic cue that can be tied to a UGameplayEffect. 
 *  This is essentially a GameplayTag + a Min/Max level range that is used to map the level of a GameplayEffect to a normalized value used by the GameplayCue system.
 */
USTRUCT(BlueprintType)
struct FGameplayEffectCue
{
	GENERATED_USTRUCT_BODY()

	FGameplayEffectCue()
		: MinLevel(0.f)
		, MaxLevel(0.f)
	{
	}

	FGameplayEffectCue(const FGameplayTag& InTag, float InMinLevel, float InMaxLevel)
		: MinLevel(InMinLevel)
		, MaxLevel(InMaxLevel)
	{
		GameplayCueTags.AddTag(InTag);
	}

	/** The attribute to use as the source for cue magnitude. If none use level */
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue)
	FGameplayAttribute MagnitudeAttribute;

	/** The minimum level that this Cue supports */
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue)
	float	MinLevel;

	/** The maximum level that this Cue supports */
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue)
	float	MaxLevel;

	/** Tags passed to the gameplay cue handler when this cue is activated */
	UPROPERTY(EditDefaultsOnly, Category = GameplayCue, meta = (Categories="GameplayCue"))
	FGameplayTagContainer GameplayCueTags;

	float NormalizeLevel(float InLevel)
	{
		float Range = MaxLevel - MinLevel;
		if (Range <= KINDA_SMALL_NUMBER)
		{
			return 1.f;
		}

		return FMath::Clamp((InLevel - MinLevel) / Range, 0.f, 1.0f);
	}
};

/** Structure that is used to combine tags from parent and child blueprints in a safe way */
USTRUCT(BlueprintType)
struct FInheritedTagContainer
{
	GENERATED_USTRUCT_BODY()

	/** Tags that I inherited and tags that I added minus tags that I removed */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Application, meta=(Tooltip="CombinedTags = Inherited - Removed + Added"))
	FGameplayTagContainer CombinedTags;

	/** Tags that I have (in addition to my parent's tags) */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Application, meta=(DisplayName="Add to Inherited"))
	FGameplayTagContainer Added;

	/** Tags that should be removed (only if my parent had them).  Note: we cannot use this to remove a tag that exists on a target. It only modifies the result of CombinedTags. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Application, meta=(DisplayName="Remove from Inherited"))
	FGameplayTagContainer Removed;

	UE_API void UpdateInheritedTagProperties(const FInheritedTagContainer* Parent);

	/** Apply the Added and Removed tags to the passed-in container (does not have to be the previously configured Parent!) */
	UE_API void ApplyTo(FGameplayTagContainer& ApplyToContainer) const;

	/** Add a tag that will appear in addition to any inherited tags */
	UE_API void AddTag(const FGameplayTag& TagToAdd);

	/** Remove a tag that will be omitted from any inherited tags */
	UE_API void RemoveTag(const FGameplayTag& TagToRemove);

	UE_API bool operator==(const FInheritedTagContainer& Other) const;
	UE_API bool operator!=(const FInheritedTagContainer& Other) const;
};

/** Gameplay effect duration policies */
UENUM()
enum class EGameplayEffectDurationType : uint8
{
	/** This effect applies instantly */
	Instant,
	/** This effect lasts forever */
	Infinite,
	/** The duration of this effect will be specified by a magnitude */
	HasDuration
};

/** Enumeration of policies for dealing with duration of a gameplay effect while stacking */
UENUM()
enum class EGameplayEffectStackingDurationPolicy : uint8
{
	/** The duration of the effect will be refreshed from any successful stack application */
	RefreshOnSuccessfulApplication,

	/** The duration of the effect will never be refreshed */
	NeverRefresh,

	/** New stacks will add their GE spec's duration onto current remaining time */
	ExtendDuration,
};

/** Enumeration of policies for dealing with the period of a gameplay effect while stacking */
UENUM()
enum class EGameplayEffectStackingPeriodPolicy : uint8
{
	/** Any progress toward the next tick of a periodic effect is discarded upon any successful stack application */
	ResetOnSuccessfulApplication,

	/** The progress toward the next tick of a periodic effect will never be reset, regardless of stack applications */
	NeverReset,
};

/** Enumeration of policies for dealing gameplay effect stacks that expire (in duration based effects). */
UENUM()
enum class EGameplayEffectStackingExpirationPolicy : uint8
{
	/** The entire stack is cleared when the active gameplay effect expires  */
	ClearEntireStack,

	/** The current stack count will be decremented by 1 and the duration refreshed. The GE is not "reapplied", just continues to exist with one less stacks. */
	RemoveSingleStackAndRefreshDuration,

	/** The duration of the gameplay effect is refreshed. This essentially makes the effect infinite in duration. This can be used to manually handle stack decrements via OnStackCountChange callback */
	RefreshDuration,
};

/** Enumeration of policies for dealing with the period of a gameplay effect when inhibition is removed */
UENUM()
enum class EGameplayEffectPeriodInhibitionRemovedPolicy : uint8
{
	/** Does not reset. The period timing will continue as if the inhibition hadn't occurred. */
	NeverReset,

	/** Resets the period. The next execution will occur one full period from when inhibition is removed. */
	ResetPeriod,

	/** Executes immediately and resets the period. */
	ExecuteAndResetPeriod,
};

/** Holds evaluated magnitude from a GameplayEffect modifier */
USTRUCT()
struct FModifierSpec
{
	GENERATED_USTRUCT_BODY()

	FModifierSpec() : EvaluatedMagnitude(0.f) { }

	float GetEvaluatedMagnitude() const { return EvaluatedMagnitude; }

private:

	/** In the event that the modifier spec requires custom magnitude calculations, this is the authoritative, last evaluated value of the magnitude */
	UPROPERTY()
	float EvaluatedMagnitude;

	/** These structures are the only ones that should internally be able to update the EvaluatedMagnitude. Any gamecode that gets its hands on FModifierSpec should never be setting EvaluatedMagnitude manually */
	friend struct FGameplayEffectSpec;
	friend struct FActiveGameplayEffectsContainer;
};

/** Saves list of modified attributes, to use for gameplay cues or later processing */
USTRUCT()
struct FGameplayEffectModifiedAttribute
{
	GENERATED_USTRUCT_BODY()

	/** The attribute that has been modified */
	UPROPERTY()
	FGameplayAttribute Attribute;

	/** Total magnitude applied to that attribute */
	UPROPERTY()
	float TotalMagnitude;

	FGameplayEffectModifiedAttribute() : TotalMagnitude(0.0f) {}
};

/** Struct used to hold the result of a gameplay attribute capture; Initially seeded by definition data, but then populated by ability system component when appropriate */
USTRUCT()
struct FGameplayEffectAttributeCaptureSpec
{
	// Allow these as friends so they can seed the aggregator, which we don't otherwise want exposed
	friend struct FActiveGameplayEffectsContainer;
	friend class UAbilitySystemComponent;

	GENERATED_USTRUCT_BODY()

	// Constructors
	UE_API FGameplayEffectAttributeCaptureSpec();
	UE_API FGameplayEffectAttributeCaptureSpec(const FGameplayEffectAttributeCaptureDefinition& InDefinition);

	/**
	 * Returns whether the spec actually has a valid capture yet or not
	 * 
	 * @return True if the spec has a valid attribute capture, false if it does not
	 */
	UE_API bool HasValidCapture() const;

	/**
	 * Attempts to calculate the magnitude of the captured attribute given the specified parameters. Can fail if the spec doesn't have
	 * a valid capture yet.
	 * 
	 * @param InEvalParams	Parameters to evaluate the attribute under
	 * @param OutMagnitude	[OUT] Computed magnitude
	 * 
	 * @return True if the magnitude was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateAttributeMagnitude(const FAggregatorEvaluateParameters& InEvalParams, OUT float& OutMagnitude) const;

	/**
	 * Attempts to calculate the magnitude of the captured attribute given the specified parameters, up to the specified evaluation channel (inclusive).
	 * Can fail if the spec doesn't have a valid capture yet.
	 * 
	 * @param InEvalParams	Parameters to evaluate the attribute under
	 * @param FinalChannel	Evaluation channel to terminate the calculation at
	 * @param OutMagnitude	[OUT] Computed magnitude
	 * 
	 * @return True if the magnitude was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateAttributeMagnitudeUpToChannel(const FAggregatorEvaluateParameters& InEvalParams, EGameplayModEvaluationChannel FinalChannel, OUT float& OutMagnitude) const;

	/**
	 * Attempts to calculate the magnitude of the captured attribute given the specified parameters, including a starting base value. 
	 * Can fail if the spec doesn't have a valid capture yet.
	 * 
	 * @param InEvalParams	Parameters to evaluate the attribute under
	 * @param InBaseValue	Base value to evaluate the attribute under
	 * @param OutMagnitude	[OUT] Computed magnitude
	 * 
	 * @return True if the magnitude was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateAttributeMagnitudeWithBase(const FAggregatorEvaluateParameters& InEvalParams, float InBaseValue, OUT float& OutMagnitude) const;

	/**
	 * Attempts to calculate the base value of the captured attribute given the specified parameters. Can fail if the spec doesn't have
	 * a valid capture yet.
	 * 
	 * @param OutBaseValue	[OUT] Computed base value
	 * 
	 * @return True if the base value was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateAttributeBaseValue(OUT float& OutBaseValue) const;

	/**
	 * Attempts to calculate the "bonus" magnitude (final - base value) of the captured attribute given the specified parameters. Can fail if the spec doesn't have
	 * a valid capture yet.
	 * 
	 * @param InEvalParams		Parameters to evaluate the attribute under
	 * @param OutBonusMagnitude	[OUT] Computed bonus magnitude
	 * 
	 * @return True if the bonus magnitude was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateAttributeBonusMagnitude(const FAggregatorEvaluateParameters& InEvalParams, OUT float& OutBonusMagnitude) const;

	/**
	 * Attempts to calculate the contribution of the specified GE to the captured attribute given the specified parameters. Can fail if the spec doesn't have
	 * a valid capture yet.
	 *
	 * @param InEvalParams		Parameters to evaluate the attribute under
	 * @param ActiveHandle		Handle of the gameplay effect to query about
	 * @param OutBonusMagnitude	[OUT] Computed bonus magnitude
	 *
	 * @return True if the bonus magnitude was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateAttributeContributionMagnitude(const FAggregatorEvaluateParameters& InEvalParams, FActiveGameplayEffectHandle ActiveHandle, OUT float& OutBonusMagnitude) const;

	/**
	 * Attempts to populate the specified aggregator with a snapshot of the backing captured aggregator. Can fail if the spec doesn't have
	 * a valid capture yet.
	 *
	 * @param OutAggregatorSnapshot	[OUT] Snapshotted aggregator, if possible
	 *
	 * @return True if the aggregator was successfully snapshotted, false if it was not
	 */
	UE_API bool AttemptGetAttributeAggregatorSnapshot(OUT FAggregator& OutAggregatorSnapshot) const;

	/**
	 * Attempts to populate the specified aggregator with all of the mods of the backing captured aggregator. Can fail if the spec doesn't have
	 * a valid capture yet.
	 *
	 * @param OutAggregatorToAddTo	[OUT] Aggregator with mods appended, if possible
	 *
	 * @return True if the aggregator had mods successfully added to it, false if it did not
	 */
	UE_API bool AttemptAddAggregatorModsToAggregator(OUT FAggregator& OutAggregatorToAddTo) const;
	
	/** Gathers made for a given capture. Note all mods are returned but only some will be qualified (use Qualified() func to determine) */
	UE_API bool AttemptGatherAttributeMods(const FAggregatorEvaluateParameters& InEvalParams, OUT TMap<EGameplayModEvaluationChannel, const TArray<FAggregatorMod>*>& OutModMap) const;
	
	/** Simple accessor to backing capture definition */
	UE_API const FGameplayEffectAttributeCaptureDefinition& GetBackingDefinition() const;

	/** Register this handle with linked aggregators */
	UE_API void RegisterLinkedAggregatorCallback(FActiveGameplayEffectHandle Handle) const;

	/** Unregister this handle with linked aggregators */
	UE_API void UnregisterLinkedAggregatorCallback(FActiveGameplayEffectHandle Handle) const;
	
	/** Return true if this capture should be recalculated if the given aggregator has changed */
	UE_API bool ShouldRefreshLinkedAggregator(const FAggregator* ChangedAggregator) const;

	/** Swaps any internal references From aggregator To aggregator. Used when cloning */
	UE_API void SwapAggregator(FAggregatorRef From, FAggregatorRef To);
		
private:

	/** Copy of the definition the spec should adhere to for capturing */
	UPROPERTY()
	FGameplayEffectAttributeCaptureDefinition BackingDefinition;

	/** Ref to the aggregator for the captured attribute */
	FAggregatorRef AttributeAggregator;
};

/** Struct used to handle a collection of captured source and target attributes */
USTRUCT()
struct FGameplayEffectAttributeCaptureSpecContainer
{
	GENERATED_USTRUCT_BODY()

public:

	UE_API FGameplayEffectAttributeCaptureSpecContainer();

	UE_API FGameplayEffectAttributeCaptureSpecContainer(FGameplayEffectAttributeCaptureSpecContainer&& Other);

	UE_API FGameplayEffectAttributeCaptureSpecContainer(const FGameplayEffectAttributeCaptureSpecContainer& Other);

	UE_API FGameplayEffectAttributeCaptureSpecContainer& operator=(FGameplayEffectAttributeCaptureSpecContainer&& Other);

	UE_API FGameplayEffectAttributeCaptureSpecContainer& operator=(const FGameplayEffectAttributeCaptureSpecContainer& Other);

	/**
	 * Add a definition to be captured by the owner of the container. Will not add the definition if its exact
	 * match already exists within the container.
	 * 
	 * @param InCaptureDefinition	Definition to capture with
	 */
	UE_API void AddCaptureDefinition(const FGameplayEffectAttributeCaptureDefinition& InCaptureDefinition);

	/**
	 * Capture source or target attributes from the specified component. Should be called by the container's owner.
	 * 
	 * @param InAbilitySystemComponent	Component to capture attributes from
	 * @param InCaptureSource			Whether to capture attributes as source or target
	 */
	UE_API void CaptureAttributes(class UAbilitySystemComponent* InAbilitySystemComponent, EGameplayEffectAttributeCaptureSource InCaptureSource);

	/**
	 * Find a capture spec within the container matching the specified capture definition, if possible.
	 * 
	 * @param InDefinition				Capture definition to use as the search basis
	 * @param bOnlyIncludeValidCapture	If true, even if a spec is found, it won't be returned if it doesn't also have a valid capture already
	 * 
	 * @return The found attribute spec matching the specified search params, if any
	 */
	UE_API const FGameplayEffectAttributeCaptureSpec* FindCaptureSpecByDefinition(const FGameplayEffectAttributeCaptureDefinition& InDefinition, bool bOnlyIncludeValidCapture) const;

	/**
	 * Determines if the container has specs with valid captures for all of the specified definitions.
	 * 
	 * @param InCaptureDefsToCheck	Capture definitions to check for
	 * 
	 * @return True if the container has valid capture attributes for all of the specified definitions, false if it does not
	 */
	UE_API bool HasValidCapturedAttributes(const TArray<FGameplayEffectAttributeCaptureDefinition>& InCaptureDefsToCheck) const;

	/** Returns whether the container has at least one spec w/o snapshotted attributes */
	UE_API bool HasNonSnapshottedAttributes() const;

	/** Registers any linked aggregators to notify this active handle if they are dirtied */
	UE_API void RegisterLinkedAggregatorCallbacks(FActiveGameplayEffectHandle Handle) const;

	/** Unregisters any linked aggregators from notifying this active handle if they are dirtied */
	UE_API void UnregisterLinkedAggregatorCallbacks(FActiveGameplayEffectHandle Handle) const;

	/** Swaps any internal references From aggregator To aggregator. Used when cloning */
	UE_API void SwapAggregator(FAggregatorRef From, FAggregatorRef To);

private:

	/** Captured attributes from the source of a gameplay effect */
	UPROPERTY()
	TArray<FGameplayEffectAttributeCaptureSpec> SourceAttributes;

	/** Captured attributes from the target of a gameplay effect */
	UPROPERTY()
	TArray<FGameplayEffectAttributeCaptureSpec> TargetAttributes;

	/** If true, has at least one capture spec that did not request a snapshot */
	UPROPERTY()
	bool bHasNonSnapshottedAttributes;
};

/**
 * GameplayEffect Specification. Tells us:
 *	-What UGameplayEffect (const data)
 *	-What Level
 *  -Who instigated
 *  
 * FGameplayEffectSpec is modifiable. We start with initial conditions and modifications be applied to it. In this sense, it is stateful/mutable but it
 * is still distinct from an FActiveGameplayEffect which in an applied instance of an FGameplayEffectSpec.
 */
USTRUCT(BlueprintType)
struct FGameplayEffectSpec
{
	GENERATED_USTRUCT_BODY()

	// --------------------------------------------------------------------------------------------------------------------------
	//	IMPORTANT: Any state added to FGameplayEffectSpec must be handled in the move/copy constructor/operator!
	// --------------------------------------------------------------------------------------------------------------------------

	UE_API FGameplayEffectSpec();

	UE_API FGameplayEffectSpec(const UGameplayEffect* InDef, const FGameplayEffectContextHandle& InEffectContext, float Level = FGameplayEffectConstants::INVALID_LEVEL);

	UE_API FGameplayEffectSpec(const FGameplayEffectSpec& Other);

	UE_API FGameplayEffectSpec(const FGameplayEffectSpec& Other, const FGameplayEffectContextHandle& InEffectContext);		//For cloning, copy all attributes, but set a new effectContext.

	UE_API FGameplayEffectSpec(FGameplayEffectSpec&& Other);

	UE_API FGameplayEffectSpec& operator=(FGameplayEffectSpec&& Other);

	UE_API FGameplayEffectSpec& operator=(const FGameplayEffectSpec& Other);

	/** Can be called manually but it is preferred to use the 3 parameter constructor */
	UE_API void Initialize(const UGameplayEffect* InDef, const FGameplayEffectContextHandle& InEffectContext, float Level = FGameplayEffectConstants::INVALID_LEVEL);

	/** Initialize the spec as a linked spec. The original spec's context is preserved except for the original GE asset tags, which are stripped out */
	UE_API void InitializeFromLinkedSpec(const UGameplayEffect* InDef, const FGameplayEffectSpec& OriginalSpec);

	/** Copies SetbyCallerMagnitudes from OriginalSpec into this */
	UE_API void CopySetByCallerMagnitudes(const FGameplayEffectSpec& OriginalSpec);

	/** Copies SetbuCallerMagnitudes, but only if magnitudes don't exist in our map (slower but preserves data) */
	UE_API void MergeSetByCallerMagnitudes(const TMap<FGameplayTag, float>& Magnitudes);

	/**
	 * Determines if the spec has capture specs with valid captures for all of the specified definitions.
	 * 
	 * @param InCaptureDefsToCheck	Capture definitions to check for
	 * 
	 * @return True if the container has valid capture attributes for all of the specified definitions, false if it does not
	 */
	UE_API bool HasValidCapturedAttributes(const TArray<FGameplayEffectAttributeCaptureDefinition>& InCaptureDefsToCheck) const;

	/** Looks for an existing modified attribute struct, may return NULL */
	UE_API const FGameplayEffectModifiedAttribute* GetModifiedAttribute(const FGameplayAttribute& Attribute) const;
	UE_API FGameplayEffectModifiedAttribute* GetModifiedAttribute(const FGameplayAttribute& Attribute);

	/** Adds a new modified attribute struct, will always add so check to see if it exists first */
	UE_API FGameplayEffectModifiedAttribute* AddModifiedAttribute(const FGameplayAttribute& Attribute);

	/**
	 * Helper function to attempt to calculate the duration of the spec from its GE definition
	 * 
	 * @param OutDefDuration	Computed duration of the spec from its GE definition; Not the actual duration of the spec
	 * 
	 * @return True if the calculation was successful, false if it was not
	 */
	UE_API bool AttemptCalculateDurationFromDef(OUT float& OutDefDuration) const;

	/**
	 * Helper function to attempt to calculate the maximum duration of the spec from its GE definition
	 *
	 * @param OutDefMaxDuration	Computed max duration of the spec from its GE definition
	 *
	 * @return True if the calculation was successful, false if it was not
	 */
	UE_API bool AttemptCalculateMaxDurationFromDef(OUT float& OutDefMaxDuration) const;

	/** Sets duration. This should only be called as the GameplayEffect is being created and applied; Ignores calls after attribute capture */
	UE_API void SetDuration(float NewDuration, bool bLockDuration);

	UE_API float GetDuration() const;

	/** Returns the Period for the effect. If DurationPolicy is Instant this will forcibly return NO_PERIOD */
	UE_API float GetPeriod() const;

	/** Sets the stack count for this GE to NewStackCount if stacking is supported. */
	UE_API void SetStackCount(int32 NewStackCount);

	/** Returns the stack count for this GE spec. */
	UE_API int32 GetStackCount() const;

	/** Set the context info: who and where this spec came from. */
	UE_API void SetContext(FGameplayEffectContextHandle NewEffectContext, bool bSkipRecaptureSourceActorTags = false);

	FGameplayEffectContextHandle GetContext() const
	{
		return EffectContext;
	}

	/** Appends all tags granted by this gameplay effect spec */
	UE_API void GetAllGrantedTags(OUT FGameplayTagContainer& OutContainer) const;

	/** Appends all blocked ability tags granted by this gameplay effect spec */
	UE_API void GetAllBlockedAbilityTags(OUT FGameplayTagContainer& OutContainer) const;

	/** Appends all tags that apply to this gameplay effect spec */
	UE_API void GetAllAssetTags(OUT FGameplayTagContainer& OutContainer) const;

	/** Sets the magnitude of a SetByCaller modifier */
	UE_API void SetSetByCallerMagnitude(FName DataName, float Magnitude);

	/** Sets the magnitude of a SetByCaller modifier */
	UE_API void SetSetByCallerMagnitude(FGameplayTag DataTag, float Magnitude);

	/** Returns the magnitude of a SetByCaller modifier. Will return 0.f and Warn if the magnitude has not been set. */
	UE_API float GetSetByCallerMagnitude(FName DataName, bool WarnIfNotFound = true, float DefaultIfNotFound = 0.f) const;

	/** Returns the magnitude of a SetByCaller modifier. Will return 0.f and Warn if the magnitude has not been set. */
	UE_API float GetSetByCallerMagnitude(FGameplayTag DataTag, bool WarnIfNotFound = true, float DefaultIfNotFound = 0.f) const;

	UE_API void SetLevel(float InLevel);

	UE_API float GetLevel() const;

	UE_API void PrintAll() const;

	UE_API FString ToSimpleString() const;

	const FGameplayEffectContextHandle& GetEffectContext() const
	{
		return EffectContext;
	}

	void DuplicateEffectContext()
	{
		EffectContext = EffectContext.Duplicate();
	}

	UE_API void CaptureAttributeDataFromTarget(UAbilitySystemComponent* TargetAbilitySystemComponent);

	/**
	 * Get the computed magnitude of the modifier on the spec with the specified index
	 * 
	 * @param ModifierIdx			Modifier to get
	 * @param bFactorInStackCount	If true, the calculation will include the stack count
	 * 
	 * @return Computed magnitude
	 */
	UE_DEPRECATED(5.6, "Versions using bFactorInStackCount are deprecated, please use GetModifierMagnitude(int32 ModifierIdx) instead.")
	UE_API float GetModifierMagnitude(int32 ModifierIdx, bool bFactorInStackCount) const;

	/**
	 * Get the computed magnitude of the modifier on the spec with the specified index
	 * 
	 * @param ModifierIdx			Modifier to get
	 * 
	 * @return Computed magnitude
	 */
	UE_API float GetModifierMagnitude(int32 ModifierIdx) const;

	/** Fills out the modifier magnitudes inside the Modifier Specs */
	UE_API void CalculateModifierMagnitudes();

	/** Recapture attributes from source and target for cloning */
	UE_API void RecaptureAttributeDataForClone(UAbilitySystemComponent* OriginalASC, UAbilitySystemComponent* NewASC);

	/** Recaptures source actor tags of this spec without modifying anything else */
	UE_API void RecaptureSourceActorTags();

	/** Helper function to initialize all of the capture definitions required by the spec */
	UE_API void SetupAttributeCaptureDefinitions();

	/** Helper function that returns the duration after applying relevant modifiers from the source and target ability system components */
	UE_API float CalculateModifiedDuration() const;

	/** Dynamically add an asset tag not originally from the source GE definition; Added to DynamicAssetTags as well as injected into the captured source spec tags */
	UE_API void AddDynamicAssetTag(const FGameplayTag& TagToAdd);

	/** Dynamically append asset tags not originally from the source GE definition; Added to DynamicAssetTags as well as injected into the captured source spec tags */
	UE_API void AppendDynamicAssetTags(const FGameplayTagContainer& TagsToAppend);

	/** Simple const accessor to the dynamic asset tags */
	UE_API const FGameplayTagContainer& GetDynamicAssetTags() const;

#if ENABLE_VISUAL_LOG
	UE_API FVisualLogStatusCategory GrabVisLogStatus() const;
#endif

private:

	UE_API void CaptureDataFromSource(bool bSkipRecaptureSourceActorTags = false);

	/** Attempts to calculate a duration-related magnitude. Helper function to allow reuse between Duration and MaxDuration. */
	UE_API bool AttemptCalculateDurationRelatedMagnitude(const FGameplayEffectModifierMagnitude& MagnitudeDef, OUT float& OutDurationValue) const;

public:

	/** GameplayEfect definition. The static data that this spec points to. */
	UPROPERTY()
	TObjectPtr<const UGameplayEffect> Def;
	
	/** A list of attributes that were modified during the application of this spec */
	UPROPERTY()
	TArray<FGameplayEffectModifiedAttribute> ModifiedAttributes;
	
	/** Attributes captured by the spec that are relevant to custom calculations, potentially in owned modifiers, etc.; NOT replicated to clients */
	UPROPERTY(NotReplicated)
	FGameplayEffectAttributeCaptureSpecContainer CapturedRelevantAttributes;

	/** other effects that need to be applied to the target if this effect is successful */
	UE_DEPRECATED(5.3, "These TargetEffectSpecs are not replicated, thus can only apply to the server. Use UAdditionalGameplayEffectComponent instead (or roll your own solution)")
	TArray< FGameplayEffectSpecHandle > TargetEffectSpecs;

	/**
	 * The duration in seconds of this effect
	 * instantaneous effects should have a duration of FGameplayEffectConstants::INSTANT_APPLICATION
	 * effects that last forever should have a duration of FGameplayEffectConstants::INFINITE_DURATION
	 */
	UPROPERTY()
	float Duration;

	/** The period in seconds of this effect, nonperiodic effects should have a period of FGameplayEffectConstants::NO_PERIOD */
	UPROPERTY()
	float Period;

	/** Captured Source Tags on GameplayEffectSpec creation */
	UPROPERTY(NotReplicated)
	FTagContainerAggregator	CapturedSourceTags;

	/** Tags from the target, captured during execute */
	UPROPERTY(NotReplicated)
	FTagContainerAggregator	CapturedTargetTags;

	/** Tags that are granted and that did not come from the UGameplayEffect def. These are replicated. */
	UPROPERTY()
	FGameplayTagContainer DynamicGrantedTags;

private:
	/** 
	 * Tags that are on this effect spec and that did not come from the UGameplayEffect def. These are replicated.
	 * To access use AddDynamicAssetTag, AppendDynamicAssetTags, or GetDynamicAssetTags as appropriate, removal is no longer supported
	 */
	UPROPERTY()
	FGameplayTagContainer DynamicAssetTags;

public:
	/** The calculated modifiers for this effect */
	UPROPERTY()
	TArray<FModifierSpec> Modifiers;

private:
	/** Total number of stacks of this effect. Use GetStackCount and SetStackCount to access. */
	UPROPERTY()
	int32 StackCount;

public:
	/** Whether the spec has had its source attribute capture completed or not yet */
	UPROPERTY(NotReplicated)
	uint32 bCompletedSourceAttributeCapture : 1;

	/** Whether the spec has had its target attribute capture completed or not yet */
	UPROPERTY(NotReplicated)
	uint32 bCompletedTargetAttributeCapture : 1;

	/** Whether the duration of the spec is locked or not; If it is, attempts to set it will fail */
	UPROPERTY(NotReplicated)
	uint32 bDurationLocked : 1;

	/** List of abilities granted by this effect */
	UE_DEPRECATED(5.3, "This variable will be removed in favor of (immutable) GASpecs that live on GameplayEffectComponents (e.g. AbilitiesGameplayEffectComponent)")
	UPROPERTY()
	TArray<FGameplayAbilitySpecDef> GrantedAbilitySpecs;

	/** Map of set by caller magnitudes */
	TMap<FName, float>			SetByCallerNameMagnitudes;
	TMap<FGameplayTag, float>	SetByCallerTagMagnitudes;

private:

	/** This tells us how we got here (who / what applied us) */
	UPROPERTY()
	FGameplayEffectContextHandle EffectContext; 

	/** The level this effect was applied at */
	UPROPERTY()
	float Level;	
};


/** This is a cut down version of the gameplay effect spec used for RPCs. */
USTRUCT()
struct FGameplayEffectSpecForRPC
{
	GENERATED_USTRUCT_BODY()

	UE_API FGameplayEffectSpecForRPC();

	UE_API FGameplayEffectSpecForRPC(const FGameplayEffectSpec& InSpec);

	/** GameplayEfect definition. The static data that this spec points to. */
	UPROPERTY()
	TObjectPtr<const UGameplayEffect> Def;

	UPROPERTY()
	TArray<FGameplayEffectModifiedAttribute> ModifiedAttributes;

	UPROPERTY()
	FGameplayEffectContextHandle EffectContext;

	UPROPERTY()
	FGameplayTagContainer AggregatedSourceTags;

	UPROPERTY()
	FGameplayTagContainer AggregatedTargetTags;

	UPROPERTY()
	float Level;

	UPROPERTY()
	float AbilityLevel;

	FGameplayEffectContextHandle GetContext() const
	{
		return EffectContext;
	}

	float GetLevel() const
	{
		return Level;
	}

	float GetAbilityLevel() const
	{
		return AbilityLevel;
	}

	UE_API FString ToSimpleString() const;

	UE_API const FGameplayEffectModifiedAttribute* GetModifiedAttribute(const FGameplayAttribute& Attribute) const;
};


/**
 * Active GameplayEffect instance
 *	-What GameplayEffect Spec
 *	-Start time
 *  -When to execute next
 *  -Replication callbacks
 */
USTRUCT(BlueprintType)
struct FActiveGameplayEffect : public FFastArraySerializerItem
{
	GENERATED_USTRUCT_BODY()

	// ---------------------------------------------------------------------------------------------------------------------------------
	//  IMPORTANT: Any new state added to FActiveGameplayEffect must be handled in the copy/move constructor/operator
	// ---------------------------------------------------------------------------------------------------------------------------------

	FActiveGameplayEffect() = default;
	UE_API FActiveGameplayEffect(FActiveGameplayEffectHandle InHandle, const FGameplayEffectSpec& InSpec, float CurrentWorldTime, float InStartServerWorldTime, FPredictionKey InPredictionKey);

	// These need to be defined because we need to omit PendingNext from move operations and the base class isn't trivially copyable
	UE_API FActiveGameplayEffect(const FActiveGameplayEffect& Other);
	UE_API FActiveGameplayEffect(FActiveGameplayEffect&& Other);
	UE_API FActiveGameplayEffect& operator=(FActiveGameplayEffect&& other);
	UE_API FActiveGameplayEffect& operator=(const FActiveGameplayEffect& other);

	float GetTimeRemaining(float WorldTime) const
	{
		float Duration = GetDuration();
		return (Duration == FGameplayEffectConstants::INFINITE_DURATION ? -1.f : Duration - (WorldTime - StartWorldTime));
	}
	
	float GetDuration() const
	{
		return Spec.GetDuration();
	}

	float GetPeriod() const
	{
		return Spec.GetPeriod();
	}

	float GetEndTime() const
	{
		float Duration = GetDuration();		
		return (Duration == FGameplayEffectConstants::INFINITE_DURATION ? -1.f : Duration + StartWorldTime);
	}

	UE_API void PrintAll() const;

	UE_API void PreReplicatedRemove(const struct FActiveGameplayEffectsContainer &InArray);
	UE_API void PostReplicatedAdd(const struct FActiveGameplayEffectsContainer &InArray);
	UE_API void PostReplicatedChange(const struct FActiveGameplayEffectsContainer &InArray);

	/** Debug string used by Fast Array serialization */
	UE_API FString GetDebugString();

	/** Refreshes the cached StartWorldTime for this effect. To be used when the server/client world time delta changes significantly to keep the start time in sync. */
	UE_API void RecomputeStartWorldTime(const FActiveGameplayEffectsContainer& InArray);

	/** Refreshes the cached StartWorldTime for this effect. To be used when the server/client world time delta changes significantly to keep the start time in sync. */
	UE_API void RecomputeStartWorldTime(const float WorldTime, const float ServerWorldTime);

	bool operator==(const FActiveGameplayEffect& Other)
	{
		return Handle == Other.Handle;
	}

	// ---------------------------------------------------------------------------------------------------------------------------------

	/** Globally unique ID for identify this active gameplay effect. Can be used to look up owner. Not networked. */
	FActiveGameplayEffectHandle Handle;

	UPROPERTY()
	FGameplayEffectSpec Spec;

	UPROPERTY()
	FPredictionKey	PredictionKey;
	
	/** Handles of Gameplay Abilities that were granted to the target by this Active Gameplay Effect */
	UPROPERTY()
	TArray<FGameplayAbilitySpecHandle> GrantedAbilityHandles;

	/** Server time this started */
	UPROPERTY()
	float StartServerWorldTime = 0.0f;

	/** Used for handling duration modifications being replicated */
	UPROPERTY(NotReplicated)
	float CachedStartServerWorldTime = 0.0f;

	UPROPERTY(NotReplicated)
	float StartWorldTime = 0.0f;

	// Not sure if this should replicate or not. If replicated, we may have trouble where IsInhibited doesn't appear to change when we do tag checks (because it was previously inhibited, but replication made it inhibited).
	UPROPERTY(NotReplicated)
	bool bIsInhibited = true;

	/** When replicated down, we cue the GC events until the entire list of active gameplay effects has been received */
	mutable bool bPendingRepOnActiveGC = false;
	mutable bool bPendingRepWhileActiveGC = false;

	bool IsPendingRemove = false;

	/** Last StackCount that the client had. Used to tell if the stackcount has changed in PostReplicatedChange */
	int32 ClientCachedStackCount = 0;

	FTimerHandle PeriodHandle;
	FTimerHandle DurationHandle;

	/** Cached pointer.  Since these ActiveGE's can be reused in-place, this should *not* be copied during copy/move operations */
	FActiveGameplayEffect* PendingNext = nullptr;
	
	/** All the bindable events for this active effect (bundled to allow easier non-const access to these events via the ASC) */
	FActiveGameplayEffectEvents EventSet;

	/** Signifies an active effect that was predicted and has now been replicated back to the client */
	bool bPostPredictObject = false;
};

DECLARE_DELEGATE_RetVal_OneParam(bool, FActiveGameplayEffectQueryCustomMatch, const FActiveGameplayEffect&);
DECLARE_DYNAMIC_DELEGATE_TwoParams(FActiveGameplayEffectQueryCustomMatch_Dynamic, FActiveGameplayEffect, Effect, bool&, bMatches);

/** Every set condition within this query must match in order for the query to match. i.e. individual query elements are ANDed together. */
USTRUCT(BlueprintType)
struct FGameplayEffectQuery
{
	GENERATED_USTRUCT_BODY()

public:
	// ctors and operators
	UE_API FGameplayEffectQuery();
	UE_API FGameplayEffectQuery(const FGameplayEffectQuery& Other);
	UE_API FGameplayEffectQuery(FActiveGameplayEffectQueryCustomMatch InCustomMatchDelegate);
	UE_API FGameplayEffectQuery(FGameplayEffectQuery&& Other);
	UE_API FGameplayEffectQuery& operator=(FGameplayEffectQuery&& Other);
	UE_API FGameplayEffectQuery& operator=(const FGameplayEffectQuery& Other);

	/** Native delegate for providing custom matching conditions. */
	FActiveGameplayEffectQueryCustomMatch CustomMatchDelegate;

	/** BP-exposed delegate for providing custom matching conditions. */
	UPROPERTY(BlueprintReadWrite, Category = Query)
	FActiveGameplayEffectQueryCustomMatch_Dynamic CustomMatchDelegate_BP;

	/** Query that is matched against tags this GE gives */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Query)
	FGameplayTagQuery OwningTagQuery;

	/** Query that is matched against tags this GE has */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Query)
	FGameplayTagQuery EffectTagQuery;

	/** Query that is matched against spec tags the source of this GE has */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Query, DisplayName = SourceSpecTagQuery)
	FGameplayTagQuery SourceTagQuery;

	/** Query that is matched against all tags the source of this GE has */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Query)
	FGameplayTagQuery SourceAggregateTagQuery;

	/** Matches on GameplayEffects which modify given attribute. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Query)
	FGameplayAttribute ModifyingAttribute;

	/** Matches on GameplayEffects which come from this source */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Query)
	TObjectPtr<const UObject> EffectSource;

	/** Matches on GameplayEffects with this definition */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Query)
	TSubclassOf<UGameplayEffect> EffectDefinition;

	/** Handles to ignore as matches, even if other criteria is met */
	TArray<FActiveGameplayEffectHandle> IgnoreHandles;

	/** Returns true if Effect matches all specified criteria of this query, including CustomMatch delegates if bound. Returns false otherwise. */
	UE_API bool Matches(const FActiveGameplayEffect& Effect) const;

	/** Returns true if Effect matches all specified criteria of this query. This does NOT check FActiveGameplayEffectQueryCustomMatch since this is performed on the spec (possibly prior to applying).
	 *	Note: it would be reasonable to support a custom delegate that operated on the FGameplayEffectSpec itself.
	 */
	UE_API bool Matches(const FGameplayEffectSpec& Effect) const;

	/** Returns true if the query is empty/default. E.g., it has no data set. */
	UE_API bool IsEmpty() const;

	// Shortcuts for easily creating common query types 

	/** Creates an effect query that will match if there are any common tags between the given tags and an ActiveGameplayEffect's owning tags */
	static UE_API FGameplayEffectQuery MakeQuery_MatchAnyOwningTags(const FGameplayTagContainer& InTags);
	/** Creates an effect query that will match if all of the given tags are in the ActiveGameplayEffect's owning tags */
	static UE_API FGameplayEffectQuery MakeQuery_MatchAllOwningTags(const FGameplayTagContainer& InTags);
	/** Creates an effect query that will match if there are no common tags between the given tags and an ActiveGameplayEffect's owning tags */
	static UE_API FGameplayEffectQuery MakeQuery_MatchNoOwningTags(const FGameplayTagContainer& InTags);
	
	/** Creates an effect query that will match if there are any common tags between the given tags and an ActiveGameplayEffect's tags */
	static UE_API FGameplayEffectQuery MakeQuery_MatchAnyEffectTags(const FGameplayTagContainer& InTags);
	/** Creates an effect query that will match if all of the given tags are in the ActiveGameplayEffect's tags */
	static UE_API FGameplayEffectQuery MakeQuery_MatchAllEffectTags(const FGameplayTagContainer& InTags);
	/** Creates an effect query that will match if there are no common tags between the given tags and an ActiveGameplayEffect's tags */
	static UE_API FGameplayEffectQuery MakeQuery_MatchNoEffectTags(const FGameplayTagContainer& InTags);

	/** Creates an effect query that will match if there are any common tags between the given tags and an ActiveGameplayEffect's source spec tags */
	static UE_API FGameplayEffectQuery MakeQuery_MatchAnySourceSpecTags(const FGameplayTagContainer& InTags);
	/** Creates an effect query that will match if all of the given tags are in the ActiveGameplayEffect's source spec tags */
	static UE_API FGameplayEffectQuery MakeQuery_MatchAllSourceSpecTags(const FGameplayTagContainer& InTags);
	/** Creates an effect query that will match if there are no common tags between the given tags and an ActiveGameplayEffect's source spec tags */
	static UE_API FGameplayEffectQuery MakeQuery_MatchNoSourceSpecTags(const FGameplayTagContainer& InTags);

	UE_API bool operator==(const FGameplayEffectQuery& Other) const;
	UE_API bool operator!=(const FGameplayEffectQuery& Other) const;
};

/**
 *	Generic querying data structure for active GameplayEffects. Lets us ask things like:
 *		Give me duration/magnitude of active gameplay effects with these tags
 *		Give me handles to all activate gameplay effects modifying this attribute.
 *		
 *	Any requirements specified in the query are required: must meet "all" not "one".
 */
USTRUCT()
struct FActiveGameplayEffectQuery
{
	GENERATED_USTRUCT_BODY()

	FActiveGameplayEffectQuery()
		: OwningTagContainer(nullptr)
		, EffectTagContainer(nullptr)
		, OwningTagContainer_Rejection(nullptr)
		, EffectTagContainer_Rejection(nullptr)
		, EffectSource(nullptr)
		, EffectDef(nullptr)
	{
	}

	FActiveGameplayEffectQuery(const FGameplayTagContainer* InOwningTagContainer)
		: OwningTagContainer(InOwningTagContainer)
		, EffectTagContainer(nullptr)
		, OwningTagContainer_Rejection(nullptr)
		, EffectTagContainer_Rejection(nullptr)
		, EffectSource(nullptr)
		, EffectDef(nullptr)
	{
	}

	/** Bind this to override the default query-matching code. */
	FActiveGameplayEffectQueryCustomMatch CustomMatch;

	/** Returns true if Effect matches the criteria of this query, which will be overridden by CustomMatch if it is bound. Returns false otherwise. */
	bool Matches(const FActiveGameplayEffect& Effect) const;

	/** used to match with InheritableOwnedTagsContainer */
	const FGameplayTagContainer* OwningTagContainer;

	/** used to match with InheritableGameplayEffectTags */
	const FGameplayTagContainer* EffectTagContainer;

	/** used to reject matches with InheritableOwnedTagsContainer */
	const FGameplayTagContainer* OwningTagContainer_Rejection;

	/** used to reject matches with InheritableGameplayEffectTags */
	const FGameplayTagContainer* EffectTagContainer_Rejection;

	/** Matches on GameplayEffects which modify given attribute */
	FGameplayAttribute ModifyingAttribute;

	/** Matches on GameplayEffects which come from this source */
	const UObject* EffectSource;

	/** Matches on GameplayEffects with this definition */
	const UGameplayEffect* EffectDef;

	/** Handles to ignore as matches, even if other criteria is met */
	TArray<FActiveGameplayEffectHandle> IgnoreHandles;
};

/** Helper struct to hold data about external dependencies for custom modifiers */
struct FCustomModifierDependencyHandle
{
	FCustomModifierDependencyHandle()
		: ActiveEffectHandles()
		, ActiveDelegateHandle()
	{}

	/** Set of handles of active gameplay effects dependent upon a particular external dependency */
	TSet<FActiveGameplayEffectHandle> ActiveEffectHandles;

	/** Delegate handle populated as a result of binding to an external dependency delegate */
	FDelegateHandle ActiveDelegateHandle;
};

/**
 * Active GameplayEffects Container
 *	-Bucket of ActiveGameplayEffects
 *	-Needed for FFastArraySerialization
 *  
 * This should only be used by UAbilitySystemComponent. All of this could just live in UAbilitySystemComponent except that we need a distinct USTRUCT to implement FFastArraySerializer.
 *
 * The preferred way to iterate through the ActiveGameplayEffectContainer is with CreateConstIterator/CreateIterator or stl style range iteration:
 * 
 *	for (const FActiveGameplayEffect& Effect : this) {}
 *	for (auto It = CreateConstIterator(); It; ++It) {}
 *
 */
USTRUCT()
struct FActiveGameplayEffectsContainer : public FFastArraySerializer
{
	GENERATED_USTRUCT_BODY();

	friend struct FActiveGameplayEffect;
	friend class UAbilitySystemComponent;
	friend struct FScopedActiveGameplayEffectLock;
	friend class AAbilitySystemDebugHUD;
	friend class FActiveGameplayEffectIterator<const FActiveGameplayEffect, FActiveGameplayEffectsContainer>;
	friend class FActiveGameplayEffectIterator<FActiveGameplayEffect, FActiveGameplayEffectsContainer>;

	typedef FActiveGameplayEffectIterator<const FActiveGameplayEffect, FActiveGameplayEffectsContainer> ConstIterator;
	typedef FActiveGameplayEffectIterator<FActiveGameplayEffect, FActiveGameplayEffectsContainer> Iterator;

	UE_API FActiveGameplayEffectsContainer();
	UE_API ~FActiveGameplayEffectsContainer();

	UAbilitySystemComponent* Owner;
	bool OwnerIsNetAuthority;

	FOnGivenActiveGameplayEffectRemoved	OnActiveGameplayEffectRemovedDelegate;

	struct DebugExecutedGameplayEffectData
	{
		FString GameplayEffectName;
		FString ActivationState;
		FGameplayAttribute Attribute;
		TEnumAsByte<EGameplayModOp::Type> ModifierOp;
		float Magnitude;
		int32 StackCount;
	};

#if ENABLE_VISUAL_LOG
	/** Stores a record of gameplay effects that have executed and their results. Useful for debugging */
	TArray<DebugExecutedGameplayEffectData> DebugExecutedGameplayEffects;

	/** Report our current state to the VisLog */
	UE_API void DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const;
#endif // ENABLE_VISUAL_LOG

	UE_API void GetActiveGameplayEffectDataByAttribute(TMultiMap<FGameplayAttribute, FActiveGameplayEffectsContainer::DebugExecutedGameplayEffectData>& EffectMap) const;

	UE_API void RegisterWithOwner(UAbilitySystemComponent* Owner);	
	
	UE_API FActiveGameplayEffect* ApplyGameplayEffectSpec(const FGameplayEffectSpec& Spec, FPredictionKey& InPredictionKey, bool& bFoundExistingStackableGE);

	UE_API FActiveGameplayEffect* GetActiveGameplayEffect(const FActiveGameplayEffectHandle Handle);

	UE_API const FActiveGameplayEffect* GetActiveGameplayEffect(const FActiveGameplayEffectHandle Handle) const;

	/** Predictively execute a given effect spec. Any attribute modifications and effect execution calculations in the effect will run and then if desired predict gameplay cues
		@note: This method will not predictively run any conditional effects that may be set up in the effect that apply post execution and will only happen if/when this spec is
		applied on the server. 
		
		@note: WARNING: This will locally perform attribute changes on your client so beware. */
	UE_API void PredictivelyExecuteEffectSpec(FGameplayEffectSpec& Spec, FPredictionKey PredictionKey, const bool bPredictGameplayCues = false);

	UE_API void ExecuteActiveEffectsFrom(FGameplayEffectSpec &Spec, FPredictionKey PredictionKey = FPredictionKey() );
	
	UE_API void ExecutePeriodicGameplayEffect(FActiveGameplayEffectHandle Handle);	// This should not be outward facing to the skill system API, should only be called by the owning AbilitySystemComponent

	UE_API bool RemoveActiveGameplayEffect(FActiveGameplayEffectHandle Handle, int32 StacksToRemove, bool bPredictionRejected = false);

	UE_API void GetGameplayEffectStartTimeAndDuration(FActiveGameplayEffectHandle Handle, float& EffectStartTime, float& EffectDuration) const;

	UE_API float GetGameplayEffectMagnitude(FActiveGameplayEffectHandle Handle, FGameplayAttribute Attribute) const;

	UE_API void SetActiveGameplayEffectLevel(FActiveGameplayEffectHandle ActiveHandle, int32 NewLevel);

	/**
	 * Update a set-by-caller magnitude for the active effect to match the new value, if possible
	 * 
	 * @param ActiveHandle		Handle of the active effect to update
	 * @param SetByCallerTag	Set-by-caller tag to update
	 * @param NewValue			New value of the set-by-caller magnitude
	 */
	UE_API void UpdateActiveGameplayEffectSetByCallerMagnitude(FActiveGameplayEffectHandle ActiveHandle, const FGameplayTag& SetByCallerTag, float NewValue);

	/**
	 * Update set-by-caller magnitudes for the active effect to match the new values, if possible; Replaces existing values
	 *
	 * @param ActiveHandle			Handle of the active effect to update
	 * @param NewSetByCallerValues	Map of set-by-caller tag to new magnitude
	 */
	UE_API void UpdateActiveGameplayEffectSetByCallerMagnitudes(FActiveGameplayEffectHandle ActiveHandle, const TMap<FGameplayTag, float>& NewSetByCallerValues);

	UE_API void SetAttributeBaseValue(FGameplayAttribute Attribute, float NewBaseValue);

	UE_API float GetAttributeBaseValue(FGameplayAttribute Attribute) const;

	UE_API float GetEffectContribution(const FAggregatorEvaluateParameters& Parameters, FActiveGameplayEffectHandle ActiveHandle, FGameplayAttribute Attribute);

	/** Actually applies given mod to the attribute */
	UE_API void ApplyModToAttribute(const FGameplayAttribute &Attribute, TEnumAsByte<EGameplayModOp::Type> ModifierOp, float ModifierMagnitude, const FGameplayEffectModCallbackData* ModData=nullptr);

	/**
	 * Get the source tags from the gameplay spec represented by the specified handle, if possible
	 * 
	 * @param Handle	Handle of the gameplay effect to retrieve source tags from
	 * 
	 * @return Source tags from the gameplay spec represented by the handle, if possible
	 */
	UE_API const FGameplayTagContainer* GetGameplayEffectSourceTagsFromHandle(FActiveGameplayEffectHandle Handle) const;

	/**
	 * Get the target tags from the gameplay spec represented by the specified handle, if possible
	 * 
	 * @param Handle	Handle of the gameplay effect to retrieve target tags from
	 * 
	 * @return Target tags from the gameplay spec represented by the handle, if possible
	 */
	UE_API const FGameplayTagContainer* GetGameplayEffectTargetTagsFromHandle(FActiveGameplayEffectHandle Handle) const;

	/**
	 * Populate the specified capture spec with the data necessary to capture an attribute from the container
	 * 
	 * @param OutCaptureSpec	[OUT] Capture spec to populate with captured data
	 */
	UE_API void CaptureAttributeForGameplayEffect(OUT FGameplayEffectAttributeCaptureSpec& OutCaptureSpec);

	UE_API void PrintAllGameplayEffects() const;

	/**
	 *	Returns the total number of gameplay effects.
	 *	NOTE this does include GameplayEffects that pending removal.
	 *	Any pending remove gameplay effects are deleted at the end of their scope lock
	 */
	inline int32 GetNumGameplayEffects() const
	{
		int32 NumPending = 0;
		FActiveGameplayEffect* PendingGameplayEffect = PendingGameplayEffectHead;
		FActiveGameplayEffect* Stop = *PendingGameplayEffectNext;
		while (PendingGameplayEffect && PendingGameplayEffect != Stop)
		{
			++NumPending;
			PendingGameplayEffect = PendingGameplayEffect->PendingNext;
		}

		return GameplayEffects_Internal.Num() + NumPending;
	}

	UE_API void CheckDuration(FActiveGameplayEffectHandle Handle);

	/** Returns which ELifetimeCondition can be used for this instance to replicate to relevant connections. This can be used to change the condition if the property is set to use COND_Dynamic in an object's GetLifetimeReplicatedProps implementation. */
	UE_API ELifetimeCondition GetReplicationCondition() const;

	/** Set whether the container is using COND_Dynamic and setting the proper condition at runtime. */
	void SetIsUsingReplicationCondition(bool bInIsUsingReplicationCondition) { bIsUsingReplicationCondition = bInIsUsingReplicationCondition; }

	/** Return whether the container is using COND_Dynamic and setting the proper condition at runtime. */
	bool IsUsingReplicationCondition() const { return bIsUsingReplicationCondition; }

	UE_API bool NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms);

	UE_API void Uninitialize();

	UE_API bool CanApplyAttributeModifiers(const UGameplayEffect *GameplayEffect, float Level, const FGameplayEffectContextHandle& EffectContext);
	
	UE_API TArray<float> GetActiveEffectsTimeRemaining(const FGameplayEffectQuery& Query) const;

	UE_API TArray<float> GetActiveEffectsDuration(const FGameplayEffectQuery& Query) const;

	UE_API TArray<TPair<float,float>> GetActiveEffectsTimeRemainingAndDuration(const FGameplayEffectQuery& Query) const;

	UE_API TArray<FActiveGameplayEffectHandle> GetActiveEffects(const FGameplayEffectQuery& Query) const;

	UE_API float GetActiveEffectsEndTime(const FGameplayEffectQuery& Query, TArray<AActor*>& Instigators) const;
	UE_API bool GetActiveEffectsEndTimeAndDuration(const FGameplayEffectQuery& Query, float& EndTime, float& Duration, TArray<AActor*>& Instigators) const;

	/** Returns an array of all of the active gameplay effect handles */
	UE_API TArray<FActiveGameplayEffectHandle> GetAllActiveEffectHandles() const;

	UE_API void ModifyActiveEffectStartTime(FActiveGameplayEffectHandle Handle, float StartTimeDiff);

	UE_API int32 RemoveActiveEffects(const FGameplayEffectQuery& Query, int32 StacksToRemove);

	/**
	 * Get the count of the effects matching the specified query (including stack count)
	 * 
	 * @return Count of the effects matching the specified query
	 */
	UE_API int32 GetActiveEffectCount(const FGameplayEffectQuery& Query, bool bEnforceOnGoingCheck = true) const;

	UE_API bool IsServerWorldTimeAvailable() const;

	UE_API float GetServerWorldTime() const;

	UE_API float GetWorldTime() const;

	UE_API bool HasReceivedEffectWithPredictedKey(FPredictionKey PredictionKey) const;

	UE_API bool HasPredictedEffectWithPredictedKey(FPredictionKey PredictionKey) const;
	
	UE_DEPRECATED(5.5, "Replaced by private SetBaseAttributeValueFromReplication that uses FGameplayAttributeData")
	UE_API void SetBaseAttributeValueFromReplication(const FGameplayAttribute& Attribute, float NewBaseValue, float OldBaseValue);

	UE_API void GetAllActiveGameplayEffectSpecs(TArray<FGameplayEffectSpec>& OutSpecCopies) const;

	UE_API void DebugCyclicAggregatorBroadcasts(struct FAggregator* Aggregator);

	/** Performs a deep copy on the source container, duplicating all gameplay effects and reconstructing the attribute aggregator map to match the passed in source. */
	UE_API void CloneFrom(const FActiveGameplayEffectsContainer& Source);

	// -------------------------------------------------------------------------------------------

	UE_DEPRECATED(4.17, "Use GetGameplayAttributeValueChangeDelegate (the delegate signature has changed)")
	UE_API FOnGameplayAttributeChange& RegisterGameplayAttributeEvent(FGameplayAttribute Attribute);

	UE_API FOnGameplayAttributeValueChange& GetGameplayAttributeValueChangeDelegate(FGameplayAttribute Attribute);

	UE_API void IncrementLock();
	UE_API void DecrementLock();
	
	inline ConstIterator CreateConstIterator() const { return ConstIterator(*this);	}
	inline Iterator CreateIterator() { return Iterator(*this);	}

	/** Recomputes the start time for all active abilities */
	UE_API void RecomputeStartWorldTimes(const float WorldTime, const float ServerWorldTime);

	/** Called every time data has been modified by the FastArraySerializer */
	UE_API void PostReplicatedReceive(const FFastArraySerializer::FPostReplicatedReceiveParameters& Parameters);

private:
	UE_API void SetBaseAttributeValueFromReplication(const FGameplayAttribute& Attribute, const FGameplayAttributeData& NewValue, const FGameplayAttributeData& OldValue);

	/**
	 *	Accessors for internal functions to get GameplayEffects directly by index.
	 *	Note this will return GameplayEffects that are pending removal!
	 *	
	 *	To iterate over all 'valid' gameplay effects, use the CreateConstIterator/CreateIterator or the stl style range iterator
	 */
	inline const FActiveGameplayEffect* GetActiveGameplayEffect(int32 idx) const
	{
		return const_cast<FActiveGameplayEffectsContainer*>(this)->GetActiveGameplayEffect(idx);
	}

	inline FActiveGameplayEffect* GetActiveGameplayEffect(int32 idx)
	{
		if (idx < GameplayEffects_Internal.Num())
		{
			return &GameplayEffects_Internal[idx];
		}

		idx -= GameplayEffects_Internal.Num();
		FActiveGameplayEffect* Ptr = PendingGameplayEffectHead;
		FActiveGameplayEffect* Stop = *PendingGameplayEffectNext;

		// Advance until the desired index or until hitting the actual end of the pending list currently in use (need to check both Ptr and Ptr->PendingNext to prevent hopping
		// the pointer too far along)
		while (idx-- > 0 && Ptr && Ptr != Stop && Ptr->PendingNext != Stop)
		{
			Ptr = Ptr->PendingNext;
		}

		return idx <= 0 ? Ptr : nullptr;
	}

	/** Our active list of Effects. Do not access this directly (Even from internal functions!) Use GetNumGameplayEffect() / GetGameplayEffect() ! */
	UPROPERTY()
	TArray<FActiveGameplayEffect>	GameplayEffects_Internal;

	UE_API void InternalUpdateNumericalAttribute(FGameplayAttribute Attribute, float NewValue, const FGameplayEffectModCallbackData* ModData, bool bFromRecursiveCall=false);

	/** Cached pointer to current mod data needed for callbacks. We cache it in the AGE struct to avoid passing it through all the delegate/aggregator plumbing */
	const struct FGameplayEffectModCallbackData* CurrentModcallbackData;
	
	/**
	 * Helper function to execute a mod on owned attributes
	 * 
	 * @param Spec			Gameplay effect spec executing the mod
	 * @param ModEvalData	Evaluated data for the mod
	 * 
	 * @return True if the mod successfully executed, false if it did not
	 */
	UE_API bool InternalExecuteMod(FGameplayEffectSpec& Spec, FGameplayModifierEvaluatedData& ModEvalData);
public:
	bool IsNetAuthority() const
	{
		return OwnerIsNetAuthority;
	}
private:
	UE_API void InternalExecutePeriodicGameplayEffect(FActiveGameplayEffect& ActiveEffect);

	/** Called internally to actually remove a GameplayEffect or to reduce its StackCount. Returns true if we resized our internal GameplayEffect array. */
	UE_API bool InternalRemoveActiveGameplayEffect(int32 Idx, int32 StacksToRemove, bool bPrematureRemoval, bool bPredictionRejected = false);
	
	/** Called both in server side creation and replication creation/deletion */
	UE_API void InternalOnActiveGameplayEffectAdded(FActiveGameplayEffect& Effect, bool bInvokePredictedEffects);
	UE_API void InternalOnActiveGameplayEffectRemoved(FActiveGameplayEffect& Effect, bool bInvokePredictedEffects, const FGameplayEffectRemovalInfo& GameplayEffectRemovalInfo);

	UE_API void RemoveActiveGameplayEffectGrantedTagsAndModifiers(const FActiveGameplayEffect& Effect, bool bInvokePredictedEffects, bool bPredictionRejected = false);
	UE_API void AddActiveGameplayEffectGrantedTagsAndModifiers(FActiveGameplayEffect& Effect, bool bInvokePredictedEffects);

	/** Updates tag dependency map when a GameplayEffect is removed */
	UE_API void RemoveActiveEffectTagDependency(const FGameplayTagContainer& Tags, FActiveGameplayEffectHandle Handle);

	/** Internal helper function to bind the active effect to all of the custom modifier magnitude external dependency delegates it contains, if any */
	UE_API void AddCustomMagnitudeExternalDependencies(FActiveGameplayEffect& Effect);

	/** Internal helper function to unbind the active effect from all of the custom modifier magnitude external dependency delegates it may have bound to, if any */
	UE_API void RemoveCustomMagnitudeExternalDependencies(FActiveGameplayEffect& Effect);

	/** Internal callback fired as a result of a custom modifier magnitude external dependency delegate firing; Updates affected active gameplay effects as necessary */
	UE_API void OnCustomMagnitudeExternalDependencyFired(TSubclassOf<UGameplayModMagnitudeCalculation> MagnitudeCalculationClass);

	/** 
	 * Updates the active GE's start time to current world time. Despite what the function name suggests, duration is not refreshed or re-calculated here.
	 * Only queries to the active effect's remaining time are affected. Call-sites should update duration on the GE spec if relevant, and should ensure that 
	 * a new expiration timer is set in the world timer manager that fires at T = (new) start time + (new) duration.
	 */
	UE_API void RestartActiveGameplayEffectDuration(FActiveGameplayEffect& ActiveGameplayEffect);

	// -------------------------------------------------------------------------------------------

	TMap<FGameplayAttribute, FAggregatorRef>		AttributeAggregatorMap;

	// DEPRECATED: use AttributeValueChangeDelegates
	TMap<FGameplayAttribute, FOnGameplayAttributeChange> AttributeChangeDelegates;
	
	TMap<FGameplayAttribute, FOnGameplayAttributeValueChange> AttributeValueChangeDelegates;

	/** Mapping of custom gameplay modifier magnitude calculation class to dependency handles for triggering updates on external delegates firing */
	TMap<FObjectKey, FCustomModifierDependencyHandle> CustomMagnitudeClassDependencies;

	/** A map to manage stacking while we are the source */
	TMap<TWeakObjectPtr<UGameplayEffect>, TArray<FActiveGameplayEffectHandle> >	SourceStackingMap;
	
	UE_API FAggregatorRef& FindOrCreateAttributeAggregator(const FGameplayAttribute& Attribute);

	UE_API void CleanupAttributeAggregator(const FGameplayAttribute& Attribute);

	UE_API void OnAttributeAggregatorDirty(FAggregator* Aggregator, FGameplayAttribute Attribute, bool FromRecursiveCall=false);

	UE_API void OnMagnitudeDependencyChange(FActiveGameplayEffectHandle Handle, const FAggregator* ChangedAgg);

	UE_API void OnStackCountChange(FActiveGameplayEffect& ActiveEffect, int32 OldStackCount, int32 NewStackCount);

	UE_API void OnPredictiveGameplayEffectStackCaughtUp(FActiveGameplayEffectHandle Handle);

	UE_API void OnDurationChange(FActiveGameplayEffect& ActiveEffect);

	UE_API void UpdateAllAggregatorModMagnitudes(FActiveGameplayEffect& ActiveEffect);

	UE_API void UpdateAggregatorModMagnitudes(const TSet<FGameplayAttribute>& AttributesToUpdate, FActiveGameplayEffect& ActiveEffect);

	/** Helper function to find the active GE that the specified spec can stack with, if any */
	UE_API FActiveGameplayEffect* FindStackableActiveGameplayEffect(const FGameplayEffectSpec& Spec);
	
	/** Helper function to handle the case of same-effect stacking overflow; Returns true if the overflow application should apply, false if it should not */
	UE_API bool HandleActiveGameplayEffectStackOverflow(const FActiveGameplayEffect& ActiveStackableGE, const FGameplayEffectSpec& OldSpec, const FGameplayEffectSpec& OverflowingSpec, FPredictionKey PredictionKey = {});

	UE_API bool ShouldUseMinimalReplication();

	void DeletePendingActiveGameplayEffect();

	mutable int32 ScopedLockCount;
	int32 PendingRemoves;
	uint32 NumConsecutiveUnmappedReferencesDebug = 0;

	FActiveGameplayEffect*	PendingGameplayEffectHead;	// Head of pending GE linked list
	FActiveGameplayEffect** PendingGameplayEffectNext;	// Points to the where to store the next pending GE (starts pointing at head, as more are added, points further down the list).

	uint8 bIsUsingReplicationCondition : 1;

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */

	inline friend Iterator begin(FActiveGameplayEffectsContainer* Container) { return Container->CreateIterator(); }
	inline friend Iterator end(FActiveGameplayEffectsContainer* Container) { return Iterator(*Container, -1); }

	inline friend ConstIterator begin(const FActiveGameplayEffectsContainer* Container) { return Container->CreateConstIterator(); }
	inline friend ConstIterator end(const FActiveGameplayEffectsContainer* Container) { return ConstIterator(*Container, -1); }
};

template<>
struct TStructOpsTypeTraits< FActiveGameplayEffectsContainer > : public TStructOpsTypeTraitsBase2< FActiveGameplayEffectsContainer >
{
	enum
	{
		WithNetDeltaSerializer = true,
		WithCopy = false,
	};
};

/**
 *	FScopedActiveGameplayEffectLock
 *	Provides a mechanism for locking the active gameplay effect list while possibly invoking callbacks into gamecode.
 *	For example, if some internal code in FActiveGameplayEffectsContainer is iterating through the active GE list
 *	or holding onto a pointer to something in that list, any changes to that list could cause memory the move out from underneath.
 *	
 *	This scope lock will queue deletions and additions until after the scope is over. The additions and deletions will actually 
 *	go through, but we will defer the memory operations to the active gameplay effect list.
 */
struct FScopedActiveGameplayEffectLock
{
	UE_API FScopedActiveGameplayEffectLock(FActiveGameplayEffectsContainer& InContainer);
	UE_API ~FScopedActiveGameplayEffectLock();

private:
	FActiveGameplayEffectsContainer& Container;
};

#define GAMEPLAYEFFECT_SCOPE_LOCK()	FScopedActiveGameplayEffectLock ActiveScopeLock(*this);


// -------------------------------------------------------------------------------------

/**
 * Gameplay Effects Data needs to be versioned (e.g. going from Monolithic to Modular)
 * Unfortunately, the Custom Package Versioning doesn't work for this case, because we're upgrading
 * outside of the Serialize function.  For that reason we want to store a UProperty that says what version
 * we're on.  Unfortunately that propagates from Parent to Child (inheritance rules) so to overcome that,
 * we have a special UStruct that always serializes its data (so it will always be loaded, not inherited).
 * Here is how to do that:
 *	1. Return false in Identical (which effectively disables delta serialization).
 *  2. Reset the value in PostInitProperties, thereby not using the inherited value on the initial preload.
 *  3. Ensure the CurrentVersion field isn't marked up as a UPROPERTY to avoid automatic copying between parent/child.
 *  4. Implement Serialize to ensure the CurrentVersion is still serialized.
 */
USTRUCT()
struct FGameplayEffectVersion
{
	GENERATED_BODY()

	/** The version the owning GameplayEffect is currently set to */
	EGameplayEffectVersion CurrentVersion;

	/** By always returning false here, we can disable delta serialization */
	bool Identical(const FGameplayEffectVersion* Other, uint32 PortFlags) const { return false; }

	/** We must use a custom serializer to make sure CurrentVersion serializes properly (without markup to avoid unwanted copying) */
	bool Serialize(FStructuredArchive::FSlot Slot);
};

template<>
struct TStructOpsTypeTraits<FGameplayEffectVersion> : public TStructOpsTypeTraitsBase2<FGameplayEffectVersion>
{
	enum
	{
		WithIdentical = true,
		WithStructuredSerializer = true,
	};
};

/**
 * UGameplayEffect
 *	The GameplayEffect definition. This is the data asset defined in the editor that drives everything.
 *  This is only blueprintable to allow for templating gameplay effects. Gameplay effects should NOT contain blueprint graphs.
 */
UCLASS(Blueprintable, PrioritizeCategories="Status Duration GameplayEffect GameplayCues Stacking", meta = (ShortTooltip = "A GameplayEffect modifies attributes and tags."), MinimalAPI)
class UGameplayEffect : public UObject, public IGameplayTagAssetInterface
{

public:
	GENERATED_UCLASS_BODY()

	// These are deprecated but remain for backwards compat, please use FGameplayEffectConstants:: instead.
	static UE_API const float INFINITE_DURATION;
	static UE_API const float INSTANT_APPLICATION;
	static UE_API const float NO_PERIOD;
	static UE_API const float INVALID_LEVEL;

	UE_DEPRECATED(5.3, "The implementation and method name did not match.  Use GetGrantedTags() to get the tags Granted to the Actor this GameplayEffect is applied to.")
	UE_API virtual void GetOwnedGameplayTags(FGameplayTagContainer& TagContainer) const override;

	UE_DEPRECATED(5.3, "The implementation and method name did not match.  Use GetGrantedTags().HasTag() to check against the tags this GameplayEffect will Grant to the Actor.")
	UE_API virtual bool HasMatchingGameplayTag(FGameplayTag TagToCheck) const override;

	UE_DEPRECATED(5.3, "The implementation and method name did not match.  Use GetGrantedTags().HasAll() to check against the tags this GameplayEffect will Grant to the Actor.")
	UE_API virtual bool HasAllMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const override;

	UE_DEPRECATED(5.3, "The implementation and method name did not match.  Use GetGrantedTags().HasAny() to check against the tags this GameplayEffect will Grant to the Actor.")
	UE_API virtual bool HasAnyMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const override;

	UE_API virtual void GetBlockedAbilityTags(FGameplayTagContainer& OutTagContainer) const;

	/** Needed to properly disable inheriting the version value from its parent. */
	UE_API virtual void PostInitProperties() override;

	/** PostLoad gets called once after the asset has been loaded.  It does not get called again on Blueprint recompile (@see PostCDOCompiled) */
	UE_API virtual void PostLoad() override;

	/** Called when the Gameplay Effect has finished loading.  This is used to catch both PostLoad (the initial load) and PostCDOCompiled (any subsequent changes) */
	UE_API virtual void OnGameplayEffectChanged();

#if WITH_EDITOR
	/** Do our upgrades in PostCDOCompiled */
	UE_API virtual void PostCDOCompiled(const FPostCDOCompiledContext& Context) override;

	/** We need to fix-up all of the SubObjects manually since the Engine doesn't fully support this */
	UE_API void PostCDOCompiledFixupSubobjects();

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/**
	 * Can we Apply this Gameplay Effect?  Note: Apply is the generic term for adding to the active container or executing it.
	 */
	UE_API bool CanApply(const FActiveGameplayEffectsContainer& ActiveGEContainer, const FGameplayEffectSpec& GESpec) const;

	/**
	 * Receive a notify that this GameplayEffect has been added to an Active Container.  ActiveGE will be the Active version of this GameplayEffect.
	 * Returns true if this GameplayEffect should be active (or false to inhibit).
	 */
	UE_API bool OnAddedToActiveContainer(FActiveGameplayEffectsContainer& ActiveGEContainer, FActiveGameplayEffect& ActiveGE) const;

	/**
     * Receive a notify that this GameplayEffect has been executed (it must be instant, as it is not added to the Container).
     * Since it is not added to the container, it will not have an associated FActiveGameplayEffect, just the FGameplayEffectSpec.
     */
	UE_API void OnExecuted(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const;

	/**
	 * Receive a notify that this GameplayEffect has been applied (this GE is either previously added to the container or executed in such cases).
	 * However, this also encompasses cases where a GE was added to a container previously and then applied again to 'stack'.  It does not happen for periodic executions of a duration GE.
	 */
	UE_API void OnApplied(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const;

	/** Returns all tags that this GE *has* and *does not* grant to any Actor. */
	const FGameplayTagContainer& GetAssetTags() const { return CachedAssetTags; }

	/** Returns all tags granted to the Target Actor of this gameplay effect. That is to say: If this GE is applied to an Actor, they get these tags. */
	const FGameplayTagContainer& GetGrantedTags() const { return CachedGrantedTags; }

	/** Returns all blocking ability tags granted by this gameplay effect definition. Blocking tags means the target will not be able to execute an ability with this asset tag. */
	const FGameplayTagContainer& GetBlockedAbilityTags() const { return CachedBlockedAbilityTags; }

	/** Returns the maximum stack size for this gameplay effect */
	UE_API int32 GetStackLimitCount() const;

	/** Returns the stack expiration policy for this gameplay effect */
	UE_API EGameplayEffectStackingExpirationPolicy GetStackExpirationPolicy() const;

	// ---------------------------------------------------------------------------------------------------------------------------------

	/**
	 * Find and return the component of class GEComponentClass on this GE, if one exists.
	 * Note: The const is a hint that these are assets that should not be modified at runtime. If you're building your own dynamic GameplayEffect: @see AddComponent, FindOrAddComponent.
	 *
	 * @return	The component matching the query (if it exists).
	 */
	template<typename GEComponentClass>
	const GEComponentClass* FindComponent() const;

	/** @return the first component that derives from the passed-in ClassToFind, if one exists. */
	UE_API const UGameplayEffectComponent* FindComponent(TSubclassOf<UGameplayEffectComponent> ClassToFind) const;

	/** 
	 * Add a GameplayEffectComponent to the GameplayEffect.  This does not check for duplicates and is guaranteed to return a new instance.
	 * 
	 * @return	The newly added instanced GameplayEffectComponent.
	 */
	template<typename GEComponentClass>
	GEComponentClass& AddComponent();

	/**
	 * Find an existing GameplayEffectComponent of the requested class, or add one if none are found.
	 * 
	 * @return	The existing instance of GEComponentClass, or a newly added one.
	 */
	template<typename GEComponentClass>
	GEComponentClass& FindOrAddComponent();

#if WITH_EDITOR
	/** Allow each Gameplay Effect Component to validate its own data.  Call this version (Super::IsDataValid) _after_ your overridden version to update EditorStatusText. */
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

protected:
	// ----------------------------------------------------------------------
	//	Upgrade Path
	// ----------------------------------------------------------------------
	
	/** Let's keep track of the version so that we can upgrade the Components properly.  This will be set properly in PostLoad after upgrades. */
	UE_API EGameplayEffectVersion GetVersion() const;

	/** Sets the Version of the class to denote it's been upgraded */
	UE_API void SetVersion(EGameplayEffectVersion Version);

	/** We should intercept the Save call and revalidate all of our deprecated values to avoid hanging onto stale data */
	UE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;

private:
	// Helper functions for converting data to use components
	UE_API void ConvertAbilitiesComponent();
	UE_API void ConvertAdditionalEffectsComponent();
	UE_API void ConvertAssetTagsComponent();
	UE_API void ConvertBlockByTagsComponent();
	UE_API void ConvertChanceToApplyComponent();
	UE_API void ConvertCustomCanApplyComponent();
	UE_API void ConvertImmunityComponent();
	UE_API void ConvertRemoveOtherComponent();
	UE_API void ConvertTagRequirementsComponent();
	UE_API void ConvertTargetTagsComponent();
	UE_API void ConvertUIComponent();
#endif

	// ----------------------------------------------------------------------
	//	Properties
	// ----------------------------------------------------------------------
public:
	/** Policy for the duration of this effect */
	UPROPERTY(EditDefaultsOnly, Category=Duration)
	EGameplayEffectDurationType DurationPolicy;

	/** Duration in seconds. 0.0 for instantaneous effects; -1.0 for infinite duration. When applying stacks onto an existing active effect, the new spec's Duration is considered. */
	UPROPERTY(EditDefaultsOnly, Category=Duration, meta=(EditCondition="DurationPolicy == EGameplayEffectDurationType::HasDuration", EditConditionHides))
	FGameplayEffectModifierMagnitude DurationMagnitude;

	/** MaxDuration in seconds. <= 0.0 for unlimited. When applying stacks onto an existing active effect, the new spec's MaxDuration is considered. */
	UPROPERTY(EditDefaultsOnly, Category = Duration, meta = (EditCondition = "DurationPolicy == EGameplayEffectDurationType::HasDuration", EditConditionHides))
	FGameplayEffectModifierMagnitude MaxDurationMagnitude;

	/** Period in seconds. 0.0 for non-periodic effects */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Duration|Period", meta=(EditCondition="DurationPolicy != EGameplayEffectDurationType::Instant", EditConditionHides))
	FScalableFloat	Period;
	
	/** If true, the effect executes on application and then at every period interval. If false, no execution occurs until the first period elapses. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Duration|Period", meta=(EditCondition="true", EditConditionHides)) // EditCondition in FGameplayEffectDetails
	bool bExecutePeriodicEffectOnApplication;

	/** How we should respond when a periodic gameplay effect is no longer inhibited */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Duration|Period", meta=(EditCondition="true", EditConditionHides)) // EditCondition in FGameplayEffectDetails
	EGameplayEffectPeriodInhibitionRemovedPolicy PeriodicInhibitionPolicy;

	/** Array of modifiers that will affect the target of this effect */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category=GameplayEffect, meta=(TitleProperty=Attribute))
	TArray<FGameplayModifierInfo> Modifiers;

	/** Array of executions that will affect the target of this effect */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = GameplayEffect)
	TArray<FGameplayEffectExecutionDefinition> Executions;

	/** Probability that this gameplay effect will be applied to the target actor (0.0 for never, 1.0 for always) */
	UE_DEPRECATED(5.3, "Chance To Apply To Target is deprecated. Use the UChanceToApplyGameplayEffectComponent instead.")
	UPROPERTY(meta = (GameplayAttribute = "True", DeprecatedProperty))
	FScalableFloat ChanceToApplyToTarget_DEPRECATED;

	UE_DEPRECATED(5.3, "Application Requirements is deprecated. Use the UCustomCanApplyGameplayEffectComponent instead.")
	UPROPERTY(meta = (DisplayName = "Application Requirement", DeprecatedProperty))
	TArray<TSubclassOf<UGameplayEffectCustomApplicationRequirement> > ApplicationRequirements_DEPRECATED;

	/** other gameplay effects that will be applied to the target of this effect if this effect applies */
	UE_DEPRECATED(5.3, "Conditional Gameplay Effects is deprecated. Use the UAdditionalEffectsGameplayEffectComponent instead.")
	UPROPERTY(BlueprintReadOnly, Category = "Deprecated|GameplayEffect", meta = (DeprecatedProperty))
	TArray<FConditionalGameplayEffect> ConditionalGameplayEffects;

	/** Effects to apply when a stacking effect "overflows" its stack count through another attempted application. Added whether the overflow application succeeds or not. Overflow occurs for each added stack if stack count limit is infinite (0 or -1)*/
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Stacking|Overflow", meta = (EditConditionHides, EditCondition = "StackingType != EGameplayEffectStackingType::None"))
	TArray<TSubclassOf<UGameplayEffect>> OverflowEffects;

	/** If true, stacking attempts made while at the stack count will fail, resulting in the duration and context not being refreshed */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Stacking|Overflow", meta = (EditConditionHides, EditCondition = "StackingType != EGameplayEffectStackingType::None"))
	bool bDenyOverflowApplication;

	/** If true, the entire stack of the effect will be cleared once it overflows */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Stacking|Overflow", meta=(EditConditionHides, EditCondition="(StackingType != EGameplayEffectStackingType::None) && bDenyOverflowApplication"))
	bool bClearStackOnOverflow;

	/** Effects to apply when this effect is made to expire prematurely (like via a forced removal, clear tags, etc.); Only works for effects with a duration */
	UE_DEPRECATED(5.3, "Premature Expiration Effect Classes is deprecated. Use the UAdditionalEffectsGameplayEffectComponent instead.")
	UPROPERTY(BlueprintReadOnly, Category="Deprecated|Expiration", meta = (DeprecatedProperty))
	TArray<TSubclassOf<UGameplayEffect>> PrematureExpirationEffectClasses;

	/** Effects to apply when this effect expires naturally via its duration; Only works for effects with a duration */
	UE_DEPRECATED(5.3, "Routine Expiration Effect Classes is deprecated. Use the UAdditionalEffectsGameplayEffectComponent instead.")
	UPROPERTY(BlueprintReadOnly, Category="Deprecated|Expiration", meta = (DeprecatedProperty))
	TArray<TSubclassOf<UGameplayEffect>> RoutineExpirationEffectClasses;

	/** If true, cues will only trigger when GE modifiers succeed being applied (whether through modifiers or executions) */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GameplayCues")
	bool bRequireModifierSuccessToTriggerCues;

	/** If true, GameplayCues will only be triggered for the first instance in a stacking GameplayEffect. */
	UPROPERTY(EditDefaultsOnly, Category = "GameplayCues")
	bool bSuppressStackingCues;

	/** Cues to trigger non-simulated reactions in response to this GameplayEffect such as sounds, particle effects, etc */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "GameplayCues")
	TArray<FGameplayEffectCue>	GameplayCues;

	/** Data for the UI representation of this effect. This should include things like text, icons, etc. Not available in server-only builds. */
	UE_DEPRECATED(5.3, "UI Data is deprecated. UGameplayEffectUIData now dervies from UGameplayEffectComponent, add it as a GameplayEffectComponent.  You can then access it with FindComponent<UGameplayEffectUIData>().")
	UPROPERTY(BlueprintReadOnly, Transient, Instanced, Category = "Deprecated|Display", meta = (DeprecatedProperty))
	TObjectPtr<class UGameplayEffectUIData> UIData;

	// ----------------------------------------------------------------------
	//	Tag Containers
	// ----------------------------------------------------------------------
	
	/** The GameplayEffect's Tags: tags the the GE *has* and DOES NOT give to the actor. */
	UE_DEPRECATED(5.3, "Inheritable Gameplay Effect Tags is deprecated. To configure, add a UAssetTagsGameplayEffectComponent.  To access, use GetAssetTags.")
	UPROPERTY(BlueprintReadOnly, Category = Deprecated, meta = (DisplayName = "GameplayEffectAssetTag", Categories="GameplayEffectTagsCategory", DeprecatedProperty))
	FInheritedTagContainer InheritableGameplayEffectTags;
	
	/** These tags are applied to the actor I am applied to */
	UE_DEPRECATED(5.3, "Inheritable Owned Tags Container is deprecated. To configure, add a UTargetTagsGameplayEffectComponent. To access, use GetGrantedTags.")
	UPROPERTY(BlueprintReadOnly, Category = Deprecated, meta = (DisplayName="GrantedTags", Categories="OwnedTagsCategory", DeprecatedProperty))
	FInheritedTagContainer InheritableOwnedTagsContainer;
	
	/** These blocked ability tags are applied to the actor I am applied to */
	UE_DEPRECATED(5.3, "Inheritable Blocked Ability Tags Container is deprecated. Use the UTargetTagsGameplayEffectComponent instead.  To access, use GetBlockedAbilityTags.")
	UPROPERTY(BlueprintReadOnly, Category = Deprecated, meta = (DisplayName="GrantedBlockedAbilityTags", Categories="BlockedAbilityTagsCategory", DeprecatedProperty))
	FInheritedTagContainer InheritableBlockedAbilityTagsContainer;
	
	/** Once Applied, these tags requirements are used to determined if the GameplayEffect is "on" or "off". A GameplayEffect can be off and do nothing, but still applied. */
	UE_DEPRECATED(5.3, "Ongoing Tag Requirements is deprecated. Use the UTargetTagRequirementsGameplayEffectComponent instead.")
	UPROPERTY(BlueprintReadOnly, Category = Deprecated, meta = (Categories="OngoingTagRequirementsCategory", DeprecatedProperty))
	FGameplayTagRequirements OngoingTagRequirements;

	/** Tag requirements for this GameplayEffect to be applied to a target. This is pass/fail at the time of application. If fail, this GE fails to apply. */
	UE_DEPRECATED(5.3, "Application Tag Requirements is deprecated. Use the UTargetTagRequirementsGameplayEffectComponent instead.")
	UPROPERTY(BlueprintReadOnly, Category = Deprecated, meta = (Categories="ApplicationTagRequirementsCategory", DeprecatedProperty))
	FGameplayTagRequirements ApplicationTagRequirements;

	/** Tag requirements that if met will remove this effect. Also prevents effect application. */
	UE_DEPRECATED(5.3, "Removal Tag Requirements is deprecated. Use the URemoveOtherGameplayEffectComponent instead.")
	UPROPERTY(BlueprintReadOnly, Category = Deprecated, meta = (Categories="ApplicationTagRequirementsCategory", DeprecatedProperty))
	FGameplayTagRequirements RemovalTagRequirements;

	/** GameplayEffects that *have* tags in this container will be cleared upon effect application. */
	UE_DEPRECATED(5.3, "Remove Gameplay Effects With Tags is deprecated. Use the UTargetTagRequirementsGameplayEffectComponent instead.")
	UPROPERTY(BlueprintReadOnly, Category = Deprecated, meta = (Categories="RemoveTagRequirementsCategory", DeprecatedProperty))
	FInheritedTagContainer RemoveGameplayEffectsWithTags;

	/** Grants the owner immunity from these source tags.  */
	UE_DEPRECATED(5.3, "Granted Application Immunity Tags is deprecated. Use the UImmunityGameplayEffectComponent instead.")
	UPROPERTY(BlueprintReadOnly, Category = Deprecated, meta = (DisplayName = "GrantedApplicationImmunityTags", Categories="GrantedApplicationImmunityTagsCategory", DeprecatedProperty))
	FGameplayTagRequirements GrantedApplicationImmunityTags;

	/** Grants immunity to GameplayEffects that match this query. Queries are more powerful but slightly slower than GrantedApplicationImmunityTags. */
	UE_DEPRECATED(5.3, "Granted Application Immunity Query is deprecated. Use the UImmunityGameplayEffectComponent instead.")
	UPROPERTY(BlueprintReadOnly, Category = Deprecated, meta = (DeprecatedProperty))
	FGameplayEffectQuery GrantedApplicationImmunityQuery;

	/** Cached !GrantedApplicationImmunityQuery.IsEmpty(). Set on PostLoad. */
	UE_DEPRECATED(5.3, "HasGrantedApplicationImmunityQuery is deprecated.  Use the UImmunityGameplayEffectComponent instead.")
	bool HasGrantedApplicationImmunityQuery = false;

	/** On Application of an effect, any active effects with this this query that matches against the added effect will be removed. Queries are more powerful but slightly slower than RemoveGameplayEffectsWithTags. */
	UE_DEPRECATED(5.3, "Remove Gameplay Effect Query is deprecated.  Use the URemoveOtherGameplayEffectComponent instead.")
	UPROPERTY(BlueprintReadOnly, Category = Tags, meta = (DisplayAfter = "RemovalTagRequirements", DeprecatedProperty))
	FGameplayEffectQuery RemoveGameplayEffectQuery;

	/** Cached !RemoveGameplayEffectsQuery.IsEmpty(). Set on PostLoad. */
	UE_DEPRECATED(5.3, "HasRemoveGameplayEffectsQuery is deprecated.  Use the URemoveOtherGameplayEffectComponent instead.")
	bool HasRemoveGameplayEffectsQuery = false;

	// ----------------------------------------------------------------------
	//	Stacking
	// ----------------------------------------------------------------------

	// TODO: 5.11 Add EditCondition DurationPolicy to Stacking rules
	/** How this GameplayEffect stacks with other instances of this same GameplayEffect */
	UE_DEPRECATED(5.7, "Stacking Type will be made private, Please use GetStackingType.")
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Stacking)
	EGameplayEffectStackingType	StackingType;

	/** Stack limit for StackingType. A value of -1 or 0 means no limit. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Stacking, meta = (EditConditionHides, EditCondition = "StackingType != EGameplayEffectStackingType::None"))
	int32 StackLimitCount;

	/** Policy for how the effect duration should be refreshed while stacking */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Stacking, meta = (EditConditionHides, EditCondition = "StackingType != EGameplayEffectStackingType::None"))
	EGameplayEffectStackingDurationPolicy StackDurationRefreshPolicy;

	/** Policy for how the effect period should be reset (or not) while stacking */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Stacking, meta = (EditConditionHides, EditCondition = "StackingType != EGameplayEffectStackingType::None"))
	EGameplayEffectStackingPeriodPolicy StackPeriodResetPolicy;

	/** Policy for how to handle duration expiring on this gameplay effect */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Stacking, meta = (EditConditionHides, EditCondition = "StackingType != EGameplayEffectStackingType::None"))
	EGameplayEffectStackingExpirationPolicy StackExpirationPolicy;

	/** If true, the calculation will include the stack count for Modifier Magnitudes */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Stacking, meta = (EditConditionHides, EditCondition = "StackingType != EGameplayEffectStackingType::None"))
	bool bFactorInStackCount;

	/** return the stacking type defined from the data asset */
	UE_API EGameplayEffectStackingType GetStackingType() const;

#if WITH_EDITOR
	/** Set stacking type */
	void SetStackingType(EGameplayEffectStackingType InType);
#endif

	// ----------------------------------------------------------------------
	//	Granted abilities
	// ----------------------------------------------------------------------

	/** Policy for what abilities this GE will grant. */
	UE_DEPRECATED(5.3, "GrantedAbilities are deprecated in favor of AbilitiesGameplayEffectComponent")
	UPROPERTY(BlueprintReadOnly, Category = "Granted Abilities")
	TArray<FGameplayAbilitySpecDef>	GrantedAbilities;

	// ----------------------------------------------------------------------
	//	Cached Component Data - Do not modify these at runtime!
	//	If you want to manipulate these, write your own GameplayEffectComponent
	//	set the data during PostLoad.
	// ----------------------------------------------------------------------
	
	/** Cached copy of all the tags this GE has. Data populated during PostLoad. */
	FGameplayTagContainer CachedAssetTags;

	/** Cached copy of all the tags this GE grants to its target. Data populated during PostLoad. */
	FGameplayTagContainer CachedGrantedTags;

	/** Cached copy of all the tags this GE applies to its target to block Gameplay Abilities. Data populated during PostLoad. */
	FGameplayTagContainer CachedBlockedAbilityTags;

protected:
	/** These Gameplay Effect Components define how this Gameplay Effect behaves when applied */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Instanced, Category = "GameplayEffect", meta = (DisplayName = "Components", TitleProperty = EditorFriendlyName, ShowOnlyInnerProperties, DisplayPriority = 0))
	TArray<TObjectPtr<UGameplayEffectComponent>> GEComponents;

#if WITH_EDITORONLY_DATA
	/** Allow us to show the Status of the class (valid configurations or invalid configurations) while configuring in the Editor */
	UPROPERTY(VisibleAnywhere, Transient, Category = Status)
	mutable FText EditorStatusText;

private:
	/** The saved version of this package (the value is not inherited from its parents). @see SetVersion and GetVersion. */
	UPROPERTY()
	FGameplayEffectVersion DataVersion;
#endif
};

template<typename GEComponentClass>
const GEComponentClass* UGameplayEffect::FindComponent() const
{
	static_assert(TIsDerivedFrom<GEComponentClass, UGameplayEffectComponent>::IsDerived, "GEComponentClass must be derived from UGameplayEffectComponent");

	for (const TObjectPtr<UGameplayEffectComponent>& GEComponent : GEComponents)
	{
		if (GEComponentClass* CastComponent = Cast<GEComponentClass>(GEComponent))
		{
			return CastComponent;
		}
	}

	return nullptr;
}

template<typename GEComponentClass>
GEComponentClass& UGameplayEffect::AddComponent()
{
	static_assert( TIsDerivedFrom<GEComponentClass, UGameplayEffectComponent>::IsDerived, "GEComponentClass must be derived from UGameplayEffectComponent");

	TObjectPtr<GEComponentClass> Instance = NewObject<GEComponentClass>(this, NAME_None, GetMaskedFlags(RF_PropagateToSubObjects) | RF_Transactional);
	GEComponents.Add(Instance);

	check(Instance);
	return *Instance;
}

template<typename GEComponentClass>
GEComponentClass& UGameplayEffect::FindOrAddComponent()
{
	static_assert(TIsDerivedFrom<GEComponentClass, UGameplayEffectComponent>::IsDerived, "GEComponentClass must be derived from UGameplayEffectComponent");

	for (const TObjectPtr<UGameplayEffectComponent>& GEComponent : GEComponents)
	{
		if (GEComponentClass* CastComponent = Cast<GEComponentClass>(GEComponent))
		{
			return *CastComponent;
		}
	}

	return AddComponent<GEComponentClass>();
}

#undef UE_API
