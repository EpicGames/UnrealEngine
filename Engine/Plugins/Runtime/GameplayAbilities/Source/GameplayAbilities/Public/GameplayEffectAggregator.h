// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayTagContainer.h"
#include "GameplayEffectTypes.h"

#define UE_API GAMEPLAYABILITIES_API

struct FGameplayEffectSpec;
struct FAggregator;

/** Data that is used in aggregator evaluation that is passed from the caller/game code */
struct FAggregatorEvaluateParameters
{
	FAggregatorEvaluateParameters()
		: SourceTags(nullptr)
		, TargetTags(nullptr)
		, IncludePredictiveMods(false) 
	{}

	/** This tag container is expected to hold all aggregrated tags of the gameplay effect's source: from ability system component, effect spec, ability, calculation specific tags, etc. */
	const FGameplayTagContainer* SourceTags;
	/** This tag container is expected to hold all aggregrated tags of the gameplay effect's target: from ability system component, effect spec, ability, calculation specific tags, etc. */
	const FGameplayTagContainer* TargetTags;

	/** Any mods with one of these handles will be ignored during evaluation */
	TArray<FActiveGameplayEffectHandle> IgnoreHandles;

	/** If any tags are specified in the filter, a mod's owning active gameplay effect's source tags must match ALL of them in order for the mod to count during evaluation */
	FGameplayTagContainer AppliedSourceTagFilter;

	/** If any tags are specified in the filter, a mod's owning active gameplay effect's target tags must match ALL of them in order for the mod to count during evaluation */
	FGameplayTagContainer AppliedTargetTagFilter;

	bool IncludePredictiveMods;
};

/** Data that is used in aggregator evaluation that is intrinsic to the aggregator itself. Not passed in from the game code (though maybe originally setup once by the game code)   */
struct FAggregatorEvaluateMetaData
{
	typedef TFunction< void(const FAggregatorEvaluateParameters&, const FAggregator*) > FCustomQualifiesFunc;

	FAggregatorEvaluateMetaData(FCustomQualifiesFunc InQualifierFunc) 
		: CustomQualifiesFunc(InQualifierFunc)
	{

	}


	FCustomQualifiesFunc CustomQualifiesFunc;
};

struct FAggregatorMod
{
	const FGameplayTagRequirements*	SourceTagReqs;
	const FGameplayTagRequirements*	TargetTagReqs;

	float EvaluatedMagnitude;		// Magnitude this mod was last evaluated at
	float StackCount;

	FActiveGameplayEffectHandle ActiveHandle;	// Handle of the active GameplayEffect we are tied to (if any)
	bool IsPredicted;
	
	bool Qualifies() const { return IsQualified; }

	/** Called to update the Qualifies bool */
	UE_API void UpdateQualifies(const FAggregatorEvaluateParameters& Parameters) const;

	/** Intended to be used by FAggregatorEvaluateMetaData::CustomQualifiesFunc to toggle qualifications of mods */
	void SetExplicitQualifies(bool NewQualifies) const { IsQualified = NewQualifies; }

private:

	/** This bool is updated by UpdateQualifiees. Think of it as a transient bool, we make a full pass on all qualifiers first (::UpdateQualifies) then while evaluating we check what the bool was set to (::Qualifies) */
	mutable bool IsQualified;
};

struct FAggregatorModInfo
{
	EGameplayModEvaluationChannel Channel;
	EGameplayModOp::Type Op;
	const FAggregatorMod* Mod;
};

/** Struct representing an individual aggregation channel/depth. Contains mods of all mod op types. */
struct FAggregatorModChannel
{
	/**
	 * Evaluates the channel's mods with the specified base value and evaluation parameters
	 * 
	 * @param InlineBaseValue	Base value to use for the evaluation
	 * @param Parameters		Additional evaluation parameters to use
	 * 
	 * @return Evaluated value based upon the channel's mods
	 */
	UE_API float EvaluateWithBase(float InlineBaseValue, const FAggregatorEvaluateParameters& Parameters) const;

	/**
	 * Evaluates a final value in reverse, attempting to determine a base value from the modifiers within the channel.
	 * Certain conditions (such as the use of override mods) can prevent this from computing correctly, at which point false
	 * will be returned. This is predominantly used for filling in base values on clients from replication for float-based attributes.
	 * 
	 * @note This will be deprecated/removed soon with the transition to struct-based attributes.
	 * 
	 * @param FinalValue	Final value to reverse evaluate
	 * @param Parameters	Evaluation parameters to use for the reverse evaluation
	 * @param ComputedValue	[OUT] Reverse evaluated base value
	 * 
	 * @return True if the reverse evaluation was successful, false if it was not
	 */
	UE_API bool ReverseEvaluate(float FinalValue, const FAggregatorEvaluateParameters& Parameters, OUT float& ComputedValue) const;
	
	/**
	 * Add a modifier to the channel
	 * 
	 * @param EvaluatedMagnitude	Magnitude of the modifier
	 * @param ModOp					Operation of the modifier
	 * @param SourceTagReqs			Source tag requirements of the modifier
	 * @param TargetTagReqs			Target tag requirements of the modifier
	 * @param bIsPredicted			Whether the mod is predicted or not
	 * @param ActiveHandle			Handle of the active gameplay effect that's applying the mod
	 */
	UE_API void AddMod(float EvaluatedMagnitude, TEnumAsByte<EGameplayModOp::Type> ModOp, const FGameplayTagRequirements* SourceTagReqs, const FGameplayTagRequirements* TargetTagReqs, bool bIsPredicted, const FActiveGameplayEffectHandle& ActiveHandle);
	
	/**
	 * Remove all mods from the channel that match the specified gameplay effect handle
	 * 
	 * @param Handle	Handle to use for removal
	 */
	UE_API void RemoveModsWithActiveHandle(const FActiveGameplayEffectHandle& Handle);

	/**
	 * Add the specified channel's mods into this channel
	 * 
	 * @param Other	Other channel to add mods from
	 */
	UE_API void AddModsFrom(const FAggregatorModChannel& Other);

	/** runs UpdateQualifies on all mods */
	UE_API void UpdateQualifiesOnAllMods(const FAggregatorEvaluateParameters& Parameters) const;

	/** Helper function for iterating through all mods within a channel */
	UE_API void ForEachMod(FAggregatorModInfo& Info, TFunction<void (const FAggregatorModInfo&) > Func) const;

	/**
	 * Populate a mapping of channel to corresponding mods
	 * 
	 * @param Channel	Enum channel associated with this channel
	 * @param OutMods	Mapping of channel enum to mods
	 */
	UE_API void GetAllAggregatorMods(EGameplayModEvaluationChannel Channel, OUT TMap<EGameplayModEvaluationChannel, const TArray<FAggregatorMod>*>& OutMods) const;

	/**
	 * Called when the mod channel's gameplay effect dependencies have potentially been swapped out for new ones, like when GE arrays are cloned.
	 * Updates mod handles appropriately.
	 * 
	 * @param SwappedDependencies	Mapping of old gameplay effect handles to new replacements
	 */
	UE_API void OnActiveEffectDependenciesSwapped(const TMap<FActiveGameplayEffectHandle, FActiveGameplayEffectHandle>& SwappedDependencies);

	/**
	 * Helper function to sum all of the mods in the specified array, using the specified modifier bias and evaluation parameters
	 * 
	 * @param InMods		Mods to sum
	 * @param Bias			Bias to apply to modifier magnitudes
	 * @param Parameters	Evaluation parameters
	 * 
	 * @return Summed value of mods
	 */
	static UE_API float SumMods(const TArray<FAggregatorMod>& InMods, float Bias, const FAggregatorEvaluateParameters& Parameters);

private:

	/** Collection of modifers within the channel, organized by modifier operation */
	TArray<FAggregatorMod> Mods[EGameplayModOp::Max];
};

/** Struct representing a container of modifier channels */
struct FAggregatorModChannelContainer
{
	/**
	 * Find or add a modifier channel for the specified enum value
	 * 
	 * @param Channel	Channel to find or add a modifier channel for
	 * 
	 * @return Modifier channel for the specified enum value
	 */
	UE_API FAggregatorModChannel& FindOrAddModChannel(EGameplayModEvaluationChannel Channel);

	/** Simple accessor to the current number of modifier channels active */
	UE_API int32 GetNumChannels() const;

	/**
	 * Evaluates the result of the specified base value run through each existing evaluation channel's modifiers in numeric order
	 * with the specified evaluation parameters. The result of the evaluation of an individual channel acts as the new base value
	 * to the channel that follows it until all channels have been evaluated.
	 * 
	 * EXAMPLE: Base Value: 2, Channel 0 has a +2 Additive Mod, Channel 1 is provided a base value of 4 to run through its modifiers
	 * 
	 * @param InlineBaseValue	Initial base value to use in the first evaluation channel
	 * @param Parameters		Additional evaluation parameters
	 * 
	 * @return Result of the specified base value run through each modifier in each evaluation channel in numeric order
	 */
	UE_API float EvaluateWithBase(float InlineBaseValue, const FAggregatorEvaluateParameters& Parameters) const;

	/**
	 * Similar to EvaluateWithBase (see comment there for details), but terminates early after evaluating the specified final channel instead of
	 * continuing through every possible channel
	 * 
	 * @param InlineBaseValue	Initial base value to use in the first evaluation channel
	 * @param Parameters		Additional evaluation parameters
	 * @param FinalChannel		Channel to terminate evaluation with (inclusive)
	 * 
	 * @return Result of the specified base value run through each modifier in each evaluation channel in numeric order
	 */
	UE_API float EvaluateWithBaseToChannel(float InlineBaseValue, const FAggregatorEvaluateParameters& Parameters, EGameplayModEvaluationChannel FinalChannel) const;

	/**
	 * Evaluates a final value in reverse, attempting to determine a base value from the modifiers within all of the channels.
	 * The operation proceeds through all channels in reverse order, with the result of the evaluation of an individual channel used
	 * as the new final value to the channel that precedes it numerically. If any channel has a condition that prevents it from
	 * computing correctly (such as an override mod), this function just returns the original final value. This is predominantly used
	 * for filling in base values on clients from replication for float-based attributes.
	 * 
	 * @note This will be deprecated/removed soon with the transition to struct-based attributes.
	 * 
	 * @param FinalValue	Final value to reverse evaluate
	 * @param Parameters	Evaluation parameters to use for the reverse evaluation
	 * 
	 * @return If possible, the base value from the reverse evaluation. If not possible, the original final value is returned.
	 */
	UE_API float ReverseEvaluate(float FinalValue, const FAggregatorEvaluateParameters& Parameters) const;

	/** Calls ::UpdateQualifies on each mod */
	UE_API void EvaluateQualificationForAllMods(const FAggregatorEvaluateParameters& Parameters) const;

	/**
	 * Removes any mods from every channel matching the specified handle
	 * 
	 * @param ActiveHandle	Handle to use for removal
	 */
	UE_API void RemoveAggregatorMod(const FActiveGameplayEffectHandle& ActiveHandle);

	/**
	 * Adds the mods from specified container to this one
	 * 
	 * @param Other	Container to add mods from
	 */
	UE_API void AddModsFrom(const FAggregatorModChannelContainer& Other);

	/** Helper function for iterating through all mods within the channel container */
	UE_API void ForEachMod( TFunction<void (const FAggregatorModInfo&) > ) const;

	/**
	 * Populate a mapping of channel to corresponding mods for debugging purposes
	 * 
	 * @param OutMods	Mapping of channel enum to mods
	 */
	UE_API void GetAllAggregatorMods(OUT TMap<EGameplayModEvaluationChannel, const TArray<FAggregatorMod>*>& OutMods) const;

	/**
	 * Called when the container's gameplay effect dependencies have potentially been swapped out for new ones, like when GE arrays are cloned.
	 * Updates each channel appropriately.
	 * 
	 * @param SwappedDependencies	Mapping of old gameplay effect handles to new replacements
	 */
	UE_API void OnActiveEffectDependenciesSwapped(const TMap<FActiveGameplayEffectHandle, FActiveGameplayEffectHandle>& SwappedDependencies);

private:

	/** Mapping of evaluation channel enumeration to actual struct representation */
	TMap<EGameplayModEvaluationChannel, FAggregatorModChannel> ModChannelsMap;
};

struct FAggregator : public TSharedFromThis<FAggregator>
{
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAggregatorDirty, FAggregator*);

	FAggregator(float InBaseValue=0.f) 
		: EvaluationMetaData(nullptr)
		, NetUpdateID(0)
		, BaseValue(InBaseValue)
		, BroadcastingDirtyCount(0)
	{}
	
	UE_API ~FAggregator();

	/** Simple accessor to base value */
	UE_API float GetBaseValue() const;
	UE_API void SetBaseValue(float NewBaseValue, bool BroadcastDirtyEvent = true);
	
	UE_API void ExecModOnBaseValue(TEnumAsByte<EGameplayModOp::Type> ModifierOp, float EvaluatedMagnitude);
	static UE_API float StaticExecModOnBaseValue(float BaseValue, TEnumAsByte<EGameplayModOp::Type> ModifierOp, float EvaluatedMagnitude);

	UE_API void AddAggregatorMod(float EvaluatedData, TEnumAsByte<EGameplayModOp::Type> ModifierOp, EGameplayModEvaluationChannel ModifierChannel, const FGameplayTagRequirements*	SourceTagReqs, const FGameplayTagRequirements* TargetTagReqs, bool IsPredicted, FActiveGameplayEffectHandle ActiveHandle = FActiveGameplayEffectHandle() );

	/** Removes all mods for the passed in handle and marks this as dirty to recalculate the aggregator */
	UE_API void RemoveAggregatorMod(FActiveGameplayEffectHandle ActiveHandle);

	/** Updates the aggregators for the past in handle, this will handle it so the UAttributeSets stats only get one update for the delta change */
	UE_API void UpdateAggregatorMod(FActiveGameplayEffectHandle ActiveHandle, const FGameplayAttribute& Attribute, const FGameplayEffectSpec& Spec, bool bWasLocallyGenerated, FActiveGameplayEffectHandle InHandle);

	/** Evaluates the Aggregator with the internal base value and given parameters */
	UE_API float Evaluate(const FAggregatorEvaluateParameters& Parameters) const;

	/** Evaluates the aggregator with the internal base value and given parameters, up to the specified evaluation channel (inclusive) */
	UE_API float EvaluateToChannel(const FAggregatorEvaluateParameters& Parameters, EGameplayModEvaluationChannel FinalChannel) const;

	/** Works backwards to calculate the base value. Used on clients for doing predictive modifiers */
	UE_API float ReverseEvaluate(float FinalValue, const FAggregatorEvaluateParameters& Parameters) const;

	/** Evaluates the Aggregator with an arbitrary base value */
	UE_API float EvaluateWithBase(float InlineBaseValue, const FAggregatorEvaluateParameters& Parameters) const;

	/** Evaluates the Aggregator to compute its "bonus" (final - base) value */
	UE_API float EvaluateBonus(const FAggregatorEvaluateParameters& Parameters) const;

	/** Evaluates the contribution from the GE associated with ActiveHandle */
	UE_API float EvaluateContribution(const FAggregatorEvaluateParameters& Parameters, FActiveGameplayEffectHandle ActiveHandle) const;

	/** Calls ::UpdateQualifies on each mod. Useful for when you need to manually inspect aggregators */
	UE_API void EvaluateQualificationForAllMods(const FAggregatorEvaluateParameters& Parameters) const;

	UE_API void TakeSnapshotOf(const FAggregator& AggToSnapshot);

	FOnAggregatorDirty OnDirty;
	FOnAggregatorDirty OnDirtyRecursive;	// Called in case where we are in a recursive dirtying chain. This will force the backing uproperty to update but not call the game code delegates

	UE_API void AddModsFrom(const FAggregator& SourceAggregator);

	UE_API void AddDependent(FActiveGameplayEffectHandle Handle);
	UE_API void RemoveDependent(FActiveGameplayEffectHandle Handle);

	/**
	 * Populate a mapping of channel to corresponding mods
	 * 
	 * @param OutMods	Mapping of channel enum to mods
	 */
	UE_API void GetAllAggregatorMods(OUT TMap<EGameplayModEvaluationChannel, const TArray<FAggregatorMod>*>& OutMods) const;

	/**
	 * Called when the aggregator's gameplay effect dependencies have potentially been swapped out for new ones, like when GE arrays are cloned.
	 * Updates the modifier channel container appropriately, as well as directly-specified dependents.
	 * 
	 * @param SwappedDependencies	Mapping of old gameplay effect handles to new replacements
	 */
	UE_API void OnActiveEffectDependenciesSwapped(const TMap<FActiveGameplayEffectHandle, FActiveGameplayEffectHandle>& SwappedDependencies);

	/** Helper function for iterating through all mods within the aggregator */
	UE_API void ForEachMod( TFunction<void (const FAggregatorModInfo&) > ) const;

	/** Custom meta data for the aggregator */
	FAggregatorEvaluateMetaData* EvaluationMetaData;

	/** NetworkID that we had our last update from. Will only be set on clients and is only used to ensure we don't recompute server base value twice. */
	int32 NetUpdateID;

private:

	UE_API void BroadcastOnDirty();

	float	BaseValue;
	FAggregatorModChannelContainer ModChannels;

	/** ActiveGE handles that we need to notify if we change. NOT copied over during snapshots. */
	TArray<FActiveGameplayEffectHandle>	Dependents;
	int32 BroadcastingDirtyCount;

	// @todo: Try to eliminate as many of these as possible
	friend struct FActiveGameplayEffectsContainer;
	friend struct FScopedAggregatorOnDirtyBatch;	// Only outside class that gets to call BroadcastOnDirty()
	friend class UAbilitySystemComponent;	// Only needed for DisplayDebug()
};

struct FAggregatorRef
{
	FAggregatorRef() { }
	FAggregatorRef(FAggregator* InData) : Data(InData) { }

	FAggregator* Get() const { return Data.Get(); }

	TSharedPtr<FAggregator>	Data;

	UE_API void TakeSnapshotOf(const FAggregatorRef& RefToSnapshot);
};

/**
 *	Allows us to batch all aggregator OnDirty calls within a scope. That is, ALL OnDirty() callbacks are
 *	delayed until FScopedAggregatorOnDirtyBatch goes out of scope.
 *	
 *	The only catch is that we store raw FAggregator*. This should only be used in scopes where aggreagtors
 *	are not deleted. There is currently no place that does. If we find to, we could add additional safety checks.
 */
struct FScopedAggregatorOnDirtyBatch
{
	UE_API FScopedAggregatorOnDirtyBatch();
	UE_API ~FScopedAggregatorOnDirtyBatch();

	static UE_API void BeginLock();
	static UE_API void EndLock();

	static UE_API void BeginNetReceiveLock();
	static UE_API void EndNetReceiveLock();

	static UE_API int32	GlobalBatchCount;
	static UE_API TSet<FAggregator*>	DirtyAggregators;

	static UE_API bool		GlobalFromNetworkUpdate;
	static UE_API int32	NetUpdateID;
};

#define AGGREGATOR_BATCH_SCOPE() FScopedAggregatorOnDirtyBatch AggregatorOnDirtyBatcher;

#undef UE_API
