// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffectAggregator.h"

#define UE_API GAMEPLAYABILITIES_API

struct FAggregatorEvaluateMetaDataLibrary
{
	static UE_API FAggregatorEvaluateMetaData MostNegativeMod_AllPositiveMods;
};

#undef UE_API
