// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AttributeSet.h"
#include "GameplayEffectAttributeCaptureDefinition.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** Enumeration for options of where to capture gameplay attributes from for gameplay effects. */
UENUM()
enum class EGameplayEffectAttributeCaptureSource : uint8
{
	/** Source (caster) of the gameplay effect. */
	Source,	
	/** Target (recipient) of the gameplay effect. */
	Target	
};


/** Struct defining gameplay attribute capture options for gameplay effects */
USTRUCT(BlueprintType)
struct FGameplayEffectAttributeCaptureDefinition
{
	GENERATED_USTRUCT_BODY()

	FGameplayEffectAttributeCaptureDefinition()
	{
		AttributeSource = EGameplayEffectAttributeCaptureSource::Source;
		bSnapshot = false;
	}

	FGameplayEffectAttributeCaptureDefinition(FGameplayAttribute InAttribute, EGameplayEffectAttributeCaptureSource InSource, bool InSnapshot)
		: AttributeToCapture(MoveTemp(InAttribute)), AttributeSource(InSource), bSnapshot(InSnapshot)
	{

	}

	/** Gameplay attribute to capture */
	UPROPERTY(EditDefaultsOnly, Category=Capture)
	FGameplayAttribute AttributeToCapture;

	/** Source of the gameplay attribute */
	UPROPERTY(EditDefaultsOnly, Category=Capture)
	EGameplayEffectAttributeCaptureSource AttributeSource;

	/** Whether the attribute should be snapshotted or not */
	UPROPERTY(EditDefaultsOnly, Category=Capture)
	bool bSnapshot;

	/** Equality/Inequality operators */
	UE_API bool operator==(const FGameplayEffectAttributeCaptureDefinition& Other) const;
	UE_API bool operator!=(const FGameplayEffectAttributeCaptureDefinition& Other) const;

	/**
	 * Get type hash for the capture definition; Implemented to allow usage in TMap
	 *
	 * @param CaptureDef Capture definition to get the type hash of
	 */
	friend uint32 GetTypeHash(const FGameplayEffectAttributeCaptureDefinition& CaptureDef)
	{
		uint32 Hash = 0;
		Hash = HashCombine(Hash, GetTypeHash(CaptureDef.AttributeToCapture));
		Hash = HashCombine(Hash, GetTypeHash(static_cast<uint8>(CaptureDef.AttributeSource)));
		Hash = HashCombine(Hash, GetTypeHash(CaptureDef.bSnapshot));
		return Hash;
	}

	UE_API FString ToSimpleString() const;
};

#undef UE_API
