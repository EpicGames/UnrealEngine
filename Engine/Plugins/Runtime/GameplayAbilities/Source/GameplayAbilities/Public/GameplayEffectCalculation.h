// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "GameplayEffectTypes.h"
#include "GameplayEffectCalculation.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** Abstract base for specialized gameplay effect calculations; Capable of specifing attribute captures */
UCLASS(BlueprintType, Blueprintable, Abstract, MinimalAPI)
class UGameplayEffectCalculation : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** Simple accessor to capture definitions for attributes */
	UE_API virtual const TArray<FGameplayEffectAttributeCaptureDefinition>& GetAttributeCaptureDefinitions() const;

protected:

	/** Attributes to capture that are relevant to the calculation */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category=Attributes)
	TArray<FGameplayEffectAttributeCaptureDefinition> RelevantAttributesToCapture;
};

#undef UE_API
