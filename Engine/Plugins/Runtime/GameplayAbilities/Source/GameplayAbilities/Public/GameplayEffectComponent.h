// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "GameplayEffectComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

struct FActiveGameplayEffect;
struct FActiveGameplayEffectsContainer;
struct FGameplayEffectSpec;
struct FPredictionKey;
class UGameplayEffect;

/**
 * Gameplay Effect Component (aka GEComponent)
 * 
 * GEComponents are what define how a GameplayEffect behaves.  Introduced in UE 5.3, there are very few calls from UGameplayEffect to UGameplayEffectComponent by design.
 * Instead of providing a larger API for all desired functionality, the implementer of a GEComponent must read the GE flow carefully and register desired callbacks
 * to achieve the desired results.  This effectively limits the implementation of GEComponents to native code for the time being.
 * 
 * GEComponents live Within a GameplayEffect (which is typically a data-only blueprint asset).  Thus, like GEs, only one GEComponent exists for all applied instances.
 * One of the unintuitive caveats of this is that GEComponent should not contain any runtime manipulated/instanced data (e.g. stored state per execution).
 * One must take careful consideration about where to store any data (and thus when it can be evaluated).  The early implementations typically work around this by
 * storing small amounts of runtime data on the desired callbacks (e.g. by binding extra parameters on the delegate).  This may explain why some functionality is still
 * in UGameplayEffect rather than a UGameplayEffectComponent.  Future implementations may need extra data stored on the FGameplayEffectSpec (i.e. Gameplay Effect Spec Components).
 * 
 * @see GameplayEffect.h for further notes, especially on the terminology used (Added vs. Executed vs. Apply).
 */
UCLASS(Abstract, Const, DefaultToInstanced, EditInlineNew, CollapseCategories, Within=GameplayEffect, MinimalAPI)
class UGameplayEffectComponent : public UObject
{
	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UGameplayEffectComponent();

	/** Returns the GameplayEffect that owns this Component (the Outer) */
	UE_API UGameplayEffect* GetOwner() const;

	/** 
	 * Can the GameplayEffectSpec apply to the passed-in ASC?  All Components of the GE must return true, or a single one can return false to prohibit the application.
	 * Note: Application and Inhibition are two separate things.  If a GE can apply, we then either Add it (if it has duration/prediction) or Execute it (if it's instant).
	 */
	virtual bool CanGameplayEffectApply(const FActiveGameplayEffectsContainer& ActiveGEContainer, const FGameplayEffectSpec& GESpec) const { return true; }

	/**
     * Called when a Gameplay Effect is Added to the ActiveGameplayEffectsContainer.  GE's are added to that container when they have duration (or are predicting locally).
	 * Note: This also occurs as a result of replication (e.g. when the server replicates a GE to the owning client -- including the 'duplicate' GE after a prediction).
     * Return if the effect should remain active, or false to inhibit.  Note: Inhibit does not remove the effect (it remains added but dormant, waiting to uninhibit).
     */
	virtual bool OnActiveGameplayEffectAdded(FActiveGameplayEffectsContainer& ActiveGEContainer, FActiveGameplayEffect& ActiveGE) const { return true; }

	/** 
	 * Called when a Gameplay Effect is executed.  GE's can only Execute on ROLE_Authority.  GE's only Execute when they're applying an instant effect (otherwise they're added to the ActiveGameplayEffectsContainer).
	 * Note: Periodic effects Execute every period (and are also added to ActiveGameplayEffectsContainer).  One may think of this as periodically executing an instant effect (and thus can only happen on the server).
	 */
	virtual void OnGameplayEffectExecuted(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const {}

	/**
	 * Called when a Gameplay Effect is initially applied, or stacked.  GE's are 'applied' in both cases of duration or instant execution.  This call does not happen periodically, nor through replication.
	 * One should favor this function over OnActiveGameplayEffectAdded & OnGameplayEffectExecuted (but all multiple may be used depending on the case).
	 */
	virtual void OnGameplayEffectApplied(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const {}

	/**
	 * Let us know that the owning GameplayEffect has been modified, thus apply an asset-related changes to the owning GameplayEffect (e.g. any of its fields)
	 */
	virtual void OnGameplayEffectChanged() {}

#if WITH_EDITOR
	/**
	 * Allow each Gameplay Effect Component to validate its own data.  Any warnings/errors will immediately show up in the Gameplay Effect when in Editor.
	 * The default implementation ensures we only have a single type of any given class.  Override this function to change that functionality and use Super::Super::IsDataValid if needed.
	 */
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif

protected:
#if WITH_EDITORONLY_DATA
	/** Friendly name for displaying in the Editor's Gameplay Effect Component Index (@see UGameplayEffect::GEComponents). We set EditCondition False here so it doesn't show up otherwise. */
	UPROPERTY(VisibleDefaultsOnly, Transient, Category=AlwaysHidden, Meta=(EditCondition=False, EditConditionHides))
	FString EditorFriendlyName;
#endif
};

// Find the same component in the parent of the passed-in GameplayEffect.  Useful for having a child component inherit properties from the parent (e.g. inherited tags).
template<typename GEComponentClass, typename LateBindGameplayEffect = UGameplayEffect> // LateBindGameplayEffect to avoid having to #include GameplayEffect.h
const GEComponentClass* FindParentComponent(const GEComponentClass& ChildComponent)
{
	const LateBindGameplayEffect* ChildGE = ChildComponent.GetOwner();
	const LateBindGameplayEffect* ParentGE = ChildGE ? Cast<LateBindGameplayEffect>(ChildGE->GetClass()->GetArchetypeForCDO()) : nullptr;
	return ParentGE ? ParentGE->template FindComponent<GEComponentClass>() : nullptr;
}

#undef UE_API
