// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffectComponent.h"
#include "GameplayAbilitySpec.h"
#include "AbilitiesGameplayEffectComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** Options on how to configure a GameplayAbilitySpec to grant an Actor */
USTRUCT()
struct FGameplayAbilitySpecConfig
{
	GENERATED_BODY()

	/** What ability to grant */
	UPROPERTY(EditDefaultsOnly, Category = "Ability Definition")
	TSubclassOf<UGameplayAbility> Ability;

	/** What level to grant this ability at */
	UPROPERTY(EditDefaultsOnly, Category = "Ability Definition", DisplayName = "Level", meta=(UIMin=0.0))
	FScalableFloat LevelScalableFloat = FScalableFloat{ 1.0f };

	/** Input ID to bind this ability to */
	UPROPERTY(EditDefaultsOnly, Category = "Ability Definition")
	int32 InputID = INDEX_NONE;

	/** What will remove this ability later */
	UPROPERTY(EditDefaultsOnly, Category = "Ability Definition")
	EGameplayEffectGrantedAbilityRemovePolicy RemovalPolicy = EGameplayEffectGrantedAbilityRemovePolicy::CancelAbilityImmediately;
};

/**
 * Grants additional Gameplay Abilities to the Target of a Gameplay Effect while active
 */
UCLASS(DisplayName="Grant Gameplay Abilities", MinimalAPI)
class UAbilitiesGameplayEffectComponent : public UGameplayEffectComponent
{
	friend class UGameplayEffect; // for upgrade path

	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UAbilitiesGameplayEffectComponent();

	/** Register for the appropriate events when we're applied */
	UE_API virtual bool OnActiveGameplayEffectAdded(FActiveGameplayEffectsContainer& ActiveGEContainer, FActiveGameplayEffect& ActiveGE) const override;

	/** Adds an entry for granting Gameplay Abilities */
	UE_API void AddGrantedAbilityConfig(const FGameplayAbilitySpecConfig& Config);

#if WITH_EDITOR
	/** Warn on misconfigured Gameplay Effect */
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif

protected:
	/** This allows us to be notified when the owning GameplayEffect has had its inhibition changed (which can happen on the initial application). */
	UE_API virtual void OnInhibitionChanged(FActiveGameplayEffectHandle ActiveGEHandle, bool bIsInhibited) const;

	UE_DEPRECATED(5.8, "Use the version that takes an FActiveGameplayEffect (because they may be PendingRemove by now, so the Handle won't resolve)")
	virtual void GrantAbilities(FActiveGameplayEffectHandle ActiveGEHandle) const {}

	UE_DEPRECATED(5.8, "Use the version that takes an FActiveGameplayEffect (because they may be PendingRemove by now, so the Handle won't resolve)")
	virtual void RemoveAbilities(FActiveGameplayEffectHandle ActiveGEHandle) const {}

	/** We should grant the configured Gameplay Abilities */
	UE_API virtual void GrantAbilities(UAbilitySystemComponent& OwningASC, const FActiveGameplayEffect& ActiveGE) const;

	/** We should remove the configured Gameplay Abilities */
	UE_API virtual void RemoveAbilities(UAbilitySystemComponent& OwningASC, const FActiveGameplayEffect& ActiveGE) const;

private:
	/** We must undo all effects when removed */
	UE_API void OnActiveGameplayEffectRemoved(const FGameplayEffectRemovalInfo& RemovalInfo) const;

protected:
	/** Abilities to Grant to the Target while this Gameplay Effect is active */
	UPROPERTY(EditDefaultsOnly, Category = GrantAbilities)
	TArray<FGameplayAbilitySpecConfig>	GrantAbilityConfigs;
};

#undef UE_API
