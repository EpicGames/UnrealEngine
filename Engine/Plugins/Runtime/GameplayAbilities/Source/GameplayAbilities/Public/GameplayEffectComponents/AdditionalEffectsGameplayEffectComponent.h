// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffect.h"
#include "GameplayEffectComponent.h"
#include "AdditionalEffectsGameplayEffectComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** Add additional Gameplay Effects that attempt to activate under certain conditions (or no conditions) */
UCLASS(CollapseCategories, DisplayName="Apply Additional Effects", MinimalAPI)
class UAdditionalEffectsGameplayEffectComponent : public UGameplayEffectComponent
{
	GENERATED_BODY()

public:
	/**
     * Called when a Gameplay Effect is Added to the ActiveGameplayEffectsContainer.  We register a callback to execute the OnComplete Gameplay Effects.
     */
	UE_API virtual bool OnActiveGameplayEffectAdded(FActiveGameplayEffectsContainer& GEContainer, FActiveGameplayEffect& ActiveGE) const override;

	/**
	 * Called when a Gameplay Effect is applied.  This executes the OnApplication Gameplay Effects.
	 */
	UE_API virtual void OnGameplayEffectApplied(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const override;

#if WITH_EDITOR
	/**
	 * There are some fields that are incompatible with one another, so let's catch them during configuration phase. 
	 */
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR


protected:
	/**
	 * Whenever the ActiveGE gets removed, we want to apply the configured OnComplete GameplayEffects
	 */
	UE_API void OnActiveGameplayEffectRemoved(const FGameplayEffectRemovalInfo& RemovalInfo, FActiveGameplayEffectsContainer* ActiveGEContainer) const;

public:
	/**
	 * This will copy all of the data (e.g. SetByCallerMagnitudes) from the GESpec that Applied this GameplayEffect to the new OnApplicationGameplayEffect Specs.
	 * One would think this is normally desirable (and how OnComplete works by default), but we default to false here for backwards compatability.
	 */
	UPROPERTY(EditDefaultsOnly, Category = OnApplication)
	bool bOnApplicationCopyDataFromOriginalSpec = false;

	/** Other gameplay effects that will be applied to the target of this effect if the owning effect applies */
	UPROPERTY(EditDefaultsOnly, Category = OnApplication)
	TArray<FConditionalGameplayEffect> OnApplicationGameplayEffects;

	/** Effects to apply when this effect completes, regardless of how it ends */
	UPROPERTY(EditDefaultsOnly, Category = OnComplete)
	TArray<TSubclassOf<UGameplayEffect>> OnCompleteAlways;

	/** Effects to apply when this effect expires naturally via its duration */
	UPROPERTY(EditDefaultsOnly, Category = OnComplete)
	TArray<TSubclassOf<UGameplayEffect>> OnCompleteNormal;

	/** Effects to apply when this effect is made to expire prematurely (e.g. via a forced removal, clear tags, etc.) */
	UPROPERTY(EditDefaultsOnly, Category = OnComplete)
	TArray<TSubclassOf<UGameplayEffect>> OnCompletePrematurely;
};

#undef UE_API
