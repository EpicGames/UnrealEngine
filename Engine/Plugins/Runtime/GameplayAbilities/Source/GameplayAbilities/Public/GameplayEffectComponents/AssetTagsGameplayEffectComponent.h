// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffect.h"
#include "GameplayEffectComponent.h"
#include "AssetTagsGameplayEffectComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** These are tags that the Gameplay Effect Asset itself 'has' (owns). These do _not_ transfer to any Actors */
UCLASS(DisplayName="Tags This Effect Has (Asset Tags)", MinimalAPI)
class UAssetTagsGameplayEffectComponent : public UGameplayEffectComponent
{
	GENERATED_BODY()

public:
	/** Setup a better EditorFriendlyName and do some initialization */
	UE_API virtual void PostInitProperties() override;

	/** Needed to properly apply FInheritedTagContainer properties */
	UE_API virtual void OnGameplayEffectChanged() override;

	/** Gets the Asset Tags inherited tag structure (as configured) */
	const FInheritedTagContainer& GetConfiguredAssetTagChanges() const { return InheritableAssetTags; }

	/** Applies the Asset Tags to the GameplayEffect (and saves those changes) */
	UE_API void SetAndApplyAssetTagChanges(const FInheritedTagContainer& TagContainerMods);

#if WITH_EDITOR
	/** Needed to properly update FInheritedTagContainer properties */
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

private:
	/** Get a cached version of the FProperty Name for PostEditChangeProperty */
	static const FName& GetInheritableAssetTagsName()
	{
		static FName NAME_InheritableAssetTags = GET_MEMBER_NAME_CHECKED(UAssetTagsGameplayEffectComponent, InheritableAssetTags);
		return NAME_InheritableAssetTags;
	}
#endif // WITH_EDITOR

private:
	/** Applies the Asset Tags to the GameplayEffect */
	UE_API void ApplyAssetTagChanges() const;

private:
	/** The GameplayEffect's Tags: tags the the GE *has* and DOES NOT give to the actor. */
	UPROPERTY(EditDefaultsOnly, Category = Tags, meta = (DisplayName = "AssetTags", Categories = "OwnedTagsCategory"))
	FInheritedTagContainer InheritableAssetTags;
};

#undef UE_API
