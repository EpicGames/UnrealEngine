// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffect.h"
#include "GameplayEffectComponent.h"
#include "BlockAbilityTagsGameplayEffectComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** Handles blocking the activation of Gameplay Abilities based on Gameplay Tags for the Target actor of the owner Gameplay Effect */
UCLASS(DisplayName="Block Abilities with Tags", MinimalAPI)
class UBlockAbilityTagsGameplayEffectComponent : public UGameplayEffectComponent
{
	GENERATED_BODY()
	
public:
	/** Setup an EditorFriendlyName and do some initialization */
	UE_API virtual void PostInitProperties() override;

	/** Needed to properly apply FInheritedTagContainer properties */
	UE_API virtual void OnGameplayEffectChanged() override;

	/** Gets the Blocked Ability Tags inherited tag structure (as configured) */
	const FInheritedTagContainer& GetConfiguredBlockedAbilityTagChanges() const { return InheritableBlockedAbilityTagsContainer; }

	/** Applies the Blocked Ability Tags to the GameplayEffect (and saves those changes) so that when the Gameplay Effect is applied, these Blocking Tags are then applied to the Target Actor */
	UE_API void SetAndApplyBlockedAbilityTagChanges(const FInheritedTagContainer& TagContainerMods);

#if WITH_EDITOR
	/** Needed to properly update FInheritedTagContainer properties */
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Validate incompatible configurations */
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

private:
	/** Get a cached version of the FProperty Name for PostEditChangeProperty */
	static const FName& GetInheritableBlockedAbilityTagsContainerPropertyName()
	{
		static FName NAME_InheritableBlockedAbilityTagsContainer = GET_MEMBER_NAME_CHECKED(UBlockAbilityTagsGameplayEffectComponent, InheritableBlockedAbilityTagsContainer);
		return NAME_InheritableBlockedAbilityTagsContainer;
	}
#endif // WITH_EDITOR

private:
	/** Applies the Blocked Ability Tags to the owning GameplayEffect */
	UE_API void ApplyBlockedAbilityTagChanges() const;

private:
	/** These tags are applied to the target actor of the Gameplay Effect.  Blocked Ability Tags prevent Gameplay Abilities with these tags from executing. */
	UPROPERTY(EditDefaultsOnly, Category = None, meta = (DisplayName = "Block Abilities w/ Tags"))
	FInheritedTagContainer InheritableBlockedAbilityTagsContainer;
};

#undef UE_API
