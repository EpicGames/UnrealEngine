// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffect.h"
#include "GameplayEffectComponent.h"
#include "CancelAbilityTagsGameplayEffectComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API


/** Enumeration representing when the ability cancellation logic for the 'CancelAbilityTagsGameplayEffectComponent' happens */
UENUM()
enum class ECancelAbilityTagsGameplayEffectComponentMode : uint8
{
	/** Component applies cancellation logic when the owning gameplay effect is Applied. Use this if we want to cancel abilities a single time on GE application, regardless of GE duration type. */
	OnApplication,

	/** Component applies cancellation logic when the owning gameplay effect is Executed. Use this if we want to cancel abilities on each periodic GE execution. */
	OnExecution
};

/** Handles canceling active Gameplay Abilities based on Gameplay Tags for the Target actor of the owner Gameplay Effect */
UCLASS(DisplayName="Cancel Abilities with Tags", MinimalAPI)
class UCancelAbilityTagsGameplayEffectComponent : public UGameplayEffectComponent
{
	GENERATED_BODY()

public:
	/** Set up an EditorFriendlyName and do some initialization */
	UE_API virtual void PostInitProperties() override;
	
	/** Needed to properly apply FInheritedTagContainer properties */
	UE_API virtual void OnGameplayEffectChanged() override;
	
	/** Needed to cancel abilities if we're canceling abilities on execution */
	UE_API virtual void OnGameplayEffectExecuted(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const override;

	/** Needed to cancel abilities if we're canceling abilities on effect application */
	UE_API virtual void OnGameplayEffectApplied(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const override;
	
	/** Applies the Canceled Ability Tags to the GameplayEffect (and saves those changes) so that when the Gameplay Effect is applied, abilities with these tags are cancelled */
	UE_API void SetAndApplyCanceledAbilityTagChanges(const FInheritedTagContainer& CanceledAbilityWithTagsMods, const FInheritedTagContainer& CanceledAbilityWithoutTagsMods);

private:
	void CancelOwnerAbilities(const FActiveGameplayEffectsContainer& ActiveGEContainer) const;

#if WITH_EDITOR
	/** Needed to properly update FInheritedTagContainer properties */
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Validate incompatible configurations */
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

	/** Get a cached version of the FProperty Name for PostEditChangeProperty */
	static const FName& GetInheritableCancelAbilitiesWithTagsContainerPropertyName()
	{
		static FName NAME_InheritableCancelAbilitiesWithTagsContainer = GET_MEMBER_NAME_CHECKED(UCancelAbilityTagsGameplayEffectComponent, InheritableCancelAbilitiesWithTagsContainer);
		return NAME_InheritableCancelAbilitiesWithTagsContainer;					
	}

	static const FName& GetInheritableCancelAbilitiesWithoutTagsContainerPropertyName()
	{
		static FName NAME_InheritableCancelAbilitiesWithoutTagsContainer = GET_MEMBER_NAME_CHECKED(UCancelAbilityTagsGameplayEffectComponent, InheritableCancelAbilitiesWithoutTagsContainer);
		return NAME_InheritableCancelAbilitiesWithoutTagsContainer;
	}
#endif // WITH_EDITOR

	/* When do we want the ability cancellation logic for this component to happen? */ 
	UPROPERTY(EditDefaultsOnly, Category = None, meta = (DisplayName = "Mode"))
	ECancelAbilityTagsGameplayEffectComponentMode ComponentMode = ECancelAbilityTagsGameplayEffectComponentMode::OnApplication;
	
	/** These tags cancel active Gameplay Abilities with these tags. */
	UPROPERTY(EditDefaultsOnly, Category = None, meta = (DisplayName = "Cancel Abilities With Tags"))
	FInheritedTagContainer InheritableCancelAbilitiesWithTagsContainer;

	/** These tags cancel active Gameplay Abilities without these tags. */
	UPROPERTY(EditDefaultsOnly, Category = None, meta = (DisplayName = "Cancel Abilities Without Tags"))
	FInheritedTagContainer InheritableCancelAbilitiesWithoutTagsContainer;
};

#undef UE_API
