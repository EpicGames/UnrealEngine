// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffectComponent.h"
#include "ScalableFloat.h"

#include "ChanceToApplyGameplayEffectComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** Applies a probablity to the application conditions of the Gameplay Effect. */
UCLASS(DisplayName="Chance To Apply This Effect", MinimalAPI)
class UChanceToApplyGameplayEffectComponent : public UGameplayEffectComponent
{
	GENERATED_BODY()
	
public:
	/** Warn of errors in the Editor */
	UE_API virtual void PostLoad() override;

	/** Determine if we can apply this GameplayEffect or not */
	UE_API virtual bool CanGameplayEffectApply(const FActiveGameplayEffectsContainer& ActiveGEContainer, const FGameplayEffectSpec& GESpec) const override;

	/** Get the configured chance to apply to target */
	const FScalableFloat& GetChanceToApplyToTarget() const { return ChanceToApplyToTarget; }

	/** Set the chance to apply to target */
	UE_API void SetChanceToApplyToTarget(const FScalableFloat& ChanceToApplyToTarget);

private:
	/** Probability that this gameplay effect will be applied to the target actor (0.0 for never, 1.0 for always) */
	UPROPERTY(EditDefaultsOnly, Category = Application, meta = (GameplayAttribute = "True"))
	FScalableFloat ChanceToApplyToTarget;
};

#undef UE_API
