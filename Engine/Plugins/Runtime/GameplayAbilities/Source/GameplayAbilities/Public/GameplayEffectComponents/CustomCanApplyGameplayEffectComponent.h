// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffectComponent.h"
#include "GameplayEffectCustomApplicationRequirement.h"

#include "CustomCanApplyGameplayEffectComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** Handles configuration of a CustomApplicationRequirement function to see if this GameplayEffect should apply */
UCLASS(DisplayName="Custom Can Apply This Effect", MinimalAPI)
class UCustomCanApplyGameplayEffectComponent : public UGameplayEffectComponent
{
	GENERATED_BODY()
	
public:
	UE_API UCustomCanApplyGameplayEffectComponent();

	/** Determine if we can apply this GameplayEffect or not */
	UE_API virtual bool CanGameplayEffectApply(const FActiveGameplayEffectsContainer& ActiveGEContainer, const FGameplayEffectSpec& GESpec) const override;

public:
	/** Custom application requirements */
	UPROPERTY(EditDefaultsOnly, Category = Application, meta = (DisplayName = "Custom Application Requirement"))
	TArray<TSubclassOf<UGameplayEffectCustomApplicationRequirement>> ApplicationRequirements;
};

#undef UE_API
