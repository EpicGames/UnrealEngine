// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffect.h"
#include "GameplayEffectComponent.h"
#include "ImmunityGameplayEffectComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** 
 * Immunity is blocking the application of _other_ GameplayEffectSpecs.
 * This registers a global handler on the ASC to block the application of other GameplayEffectSpecs.
 */
UCLASS(DisplayName="Immunity to Other Effects", MinimalAPI)
class UImmunityGameplayEffectComponent : public UGameplayEffectComponent
{
	friend class UGameplayEffect; // Needed for upgrade path
	GENERATED_BODY()

public:
	/** Constructor to set EditorFriendlyName */
	UE_API UImmunityGameplayEffectComponent();

	/** We need to register our callback to check for Immunity */
	UE_API virtual bool OnActiveGameplayEffectAdded(FActiveGameplayEffectsContainer& ActiveGEContainer, FActiveGameplayEffect& ActiveGE) const override;

#if WITH_EDITOR
	/**
	 * Warn about instant Gameplay Effects as they do not persist over time (and therefore cannot grant immunity)
	 */
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

protected:
	/** We register with the AbilitySystemComponent to try and block any GESpecs we think we should be immune to */
	UE_API bool AllowGameplayEffectApplication(const FActiveGameplayEffectsContainer& ActiveGEContainer, const FGameplayEffectSpec& GESpecToConsider, FActiveGameplayEffectHandle ImmunityActiveGE) const;

public:
	/** Grants immunity to GameplayEffects that match this query. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = None)
	TArray<FGameplayEffectQuery> ImmunityQueries;
};

#undef UE_API
