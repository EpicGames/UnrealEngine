// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffect.h"
#include "GameplayEffectComponent.h"
#include "RemoveOtherGameplayEffectComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

struct FGameplayEffectRemovalInfo;

/** Remove other Gameplay Effects based on certain conditions */
UCLASS(DisplayName="Remove Other Effects", MinimalAPI)
class URemoveOtherGameplayEffectComponent : public UGameplayEffectComponent
{
	GENERATED_BODY()

public:
	/** Constructor to set EditorFriendlyName */
	UE_API URemoveOtherGameplayEffectComponent();

	/**
	 * We will re-run RemoveGameplayEffectQueries every time the owning Gameplay Effect is applied.
	 */
	UE_API virtual void OnGameplayEffectApplied(FActiveGameplayEffectsContainer& ActiveGEContainer, FGameplayEffectSpec& GESpec, FPredictionKey& PredictionKey) const override;

#if WITH_EDITOR
	/**
	 * Warn about periodic Gameplay Effects, that you probably instead want the OngoingTagRequirements in TagRequirementsGameplayEffectComponent
	 */
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

public:
	/** On Application of the owning Gameplay Effect, any Active GameplayEffects that *match* these queries will be removed. */
	UPROPERTY(EditDefaultsOnly, Category = None)
	TArray<FGameplayEffectQuery> RemoveGameplayEffectQueries;
};

#undef UE_API
