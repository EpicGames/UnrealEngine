// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffect.h"
#include "GameplayEffectComponent.h"
#include "TargetTagRequirementsGameplayEffectComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

struct FGameplayEffectRemovalInfo;

/** Specifies tag requirements that the Target (owner of the Gameplay Effect) must have if this GE should apply or continue to execute */
UCLASS(DisplayName="Require Tags to Apply/Continue This Effect", MinimalAPI)
class UTargetTagRequirementsGameplayEffectComponent : public UGameplayEffectComponent
{
	GENERATED_BODY()

public:
	/** Constructor to set EditorFriendlyName */
	UE_API UTargetTagRequirementsGameplayEffectComponent();

	/** Can we apply to the ActiveGEContainer? */
	UE_API virtual bool CanGameplayEffectApply(const FActiveGameplayEffectsContainer& ActiveGEContainer, const FGameplayEffectSpec& GESpec) const override;

	/** Once we've applied, we need to register for ongoing requirements */
	UE_API virtual bool OnActiveGameplayEffectAdded(FActiveGameplayEffectsContainer& GEContainer, FActiveGameplayEffect& ActiveGE) const override;

#if WITH_EDITOR
	/**
	 * Validate incompatible configurations
	 */
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

private:
	/** We need to be notified when we're removed to unregister some callbacks */
	void OnGameplayEffectRemoved(const FGameplayEffectRemovalInfo& GERemovalInfo, UAbilitySystemComponent* ASC, TArray<TTuple<FGameplayTag, FDelegateHandle>> AllBoundEvents) const;

	/** We need to register a callback for when the owner changes its tags.  When that happens, we need to figure out if our GE should continue to execute */
	void OnTagChanged(const FGameplayTag GameplayTag, int32 NewCount, FActiveGameplayEffectHandle ActiveGEHandle) const;

	/** Have requirements to remove this Gameplay Effect been met? */
	bool HaveRemovalRequirementsBeenMet(const FGameplayTagContainer& TargetOwnedTags, bool bNetAuthority) const;

public:
	/** Tag requirements the target must have for this GameplayEffect to be applied. This is pass/fail at the time of application. If fail, this GE fails to apply. */
	UPROPERTY(EditDefaultsOnly, Category = Tags)
	FGameplayTagRequirements ApplicationTagRequirements;

	/** Once Applied, these tags requirements are used to determined if the GameplayEffect is "on" or "off". A GameplayEffect can be off and do nothing, but still applied. */
	UPROPERTY(EditDefaultsOnly, Category = Tags)
	FGameplayTagRequirements OngoingTagRequirements;

	/** Tag requirements that if met will remove this effect. Also prevents effect application. */
	UPROPERTY(EditDefaultsOnly, Category = Tags)
	FGameplayTagRequirements RemovalTagRequirements;
};

#undef UE_API
