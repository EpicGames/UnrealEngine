// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffect.h"
#include "GameplayEffectComponent.h"
#include "TargetTagsGameplayEffectComponent.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** Handles granting Tags to the Target (sometimes referred to as the Owner) of the Gameplay Effect */
UCLASS(DisplayName="Grant Tags to Target Actor", MinimalAPI)
class UTargetTagsGameplayEffectComponent : public UGameplayEffectComponent
{
	GENERATED_BODY()

public:
	/** Setup an EditorFriendlyName and do some initialization */
	UE_API virtual void PostInitProperties() override;

	/** Needed to properly load FInheritedTagContainer properties */
	UE_API virtual void OnGameplayEffectChanged() override;

	/** Gets the (Granted) Target Tags inherited tag structure (as configured) */
	const FInheritedTagContainer& GetConfiguredTargetTagChanges() const { return InheritableGrantedTagsContainer; }

	/** Applies the (Granted) Target Tags to the GameplayEffect (and saves those changes) so that it may grant these Tags the Target Actor when Applied */
	UE_API void SetAndApplyTargetTagChanges(const FInheritedTagContainer& TagContainerMods);

#if WITH_EDITOR
	/** Needed to properly update FInheritedTagContainer properties */
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Validate incompatible configurations */
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

private:
	/** Get a cached version of the FProperty Name for PostEditChangeProperty */
	static const FName& GetInheritableGrantedTagsContainerName()
	{
		static FName NAME_InheritableGrantedTagsContainer = GET_MEMBER_NAME_CHECKED(UTargetTagsGameplayEffectComponent, InheritableGrantedTagsContainer);
		return NAME_InheritableGrantedTagsContainer;
	}
#endif // WITH_EDITOR

private:
	/** Applies the configured (Granted) Target Tags to the owning GameplayEffect. */
	UE_API void ApplyTargetTagChanges() const;

private:
	/** These tags are applied (granted) to the target actor of the Gameplay Effect.  The Target would then "own" these Tags. */
	UPROPERTY(EditDefaultsOnly, Category = None, meta = (DisplayName = "Add Tags", Categories = "OwnedTagsCategory"))
	FInheritedTagContainer InheritableGrantedTagsContainer;
};

#undef UE_API
