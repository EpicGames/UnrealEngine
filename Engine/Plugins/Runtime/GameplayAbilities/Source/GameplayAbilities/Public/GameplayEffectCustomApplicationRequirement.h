// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "GameplayEffectCustomApplicationRequirement.generated.h"

#define UE_API GAMEPLAYABILITIES_API

struct FGameplayEffectSpec;
class UGameplayEffect;
class UAbilitySystemComponent;

/** Class used to perform custom gameplay effect modifier calculations, either via blueprint or native code */ 
UCLASS(BlueprintType, Blueprintable, Abstract, MinimalAPI)
class UGameplayEffectCustomApplicationRequirement : public UObject
{
	GENERATED_BODY()

public:
	/** Return whether the gameplay effect should be applied or not */
	UFUNCTION(BlueprintNativeEvent, Category="Calculation")
	UE_API bool CanApplyGameplayEffect(const UGameplayEffect* GameplayEffect, const FGameplayEffectSpec& Spec, UAbilitySystemComponent* ASC) const;
};

#undef UE_API
