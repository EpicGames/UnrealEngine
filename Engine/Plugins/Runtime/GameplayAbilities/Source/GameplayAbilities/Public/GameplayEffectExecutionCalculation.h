// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "GameplayTagContainer.h"
#include "GameplayEffectTypes.h"
#include "GameplayEffectAggregator.h"
#include "GameplayEffect.h"
#include "GameplayEffectCalculation.h"
#include "GameplayEffectExecutionCalculation.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

/** Struct representing parameters for a custom gameplay effect execution. Should not be held onto via reference, used just for the scope of the execution */
USTRUCT(BlueprintType)
struct FGameplayEffectCustomExecutionParameters
{
	GENERATED_USTRUCT_BODY()

public:

	// Constructors
	UE_API FGameplayEffectCustomExecutionParameters();
	UE_API FGameplayEffectCustomExecutionParameters(FGameplayEffectSpec& InOwningSpec, const TArray<FGameplayEffectExecutionScopedModifierInfo>& InScopedMods, UAbilitySystemComponent* InTargetAbilityComponent, const FGameplayTagContainer& InPassedIntags, const FPredictionKey& InPredictionKey);
	UE_API FGameplayEffectCustomExecutionParameters(FGameplayEffectSpec& InOwningSpec, const TArray<FGameplayEffectExecutionScopedModifierInfo>& InScopedMods, UAbilitySystemComponent* InTargetAbilityComponent, const FGameplayTagContainer& InPassedIntags, const FPredictionKey& InPredictionKey, const TArray<FActiveGameplayEffectHandle>& InIgnoreHandles);

	/** Simple accessor to owning gameplay spec */
	UE_API const FGameplayEffectSpec& GetOwningSpec() const;

	/** Non const access. Be careful with this, especially when modifying a spec after attribute capture. */
	UE_API FGameplayEffectSpec* GetOwningSpecForPreExecuteMod() const;

	/** Simple accessor to target ability system component */
	UE_API UAbilitySystemComponent* GetTargetAbilitySystemComponent() const;

	/** Simple accessor to source ability system component (could be null!) */
	UE_API UAbilitySystemComponent* GetSourceAbilitySystemComponent() const;
	
	/** Simple accessor to the Passed In Tags to this execution */
	UE_API const FGameplayTagContainer& GetPassedInTags() const;

	UE_API TArray<FActiveGameplayEffectHandle> GetIgnoreHandles() const;
	
	UE_API FPredictionKey GetPredictionKey() const;

	/**
	 * Attempts to calculate the magnitude of a captured attribute given the specified parameters. Can fail if the gameplay spec doesn't have
	 * a valid capture for the attribute.
	 * 
	 * @param InCaptureDef	Attribute definition to attempt to calculate the magnitude of
	 * @param InEvalParams	Parameters to evaluate the attribute under
	 * @param OutMagnitude	[OUT] Computed magnitude
	 * 
	 * @return True if the magnitude was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateCapturedAttributeMagnitude(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, const FAggregatorEvaluateParameters& InEvalParams, OUT float& OutMagnitude) const;
	
	/**
	 * Attempts to calculate the magnitude of a captured attribute given the specified parameters, including a starting base value. 
	 * Can fail if the gameplay spec doesn't have a valid capture for the attribute.
	 * 
	 * @param InCaptureDef	Attribute definition to attempt to calculate the magnitude of
	 * @param InEvalParams	Parameters to evaluate the attribute under
	 * @param InBaseValue	Base value to evaluate the attribute under
	 * @param OutMagnitude	[OUT] Computed magnitude
	 * 
	 * @return True if the magnitude was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateCapturedAttributeMagnitudeWithBase(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, const FAggregatorEvaluateParameters& InEvalParams, float InBaseValue, OUT float& OutMagnitude) const;

	/**
	 * Attempts to calculate the base value of a captured attribute given the specified parameters. Can fail if the gameplay spec doesn't have
	 * a valid capture for the attribute.
	 * 
	 * @param InCaptureDef	Attribute definition to attempt to calculate the base value of
	 * @param OutBaseValue	[OUT] Computed base value
	 * 
	 * @return True if the base value was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateCapturedAttributeBaseValue(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, OUT float& OutBaseValue) const;

	/**
	 * Attempts to calculate the bonus magnitude of a captured attribute given the specified parameters. Can fail if the gameplay spec doesn't have
	 * a valid capture for the attribute.
	 * 
	 * @param InCaptureDef		Attribute definition to attempt to calculate the bonus magnitude of
	 * @param InEvalParams		Parameters to evaluate the attribute under
	 * @param OutBonusMagnitude	[OUT] Computed bonus magnitude
	 * 
	 * @return True if the bonus magnitude was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateCapturedAttributeBonusMagnitude(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, const FAggregatorEvaluateParameters& InEvalParams, OUT float& OutBonusMagnitude) const;
	
	/**
	 * Attempts to populate the specified aggregator with a snapshot of a backing captured aggregator. Can fail if the gameplay spec doesn't have
	 * a valid capture for the attribute.
	 * 
	 * @param InCaptureDef				Attribute definition to attempt to snapshot
	 * @param OutSnapshottedAggregator	[OUT] Snapshotted aggregator, if possible
	 * 
	 * @return True if the aggregator was successfully snapshotted, false if it was not
	 */
	UE_API bool AttemptGetCapturedAttributeAggregatorSnapshot(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, OUT FAggregator& OutSnapshottedAggregator) const;

	/** 
	 *	Returns all modifiers for a given captured def. Note the returned list is a direct reference to the internal list stored in the attribute aggregators (possibly a snapshot copy or possibly the "real" ones).
	 *	Also note that all modifiers are returned, even ones that do not qualify for the given InEvalParams. You will still need to check ::Qualifies() on the mods themselves to see if they "count".
	 *  (we are returning a direct reference to the internal mod list, so we do not want to remove them or force a copy).
	 *	
	 *	Consider using ForEachQualifiedAttributeMod when you want to "do something for every qualifier mod".
	 */
	UE_API bool AttemptGatherAttributeMods(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, const FAggregatorEvaluateParameters& InEvalParams, OUT TMap<EGameplayModEvaluationChannel, const TArray<FAggregatorMod>*>& OutModMap) const;

	/** Runs given TFunction on every qualifier mod for a given AttributeCaptureDefinition */
	UE_API bool ForEachQualifiedAttributeMod(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef, const FAggregatorEvaluateParameters& InEvalParams, TFunction< void(EGameplayModEvaluationChannel, EGameplayModOp::Type, const FAggregatorMod&) >) const;

	/**
	 * Attempts to calculate the magnitude of a transient aggregator given the specified parameters.
	 * 
	 * @param InAggregatorIdentifier	Tag identifying the transient aggregator to attempt to calculate the magnitude of
	 * @param InEvalParams				Parameters to evaluate the aggregator under
	 * @param OutMagnitude				[OUT] Computed magnitude
	 * 
	 * @return True if the magnitude was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateTransientAggregatorMagnitude(const FGameplayTag& InAggregatorIdentifier, const FAggregatorEvaluateParameters& InEvalParams, OUT float& OutMagnitude) const;
	
	/**
	 * Attempts to calculate the magnitude of a transient aggregator given the specified parameters, including a starting base value.
	 * 
	 * @param InAggregatorIdentifier	Tag identifying the transient aggregator to attempt to calculate the magnitude of
	 * @param InEvalParams				Parameters to evaluate the attribute under
	 * @param InBaseValue				Base value to evaluate the attribute under
	 * @param OutMagnitude				[OUT] Computed magnitude
	 * 
	 * @return True if the magnitude was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateTransientAggregatorMagnitudeWithBase(const FGameplayTag& InAggregatorIdentifier, const FAggregatorEvaluateParameters& InEvalParams, float InBaseValue, OUT float& OutMagnitude) const;

	/**
	 * Attempts to calculate the base value of a transient aggregator given the specified parameters.
	 * 
	 * @param InAggregatorIdentifier	Tag identifying the transient aggregator to attempt to calculate the base value of
	 * @param OutBaseValue				[OUT] Computed base value
	 * 
	 * @return True if the base value was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateTransientAggregatorBaseValue(const FGameplayTag& InAggregatorIdentifier, OUT float& OutBaseValue) const;

	/**
	 * Attempts to calculate the bonus magnitude of a transient aggregator given the specified parameters.
	 * 
	 * @param InAggregatorIdentifier		Tag identifying the transient aggregator to attempt to calculate the bonus magnitude of
	 * @param InEvalParams					Parameters to evaluate the attribute under
	 * @param OutBonusMagnitude				[OUT] Computed bonus magnitude
	 * 
	 * @return True if the bonus magnitude was successfully calculated, false if it was not
	 */
	UE_API bool AttemptCalculateTransientAggregatorBonusMagnitude(const FGameplayTag& InAggregatorIdentifier, const FAggregatorEvaluateParameters& InEvalParams, OUT float& OutBonusMagnitude) const;
	
	/**
	 * Attempts to populate the specified aggregator with a snapshot of a backing transient aggregator. Can fail if the transient aggregator doesn't exist as a result
	 * of no scoped mods targeting it.
	 * 
	 * @param InAggregatorIdentifier	Tag identifying the transient aggregator to attempt to snapshot
	 * @param OutSnapshottedAggregator	[OUT] Snapshotted aggregator, if possible
	 * 
	 * @return True if the aggregator was successfully snapshotted, false if it was not
	 */
	UE_API bool AttemptGetCapturedAttributeAggregatorSnapshot(const FGameplayTag& InAggregatorIdentifier, OUT FAggregator& OutSnapshottedAggregator) const;

private:

	/** Mapping of capture definition to aggregator with scoped modifiers added in; Used to process scoped modifiers w/o modifying underlying aggregators in the capture */
	TMap<FGameplayEffectAttributeCaptureDefinition, FAggregator> ScopedModifierAggregators;

	/** Mapping of tag identifier to aggregators with scoped modifiers added in; Used to process "temporary variable" support w/o requiring backed attribute capture definitions in scoped modifiers */
	TMap<FGameplayTag, FAggregator> ScopedTransientAggregators;

	/** Owning gameplay effect spec */
	FGameplayEffectSpec* OwningSpec;

	/** Target ability system component of the execution */
	TWeakObjectPtr<UAbilitySystemComponent> TargetAbilitySystemComponent;

	/** The extra tags that were passed in to this execution */
	FGameplayTagContainer PassedInTags;

	TArray<FActiveGameplayEffectHandle> IgnoreHandles;
	
	FPredictionKey PredictionKey;
};

/** Struct representing the output of a custom gameplay effect execution. */
USTRUCT(BlueprintType)
struct FGameplayEffectCustomExecutionOutput
{
	GENERATED_USTRUCT_BODY()

public:

	/** Constructor */
	UE_API FGameplayEffectCustomExecutionOutput();

	/** Mark that the execution has manually handled the stack count and the GE system should not attempt to automatically act upon it for emitted modifiers */
	UE_API void MarkStackCountHandledManually();

	/** Simple accessor for determining whether the execution has manually handled the stack count or not */
	UE_API bool IsStackCountHandledManually() const;

	/** Accessor for determining if GameplayCue events have already been handled */
	UE_API bool AreGameplayCuesHandledManually() const;

	/** Mark that the execution wants conditional gameplay effects to trigger */
	UE_API void MarkConditionalGameplayEffectsToTrigger();

	/** Mark that the execution wants conditional gameplay effects to trigger */
	UE_API void MarkGameplayCuesHandledManually();

	/** Simple accessor for determining whether the execution wants conditional gameplay effects to trigger or not */
	UE_API bool ShouldTriggerConditionalGameplayEffects() const;

	/** Add the specified evaluated data to the execution's output modifiers */
	UE_API void AddOutputModifier(const FGameplayModifierEvaluatedData& InOutputMod);

	/** Simple accessor to output modifiers of the execution */
	UE_API const TArray<FGameplayModifierEvaluatedData>& GetOutputModifiers() const;

	/** Simple accessor to output modifiers of the execution */
	UE_API void GetOutputModifiers(OUT TArray<FGameplayModifierEvaluatedData>& OutOutputModifiers) const;

	/** Returns direct access to output modifiers of the execution (avoid copy) */
	TArray<FGameplayModifierEvaluatedData>& GetOutputModifiersRef() { return OutputModifiers; }

private:

	/** Modifiers emitted by the execution */
	UPROPERTY()
	TArray<FGameplayModifierEvaluatedData> OutputModifiers;

	/** If true, the execution wants to trigger conditional gameplay effects when it completes */
	UPROPERTY()
	uint32 bTriggerConditionalGameplayEffects : 1;
	
	/** If true, the execution itself has manually handled the stack count of the effect and the GE system doesn't have to automatically handle it */
	UPROPERTY()
	uint32 bHandledStackCountManually : 1;

	/** If true, the execution itself has manually invoked all gameplay cues and the GE system doesn't have to automatically handle them. */
	UPROPERTY()
	uint32 bHandledGameplayCuesManually : 1;
};

UCLASS(BlueprintType, Blueprintable, Abstract, MinimalAPI)
class UGameplayEffectExecutionCalculation : public UGameplayEffectCalculation
{
	GENERATED_UCLASS_BODY()

protected:

	/** Used to indicate if this execution uses Passed In Tags */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Attributes)
	bool bRequiresPassedInTags;

#if WITH_EDITORONLY_DATA

protected:

	/** Any attribute in this list will not show up as a valid option for scoped modifiers; Used to allow attribute capture for internal calculation while preventing modification */
	UPROPERTY(EditDefaultsOnly, Category=Attributes)
	TArray<FGameplayEffectAttributeCaptureDefinition> InvalidScopedModifierAttributes;

	/** Any tag in this container will show up as a valid "temporary variable" for scoped modifiers; Used to allow for data-driven variable support that doesn't rely on scoped modifiers */
	UPROPERTY(EditDefaultsOnly, Category=NonAttributeCalculation)
	FGameplayTagContainer ValidTransientAggregatorIdentifiers;

public:
	/**
	 * Gets the collection of capture attribute definitions that the calculation class will accept as valid scoped modifiers
	 * 
	 * @param OutScopableModifiers	[OUT] Array to populate with definitions valid as scoped modifiers
	 */
	UE_API virtual void GetValidScopedModifierAttributeCaptureDefinitions(OUT TArray<FGameplayEffectAttributeCaptureDefinition>& OutScopableModifiers) const;

	/**
	 * Gets the collection of identifiers of valid transient aggregators ("temporary variable aggregators")
	 * 
	 * @return Array of valid transient aggregator identifiers
	 */
	UE_API virtual const FGameplayTagContainer& GetValidTransientAggregatorIdentifiers() const;

	/** Returns if this execution requires passed in tags */
	UE_API virtual bool DoesRequirePassedInTags() const;

#endif // #if WITH_EDITORONLY_DATA

public:

	/**
	 * Called whenever the owning gameplay effect is executed. Allowed to do essentially whatever is desired, including generating new
	 * modifiers to instantly execute as well.
	 * 
	 * @note: Native subclasses should override the auto-generated Execute_Implementation function and NOT this one.
	 * 
	 * @param ExecutionParams		Parameters for the custom execution calculation
	 * @param OutExecutionOutput	[OUT] Output data populated by the execution detailing further behavior or results of the execution
	 */
	UFUNCTION(BlueprintNativeEvent, Category="Calculation")
	UE_API void Execute(const FGameplayEffectCustomExecutionParameters& ExecutionParams, FGameplayEffectCustomExecutionOutput& OutExecutionOutput) const;
};



// -------------------------------------------------------------------------
//	Helper macros for declaring attribute captures 
// -------------------------------------------------------------------------

#define DECLARE_ATTRIBUTE_CAPTUREDEF(P) \
	FProperty* P##Property; \
	FGameplayEffectAttributeCaptureDefinition P##Def; \

#define DEFINE_ATTRIBUTE_CAPTUREDEF(S, P, T, B) \
{ \
	P##Property = FindFieldChecked<FProperty>(S::StaticClass(), GET_MEMBER_NAME_CHECKED(S, P)); \
	P##Def = FGameplayEffectAttributeCaptureDefinition(P##Property, EGameplayEffectAttributeCaptureSource::T, B); \
}

#undef UE_API
