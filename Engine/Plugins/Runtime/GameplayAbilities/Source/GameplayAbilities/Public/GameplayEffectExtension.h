// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AttributeSet.h"

// This file remains as legacy cruft. FGameplayEffectModCallbackData is used throughout the system but the original GameplayEffectExtension classes have been removed.
// This header remains to avoid breaking projects that had to include it for FGameplayEffectModCallbackData.

class UAbilitySystemComponent;
struct FGameplayEffectSpec;
struct FGameplayModifierEvaluatedData;

struct FGameplayEffectModCallbackData
{
	FGameplayEffectModCallbackData(const FGameplayEffectSpec& InEffectSpec, FGameplayModifierEvaluatedData& InEvaluatedData, UAbilitySystemComponent& InTarget)
		: EffectSpec(InEffectSpec)
		, EvaluatedData(InEvaluatedData)
		, Target(InTarget)
	{
	}

	const struct FGameplayEffectSpec&		EffectSpec;		// The spec that the mod came from
	struct FGameplayModifierEvaluatedData&	EvaluatedData;	// The 'flat'/computed data to be applied to the target

	class UAbilitySystemComponent &Target;		// Target we intend to apply to
};