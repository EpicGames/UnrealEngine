// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActiveGameplayEffectHandle.h"
#include "CoreMinimal.h"
#include "GameplayEffectAttributeCaptureDefinition.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "Engine/NetSerialization.h"
#include "GameFramework/Actor.h"
#include "GameplayTagContainer.h"
#include "AttributeSet.h"
#include "Net/Serialization/FastArraySerializer.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "GameplayEffectTypes.generated.h"

#define UE_API GAMEPLAYABILITIES_API

#define SKILL_SYSTEM_AGGREGATOR_DEBUG 1

#if SKILL_SYSTEM_AGGREGATOR_DEBUG
	#define SKILL_AGG_DEBUG( Format, ... ) *FString::Printf(Format, ##__VA_ARGS__)
#else
	#define SKILL_AGG_DEBUG( Format, ... ) nullptr
#endif

class Error;
class UAbilitySystemComponent;
class UGameplayAbility;
class UNetConnection;
struct FActiveGameplayEffect;
struct FGameplayEffectModCallbackData;
struct FGameplayEffectSpec;
struct FHitResult;
struct FMinimalReplicationTagCountMapForNetSerializer;
struct FNetGameplayTagCountContainerStateForNetSerialize;
namespace UE::Net
{
	struct FGameplayTagCountContainerNetSerializer;
	struct FGameplayEffectContextHandleAccessorForNetSerializer;
	class FMinimalReplicationTagCountMapReplicationFragment;
}

/** Wrappers to convert enum to string. These are fairly slow */
GAMEPLAYABILITIES_API FString EGameplayModOpToString(int32 Type);
GAMEPLAYABILITIES_API FString EGameplayModToString(int32 Type);
GAMEPLAYABILITIES_API FString EGameplayModEffectToString(int32 Type);
GAMEPLAYABILITIES_API FString EGameplayCueEventToString(int32 Type);

/** Valid gameplay modifier evaluation channels; Displayed and renamed via game-specific aliases and options */
UENUM()
enum class EGameplayModEvaluationChannel : uint8
{
	Channel0 UMETA(Hidden),
	Channel1 UMETA(Hidden),
	Channel2 UMETA(Hidden),
	Channel3 UMETA(Hidden),
	Channel4 UMETA(Hidden),
	Channel5 UMETA(Hidden),
	Channel6 UMETA(Hidden),
	Channel7 UMETA(Hidden),
	Channel8 UMETA(Hidden),
	Channel9 UMETA(Hidden),

	// Always keep last
	Channel_MAX UMETA(Hidden)
};


/** Struct representing evaluation channel settings for a gameplay modifier */
USTRUCT()
struct FGameplayModEvaluationChannelSettings
{
	GENERATED_USTRUCT_BODY()
	
	/** Constructor */
	UE_API FGameplayModEvaluationChannelSettings();

	/**
	 * Get the modifier evaluation channel to use
	 * 
	 * @return	Either the channel directly specified within the settings, if valid, or Channel0 in the event of a game not using modifier
	 *			channels or in the case of an invalid channel being specified within the settings
	 */
	UE_API EGameplayModEvaluationChannel GetEvaluationChannel() const;

	/** Editor-only constants to aid in hiding evaluation channel settings when appropriate */
#if WITH_EDITORONLY_DATA
	static UE_API const FName ForceHideMetadataKey;
	static UE_API const FString ForceHideMetadataEnabledValue;
#endif // #if WITH_EDITORONLY_DATA

	UE_API void SetEvaluationChannel(EGameplayModEvaluationChannel NewChannel);

	UE_API bool operator==(const FGameplayModEvaluationChannelSettings& Other) const;
	UE_API bool operator!=(const FGameplayModEvaluationChannelSettings& Other) const;

protected:

	/** Channel the settings would prefer to use, if possible/valid */
	UPROPERTY(EditDefaultsOnly, Category=EvaluationChannel)
	EGameplayModEvaluationChannel Channel;

	// Allow the details customization as a friend so it can handle custom display of the struct
	friend class FGameplayModEvaluationChannelSettingsDetails;
};


UENUM(BlueprintType)
namespace EGameplayModOp
{
	/**
	 * Defines the ways that mods will modify attributes. Values of the same type are aggregated, and then applied in the following equation:
	 * ((BaseValue + AddBase) * MultiplyAdditive / DivideAdditive * MultiplyCompound) + AddFinal
	 */
	enum Type : int
	{		
		/** Adds to the Base value. This happens first, before all other mods are considered. */
		AddBase				UMETA(DisplayName="Add (Base)"),

		/** Multipliers are added together first, then multiplied against prev result. E.g. 50% + 50% = 100% in values is 1.5 + 1.5 = 2.0. */
		MultiplyAdditive	UMETA(DisplayName = "Multiply (Additive)"),

		/** Divisors are added together, then divided against the prev result. E.g. 1/2 + 1/2 = 1/3 in values is 2 + 2 = 3. */
		DivideAdditive		UMETA(DisplayName = "Divide (Additive)"),

		// !!! Don't get tripped up here:  Override is 3 (due to existing serialized assets) !!!

		/** Multiply the prev result by this value. E.g. two values of 1.5 compounded: 1.5 * 1.5 = 2.25. */
		MultiplyCompound = 4 UMETA(DisplayName = "Multiply (Compound)"),

		/** Add this value to the final computed result. */
		AddFinal			UMETA(DisplayName = "Add (Final)"),
		
		/** This must always be the last value (used in iteration code). */
		Max					UMETA(Hidden, DisplayName="Invalid"),

		// Backwards compatible names
		Additive = 0		UMETA(Hidden), // Backwards compat name
		Multiplicitive = 1	 UMETA(Hidden), // Backwards compat name
		Division = 2		UMETA(Hidden), // Backwards compat name

		// Due to UI being displayed in the same order as the operations, we want Override at the bottom.

		/** Override the value, regardless of what the computation provides. */
		Override = 3		UMETA(DisplayName="Override"),
	};
}

namespace GameplayEffectUtilities
{
	/**
	 * Helper function to retrieve the modifier bias based upon modifier operation
	 * 
	 * @param ModOp	Modifier operation to retrieve the modifier bias for
	 * 
	 * @return Modifier bias for the specified operation
	 */
	GAMEPLAYABILITIES_API float GetModifierBiasByModifierOp(EGameplayModOp::Type ModOp);

	/**
	 * Helper function to compute the stacked modifier magnitude from a base magnitude, given a stack count and modifier operation
	 * 
	 * @param BaseComputedMagnitude	Base magnitude to compute from
	 * @param StackCount			Stack count to use for the calculation
	 * @param ModOp					Modifier operation to use
	 * 
	 * @return Computed modifier magnitude with stack count factored in
	 */
	GAMEPLAYABILITIES_API float ComputeStackedModifierMagnitude(float BaseComputedMagnitude, int32 StackCount, EGameplayModOp::Type ModOp);
}

/** Defines how multiple instances of the same GameplayEffect definition stack when applied (non-instant GE's). */
UENUM()
enum class EGameplayEffectStackingType : uint8
{
	/** No stacking. Each application is a separate instance. */
	None UMETA(DisplayName = "No Stacking"),

	/** Stacks only when the same source reapplies the effect. */
	AggregateBySource UMETA(DisplayName = "Stack Per Source"),

	/** Stacks only when the effect is reapplied to the same target. */
	AggregateByTarget UMETA(DisplayName = "Stack Per Target"),
};




/** Data that describes what happened in an attribute modification. This is passed to ability set callbacks */
USTRUCT(BlueprintType)
struct FGameplayModifierEvaluatedData
{
	GENERATED_USTRUCT_BODY()

	FGameplayModifierEvaluatedData()
		: Attribute()
		, ModifierOp(EGameplayModOp::Additive)
		, Magnitude(0.f)
		, IsValid(false)
	{
	}

	FGameplayModifierEvaluatedData(const FGameplayAttribute& InAttribute, TEnumAsByte<EGameplayModOp::Type> InModOp, float InMagnitude, FActiveGameplayEffectHandle InHandle = FActiveGameplayEffectHandle())
		: Attribute(InAttribute)
		, ModifierOp(InModOp)
		, Magnitude(InMagnitude)
		, Handle(InHandle)
		, IsValid(true)
	{
	}

	/** What attribute was modified */
	UPROPERTY()
	FGameplayAttribute Attribute;

	/** The numeric operation of this modifier: Override, Add, Multiply, etc  */
	UPROPERTY()
	TEnumAsByte<EGameplayModOp::Type> ModifierOp;

	/** The raw magnitude of the applied attribute, this is generally before being clamped */
	UPROPERTY()
	float Magnitude;

	/** Handle of the active gameplay effect that originated us. Will be invalid in many cases */
	UPROPERTY()
	FActiveGameplayEffectHandle	Handle;

	/** True if something was evaluated */
	UPROPERTY()
	bool IsValid;

	FString ToSimpleString() const
	{
		return FString::Printf(TEXT("%s %s EvalMag: %f"), *Attribute.GetName(), *EGameplayModOpToString(ModifierOp), Magnitude);
	}
};


/**
 * Data structure that stores an instigator and related data, such as positions and targets
 * Games can subclass this structure and add game-specific information
 * It is passed throughout effect execution so it is a great place to track transient information about an execution
 */
USTRUCT()
struct FGameplayEffectContext
{
	GENERATED_USTRUCT_BODY()

	FGameplayEffectContext()
	: AbilityLevel(1)
	, WorldOrigin(ForceInitToZero)
	, bHasWorldOrigin(false)
	, bReplicateSourceObject(false)
	, bReplicateInstigator(false)
	, bReplicateEffectCauser(false)
	{
	}

	FGameplayEffectContext(AActor* InInstigator, AActor* InEffectCauser)
	: AbilityLevel(1)
	, WorldOrigin(ForceInitToZero)
	, bHasWorldOrigin(false)
	, bReplicateSourceObject(false)
	, bReplicateInstigator(false)
	, bReplicateEffectCauser(false)
	{
		FGameplayEffectContext::AddInstigator(InInstigator, InEffectCauser);
	}

	virtual ~FGameplayEffectContext() = default;

	/** Returns the list of gameplay tags applicable to this effect, defaults to the owner's tags. SpecTagContainer remains untouched by default. */
	UE_API virtual void GetOwnedGameplayTags(OUT FGameplayTagContainer& ActorTagContainer, OUT FGameplayTagContainer& SpecTagContainer) const;

	/** Sets the instigator and effect causer. Instigator is who owns the ability that spawned this, EffectCauser is the actor that is the physical source of the effect, such as a weapon. They can be the same. */
	UE_API virtual void AddInstigator(class AActor *InInstigator, class AActor *InEffectCauser);

	/** Sets the ability that was used to spawn this */
	UE_API virtual void SetAbility(const UGameplayAbility* InGameplayAbility);

	/** Returns the immediate instigator that applied this effect */
	virtual AActor* GetInstigator() const
	{
		return Instigator.Get();
	}

	/** Returns the CDO of the ability used to instigate this context */
	UE_API const UGameplayAbility* GetAbility() const;

	/** Returns the specific instance that instigated this, may not always be set */
	UE_API const UGameplayAbility* GetAbilityInstance_NotReplicated() const;

	/** Gets the ability level this was evaluated at */
	int32 GetAbilityLevel() const
	{
		return AbilityLevel;
	}

	/** Returns the ability system component of the instigator of this effect */
	virtual UAbilitySystemComponent* GetInstigatorAbilitySystemComponent() const
	{
		return InstigatorAbilitySystemComponent.Get();
	}

	/** Returns the physical actor tied to the application of this effect */
	virtual AActor* GetEffectCauser() const
	{
		return EffectCauser.Get();
	}

	/** Modify the effect causer actor, useful when that information is added after creation */
	void SetEffectCauser(AActor* InEffectCauser)
	{
		EffectCauser = InEffectCauser;
		bReplicateEffectCauser = CanActorReferenceBeReplicated(InEffectCauser);
	}

	/** Should always return the original instigator that started the whole chain. Subclasses can override what this does */
	virtual AActor* GetOriginalInstigator() const
	{
		return Instigator.Get();
	}

	/** Returns the ability system component of the instigator that started the whole chain */
	virtual UAbilitySystemComponent* GetOriginalInstigatorAbilitySystemComponent() const
	{
		return InstigatorAbilitySystemComponent.Get();
	}

	/** Sets the object this effect was created from. */
	virtual void AddSourceObject(const UObject* NewSourceObject)
	{
		SourceObject = MakeWeakObjectPtr(const_cast<UObject*>(NewSourceObject));
		bReplicateSourceObject = NewSourceObject && NewSourceObject->IsSupportedForNetworking();
	}

	/** Returns the object this effect was created from. */
	virtual UObject* GetSourceObject() const
	{
		return SourceObject.Get();
	}

	/** Add actors to the stored actor list */
	UE_API virtual void AddActors(const TArray<TWeakObjectPtr<AActor>>& IActor, bool bReset = false);

	/** Add a hit result for targeting */
	UE_API virtual void AddHitResult(const FHitResult& InHitResult, bool bReset = false);

	/** Returns actor list, may be empty */
	virtual const TArray<TWeakObjectPtr<AActor>>& GetActors() const
	{
		return Actors;
	}

	/** Returns hit result, this can be null */
	virtual const FHitResult* GetHitResult() const
	{
		return const_cast<FGameplayEffectContext*>(this)->GetHitResult();
	}

	/** Returns hit result, this can be null */
	virtual FHitResult* GetHitResult()
	{
		return HitResult.Get();
	}

	/** Adds an origin point */
	UE_API virtual void AddOrigin(FVector InOrigin);

	/** Returns origin point, may be invalid if HasOrigin is false */
	virtual const FVector& GetOrigin() const
	{
		return WorldOrigin;
	}

	/** Returns true if GetOrigin will give valid information */
	virtual bool HasOrigin() const
	{
		return bHasWorldOrigin;
	}

	/** Returns debug string */
	UE_API virtual FString ToString() const;

	/** Returns the actual struct used for serialization, subclasses must override this! */
	virtual UScriptStruct* GetScriptStruct() const
	{
		return FGameplayEffectContext::StaticStruct();
	}

	/** Creates a copy of this context, used to duplicate for later modifications */
	virtual FGameplayEffectContext* Duplicate() const
	{
		FGameplayEffectContext* NewContext = new FGameplayEffectContext();
		*NewContext = *this;
		if (GetHitResult())
		{
			// Does a deep copy of the hit result
			NewContext->AddHitResult(*GetHitResult(), true);
		}
		return NewContext;
	}

	/** True if this was instigated by a locally controlled actor */
	UE_API virtual bool IsLocallyControlled() const;

	/** True if this was instigated by a locally controlled player */
	UE_API virtual bool IsLocallyControlledPlayer() const;

	/** Custom serialization, subclasses must override this */
	UE_API virtual bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

protected:
	static UE_API bool CanActorReferenceBeReplicated(const AActor* Actor);

	// The object pointers here have to be weak because contexts aren't necessarily tracked by GC in all cases

	/** Instigator actor, the actor that owns the ability system component */
	UPROPERTY()
	TWeakObjectPtr<AActor> Instigator;

	/** The physical actor that actually did the damage, can be a weapon or projectile */
	UPROPERTY()
	TWeakObjectPtr<AActor> EffectCauser;

	/** The ability CDO that is responsible for this effect context (replicated) */
	UPROPERTY()
	TWeakObjectPtr<UGameplayAbility> AbilityCDO;

	/** The ability instance that is responsible for this effect context (NOT replicated) */
	UPROPERTY(NotReplicated)
	TWeakObjectPtr<UGameplayAbility> AbilityInstanceNotReplicated;

	/** The level this was executed at */
	UPROPERTY()
	int32 AbilityLevel;

	/** Object this effect was created from, can be an actor or static object. Useful to bind an effect to a gameplay object */
	UPROPERTY()
	TWeakObjectPtr<UObject> SourceObject;

	/** The ability system component that's bound to instigator */
	UPROPERTY(NotReplicated)
	TWeakObjectPtr<UAbilitySystemComponent> InstigatorAbilitySystemComponent;

	/** Actors referenced by this context */
	UPROPERTY()
	TArray<TWeakObjectPtr<AActor>> Actors;

	/** Trace information - may be nullptr in many cases */
	TSharedPtr<FHitResult>	HitResult;

	/** Stored origin, may be invalid if bHasWorldOrigin is false */
	UPROPERTY()
	FVector	WorldOrigin;

	UPROPERTY()
	uint8 bHasWorldOrigin:1;

	/** True if the SourceObject can be replicated. This bool is not replicated itself. */
	UPROPERTY(NotReplicated)
	uint8 bReplicateSourceObject:1;
	
	/** True if the Instigator can be replicated. This bool is not replicated itself. */
	UPROPERTY(NotReplicated)	
	uint8 bReplicateInstigator:1;

	/** True if the Instigator can be replicated. This bool is not replicated itself. */
	UPROPERTY(NotReplicated)	
	uint8 bReplicateEffectCauser:1;
};

template<>
struct TStructOpsTypeTraits< FGameplayEffectContext > : public TStructOpsTypeTraitsBase2< FGameplayEffectContext >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true		// Necessary so that TSharedPtr<FHitResult> Data is copied around
	};
};


/**
 * Handle that wraps a FGameplayEffectContext or subclass, to allow it to be polymorphic and replicate properly
 */
USTRUCT(BlueprintType)
struct FGameplayEffectContextHandle
{
	GENERATED_USTRUCT_BODY()

	FGameplayEffectContextHandle()
	{
	}

	virtual ~FGameplayEffectContextHandle() = default;

	/** Constructs from an existing context, should be allocated by new */
	explicit FGameplayEffectContextHandle(FGameplayEffectContext* DataPtr)
	{
		Data = TSharedPtr<FGameplayEffectContext>(DataPtr);
	}

	/** Sets from an existing context, should be allocated by new */
	void operator=(FGameplayEffectContext* DataPtr)
	{
		Data = TSharedPtr<FGameplayEffectContext>(DataPtr);
	}

	void Clear()
	{
		Data.Reset();
	}

	bool IsValid() const
	{
		return Data.IsValid();
	}

	/** Returns Raw effet context, may be null */
	FGameplayEffectContext* Get()
	{
		return IsValid() ? Data.Get() : nullptr;
	}
	const FGameplayEffectContext* Get() const
	{
		return IsValid() ? Data.Get() : nullptr;
	}

	/** Returns the list of gameplay tags applicable to this effect, defaults to the owner's tags */
	void GetOwnedGameplayTags(OUT FGameplayTagContainer& ActorTagContainer, OUT FGameplayTagContainer& SpecTagContainer) const
	{
		if (IsValid())
		{
			Data->GetOwnedGameplayTags(ActorTagContainer, SpecTagContainer);
		}
	}

	/** Sets the instigator and effect causer. Instigator is who owns the ability that spawned this, EffectCauser is the actor that is the physical source of the effect, such as a weapon. They can be the same. */
	void AddInstigator(class AActor *InInstigator, class AActor *InEffectCauser)
	{
		if (IsValid())
		{
			Data->AddInstigator(InInstigator, InEffectCauser);
		}
	}

	/** Sets Ability instance and CDO parameters on context */
	void SetAbility(const UGameplayAbility* InGameplayAbility)
	{
		if (IsValid())
		{
			Data->SetAbility(InGameplayAbility);
		}
	}

	/** Returns the immediate instigator that applied this effect */
	virtual AActor* GetInstigator() const
	{
		if (IsValid())
		{
			return Data->GetInstigator();
		}
		return nullptr;
	}

	/** Returns the Ability CDO */
	const UGameplayAbility* GetAbility() const
	{
		if (IsValid())
		{
			return Data->GetAbility();
		}
		return nullptr;
	}

	/** Returns the Ability Instance (never replicated) */
	const UGameplayAbility* GetAbilityInstance_NotReplicated() const
	{
		if (IsValid())
		{
			return Data->GetAbilityInstance_NotReplicated();
		}
		return nullptr;
	}

	/** Returns level this was executed at */
	int32 GetAbilityLevel() const
	{
		if (IsValid())
		{
			return Data->GetAbilityLevel();
		}
		return 1;
	}

	/** Returns the ability system component of the instigator of this effect */
	virtual UAbilitySystemComponent* GetInstigatorAbilitySystemComponent() const
	{
		if (IsValid())
		{
			return Data->GetInstigatorAbilitySystemComponent();
		}
		return nullptr;
	}

	/** Returns the physical actor tied to the application of this effect */
	virtual AActor* GetEffectCauser() const
	{
		if (IsValid())
		{
			return Data->GetEffectCauser();
		}
		return nullptr;
	}

	/** Should always return the original instigator that started the whole chain. Subclasses can override what this does */
	AActor* GetOriginalInstigator() const
	{
		if (IsValid())
		{
			return Data->GetOriginalInstigator();
		}
		return nullptr;
	}

	/** Returns the ability system component of the instigator that started the whole chain */
	UAbilitySystemComponent* GetOriginalInstigatorAbilitySystemComponent() const
	{
		if (IsValid())
		{
			return Data->GetOriginalInstigatorAbilitySystemComponent();
		}
		return nullptr;
	}

	/** Sets the object this effect was created from. */
	void AddSourceObject(const UObject* NewSourceObject)
	{
		if (IsValid())
		{
			Data->AddSourceObject(NewSourceObject);
		}
	}

	/** Returns the object this effect was created from. */
	UObject* GetSourceObject() const
	{
		if (IsValid())
		{
			return Data->GetSourceObject();
		}
		return nullptr;
	}

	/** Returns if the instigator is locally controlled */
	bool IsLocallyControlled() const
	{
		if (IsValid())
		{
			return Data->IsLocallyControlled();
		}
		return false;
	}

	/** Returns if the instigator is locally controlled and a player */
	bool IsLocallyControlledPlayer() const
	{
		if (IsValid())
		{
			return Data->IsLocallyControlledPlayer();
		}
		return false;
	}

	/** Add actors to the stored actor list */
	void AddActors(const TArray<TWeakObjectPtr<AActor>>& InActors, bool bReset = false)
	{
		if (IsValid())
		{
			Data->AddActors(InActors, bReset);
		}
	}

	/** Add a hit result for targeting */
	void AddHitResult(const FHitResult& InHitResult, bool bReset = false)
	{
		if (IsValid())
		{
			Data->AddHitResult(InHitResult, bReset);
		}
	}

	/** Returns actor list, may be empty */
	const TArray<TWeakObjectPtr<AActor>> GetActors()
	{
		if (IsValid())
		{
			return Data->GetActors();
		}

		return {};
	}

	/** Returns hit result, this can be null */
	const FHitResult* GetHitResult() const
	{
		if (IsValid())
		{
			return Data->GetHitResult();
		}
		return nullptr;
	}

	/** Adds an origin point */
	void AddOrigin(FVector InOrigin)
	{
		if (IsValid())
		{
			Data->AddOrigin(InOrigin);
		}
	}

	/** Returns origin point, may be invalid if HasOrigin is false */
	virtual const FVector& GetOrigin() const
	{
		if (IsValid())
		{
			return Data->GetOrigin();
		}
		return FVector::ZeroVector;
	}

	/** Returns true if GetOrigin will give valid information */
	virtual bool HasOrigin() const
	{
		if (IsValid())
		{
			return Data->HasOrigin();
		}
		return false;
	}

	/** Returns debug string */
	FString ToString() const
	{
		return IsValid() ? Data->ToString() : FString(TEXT("NONE"));
	}

	/** Creates a deep copy of this handle, used before modifying */
	FGameplayEffectContextHandle Duplicate() const
	{
		if (IsValid())
		{
			FGameplayEffectContext* NewContext = Data->Duplicate();
			return FGameplayEffectContextHandle(NewContext);
		}
		else
		{
			return FGameplayEffectContextHandle();
		}
	}

	/** Comparison operator */
	bool operator==(FGameplayEffectContextHandle const& Other) const
	{
		if (Data.IsValid() != Other.Data.IsValid())
		{
			return false;
		}
		if (Data.Get() != Other.Data.Get())
		{
			return false;
		}
		return true;
	}

	/** Comparison operator */
	bool operator!=(FGameplayEffectContextHandle const& Other) const
	{
		return !(FGameplayEffectContextHandle::operator==(Other));
	}

	/** Custom serializer, handles polymorphism of context */
	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

private:
	friend UE::Net::FGameplayEffectContextHandleAccessorForNetSerializer;

	TSharedPtr<FGameplayEffectContext> Data;
};

template<>
struct TStructOpsTypeTraits<FGameplayEffectContextHandle> : public TStructOpsTypeTraitsBase2<FGameplayEffectContextHandle>
{
	enum
	{
		WithCopy = true,		// Necessary so that TSharedPtr<FGameplayEffectContext> Data is copied around
		WithNetSerializer = true,
		WithIdenticalViaEquality = true,
	};
};


/** Data struct for containing information pertinent to GameplayEffects as they are removed */
USTRUCT(BlueprintType)
struct FGameplayEffectRemovalInfo
{
	GENERATED_USTRUCT_BODY()
	
	/** True when the gameplay effect's duration has not expired, meaning the gameplay effect is being forcefully removed.  */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Removal")
	bool bPrematureRemoval = false;

	/** True when the effect is being removed due to prediction rejection */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Removal")
	bool bPredictionRejected = false;

	/** Number of Stacks this gameplay effect had before it was removed. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Removal")
	int32 StackCount = 0;

	/** Actor this gameplay effect was targeting. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Removal")
	FGameplayEffectContextHandle EffectContext;
	
	/** Which AbilitySystemComponent this Gameplay Effect was applied to */
	UPROPERTY(Transient)
	TObjectPtr<UAbilitySystemComponent> OwningASC = {};

	/** The Effect being Removed */
	const FActiveGameplayEffect* ActiveEffect = nullptr;
};


/** Metadata about a gameplay cue execution */
USTRUCT(BlueprintType, meta = (HasNativeBreak = "/Script/GameplayAbilities.AbilitySystemBlueprintLibrary.BreakGameplayCueParameters", HasNativeMake = "/Script/GameplayAbilities.AbilitySystemBlueprintLibrary.MakeGameplayCueParameters"))
struct FGameplayCueParameters
{
	GENERATED_USTRUCT_BODY()

	FGameplayCueParameters()
	: NormalizedMagnitude(0.0f)
	, RawMagnitude(0.0f)
	, Location(ForceInitToZero)
	, Normal(ForceInitToZero)
	, GameplayEffectLevel(1)
	, AbilityLevel(1)
	{}

	/** Projects can override this via UAbilitySystemGlobals */
	UE_API FGameplayCueParameters(const struct FGameplayEffectSpecForRPC &Spec);
	UE_API FGameplayCueParameters(const struct FGameplayEffectContextHandle& EffectContext);

	UE_API bool operator==(const FGameplayCueParameters& Other) const;
	bool operator!=(const FGameplayCueParameters& Other) const
	{
		return !(*this == Other);
	}

	/** Magnitude of source gameplay effect, normalzed from 0-1. Use this for "how strong is the gameplay effect" (0=min, 1=,max) */
	UPROPERTY(BlueprintReadWrite, Category=GameplayCue)
	float NormalizedMagnitude;

	/** Raw final magnitude of source gameplay effect. Use this is you need to display numbers or for other informational purposes. */
	UPROPERTY(BlueprintReadWrite, Category=GameplayCue)
	float RawMagnitude;

	/** Effect context, contains information about hit result, etc */
	UPROPERTY(BlueprintReadWrite, Category=GameplayCue)
	FGameplayEffectContextHandle EffectContext;

	/** The tag name that matched this specific gameplay cue handler */
	UPROPERTY(BlueprintReadWrite, Category=GameplayCue, NotReplicated)
	mutable FGameplayTag MatchedTagName;

	/** The original tag of the gameplay cue */
	UPROPERTY(BlueprintReadWrite, Category=GameplayCue, NotReplicated)
	mutable FGameplayTag OriginalTag;

	/** The aggregated source tags taken from the effect spec */
	UPROPERTY(BlueprintReadWrite, Category=GameplayCue)
	FGameplayTagContainer AggregatedSourceTags;

	/** The aggregated target tags taken from the effect spec */
	UPROPERTY(BlueprintReadWrite, Category=GameplayCue)
	FGameplayTagContainer AggregatedTargetTags;

	/** Location cue took place at */
	UPROPERTY(BlueprintReadWrite, Category=GameplayCue)
	FVector_NetQuantize10 Location;

	/** Normal of impact that caused cue */
	UPROPERTY(BlueprintReadWrite, Category=GameplayCue)
	FVector_NetQuantizeNormal Normal;

	/** Instigator actor, the actor that owns the ability system component */
	UPROPERTY(BlueprintReadWrite, Category=GameplayCue)
	TWeakObjectPtr<AActor> Instigator;

	/** The physical actor that actually did the damage, can be a weapon or projectile */
	UPROPERTY(BlueprintReadWrite, Category=GameplayCue)
	TWeakObjectPtr<AActor> EffectCauser;

	/** Object this effect was created from, can be an actor or static object. Useful to bind an effect to a gameplay object */
	UPROPERTY(BlueprintReadWrite, Category=GameplayCue)
	TWeakObjectPtr<const UObject> SourceObject;

	/** PhysMat of the hit, if there was a hit. */
	UPROPERTY(BlueprintReadWrite, Category = GameplayCue)
	TWeakObjectPtr<const UPhysicalMaterial> PhysicalMaterial;

	/** If originating from a GameplayEffect, the level of that GameplayEffect */
	UPROPERTY(BlueprintReadWrite, Category = GameplayCue)
	int32 GameplayEffectLevel;

	/** If originating from an ability, this will be the level of that ability */
	UPROPERTY(BlueprintReadWrite, Category = GameplayCue)
	int32 AbilityLevel;

	/** Could be used to say "attach FX to this component always" */
	UPROPERTY(BlueprintReadWrite, Category = GameplayCue)
	TWeakObjectPtr<USceneComponent> TargetAttachComponent;

	/** If we're using a minimal replication proxy, should we replicate location for this cue */
	UPROPERTY(BlueprintReadWrite, Category = GameplayCue)
	bool bReplicateLocationWhenUsingMinimalRepProxy = false;

	/** If originating from a GameplayEffect, whether that GameplayEffect is still Active */
	bool bGameplayEffectActive = true;

	/** Optimized serializer */
	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** Returns true if this is locally controlled, using fallback actor if nothing else available */
	UE_API bool IsInstigatorLocallyControlled(AActor* FallbackActor = nullptr) const;

	/** Fallback actor is used if the parameters have nullptr for instigator and effect causer */
	UE_API bool IsInstigatorLocallyControlledPlayer(AActor* FallbackActor=nullptr) const;

	/** Returns the actor that instigated this originally, generally attached to an ability system component */
	UE_API AActor* GetInstigator() const;

	/** Returns the actor that physically caused the damage, could be a projectile or weapon */
	UE_API AActor* GetEffectCauser() const;

	/** Returns the object that originally caused this, game-specific but usually not an actor */
	UE_API const UObject* GetSourceObject() const;
};

template<>
struct TStructOpsTypeTraits<FGameplayCueParameters> : public TStructOpsTypeTraitsBase2<FGameplayCueParameters>
{
	enum
	{
		WithNetSerializer = true		
	};
};


UENUM(BlueprintType)
namespace EGameplayCueEvent
{
	/** Indicates what type of action happened to a specific gameplay cue tag. Sometimes you will get multiple events at once */
	enum Type : int
	{
		/** Called when a GameplayCue with duration is first activated, this will only be called if the client witnessed the activation */
		OnActive,

		/** Called when a GameplayCue with duration is first seen as active, even if it wasn't actually just applied (Join in progress, etc) */
		WhileActive,

		/** Called when a GameplayCue is executed, this is used for instant effects or periodic ticks */
		Executed,

		/** Called when a GameplayCue with duration is removed */
		Removed
	};
}

DECLARE_DELEGATE_OneParam(FOnGameplayAttributeEffectExecuted, struct FGameplayModifierEvaluatedData&);
DECLARE_DELEGATE(FDeferredTagChangeDelegate);

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGameplayEffectTagCountChanged, const FGameplayTag, int32);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGivenActiveGameplayEffectRemoved, const FActiveGameplayEffect&);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnActiveGameplayEffectRemoved_Info, const FGameplayEffectRemovalInfo&);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnActiveGameplayEffectStackChange, FActiveGameplayEffectHandle, int32 /*NewStackCount*/, int32 /*PreviousStackCount*/);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnActiveGameplayEffectTimeChange, FActiveGameplayEffectHandle, float /*NewStartTime*/, float /*NewDuration*/);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnActiveGameplayEffectInhibitionChanged, FActiveGameplayEffectHandle, bool /*bIsInhibited*/);

/** Callback struct for different types of gameplay effect changes */
struct FActiveGameplayEffectEvents
{
	FOnActiveGameplayEffectRemoved_Info OnEffectRemoved;
	FOnActiveGameplayEffectStackChange OnStackChanged;
	FOnActiveGameplayEffectTimeChange OnTimeChanged;
	FOnActiveGameplayEffectInhibitionChanged OnInhibitionChanged;
};

// This is deprecated, use FOnGameplayAttributeValueChange
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGameplayAttributeChange, float, const FGameplayEffectModCallbackData*);


/** Temporary parameter struct used when an attribute has changed */
struct FOnAttributeChangeData
{
	FOnAttributeChangeData()
		: NewValue(0.0f)
		, OldValue(0.0f)
		, GEModData(nullptr)
	{ }

	FGameplayAttribute Attribute;

	float	NewValue;
	float	OldValue;
	const FGameplayEffectModCallbackData* GEModData;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnGameplayAttributeValueChange, const FOnAttributeChangeData&);

DECLARE_DELEGATE_RetVal(FGameplayTagContainer, FGetGameplayTags);

DECLARE_DELEGATE_RetVal_OneParam(FOnGameplayEffectTagCountChanged&, FRegisterGameplayTagChangeDelegate, FGameplayTag);


UENUM(BlueprintType)
namespace EGameplayTagEventType
{
	/** Rather a tag was added or removed, used in callbacks */
	enum Type : int
	{		
		/** Event only happens when tag is new or completely removed */
		NewOrRemoved,

		/** Event happens any time tag "count" changes */
		AnyCountChange		
	};
}

/**
 * Enum to describe replication rules for each tag
 * Any state besides "None" must be added on the server and can be predicted on clients.
 */
UENUM()
enum class EGameplayTagReplicationState : uint8
{
	None,				// Tag will not replicate
	SimulatedTagOnly,		// tag will be replicated to simulated proxies, but not the owner. Used for GameplayEffect tags
	TagOnly,			// Tag will replicate to all without count
	CountToOwner,		// Tag will replicate to all and Count will replicate to owner, Functions same as TagAndCountToAll with Iris
	TagAndCountToAll	// Tag and Count will replicate to all
};

USTRUCT()
struct FGameplayTagCountItem
{
	GENERATED_USTRUCT_BODY()
	
	FGameplayTagCountItem()
	: Tag(FGameplayTag())
	, Count(0)
	, ReplicationState(EGameplayTagReplicationState::None)
	{}

	FGameplayTagCountItem(const FGameplayTag& InTag, const int32 InCount = 1, EGameplayTagReplicationState TagRepState = EGameplayTagReplicationState::None)
	: Tag(InTag)
	, Count(InCount)
	, ReplicationState(TagRepState)
	{}

	UPROPERTY()
	FGameplayTag Tag;

	UPROPERTY()
	int32 Count;

	UPROPERTY()
	EGameplayTagReplicationState ReplicationState;

	bool operator == (const FGameplayTagCountItem& Other) const
	{
		return Tag.MatchesTagExact(Other.Tag);
	}

	bool operator == (const FGameplayTag& OtherTag) const
	{
		return Tag.MatchesTagExact(OtherTag);
	}
};

/**
 * Struct that tracks the number/count of tag applications within it. Explicitly tracks the tags added or removed,
 * while simultaneously tracking the count of parent tags as well. Events/delegates are fired whenever the tag counts
 * of any tag (explicit or parent) are modified.
 */
USTRUCT()
struct FGameplayTagCountContainer
{
	GENERATED_BODY()
	
	/**
	 * Check if the count container has a gameplay tag that matches against the specified tag (expands to include parents of asset tags)
	 * 
	 * @param TagToCheck	Tag to check for a match
	 * 
	 * @return True if the count container has a gameplay tag that matches, false if not
	 */
	inline bool HasMatchingGameplayTag(FGameplayTag TagToCheck) const
	{
		return GameplayTagCountMap.FindRef(TagToCheck) > 0;
	}

	/**
	 * Check if the count container has gameplay tags that matches against all of the specified tags (expands to include parents of asset tags)
	 * 
	 * @param TagContainer			Tag container to check for a match. If empty will return true
	 * 
	 * @return True if the count container matches all of the gameplay tags
	 */
	inline bool HasAllMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const
	{
		// if the TagContainer count is 0 return bCountEmptyAsMatch;
		if (TagContainer.Num() == 0)
		{
			return true;
		}

		bool AllMatch = true;
		for (const FGameplayTag& Tag : TagContainer)
		{
			if (GameplayTagCountMap.FindRef(Tag) <= 0)
			{
				AllMatch = false;
				break;
			}
		}		
		return AllMatch;
	}
	
	/**
	 * Check if the count container has gameplay tags that matches against any of the specified tags (expands to include parents of asset tags)
	 * 
	 * @param TagContainer			Tag container to check for a match. If empty will return false
	 * 
	 * @return True if the count container matches any of the gameplay tags
	 */
	inline bool HasAnyMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const
	{
		if (TagContainer.Num() == 0)
		{
			return false;
		}

		bool AnyMatch = false;
		for (const FGameplayTag& Tag : TagContainer)
		{
			if (GameplayTagCountMap.FindRef(Tag) > 0)
			{
				AnyMatch = true;
				break;
			}
		}
		return AnyMatch;
	}
	
	/**
	 * Update the specified container of tags by the specified delta, potentially causing an additional or removal from the explicit tag list
	 * 
	 * @param Container		Container of tags to update
	 * @param CountDelta	Delta of the tag count to apply
	 */
	inline void UpdateTagCount(const FGameplayTagContainer& Container, int32 CountDelta)
	{
		if (CountDelta != 0)
		{
			bool bUpdatedAny = false;
			TArray<FDeferredTagChangeDelegate> DeferredTagChangeDelegates;
			for (auto TagIt = Container.CreateConstIterator(); TagIt; ++TagIt)
			{
				bUpdatedAny |= UpdateTagMapDeferredParentRemoval_Internal(*TagIt, CountDelta, DeferredTagChangeDelegates, EGameplayTagReplicationState::None);
			}

			if (bUpdatedAny && CountDelta < 0)
			{
				ExplicitTags.FillParentTags();
			}

			for (FDeferredTagChangeDelegate& Delegate : DeferredTagChangeDelegates)
			{
				Delegate.Execute();
			}
		}
	}
	
	/**
	 * Update the specified tag by the specified delta, potentially causing an additional or removal from the explicit tag list
	 * 
	 * @param Tag						Tag to update
	 * @param CountDelta				Delta of the tag count to apply
	 * 
	 * @return True if tag was *either* added or removed. (E.g., we had the tag and now dont. or didnt have the tag and now we do. We didn't just change the count (1 count -> 2 count would return false).
	 */
	inline bool UpdateTagCount(const FGameplayTag& Tag, int32 CountDelta, EGameplayTagReplicationState TagRepState)
	{
		if (Tag.IsValid() && CountDelta != 0)
		{
			return UpdateTagMap_Internal(Tag, CountDelta, TagRepState);
		}

		return false;
	}

	/**
	 * Update the specified tag by the specified delta, potentially causing an additional or removal from the explicit tag list.
	 * Calling code MUST call FillParentTags followed by executing the returned delegates.
	 * 
	 * @param Tag						Tag to update
	 * @param CountDelta				Delta of the tag count to apply
	 * @param DeferredTagChangeDelegates		Delegates to be called after this code runs
	 * 
	 * @return True if tag was *either* added or removed. (E.g., we had the tag and now dont. or didnt have the tag and now we do. We didn't just change the count (1 count -> 2 count would return false).
	 */
	inline bool UpdateTagCount_DeferredParentRemoval(const FGameplayTag& Tag, int32 CountDelta, TArray<FDeferredTagChangeDelegate>& DeferredTagChangeDelegates)
	{
		if (CountDelta != 0)
		{
			return UpdateTagMapDeferredParentRemoval_Internal(Tag, CountDelta, DeferredTagChangeDelegates, EGameplayTagReplicationState::None);
		}

		return false;
	}

	/**
	 * Set the specified tag count to a specific value
	 * 
	 * @param Tag			Tag to update
	 * @param Count			New count of the tag
	 * @param TagRepState	The replication state for the tag, if it is being added it will be added with this state
	 * 
	 * @return True if tag was *either* added or removed. (E.g., we had the tag and now dont. or didnt have the tag and now we do. We didn't just change the count (1 count -> 2 count would return false).
	 */
	inline bool SetTagCount(const FGameplayTag& Tag, int32 NewCount, EGameplayTagReplicationState TagRepState = EGameplayTagReplicationState::None)
	{
		if (!Tag.IsValid())
		{
			return false;
		}

		int32 ExistingCount = 0;
		const int32 Idx  = Items.Find(Tag);
		if (Idx != INDEX_NONE)
		{
			ExistingCount = Items[Idx].Count;
		}

		int32 CountDelta = NewCount - ExistingCount;
		if (CountDelta != 0)
		{
			return UpdateTagMap_Internal(Tag, CountDelta, TagRepState);
		}

		return false;
	}

	/**
	* return the hierarchical count for a specified tag
	* e.g. if A.B & A.C were added, GetTagCount("A") would return 2.
	*
	* @param Tag			Tag to update
	*
	* @return the count of the passed in tag
	*/
	inline int32 GetTagCount(const FGameplayTag& Tag) const
	{
		if (const int32* Ptr = GameplayTagCountMap.Find(Tag))
		{
			return *Ptr;
		}

		return 0;
	}

	/**
	* return how many times the exact specified tag has been added to the container (ignores the tag hierarchy)
	* e.g. if A.B & A.C were added, GetExplicitTagCount("A") would return 0, and GetExplicitTagCount("A.B") would return 1.
	*
	* @param Tag			Tag to update
	*
	* @return the count of the passed in tag
	*/
	inline int32 GetExplicitTagCount(const FGameplayTag& Tag) const
	{
		const int32 Idx  = Items.Find(Tag);
		if (Idx != INDEX_NONE)
		{
			return Items[Idx].Count;
		}

		return 0;
	}

	/** Fills in ParentTags from GameplayTags */
	void FillParentTags()
	{
		ExplicitTags.FillParentTags();
	}

	/**
	 *	Broadcasts the AnyChange event for this tag. This is called when the stack count of the backing gameplay effect change.
	 *	It is up to the receiver of the broadcasted delegate to decide what to do with this.
	 */
	UE_API void Notify_StackCountChange(const FGameplayTag& Tag);

	/**
	 * Return delegate that can be bound to for when the specific tag's count changes to or off of zero
	 *
	 * @param Tag	Tag to get a delegate for
	 * 
	 * @return Delegate for when the specified tag's count changes to or off of zero
	 */
	UE_API FOnGameplayEffectTagCountChanged& RegisterGameplayTagEvent(const FGameplayTag& Tag, EGameplayTagEventType::Type EventType=EGameplayTagEventType::NewOrRemoved);
	
	/**
	 * Return delegate that can be bound to for when the any tag's count changes to or off of zero
	 * 
	 * @return Delegate for when any tag's count changes to or off of zero
	 */
	FOnGameplayEffectTagCountChanged& RegisterGenericGameplayEvent()
	{
		return OnAnyTagChangeDelegate;
	}

	/** Simple accessor to the explicit gameplay tag list */
	const FGameplayTagContainer& GetExplicitGameplayTags() const
	{
		return ExplicitTags;
	}

	/**
	 * Removes all of the tags. Does not notify any delegates.
	 * 
	 * @param bResetCallbacks	If true, also remove all of the registered tag count change delegates
	 */
	UE_API void Reset(bool bResetCallbacks = true);

	bool NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParams);

	UE_API void SetOwner(UAbilitySystemComponent* ASC);

	bool operator==(const FGameplayTagCountContainer& Other) const
	{
		if (Items.Num() != Other.Items.Num())
		{
			return false;
		}

		int32 Idx = INDEX_NONE;
		for (const FGameplayTagCountItem& Item : Items)
		{
			Idx = Other.Items.Find(Item);
			if (Idx == INDEX_NONE || Item.Count != Other.Items[Idx].Count)
			{
				return false;
			}
		}

		return true;
	}

private:

	friend UE::Net::FGameplayTagCountContainerNetSerializer;
	friend FNetGameplayTagCountContainerStateForNetSerialize;

	struct FDelegateInfo
	{
		FOnGameplayEffectTagCountChanged	OnNewOrRemove;
		FOnGameplayEffectTagCountChanged	OnAnyChange;
	};

	/** Array of explicit tags and their count */
	TArray<FGameplayTagCountItem> Items;

	/** Map of tag to delegate that will be fired when the count for the key tag changes to or away from zero */
	TMap<FGameplayTag, FDelegateInfo> GameplayTagEventMap;

	/** Map of tag to active count of that tag */
	TMap<FGameplayTag, int32> GameplayTagCountMap;

	/** Delegate fired whenever any tag's count changes to or away from zero */
	FOnGameplayEffectTagCountChanged OnAnyTagChangeDelegate;

	/** Container of tags that were explicitly added */
	FGameplayTagContainer ExplicitTags;

	/** We want to track the owner for the container so that we can give valuable logging about tag replication mismatches. */
	UPROPERTY()
	TObjectPtr<class UAbilitySystemComponent> Owner;

	/** Internal helper function to adjust the explicit tag list & corresponding maps/delegates/etc. as necessary */
	UE_API bool UpdateTagMap_Internal(const FGameplayTag& Tag, int32 CountDelta, EGameplayTagReplicationState TagRepState = EGameplayTagReplicationState::None);

	/** Internal helper function to adjust the explicit tag list & corresponding maps/delegates/etc. as necessary. This does not call FillParentTags or any of the tag change delegates. These delegates are returned and must be executed by the caller. */
	UE_API bool UpdateTagMapDeferredParentRemoval_Internal(const FGameplayTag& Tag, int32 CountDelta, TArray<FDeferredTagChangeDelegate>& DeferredTagChangeDelegates, EGameplayTagReplicationState TagRepState = EGameplayTagReplicationState::None);

	/** Internal helper function to adjust the explicit tag list & corresponding map. */
	UE_API bool UpdateExplicitTags(const FGameplayTag& Tag, int32 CountDelta, bool bDeferParentTagsOnRemove, EGameplayTagReplicationState TagRepState = EGameplayTagReplicationState::None);

	/** Internal helper function to collect the delegates that need to be called when Tag has its count changed by CountDelta. */
	UE_API bool GatherTagChangeDelegates(const FGameplayTag& Tag, int32 CountDelta, TArray<FDeferredTagChangeDelegate>& TagChangeDelegates, EGameplayTagReplicationState TagRepState);
};

template<>
struct TStructOpsTypeTraits< FGameplayTagCountContainer > : public TStructOpsTypeTraitsBase2< FGameplayTagCountContainer >
{
	enum 
	{
		WithNetDeltaSerializer = true,
   };
};

/**
 * Struct used to update a blueprint property with a gameplay tag count.
 * The property is automatically updated as the gameplay tag count changes.
 * It only supports boolean, integer, and float properties.
 */
USTRUCT()
struct FGameplayTagBlueprintPropertyMapping
{
	GENERATED_BODY()

public:

	FGameplayTagBlueprintPropertyMapping() {}
	FGameplayTagBlueprintPropertyMapping(const FGameplayTagBlueprintPropertyMapping& Other)
	{
		// Don't copy handle
		TagToMap = Other.TagToMap;
		PropertyToEdit = Other.PropertyToEdit;
		PropertyName = Other.PropertyName;
		PropertyGuid = Other.PropertyGuid;
	}

	// Pretty weird that the assignment operator copies the handle when the copy constructor doesn't - bug?
	FGameplayTagBlueprintPropertyMapping& operator=(const FGameplayTagBlueprintPropertyMapping& Other) = default;

	/** Gameplay tag being counted. */
	UPROPERTY(EditAnywhere, Category = GameplayTagBlueprintProperty)
	FGameplayTag TagToMap;

	/** Property to update with the gameplay tag count. */
	UPROPERTY(VisibleAnywhere, Category = GameplayTagBlueprintProperty)
	TFieldPath<FProperty> PropertyToEdit;

	/** Name of property being edited. */
	UPROPERTY(VisibleAnywhere, Category = GameplayTagBlueprintProperty)
	FName PropertyName;

	/** Guid of property being edited. */
	UPROPERTY(VisibleAnywhere, Category = GameplayTagBlueprintProperty)
	FGuid PropertyGuid;

	/** Handle to delegate bound on the ability system component. */
	FDelegateHandle DelegateHandle;
};


/**
 * Struct used to manage gameplay tag blueprint property mappings.
 * It registers the properties with delegates on an ability system component.
 * This struct can not be used in containers (such as TArray) since it uses a raw pointer
 * to bind the delegate and it's address could change causing an invalid binding.
 */
USTRUCT()
struct FGameplayTagBlueprintPropertyMap
{
	GENERATED_BODY()

public:

	UE_API FGameplayTagBlueprintPropertyMap();
	UE_API FGameplayTagBlueprintPropertyMap(const FGameplayTagBlueprintPropertyMap& Other);
	UE_API ~FGameplayTagBlueprintPropertyMap();

	/** Call this to initialize and bind the properties with the ability system component. */
	UE_API void Initialize(UObject* Owner, class UAbilitySystemComponent* ASC);

	/** Call to manually apply the current tag state, can handle cases where callbacks were skipped */
	UE_API void ApplyCurrentTags();

#if WITH_EDITOR
	/** This can optionally be called in the owner's IsDataValid() for data validation. */
	UE_API EDataValidationResult IsDataValid(const UObject* ContainingAsset, class FDataValidationContext& Context) const;
#endif // #if WITH_EDITOR

protected:

	UE_API void Unregister();

	UE_API void GameplayTagEventCallback(const FGameplayTag Tag, int32 NewCount, TWeakObjectPtr<UObject> RegisteredOwner);

	UE_API bool IsPropertyTypeValid(const FProperty* Property) const;

	UE_API EGameplayTagEventType::Type GetGameplayTagEventType(const FProperty* Property) const;

protected:

	TWeakObjectPtr<UObject> CachedOwner;
	TWeakObjectPtr<UAbilitySystemComponent> CachedASC;

	UPROPERTY(EditAnywhere, Category = GameplayTagBlueprintProperty)
	TArray<FGameplayTagBlueprintPropertyMapping> PropertyMappings;
};


/** Encapsulate require and ignore tags */
USTRUCT(BlueprintType)
struct FGameplayTagRequirements
{
	GENERATED_USTRUCT_BODY()

	/** All of these tags must be present */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayModifier, meta=(DisplayName="Must Have Tags"))
	FGameplayTagContainer RequireTags;

	/** None of these tags may be present */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayModifier, meta=(DisplayName="Must Not Have Tags"))
	FGameplayTagContainer IgnoreTags;

	/** Build up a more complex query that can't be expressed with RequireTags/IgnoreTags alone */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayModifier, meta = (DisplayName = "Query Must Match"))
	FGameplayTagQuery TagQuery;

	/** True if all required tags and no ignore tags found */
	UE_API bool	RequirementsMet(const FGameplayTagContainer& Container) const;

	/** True if neither RequireTags or IgnoreTags has any tags */
	UE_API bool	IsEmpty() const;

	/** Return debug string */
	UE_API FString ToString() const;

	UE_API bool operator==(const FGameplayTagRequirements& Other) const;
	UE_API bool operator!=(const FGameplayTagRequirements& Other) const;

	/** Converts the RequireTags and IgnoreTags fields into an equivalent FGameplayTagQuery */
	[[nodiscard]] UE_API FGameplayTagQuery ConvertTagFieldsToTagQuery() const;
};


/** Structure used to combine tags from different sources during effect execution */
USTRUCT()
struct FTagContainerAggregator
{
	GENERATED_USTRUCT_BODY()

	FTagContainerAggregator() : CacheIsValid(false) {}

	FTagContainerAggregator(FTagContainerAggregator&& Other)
		: CapturedActorTags(MoveTemp(Other.CapturedActorTags))
		, CapturedSpecTags(MoveTemp(Other.CapturedSpecTags))
		, CachedAggregator(MoveTemp(Other.CachedAggregator))
		, CacheIsValid(Other.CacheIsValid)
	{
	}

	FTagContainerAggregator(const FTagContainerAggregator& Other)
		: CapturedActorTags(Other.CapturedActorTags)
		, CapturedSpecTags(Other.CapturedSpecTags)
		, CachedAggregator(Other.CachedAggregator)
		, CacheIsValid(Other.CacheIsValid)
	{
	}

	FTagContainerAggregator& operator=(FTagContainerAggregator&& Other)
	{
		CapturedActorTags = MoveTemp(Other.CapturedActorTags);
		CapturedSpecTags = MoveTemp(Other.CapturedSpecTags);
		CachedAggregator = MoveTemp(Other.CachedAggregator);
		CacheIsValid = Other.CacheIsValid;
		return *this;
	}

	FTagContainerAggregator& operator=(const FTagContainerAggregator& Other)
	{
		CapturedActorTags = Other.CapturedActorTags;
		CapturedSpecTags = Other.CapturedSpecTags;
		CachedAggregator = Other.CachedAggregator;
		CacheIsValid = Other.CacheIsValid;
		return *this;
	}

	/** Returns tags from the source or target actor */
	UE_API FGameplayTagContainer& GetActorTags();
	UE_API const FGameplayTagContainer& GetActorTags() const;

	/** Get tags that came from the effect spec */
	UE_API FGameplayTagContainer& GetSpecTags();
	UE_API const FGameplayTagContainer& GetSpecTags() const;

	/** Returns combination of spec and actor tags */
	UE_API const FGameplayTagContainer* GetAggregatedTags() const;

private:

	UPROPERTY()
	FGameplayTagContainer CapturedActorTags;

	UPROPERTY()
	FGameplayTagContainer CapturedSpecTags;

	mutable FGameplayTagContainer CachedAggregator;
	mutable bool CacheIsValid;
};


/** Allows blueprints to generate a GameplayEffectSpec once and then reference it by handle, to apply it multiple times/multiple targets. */
USTRUCT(BlueprintType)
struct FGameplayEffectSpecHandle
{
	GENERATED_USTRUCT_BODY()

	UE_API FGameplayEffectSpecHandle();
	UE_API FGameplayEffectSpecHandle(FGameplayEffectSpec* DataPtr);

	/** Internal pointer to effect spec */
	TSharedPtr<FGameplayEffectSpec>	Data;

	void Clear()
	{
		Data.Reset();
	}

	bool IsValid() const
	{
		return Data.IsValid();
	}

	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** Comparison operator */
	bool operator==(FGameplayEffectSpecHandle const& Other) const
	{
		// Both invalid structs or both valid and Pointer compare (???) // deep comparison equality
		bool bBothValid = IsValid() && Other.IsValid();
		bool bBothInvalid = !IsValid() && !Other.IsValid();
		return (bBothInvalid || (bBothValid && (Data.Get() == Other.Data.Get())));
	}

	/** Comparison operator */
	bool operator!=(FGameplayEffectSpecHandle const& Other) const
	{
		return !(FGameplayEffectSpecHandle::operator==(Other));
	}
};

template<>
struct TStructOpsTypeTraits<FGameplayEffectSpecHandle> : public TStructOpsTypeTraitsBase2<FGameplayEffectSpecHandle>
{
	enum
	{
		WithCopy = true,
		WithNetSerializer = true,
		WithIdenticalViaEquality = true,
	};
};


/** Map that stores count of tags, in a form that is optimized for replication */
USTRUCT()
struct FMinimalReplicationTagCountMap
{
	GENERATED_USTRUCT_BODY()

	FMinimalReplicationTagCountMap()
		: Owner(nullptr)
	{
		MapID = 0;
	}

	void AddTag(const FGameplayTag& Tag)
	{
		if (!Tag.IsValid())
		{
			return;
		}
		
		MapID++;
		TagMap.FindOrAdd(Tag)++;
	}

	UE_API void RemoveTag(const FGameplayTag& Tag);

	void AddTags(const FGameplayTagContainer& Container)
	{
		for (const FGameplayTag& Tag : Container)
		{
			AddTag(Tag);
		}
	}

	void RemoveTags(const FGameplayTagContainer& Container)
	{
		for (const FGameplayTag& Tag : Container)
		{
			RemoveTag(Tag);
		}
	}

	void SetTagCount(const FGameplayTag& Tag, int32 NewCount)
	{
		if (!Tag.IsValid())
		{
			return;
		}

		MapID++;
		int32& Count = TagMap.FindOrAdd(Tag);
		Count = FMath::Max(0, NewCount);
		
		if (Count == 0)
		{
			// Remove from map so that we do not replicate
			TagMap.Remove(Tag);
		}
	}

	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	TMap<FGameplayTag, int32>	TagMap;

	UPROPERTY()
	TObjectPtr<class UAbilitySystemComponent> Owner;

	/** Comparison operator */
	bool operator==(FMinimalReplicationTagCountMap const& Other) const
	{
		return (MapID == Other.MapID);
	}

	/** Comparison operator */
	bool operator!=(FMinimalReplicationTagCountMap const& Other) const
	{
		return !(FMinimalReplicationTagCountMap::operator==(Other));
	}

	int32 MapID;

	/** If true, we will skip updating the Owner ASC if we replicate on a connection owned by the ASC */
	void SetRequireNonOwningNetConnection(bool b) { bRequireNonOwningNetConnection = b; }

	UE_API void SetOwner(UAbilitySystemComponent* ASC);

	// Removes all tags that this container is granting
	UE_API void RemoveAllTags();

private:
	friend FMinimalReplicationTagCountMapForNetSerializer;
	friend UE::Net::FMinimalReplicationTagCountMapReplicationFragment;

	bool bRequireNonOwningNetConnection = false;
	TWeakObjectPtr<UNetConnection> LastConnection;

	UE_API void UpdateOwnerTagMap();
};

template<>
struct TStructOpsTypeTraits<FMinimalReplicationTagCountMap> : public TStructOpsTypeTraitsBase2<FMinimalReplicationTagCountMap>
{
	enum
	{
		WithCopy = true,
		WithNetSerializer = true,
		WithIdenticalViaEquality = true,
	};
};

DECLARE_MULTICAST_DELEGATE(FOnExternalGameplayModifierDependencyChange);

#undef UE_API
