// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffectComponent.h"
#include "GameplayEffectUIData.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/**
 * UGameplayEffectUIData
 * Base class to provide game-specific data about how to describe a Gameplay Effect in the UI. Subclass with data to use in your game.
 * In Unreal Engine 5.3, this now derives from UGameplayEffectComponent so you can use it directly as a GameplayEffectComponent.
 */
UCLASS(Blueprintable, Abstract, EditInlineNew, CollapseCategories, MinimalAPI)
class UGameplayEffectUIData : public UGameplayEffectComponent
{
	GENERATED_BODY()
};

#undef UE_API
