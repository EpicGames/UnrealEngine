// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayEffectUIData.h"
#include "GameplayEffectUIData_TextOnly.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/**
 * UI data that contains only text. This is mostly used as an example of a subclass of UGameplayEffectUIData.
 * If your game needs only text, this is a reasonable class to use. To include more data, make a custom subclass of UGameplayEffectUIData.
 */
UCLASS(DisplayName="UI Data (Text Only)", MinimalAPI)
class UGameplayEffectUIData_TextOnly : public UGameplayEffectUIData
{
	GENERATED_BODY()
	
public:
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Data, meta = (MultiLine = "true"))
	FText Description;
};

#undef UE_API
