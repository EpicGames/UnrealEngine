// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayEffectCalculation.h"
#include "GameplayEffect.h"
#include "GameplayModMagnitudeCalculation.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** Class used to perform custom gameplay effect modifier calculations, either via blueprint or native code */ 
UCLASS(BlueprintType, Blueprintable, Abstract, MinimalAPI)
class UGameplayModMagnitudeCalculation : public UGameplayEffectCalculation
{

public:
	GENERATED_UCLASS_BODY()

	/**
	 * Calculate the base magnitude of the gameplay effect modifier, given the specified spec. Note that the owning spec def can still modify this base value
	 * with a coeffecient and pre/post multiply adds. see FCustomCalculationBasedFloat::CalculateMagnitude for details.
	 * 
	 * @param Spec	Gameplay effect spec to use to calculate the magnitude with
	 * 
	 * @return Computed magnitude of the modifier
	 */
	UFUNCTION(BlueprintNativeEvent, Category="Calculation")
	UE_API float CalculateBaseMagnitude(const FGameplayEffectSpec& Spec) const;

	/**
	 * If the magnitude resultant from the custom calculation depends on game code-specific conditions that are not under the purview of the ability system,
	 * this method should be overridden to provide a multicast delegate that will fire when the reliant conditions change, so that the magnitude can be recalculated
	 * and updated.
	 * 
	 * @param Spec	Gameplay effect spec that is requesting the delegate
	 * @param World	UWorld that is requesting the delegate
	 * 
	 * @return Multicast delegate that will fire when this calculation's external dependencies change, if any
	 */
	UE_API virtual FOnExternalGameplayModifierDependencyChange* GetExternalModifierDependencyMulticast(const FGameplayEffectSpec& Spec, UWorld* World) const;

	/** Simple accessor to bAllowNonNetAuthorityDependencyRegistration with some validation: Read the comment on that variable for usage!!! */
	UE_API bool ShouldAllowNonNetAuthorityDependencyRegistration() const;

protected:
	
	/** Convenience method to get attribute magnitude during a CalculateMagnitude call */
	UE_API bool GetCapturedAttributeMagnitude(const FGameplayEffectAttributeCaptureDefinition& Def, const FGameplayEffectSpec& Spec, const FAggregatorEvaluateParameters& EvaluationParameters, OUT float& Magnitude) const;

	/** 
	 * Whether the calculation allows non-net authorities to register the external dependency multi-cast delegate or not; Effectively
	 * whether clients are allowed to perform the custom calculation themselves or not
	 * 
	 * @Note:	This is an advanced use case that should only be enabled under very specific circumstances. This is designed for games
	 *			that are utilizing network dormancy and may want to trust the client to update non-gameplay critical attributes client-side without
	 *			causing a network dormancy flush. Usage of this flag is *NOT* compatible with attribute capture within the calculation and will trigger
	 *			an ensure if used in tandem. Clients are incapable of performing custom calculations that require attribute captures. In general,
	 *			if your game is not using network dormancy, this should always remain disabled.
	 *
	 * @Note:	If enabling this for a custom calculation, be sure that the external delegate is sourced from something guaranteed to be on the client to avoid
	 *			timing issues. As an example, binding to a delegate on a GameState is potentially unreliable, as the client reference to that actor may not be
	 *			available during registration.
	 */
	UPROPERTY(EditDefaultsOnly, Category=ExternalDependencies, AdvancedDisplay)
	bool bAllowNonNetAuthorityDependencyRegistration;

	/**
	 * Gets the captured magnitude value for the given Attribute
	 * For this to work correctly, the Attribute needs to be added to the Relevant Attributes to Capture array
	 *
	 * @param EffectSpec The Gameplay Effect Spec to get the info from
	 * @param Attribute The attribute to query
	 *
	 * @return The magnitude value if found, zero otherwise
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Ability|GameplayEffect", meta=(DisplayName="Get Captured Attribute Magnitude", ScriptName="GetCapturedAttributeMagnitude"))
	UE_API float K2_GetCapturedAttributeMagnitude(const FGameplayEffectSpec& EffectSpec, FGameplayAttribute Attribute, const FGameplayTagContainer& SourceTags, const FGameplayTagContainer& TargetTags) const;

	/**
	 * Extracts the Set by Caller Magnitude from a Gameplay Effect Spec
	 *
	 * @param EffectSpec The Gameplay Effect Spec to get the info from
	 * @param Tag The effect tag to query
	 * 
	 * @return The magnitude value if found, zero otherwise
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Ability|GameplayEffect")
	UE_API float GetSetByCallerMagnitudeByTag(const FGameplayEffectSpec& EffectSpec, const FGameplayTag& Tag) const;

	/**
	 * Extracts the Set by Caller Magnitude from a Gameplay Effect Spec
	 *
	 * @param EffectSpec The Gameplay Effect Spec to get the info from
	 * @param MagnitudeName The effect name to query
	 * 
	 * @return The magnitude value if found, zero otherwise
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Ability|GameplayEffect")
	UE_API float GetSetByCallerMagnitudeByName(const FGameplayEffectSpec& EffectSpec, const FName& MagnitudeName) const;

	/**
	 * Copies and returns the source aggregated tags from a Gameplay Effect Spec
	 *
	 * @param EffectSpec The Gameplay Effect Spec to get the info from
	 * 
	 * @return Gameplay Tag Container with the copied tags. The container will be empty if no captured source tags exist.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Ability|GameplayEffect")
	UE_API FGameplayTagContainer GetSourceAggregatedTags(const FGameplayEffectSpec& EffectSpec) const;

	/**
	 * Returns the source actor tags from a Gameplay Effect Spec
	 *
	 * @param EffectSpec The Gameplay Effect Spec to get the info from
	 * 
	 * @return Gameplay Tag Container with the copied tags. The container will be empty if no captured source tags exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	UE_API const FGameplayTagContainer& GetSourceActorTags(const FGameplayEffectSpec& EffectSpec) const;

	/**
	 * Returns the source spec tags from a Gameplay Effect Spec
	 *
	 * @param EffectSpec The Gameplay Effect Spec to get the info from
	 * 
	 * @return Gameplay Tag Container with the copied tags. The container will be empty if no captured source tags exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	UE_API const FGameplayTagContainer& GetSourceSpecTags(const FGameplayEffectSpec& EffectSpec) const;

	/**
	 * Copies and returns the target aggregated tags from a Gameplay Effect Spec
	 *
	 * @param EffectSpec The Gameplay Effect Spec to get the info from
	 * 
	 * @return Gameplay Tag Container with the copied tags. The container will be empty if no captured source tags exist.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Ability|GameplayEffect")
	UE_API FGameplayTagContainer GetTargetAggregatedTags(const FGameplayEffectSpec& EffectSpec) const;

	/**
	 * Returns the target actor tags from a Gameplay Effect Spec
	 * Useful for Modifier Magnitude Calculations
	 *
	 * @param EffectSpec The Gameplay Effect Spec to get the info from
	 * 
	 * @return Gameplay Tag Container with the copied tags. The container will be empty if no captured source tags exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	UE_API const FGameplayTagContainer& GetTargetActorTags(const FGameplayEffectSpec& EffectSpec) const;

	/**
	 * Returns the target spec tags from a Gameplay Effect Spec
	 * Useful for Modifier Magnitude Calculations
	 *
	 * @param EffectSpec The Gameplay Effect Spec to get the info from
	 * 
	 * @return Gameplay Tag Container with the copied tags. The container will be empty if no captured source tags exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayEffect")
	UE_API const FGameplayTagContainer& GetTargetSpecTags(const FGameplayEffectSpec& EffectSpec) const;
};

#undef UE_API
