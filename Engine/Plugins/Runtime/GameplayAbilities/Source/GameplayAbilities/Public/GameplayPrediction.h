// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/NetDriver.h"
#include "Engine/NetSerialization.h"
#include "Net/Serialization/FastArraySerializer.h"
#include "UObject/ObjectKey.h"
#include "Templates/TypeCompatibleBytes.h"
#include "GameplayPrediction.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;
namespace UE::Net
{
	struct FPredictionKeyNetSerializer;
}

DECLARE_DELEGATE(FPredictionKeyEvent);

/**
 *	
 *	Overview of Gameplay Ability Prediction
 *	
 *	High Level Goals:
 *	At the GameplayAbility level (implementing an ability) prediction is transparent. An ability says "Do X->Y->Z", and we will automatically predict the parts of that that we can. 
 *	We wish to avoid having logic such as "If Authority: Do X. Else: Do predictive version of X" in the ability itself.
 *	
 *	At this point, not all cases are solved, but we have a very solid framework for working with client side prediction.
 *
 *	When we say "client side prediction" we really mean client predicting game simulation state. Things can still be 'completely client side' without having to work within a prediction system.
 *	For example, footsteps are completely client side and never interact with this system. But clients predicting their mana going from 100 to 90 when they cast a spell is 'client side prediction'.
 *		
 *	What do we currently predict?
 *	-Initial GameplayAbility activation (and chained activation with caveats)
 *	-Triggered Events
 *	-GameplayEffect application:
 *		-Attribute modification	(EXCEPTIONS: Executions do not currently predict, only attribute modifiers)
 *		-GameplayTag modification
 *	-Gameplay Cue events (both from within predictive gameplay effect and on their own)
 *		
 *	-Montages
 *	-Movement (built into UE UCharacterMovement)
 *	
 *	
 *	Some things we don't predict (most of these we potentially could, but currently dont):
 *	-GameplayEffect removal
 *	-GameplayEffect periodic effects (dots ticking)
 *	
 *	
 *	Problems we attempt to solve:
 *	1. "Can I do this?" Basic protocol for prediction.
 *	2. "Undo" How to undo side effects when a prediction fails.
 *	3. "Redo" How to avoid replaying side effects that we predicted locally but that also get replicated from the server.
 *	4. "Completeness" How to be sure we /really/ predicted all side effects.
 *	5. "Dependencies" How to manage dependent prediction and chains of predicted events.
 *	6. "Override" How to override state predictively that is otherwise replicated/owned by the server.
 *	
 *	---------------------------------------------------------
 *	
 *	Implementation Details
 *	
 *	*** PredictionKey ***
 *	
 *	A fundamental concept in this system is the Prediction Key (FPredictionKey). A prediction key on its own is simply a unique ID that is generated in a central place on the client. The client will send its prediction key to the server, 
 *	and associate predictive actions and side effects with this key. The server may respond with an accept/reject for the prediction key, and will also associate the server-side created side effects with this prediction key.
 *	
 *	(IMPORTANT) FPredictionKey always replicate client -> server, but when replicating server -> clients they *only* replicate to the client that sent the prediction key to the server in the first place. 
 *	This happens in FPredictionKey::NetSerialize. All other clients will receive an invalid (0) prediction key when a prediction key sent from a client is replicated back down through a replicated property.
 *	
 *	
 *	*** Ability Activation ***
 *	
 *	Ability Activation is a first class predictive action -- it generates an initial prediction key. Whenever a client predictively activates an ability, it explicitly asks the server and the server explicitly responds. Once an ability has been
 *	predictively activated (but the request has not yet been sent), the client has a valid 'prediction window' where predictive side effects can happen which are not explicitly 'asked about'. (E.g., we do not explicitly ask 'Can I decrement mana, Can I
 *	put this ability on cooldown. Those actions are considered logically atomic with activating an ability). You can think of this prediction window as being the initial callstack of ActivateAbility. Once ActivateAbility ends, your
 *  prediction window (and therefore your prediction key) is no longer valid. This is important, because many things can invalidate your prediction window such as any timers or latent nodes in your Blueprint; we do not predict over multiple frames.
 *	
 *	
 *	AbilitySystemComponent provides a set of functions for communicating ability activation between clients and server: TryActivateAbility -> ServerTryActivateAbility ->  ClientActivateAbility(Failed/Succeed).
 *	
 *	1. Client calls TryActivateAbility which generates a new FPredictionKey and calls ServerTryActivateAbility.
 *	2. Client continues (before hearing back from server) and calls ActivateAbility with the generated PredictionKey associated with the Ability's ActivationInfo.
 *	3. Any side effects that happen /before the call to ActivateAbility finish/ have the generated FPredictionKey associated with them.
 *	4. Server decides if the ability really happened in ServerTryActivateAbility, calls ClientActivateAbility(Failed/Succeed) and sets UAbilitySystemComponent::ReplicatedPredictionKey to the generated key that was sent with the request by the client.
 *	5. If client receives ClientAbilityFailed, it immediately kills the ability and rolls back side effects that were associated with the prediction key.
 *		5a. 'Rolling back' logic is registered via FPredictionKeyDelegates and FPredictionKey::NewRejectedDelegate/NewCaughtUpDelegate/NewRejectOrCaughtUpDelegate.
 *		5b. ClientAbilityFailed is really the only case where we 'reject' prediction keys and thus all of our current predictions rely on if an ability activates or not.
 *	6. If ServerTryActivateAbility succeeds, client must wait until property replication catches up (the Succeed RPC will be sent immediately, property replication will happen on its own). Once the ReplicatedPredictionKey catches up to the
 *		key used previous steps, the client can undo its predictive side effects.
 *		See FReplicatedPredictionKeyItem::OnRep for the CatchUpTo logic. See UAbilitySystemComponent::ReplicatedPredictionKeyMap for how the keys actually get replicated. See ~FScopedPredictionWindow where the server acknowledges keys.
 *						
 *			 
 *	*** GameplayEffect Prediction ***
 *
 *	GameplayEffects are considered side effects of ability activation and are not separately accepted/rejected.
 *	
 *	1. GameplayEffects are only applied on clients if there is a valid prediction key. (If no prediction key, it simply skips the application on client).
 *	2. Attributes, GameplayCues, and GameplayTags are all predicted if the GameplayEffect is predicted.
 *	3. When the FActiveGameplayEffect is created, it stores the prediction key (FActiveGameplayEffect::PredictionKey)
 *		3a. Instant effects are explained below in "Attribute Prediction".
 *	4. On the server, the same prediction key is also set on the server's FActiveGameplayEffect that will be replicated down.
 *	5. As a client, if you get a replicated FActiveGameplayEffect with a valid prediction key on it, you check to see if you have an ActiveGameplayEffect with that same key, if there is match, we do not apply
 *		the 'on applied' type of logic, e.g., GameplayCues. The solves the "Redo" problem. However we will have 2 of the 'same' GameplayEffects in our ActiveGameplayEffects container, temporarily:
 *	6. At the same time, FReplicatedPredictionKeyItem::OnRep will catch up and the predictive effects will be removed. When they are removed in this case, we again check PredictionKey and decide 
 *		if we should not do the 'On Remove' logic / GameplayCue.
 *		
 *	At this point, we have effectively predicted a gameplay effect as a side effect and handled the 'Undo' and 'Redo' problems.
 *		
 *  See FActiveGameplayEffectsContainer::ApplyGameplayEffectSpec where it registers what to do when caught-up (RemoveActiveGameplayEffect_NoReturn).
 *	See FActiveGameplayEffect::PostReplicatedAdd, FActiveGameplayEffect::PreReplicatedRemove, and FActiveGameplayCue::PostReplicatedAdd for examples of how FPredictionKey is associated with the GE's and GC's.
 *
 *	*** Attribute Prediction ***
 *	
 *	Since attributes are replicated as standard uproperties, predicting modification to them can be tricky ("Override" problem). Instantaneous modification can be even harder since these are non stateful by nature.
 *	(E.g., rolling back an attribute mod is difficult if there is no book keeping past the modification). This makes the "Undo" and "Redo" problem also hard in this case.
 *	
 *	The basic plan of attack is to treat attribute prediction as delta prediction rather than absolute value prediction. We do not predict that we have 90 mana, we predict that we have -10 mana from the server value, until 
 *	the server confirms our prediction key. Basically, treat instant modifications as /infinite duration modifications/ to attributes while they are done predictively. The solves "Undo" and "Redo".
 *	
 *	For the "override" problem, we can handle this in the properties OnRep by treating the replicated (server) value as the 'base value' instead of 'final value' of the attribute, and to 
 *	reaggregate our 'final value' after a replication happens.
 *	
 *	
 *	1. We treat predictive instant gameplay effects as infinite duration gameplay effects. See UAbilitySystemComponent::ApplyGameplayEffectSpecToSelf.
 *	2. We have to *always* receive RepNotify calls on our attributes (not just when there is a change from last local value, since we will predict the change ahead of time). Done with REPNOTIFY_Always.
 *	3. In the attribute RepNotify, we call into the AbilitySystemComponent::ActiveGameplayEffects to update our 'final value' give the new 'base value'. the GAMEPLAYATTRIBUTE_REPNOTIFY can do this.
 *	4. Everything else will work like above (GameplayEffect prediction) : when the prediction key is caught up, the predictive GameplayEffect is removed and we will return to the server given value.
 *	
 *	
 *	Example:
 *	
 *	void UMyHealthSet::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
 *	{
 *		Super::GetLifetimeReplicatedProps(OutLifetimeProps);
 *
 *		DOREPLIFETIME_CONDITION_NOTIFY(UMyHealthSet, Health, COND_None, REPNOTIFY_Always);
 *	}
 *	
 *  void UMyHealthSet::OnRep_Health()
 *  {
 *		GAMEPLAYATTRIBUTE_REPNOTIFY(UMyHealthSet, Health);
 *  }
 *  
 *  *** Gameplay Cue Events ***
 *  
 *  Outside of GameplayEffects which are already explained, Gameplay Cues can be activated on their own. These functions (UAbilitySystemComponent::ExecuteGameplayCue etc)  take network role and prediction keys into account.
 *  
 *  1. In UAbilitySystemComponent::ExecuteGameplayCue, if authority then do the multicast event (with replication key). If non authority but w/ a valid prediction key, predict the GameplayCue.
 *  2. On the receiving end (NetMulticast_InvokeGameplayCueExecuted etc), if there is a replication key, then don't do the event (assume you predicted it).
 *  
 *  Remember that FPredictionKeys only replicate to the originating owner. This is an intrinsic property of FReplicationKey.
 *	
 *	*** Triggered Data Prediction ***
 *	
 *	Triggered Data is currently used to activate abilities. Essentially this all goes through the same code path as ActivateAbility. Rather than the ability being activated from input press, it is activated from
 *	another game code driven event. Clients are able to predictively execute these events which predictively activate abilities. 
 *	
 *	There are some nuances to however, since the server will also run the code that triggers events. The server won't just wait to hear from the client. The server will keep a list of triggered abilities that have been
 *	activated from a predictive ability. When receiving a TryActivate from a triggered ability, the server will look to see if /it/ has already run this ability, and respond with that information.
 *	
 *	The issue is we do not properly rollback these operations. There is work left to do on Triggered Events and replication. (explained at the end).
 * 
 *	
 *	---------------------------------------------------------	
 *	
 *	Advanced topic!	
 *	
 *	*** Dependencies ***
 *	
 *	We can have situations such as "Ability X activates and immediately triggers an event which activates Ability Y which triggers another Ability Z". The dependency chain is X->Y->Z.
 *	Each of those abilities could be rejected by the server. If Y is rejected, then Z also never happened, but the server never tries to run Z, so the server doesn't explicitly decide 'no Z can't run'.
 *	
 *	To handle this, we have a concept of a Base PredictionKey, which is a member of FPredictionKey. When calling TryActivateAbility, we pass in the current PredictionKey (if applicable). That prediction key
 *	is used as the base for any new prediction keys generated. We build a chain of keys this way, and can then invalidate Z if Y is rejected.
 *	
 *	This is slightly more nuanced though. In the X->Y->Z case, the server will only receive the PredictionKey for X before trying to run the chain itself. E.g., it will TryActivate Y and Z with the original prediction key
 *	sent to it from the client, whereas the client will generate a new PredictionKey each time it calls TryActivateAbility. The client *has* to generate a new PredictionKey for each ability activate, since each activate
 *	is not logically atomic. Each side effect produced in the chain of events has to have a unique PredictionKey. We cannot have GameplayEffects produced in X have the same PredictionKey produced in Z.
 *	
 *	To get around this, The prediction key of X is considered the Base key for Y and Z. The dependency from Y to Z is kept completely client side, which is done in by FPredictionKeyDelegates::AddDependancy. We add delegates
 *	to reject/catchup Z if Y rejected/confirmed.
 *	
 *	This dependency system allows us to have multiple predictive actions that are not logically atomic within a single prediction window/scope.
 *  
 *  There is a problem though: because the dependencies are kept client side, the server does not actually know if it had previously rejected a dependent action. You can design around this issue by using activation tags
 *  in your gameplay abilities. For instance, when predicting dependents GA_Combo1 -> GA_Combo2, you could make GA_Combo2 only activate if it has a GameplayTag given by GA_Combo1. Thus a rejection of GA_Combo1 would also
 *  cause the server to reject the activation of GA_Combo2.
 *	
 *	
 *	*** Additional Prediction Windows (within an Ability) ***
 *	
 *	As stated, A prediction key is only usable during a single logical scope. Once ActivateAbility returns, we are essentially done with that key. If the ability is waiting on an external event or timer, it's possible
 *	we will have already received a confirm/reject from the server by the time we're ready to continue execution. Thus any additional side effects produced after the initial activation can no longer be tied to the lifespan of the original key.
 *	
 *	This isn't that bad, except that abilities will sometimes want to react to player input. For example, 'a hold down and charge' ability wants to instantly predict some stuff when the button is released. It is possible
 *	to create a new prediction window within an ability with FScopedPredictionWindow. 
 *	
 *	FScopedPredictionWindows provides a way to send the server a new prediction key and have the server pick up and use that key within the same logical scope.
 *	
 *	UAbilityTask_WaitInputRelease::OnReleaseCallback is a good example. The flow of events is as followed:
 *	1. Client enters UAbilityTask_WaitInputRelease::OnReleaseCallback and starts a new FScopedPredictionWindow. This creates a new prediction key for this scope (FScopedPredictionWindow::ScopedPredictionKey).
 *	2. Client calls AbilitySystemComponent->ServerInputRelease which passes ScopedPrediction.ScopedPredictionKey as a parameter.
 *	3. Server runs ServerInputRelease_Implementation which takes the passed in PredictionKey and sets it as UAbilitySystemComponent::ScopedPredictionKey with an FScopedPredictionWindow.
 *	4. Server runs UAbilityTask_WaitInputRelease::OnReleaseCallback /within the same scope/
 *	5. When the server hits the FScopedPredictionWindow in ::OnReleaseCallback, it gets the prediction key from UAbilitySystemComponent::ScopedPredictionKey. That is now used for all side effects within this logical scope.
 *	6. Once the server ends this scoped prediction window, the prediction key used is finished and set to ReplicatedPredictionKey.
 *	7. All side effects created in this scope now share a key between client and server.
 *	
 *	The key to this working is that ::OnReleaseCallback calls ::ServerInputRelease which calls ::OnReleaseCallback on the server. There is no room for anything else to happen and use the given prediction key.
 *	
 *	While there is no "Try/Failed/Succeed" calls in this example, all side effects are procedurally grouped/atomic. This solves the "Undo" and "Redo" problems for any arbitrary function calls that run on the server and client.
 *	
 *	
 *	---------------------------------------------------------
 *	
 *	Unsupported / Issues/ Todo
 *	
 *	Triggered events do not explicitly replicate. E.g., if a triggered event only runs on the server, the client will never hear about it. This also prevents us from doing cross player/AI etc events. Support
 *	for this should eventually be added and it should follow the same pattern that GameplayEffect and GameplayCues follow (predict triggered event with a prediction key, ignore the RPC event if it has a prediction key).
 *	
 *	Big caveat with this whole system: Rollback of any chained activations (including triggered events) is currently not possible out of the box. The reason for this is because each ServerTryActivateAbility is going to be responded to in order.
 *  Let's chain dependent GA's as an example: GA_Mispredict -> GA_Predict1. In this example, when GA_Mispredict is activated and predicted locally it will immediately also activate GA_Predict1. The client sends ServerTryActivateAbility for GA_Mispredict,
 *  and the server rejects it (sending back a ClientActivateAbilityFailed). As it stands, we don't have any delegates that reject dependent abilities on the client (and the server isn't even aware there are dependencies).  On the server, it also
 *  receives a ServerTryActivateAbility for GA_Predict1. Assuming that succeeds, the Client and Server are now both executing GA_Predict1 even though GA_Mispredict never happened. You can design around this by using the tag system to ensure GA_Mispredict succeeded.
 *
 *	*** Predicting "Meta" Attributes such as Damage/Healing vs "real" attributes such as Health ***
 *	
 *	We are unable to apply meta attributes predictively. Meta attributes only work on instant effects, in the back end of GameplayEffect (Pre/Post Modify Attribute on the UAttributeSet). These events are not called when
 *	applying duration-based gameplay effects. E.g., a GameplayEffect that modifies damage for 5 seconds doesn't make sense. 
 *	
 *	In order to support this, we would probably add some limited support for duration based meta attributes, and move the transform of the instant gameplay effect from the front end (UAbilitySystemComponent::ApplyGameplayEffectSpecToSelf)
 *	to the backend (UAttributeSet::PostModifyAttribute). 
 * 
 *			 
 *	*** Predicting ongoing multiplicative GameplayEffects ***
 *	
 *	There are also limitations when predicting % based gameplay effects. Since the server replicates down the 'final value' of an attribute, but not the entire aggregator chain of what is modifying it, we may run into cases where
 *	the client cannot accurately predict new gameplay effects.
 *	
 *	For example: 
 *	-Client has a perm +10% movement speed buff with base movement speed of 500 -> 550 is the final movement speed for this client.
 *	-Client has an ability which grants an additional 10% movement speed buff. It is expected to *sum* the % based multipliers for a final 20% bonus to 500 -> 600 movement speed.
 *	-However on the client, we just apply a 10% buff to 550 -> 605.
 *	
 *	This will need to be fixed by replicating down the aggregator chain for attributes. We already replicate some of this data, but not the full modifier list. We will need to look into supporting this eventually.
 *	 
 *	
 *	*** "Weak Prediction" ***
 *	
 *	We will probably still have cases that do not fit well into this system. Some situations will exist where a prediction key exchange is not feasible. For example, an ability where any one that player collides with/touches
 *	receives a GameplayEffect that slows them and their material blue. Since we can't send Server RPCs every time this happens (and the server couldn't necessarily handle the message at its point in the simulation), there is no 
 *	way to correlate the gameplay effect side effects between client and server.
 *	
 *	One approach here may be to think about a weaker form of prediction. One where there is not a fresh prediction key used and instead the server assumes the client will predict all side effects from an entire ability. This would
 *	at least solve the "redo" problem but would not solve the "completeness" problem. If the client side prediction could be made as minimal as possible - for example only predicting an initial particle effect rather than
 *	predicting the state and attribute change - then the problems get less severe.
 *	
 *	I can envision a weak prediction mode which is what (certain abilities? All abilities?) fall back to when there is no fresh prediction key that can accurately correlate side effects. When in weak prediction mode, perhaps
 *	only certain actions can be predicted - for example GameplayCue execute events, but not OnAdded/OnRemove events.
 *	
 *	
 */



/**
 *	FPredictionKey is a generic way of supporting Clientside Prediction in the GameplayAbility system.
 *	A FPredictionKey is essentially an ID for identifying predictive actions and side effects that are
 *	done on a client. UAbilitySystemComponent supports synchronization of the prediction key and its side effects
 *	between client and server.
 *	
 *	Essentially, anything can be associated with a PredictionKey, for example activating an Ability.
 *	The client can generate a fresh PredictionKey and sends it to the server in its ServerTryActivateAbility call.
 *	The server can confirm or reject this call (ClientActivateAbilitySucceed/Failed). 
 *	
 *	While the client is predicting its ability, it is creating side effects (GameplayEffects, TriggeredEvents, Animations, etc).
 *	As the client predicts these side effects, it associates each one with the prediction key generated at the start of the ability
 *	activation.
 *	
 *	If the ability activation is rejected, the client can immediately revert these side effects. 
 *	If the ability activation is accepted, the client must wait until the replicated side effects are sent to the server.
 *		(the ClientActivatbleAbilitySucceed RPC will be immediately sent. Property replication may happen a few frames later).
 *		Once replication of the server created side effects is finished, the client can undo its locally predictive side effects.
 *		
 *	The main things FPredictionKey itself provides are:
 *		-Unique ID and a system for having dependant chains of Prediction Keys ("Current" and "Base" integers)
 *		-A special implementation of ::NetSerialize *** which only serializes the prediction key to the predicting client ***
 *			-This is important as it allows us to serialize prediction keys in replicated state, knowing that only clients that gave the server the prediction key will actually see them!
 *	
 */

PRAGMA_DISABLE_DEPRECATION_WARNINGS // PredictiveConnection
USTRUCT()
struct FPredictionKey
{
	GENERATED_USTRUCT_BODY()

	typedef int16 KeyType;
	FPredictionKey() = default;

	/** The unique ID of this prediction key */
	UPROPERTY()
	int16	Current = 0;

	/** If non 0, the original prediction key this was created from (in a dependency chain) */
	UPROPERTY(NotReplicated)
	int16	Base = 0;

	/** True if this was created as a server initiated activation key, used to identify server activations but cannot be used for prediction */
	UPROPERTY()
	bool bIsServerInitiated = false;

	/** Construct a new prediction key with no dependencies */
	static UE_API FPredictionKey CreateNewPredictionKey(const UAbilitySystemComponent*);

	/** Construct a new server initiation key, for abilities activated on the server */
	static UE_API FPredictionKey CreateNewServerInitiatedKey(const UAbilitySystemComponent*);

	/** Create a new dependent prediction key: keep our existing base or use the current key as the base. */
	UE_API void GenerateDependentPredictionKey();

	/** Creates new delegate called only when this key is rejected. */
	UE_API FPredictionKeyEvent& NewRejectedDelegate();

	/** Creates new delegate called only when replicated state catches up to this key. */
	UE_API FPredictionKeyEvent& NewCaughtUpDelegate();
	
	/** Add a new delegate that is called if the key is rejected or caught up to. */
	UE_API void NewRejectOrCaughtUpDelegate(FPredictionKeyEvent Event);
	
	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** A key is valid if it's non-zero, prediction keys for other clients will serialize down as 0 and be invalid */
	bool IsValidKey() const
	{
		return Current > 0;
	}

	/** A key was generated by the local client if it's valid and not a server key, prediction keys for other clients will serialize down as 0 and be invalid */
	bool IsLocalClientKey() const
	{
		return Current > 0 && !bIsServerInitiated;
	}

	/** True if this was created as a server initiated activation key, used to identify server activations but cannot be used for prediction */
	bool IsServerInitiatedKey() const
	{
		return bIsServerInitiated;
	}

	/** Can this key be used for more predictive actions, or has it already been sent off to the server? */
	bool IsValidForMorePrediction() const
	{
		return IsLocalClientKey();
	}

	/** Was this PredictionKey received from a NetSerialize or created locally? */
	bool WasReceived() const
	{
		return PredictiveConnectionObjectKey != FObjectKey();
	}

	bool WasLocallyGenerated() const
	{
		return (Current > 0) && (PredictiveConnectionObjectKey == FObjectKey());
	}

	bool operator==(const FPredictionKey& Other) const
	{
		// we're omitting Base here because it's not replicated
		return Current == Other.Current && bIsServerInitiated == Other.bIsServerInitiated;
	}

	bool operator!=(const FPredictionKey& Other) const
	{
		return !(*this == Other);
	}

	FString ToString() const
	{
		return bIsServerInitiated ? FString::Printf(TEXT("[Srv: %d]"), Current) : FString::Printf(TEXT("[%d/%d]"), Current, Base);
	}

	friend uint32 GetTypeHash(const FPredictionKey& InKey)
	{
		// we're omitting Base here because it's not replicated
		return ((InKey.Current << 1) | (InKey.bIsServerInitiated & 1));
	}

	uint64 GetPredictiveConnectionKey() const 
	{ 
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return BitCast<uint64>(PredictiveConnectionObjectKey.GetRemoteId());
#else
		return BitCast<uint64>(PredictiveConnectionObjectKey);
#endif
	}

private:
	friend UE::Net::FPredictionKeyNetSerializer;

	UE_API void GenerateNewPredictionKey();

	explicit FPredictionKey(int32 Key)
		: Current(static_cast<KeyType>(Key))
	{
		check(Key >= 0 && Key <= std::numeric_limits<KeyType>::max());
	}

	/** On the server, uniquely identifies network connection this was serialized on/from.  See NetSerialize for additional information. */
	FObjectKey PredictiveConnectionObjectKey;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

template<>
struct TStructOpsTypeTraits<FPredictionKey> : public TStructOpsTypeTraitsBase2<FPredictionKey>
{
	enum
	{
		WithNetSerializer = true,
		WithIdenticalViaEquality = true
	};
};

// -----------------------------------------------------------------

/**
 *	This is a data structure for registering delegates associated with prediction key rejection and replicated state 'catching up'.
 *	Delegates should be registered that revert side effects created with prediction keys.
 *	
 */

struct FPredictionKeyDelegates
{

public:

	struct FDelegates
	{
	public:

		/** This delegate is called if the prediction key is associated with an action that is explicitly rejected by the server. */
		TArray<FPredictionKeyEvent>	RejectedDelegates;

		/** This delegate is called when replicated state has caught up with the prediction key. Doesnt imply rejection or acceptance. */
		TArray<FPredictionKeyEvent>	CaughtUpDelegates;
	};

	TMap<FPredictionKey::KeyType, FDelegates>	DelegateMap;

	static FPredictionKeyDelegates& Get();

	static FPredictionKeyEvent&	NewRejectedDelegate(FPredictionKey::KeyType Key);
	static FPredictionKeyEvent&	NewCaughtUpDelegate(FPredictionKey::KeyType Key);
	static void NewRejectOrCaughtUpDelegate(FPredictionKey::KeyType Key, FPredictionKeyEvent NewEvent);

	static void	BroadcastRejectedDelegate(FPredictionKey::KeyType Key);
	static void	BroadcastCaughtUpDelegate(FPredictionKey::KeyType Key);

	static void Reject(FPredictionKey::KeyType Key);
	static void CatchUpTo(FPredictionKey::KeyType Key);

	static void AddDependency(FPredictionKey::KeyType ThisKey, FPredictionKey::KeyType DependsOn);

};


// -----------------------------------------------------------------

class UAbilitySystemComponent;
class UGameplayAbility;

/**
 *	A structure for allowing scoped prediction windows.
 */

struct FScopedPredictionWindow
{
	/** To be called on server when a new prediction key is received from the client (In an RPC). 
	 *	InSetReplicatedPredictionKey should be set to false in cases where we want a scoped prediction key but have already repped the prediction key.
	 *	(For example, cached target data will restore the prediction key that the TD was sent with, but this key was already repped down as confirmed when received)
	 **/
	UE_API FScopedPredictionWindow(UAbilitySystemComponent* AbilitySystemComponent, FPredictionKey InPredictionKey, bool InSetReplicatedPredictionKey = true);

	/** To be called in the callsite where the predictive code will take place. This generates a new PredictionKey and acts as a synchonization point between client and server for that key.  */
	UE_API FScopedPredictionWindow(UAbilitySystemComponent* AbilitySystemComponent, bool CanGenerateNewKey=true);

	UE_API ~FScopedPredictionWindow();

private:

	TWeakObjectPtr<UAbilitySystemComponent> Owner;
	bool ClearScopedPredictionKey;
	bool SetReplicatedPredictionKey;
	FPredictionKey RestoreKey;

#if !UE_BUILD_SHIPPING
	FOnSendRPC DebugSavedOnSendRPC;
	TWeakObjectPtr<UNetDriver> DebugSavedNetDriver;
	TOptional<FPredictionKey::KeyType> DebugBaseKeyOfChain;
#endif
};

/**
 * Possible results when evaluating how to deal with an FPredictionKey
 */
enum class EGasPredictionKeyResult : uint8
{
	// Silently drop the key (don't acknowledge it at all)
	SilentlyDrop,

	// Accept the key (e.g. server acknowledges the event happened)
	Accept,

	// Reject the key (e.g. server says the event never happened)
	Reject
};

/**
 * Discard the predictions that occur within this window.
 * This is useful in a case where you don't intend to send a chain of generated events (FPredictionKeys) to the server.
 * For example, we use this in a case where we want to play a Montage locally, and we notify the server unreliably (i.e. it is an optional event).
 * Note: Nothing stops something within this scope from creating a valid key, then sending that valid key to the server.  This is meant for generated keys that aren't already going to be sent.
 */
struct FScopedDiscardPredictions
{
	// Pass in the AbilitySystemComponent to discard predictions on.  Optionally, if we want to do something other than just Drop the prediction events you can override that.
	UE_API explicit FScopedDiscardPredictions(UAbilitySystemComponent* AbilitySystemComponent, EGasPredictionKeyResult HowToHandlePredictions = EGasPredictionKeyResult::SilentlyDrop);

	// When the scope ends, we will perform the final drop/ack/nack of the prediction chain
	UE_API ~FScopedDiscardPredictions();

private:
	// Weak pointer to the 'Owning' ASC.  If this ends up being invalid/null we don't do anything.
	TWeakObjectPtr<UAbilitySystemComponent> Owner;

	// The key that we're going to restore on the Owner
	FPredictionKey KeyToRestoreOnOwner;

	// How we're going to handle the resulting chain of predictions
	EGasPredictionKeyResult PredictionKeyChainResult;

	// This is the key that we're going to eventually acknowledge according to PredictionKeyChainResult
	FPredictionKey BaseKeyToAck;
};

// -----------------------------------------------------------------


/**
 *	This is the structure that replicates prediction keys back to clients, from the server (via property replication).
 *	This is done via a FastArray so that each predictionkey is individually ack'd, rather than just replicating "highest numbered key".
 *
 *	"Highest numbered key" fails with packet loss. For example:
 *
 *	Pkt1: {+Tag=X, ReplicatedKey=1)
 *	Pkt2: (ReplicatedKey=2)
 *
 *	If Pkt1 is dropped, after Pkt2 is already in flight, client receives ReplicatedKey=2 and will remove irs predictive Tag=X.
 *	The state in Pkt1 will be resent, after the n'ack is detected. But the damage will have been done: Client thought it was up to date
 *	but was missing a gap.
 *
 */

struct FReplicatedPredictionKeyMap;

USTRUCT()
struct FReplicatedPredictionKeyItem : public FFastArraySerializerItem
{
	GENERATED_USTRUCT_BODY()

	// As we know that FReplicatedPredictionKeyItem is well behaved and does not leak outside of FReplicatedPredictionKeyMap we allow ReplicationID and ReplicationKey to be copied around to avoid issues when instantiating from archetype or CDO
	FReplicatedPredictionKeyItem();
	FReplicatedPredictionKeyItem(const FReplicatedPredictionKeyItem& Other);
	FReplicatedPredictionKeyItem(FReplicatedPredictionKeyItem&& Other);
	FReplicatedPredictionKeyItem& operator=(FReplicatedPredictionKeyItem&& other);
	FReplicatedPredictionKeyItem& operator=(const FReplicatedPredictionKeyItem& other);

	UPROPERTY()
	FPredictionKey PredictionKey;
	
	void PostReplicatedAdd(const struct FReplicatedPredictionKeyMap &InArray) { OnRep(InArray); }
	void PostReplicatedChange(const struct FReplicatedPredictionKeyMap &InArray) { OnRep(InArray); }

	FString GetDebugString() { return PredictionKey.ToString(); }

private:

	void OnRep(const struct FReplicatedPredictionKeyMap& InArray);
};

USTRUCT()
struct FReplicatedPredictionKeyMap : public FFastArraySerializer
{
	GENERATED_USTRUCT_BODY()

	FReplicatedPredictionKeyMap();

	UPROPERTY()
	TArray<FReplicatedPredictionKeyItem> PredictionKeys;

	void ReplicatePredictionKey(FPredictionKey Key);

	bool NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms);

	FString GetDebugString() const;

	static const int32 KeyRingBufferSize;

	
};

template<>
struct TStructOpsTypeTraits< FReplicatedPredictionKeyMap > : public TStructOpsTypeTraitsBase2< FReplicatedPredictionKeyMap >
{
	enum
	{
		WithNetDeltaSerializer = true,
	};
};

#undef UE_API
