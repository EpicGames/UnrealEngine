// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameplayTagContainer.h"
#include "Engine/DataAsset.h"
#include "GameplayEffectTypes.h"
#include "GameplayEffect.h"
#include "GameplayTagResponseTable.generated.h"

#define UE_API GAMEPLAYABILITIES_API

class UAbilitySystemComponent;

USTRUCT()
struct FGameplayTagReponsePair
{
	GENERATED_USTRUCT_BODY()

	/** Tag that triggers this response */
	UPROPERTY(EditAnywhere, Category="Response")
	FGameplayTag	Tag;
	
	/** Deprecated. Replaced with ResponseGameplayEffects */
	UPROPERTY()
	TSubclassOf<UGameplayEffect> ResponseGameplayEffect;

	/** The GameplayEffects to apply in reponse to the tag */
	UPROPERTY(EditAnywhere, Category="Response")
	TArray<TSubclassOf<UGameplayEffect> > ResponseGameplayEffects;

	/** The max "count" this response can achieve. This will not prevent counts from being applied, but will be used when calculating the net count of a tag. 0=no cap. */
	UPROPERTY(EditAnywhere, Category="Response", meta=(ClampMin = "0"))
	int32 SoftCountCap = 0;
};

USTRUCT()
struct FGameplayTagResponseTableEntry
{
	GENERATED_USTRUCT_BODY()

	/** Tags that count as "positive" toward to final response count. If the overall count is positive, this ResponseGameplayEffect is applied. */
	UPROPERTY(EditAnywhere, Category="Response")
	FGameplayTagReponsePair		Positive;

	/** Tags that count as "negative" toward to final response count. If the overall count is negative, this ResponseGameplayEffect is applied. */
	UPROPERTY(EditAnywhere, Category="Response")
	FGameplayTagReponsePair		Negative;
};

/**
 *	A data driven table for applying gameplay effects based on tag count. This allows designers to map a 
 *	"tag count" -> "response Gameplay Effect" relationship.
 *	
 *	For example, "for every count of "Status.Haste" I get 1 level of GE_Response_Haste. This class facilitates
 *	building this table and automatically registering and responding to tag events on the ability system component.
 */
UCLASS(MinimalAPI)
class UGameplayTagReponseTable : public UDataAsset
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category="Response")
	TArray<FGameplayTagResponseTableEntry>	Entries;

	/** Registers for tag events for the given ability system component. Note this will happen to every spawned ASC, we may want to allow games
	 *	to limit what classe this is called on, or potentially build into the table class restrictions for each response entry.
	 */
	UE_API void RegisterResponseForEvents(UAbilitySystemComponent* ASC);

	UE_API virtual void PostLoad() override;

protected:

	UFUNCTION()
	UE_API void TagResponseEvent(const FGameplayTag Tag, int32 NewCount, UAbilitySystemComponent* ASC, int32 idx);
	
	/** Temporary structs to avoid extra heap allocations every time we recalculate tag count */
	mutable FGameplayEffectQuery Query;

	FGameplayEffectQuery& MakeQuery(const FGameplayTag& Tag) const
	{
		Query.OwningTagQuery.ReplaceTagFast(Tag);
		return Query;
	}

	// ----------------------------------------------------

	struct FGameplayTagResponseAppliedInfo
	{
		TArray<FActiveGameplayEffectHandle> PositiveHandles;
		TArray<FActiveGameplayEffectHandle> NegativeHandles;
	};

	TMap< TWeakObjectPtr<UAbilitySystemComponent>, TArray< FGameplayTagResponseAppliedInfo> > RegisteredASCs;

	float LastASCPurgeTime;

	UE_API void Remove(UAbilitySystemComponent* ASC, TArray<FActiveGameplayEffectHandle>& Handles);

	UE_API void AddOrUpdate(UAbilitySystemComponent* ASC, const TArray<TSubclassOf<UGameplayEffect> >& ResponseGameplayEffects, int32 TotalCount, TArray<FActiveGameplayEffectHandle>& Handles);

	UE_API int32 GetCount(const FGameplayTagReponsePair& Pair, UAbilitySystemComponent* ASC) const;
};

#undef UE_API
