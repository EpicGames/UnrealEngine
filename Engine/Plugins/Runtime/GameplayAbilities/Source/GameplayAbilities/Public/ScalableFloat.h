// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/CurveTable.h"
#include "DataRegistryId.h"
#include "ScalableFloat.generated.h"

#define UE_API GAMEPLAYABILITIES_API

/** Generic numerical value in the form Value * Curve[Level] */
USTRUCT(BlueprintType)
struct FScalableFloat
{
	GENERATED_USTRUCT_BODY()

	FScalableFloat()
		: Value(0.f)
		, LocalCachedCurveID(INDEX_NONE)
		, CachedCurve(nullptr)
	{
	}

	FScalableFloat(float InInitialValue)
		: Value(InInitialValue)
		, LocalCachedCurveID(INDEX_NONE)
		, CachedCurve(nullptr)
	{
	}

	~FScalableFloat() = default;

public:

	/** Raw value, is multiplied by curve */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ScalableFloat)
	float	Value;

private:
	/** Value of GetGlobalCachedCurveID when CachedCurve was set */
	mutable int32 LocalCachedCurveID;

public:
	/** Curve that is evaluated at a specific level. If found, it is multipled by Value */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ScalableFloat)
	FCurveTableRowHandle	Curve;

	/** Name of Data Registry containing curve to use. If set the RowName inside Curve is used as the item name */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ScalableFloat)
	FDataRegistryType		RegistryType;

	/** Returns the scaled value at a given level */
	UE_API float GetValueAtLevel(float Level, const FString* ContextString = nullptr) const;

	/** Returns the scaled value at level 0 */
	UE_API float GetValue(const FString* ContextString = nullptr) const;

	/** Used when using a scalable float as a bool */
	UE_API bool AsBool(float Level = 0, const FString* ContextString = nullptr) const;

	/** Returns the value as an int32 */
	UE_API int32 AsInteger(float Level = 0, const FString* ContextString = nullptr) const;

	/** Gets the value and possible curve at a given level, returning false if it failed to find a good value */
	UE_API bool EvaluateCurveAtLevel(float& OutValue, const FRealCurve*& OutCurve, float Level, const FString& ContextString, bool bWarnIfInvalid = true) const;

	/** True if there is no curve lookup */
	bool IsStatic() const
	{
		return Curve.RowName.IsNone();
	}

	/** Sets raw value */
	UE_API void SetValue(float NewValue);

	/** Overrides raw value and curve reference */
	UE_API void SetScalingValue(float InCoeffecient, FName InRowName, UCurveTable * InTable);

	/** Returns static value, only safe if this has no curve reference */
	float GetValueChecked() const
	{
		check(IsStatic());
		return Value;
	}

	/** Outputs human readable string */
	UE_API FString ToSimpleString() const;

	/** Error checking: Returns false if this has an invalid curve reference but will not print warnings */
	UE_API bool IsValid() const;

	/** Error checking: Checks if float is valid and prints detailed warnings if not valid */
	UE_DEPRECATED(5.6, "Use IsDataValid and manually output the errors")
	UE_API bool IsValidWithWarnings(const FString& ContextString) const;

#if WITH_EDITOR
	/** Error checking must be performed by the owning class, then call this to get specific errors */
	UE_API EDataValidationResult IsDataValid(class FDataValidationContext& Context, const FString& PathName) const;
#endif

	/** Equality/Inequality operators */
	UE_API bool operator==(const FScalableFloat& Other) const;
	UE_API bool operator!=(const FScalableFloat& Other) const;

	/** copy operator to prevent duplicate handles */
	UE_API FScalableFloat& operator=(const FScalableFloat& Src);

	/** Used to upgrade a float or int8/int16/int32 property into an FScalableFloat */
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
	UE_API void PostSerialize(const FArchive& Ar);

private:
	/** Cached direct pointer to the RealCurve we should evaluate */
	mutable const FRealCurve* CachedCurve;

	/** Clear cached data */
	void ClearCachedCurve() const
	{
		LocalCachedCurveID = INDEX_NONE;
		CachedCurve = nullptr;
	}

};

template<>
struct TStructOpsTypeTraits<FScalableFloat>
	: public TStructOpsTypeTraitsBase2<FScalableFloat>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
		WithPostSerialize = true,
	};
};

/** Macro to call IsValidWithWarnings with a correct error info. Assumed to be called within a UObject */
#define SCALABLEFLOAT_REPORTERROR(Scalable)\
	Scalable.IsValidWithWarnings(FString::Printf(TEXT("%s.%s"), *GetPathName(), TEXT(#Scalable)))\

/** Macro to call IsValidWithWarnings with a correct error info */
#define SCALABLEFLOAT_REPORTERROR_WITHPATHNAME(Scalable, PathNameString)\
	Scalable.IsValidWithWarnings(FString::Printf(TEXT("%s.%s"), *PathNameString, TEXT(#Scalable)))

#undef UE_API
