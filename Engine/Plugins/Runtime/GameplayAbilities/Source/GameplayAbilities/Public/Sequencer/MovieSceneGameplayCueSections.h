// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneSection.h"
#include "Sections/MovieSceneHookSection.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "MovieSceneClipboard.h"
#include "GameplayCueInterface.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneGameplayCueSections.generated.h"


USTRUCT()
struct FMovieSceneGameplayCueKey
{
	GENERATED_BODY()

	FMovieSceneGameplayCueKey()
		: Location(ForceInitToZero)
		, Normal(ForceInitToZero)
		, NormalizedMagnitude(0.0f)
		, PhysicalMaterial(nullptr)
		, GameplayEffectLevel(1)
		, AbilityLevel(1)
		, bAttachToBinding(true)
	{}

	UPROPERTY(EditAnywhere, Category="Gameplay Cue")
	FGameplayCueTag Cue;

	/** Location cue took place at - relative to the attached component if applicable */
	UPROPERTY(EditAnywhere, Category="Position")
	FVector Location;

	/** Normal of impact that caused cue */
	UPROPERTY(EditAnywhere, Category="Position")
	FVector Normal;

	/** When attached to a skeletal mesh component, specifies a socket to trigger the cue at */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Position")
	FName AttachSocketName;

	/** Magnitude of source gameplay effect, normalzed from 0-1. Use this for "how strong is the gameplay effect" (0=min, 1=,max) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Gameplay Cue", meta=(ClampMin=0.0, ClampMax=1.0))
	float NormalizedMagnitude;

	/** Instigator actor, the actor that owns the ability system component. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Gameplay Cue")
	FMovieSceneObjectBindingID Instigator;

	/** The physical actor that actually did the damage, can be a weapon or projectile */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Gameplay Cue")
	FMovieSceneObjectBindingID EffectCauser;

	/** PhysMat of the hit, if there was a hit. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Gameplay Cue")
	TObjectPtr<const UPhysicalMaterial> PhysicalMaterial;

	/** The level of that GameplayEffect */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Gameplay Cue")
	int32 GameplayEffectLevel;

	/** If originating from an ability, this will be the level of that ability */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Gameplay Cue")
	int32 AbilityLevel;

	/** Attach the gameplay cue to the track's bound object in sequencer */
	UPROPERTY(EditAnywhere, Category="Position")
	bool bAttachToBinding;
};


USTRUCT()
struct FMovieSceneGameplayCueChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	/**
	 * Access a mutable interface for this channel's data
	 *
	 * @return An object that is able to manipulate this channel's data
	 */
	inline TMovieSceneChannelData<FMovieSceneGameplayCueKey> GetData()
	{
		return TMovieSceneChannelData<FMovieSceneGameplayCueKey>(&KeyTimes, &KeyValues, this, &KeyHandles);
	}

	/**
	 * Access a constant interface for this channel's data
	 *
	 * @return An object that is able to interrogate this channel's data
	 */
	inline TMovieSceneChannelData<const FMovieSceneGameplayCueKey> GetData() const
	{
		return TMovieSceneChannelData<const FMovieSceneGameplayCueKey>(&KeyTimes, &KeyValues);
	}

public:

	// ~ FMovieSceneChannel Interface
	virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	virtual int32 GetNumKeys() const override;
	virtual void Reset() override;
	virtual void Offset(FFrameNumber DeltaPosition) override;

private:

	/** Array of times for each key */
	UPROPERTY(meta=(KeyTimes))
	TArray<FFrameNumber> KeyTimes;

	/** Array of values that correspond to each key time */
	UPROPERTY(meta=(KeyValues), AssetRegistrySearchable)
	TArray<FMovieSceneGameplayCueKey> KeyValues;

	FMovieSceneKeyHandleMap KeyHandles;
};

template<>
struct TMovieSceneChannelTraits<FMovieSceneGameplayCueChannel> : TMovieSceneChannelTraitsBase<FMovieSceneGameplayCueChannel>
{
	enum { SupportsDefaults = false };
};

inline bool EvaluateChannel(const FMovieSceneGameplayCueChannel* InChannel, FFrameTime InTime, FMovieSceneGameplayCueKey& OutValue)
{
	return false;
}

#if WITH_EDITOR
namespace MovieSceneClipboard
{
	template<> inline FName GetKeyTypeName<FMovieSceneGameplayCueKey>()
	{
		return "FMovieSceneGameplayCueKey";
	}
}
#endif // WITH_EDITOR

/**
 * Implements a movie scene section that triggers gameplay cues
 */
UCLASS(MinimalAPI)
class UMovieSceneGameplayCueTriggerSection
	: public UMovieSceneHookSection
{
public:

	GENERATED_BODY()

	UMovieSceneGameplayCueTriggerSection(const FObjectInitializer&);

private:

	virtual EMovieSceneChannelProxyType CacheChannelProxy() override;
	virtual void Trigger(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;
	virtual TArrayView<const FFrameNumber> GetTriggerTimes() const { return Channel.GetData().GetTimes(); }

	UPROPERTY(AssetRegistrySearchable)
	FMovieSceneGameplayCueChannel Channel;
};


/**
 * Implements a movie scene section that triggers gameplay cues
 */
UCLASS(MinimalAPI)
class UMovieSceneGameplayCueSection
	: public UMovieSceneHookSection
{
public:

	GENERATED_BODY()

	UMovieSceneGameplayCueSection(const FObjectInitializer&);

private:

	virtual void Begin(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;
	virtual void Update(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;
	virtual void End(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;

	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category="Cue", meta=(ShowOnlyInnerProperties))
	FMovieSceneGameplayCueKey Cue;
};

