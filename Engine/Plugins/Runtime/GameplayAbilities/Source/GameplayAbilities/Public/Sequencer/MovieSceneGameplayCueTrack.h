// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "Delegates/DelegateCombinations.h"
#include "GameplayEffectTypes.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneGameplayCueTrack.generated.h"

class AActor;

DECLARE_DYNAMIC_DELEGATE_FourParams(FMovieSceneGameplayCueEvent, AActor*, Target, FGameplayTag, GameplayTag, const FGameplayCueParameters&, Parameters, EGameplayCueEvent::Type, Event);

/**
 * Implements a movie scene track that triggers gameplay cues
 */
UCLASS(MinimalAPI)
class UMovieSceneGameplayCueTrack
	: public UMovieSceneNameableTrack
{
	GENERATED_BODY()

public:

	/** Default constructor. */
	UMovieSceneGameplayCueTrack()
	{
#if WITH_EDITORONLY_DATA
		TrackTint = FColor(0, 96, 128, 150);
#endif
	}

	static FMovieSceneGameplayCueEvent OnHandleCueEvent;

	/** Override the default function for invoking Gameplay Cues from sequencer tracks */
	UFUNCTION(BlueprintCallable, Category = "Ability|GameplayCue")
	static void SetSequencerTrackHandler(FMovieSceneGameplayCueEvent InGameplayCueTrackHandler);

public:

	// UMovieSceneTrack interface

	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual bool IsEmpty() const override;
	virtual void RemoveAllAnimationData() override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool SupportsMultipleRows() const override { return true; }

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

private:
	
	/** The track's sections. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};
