// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/PolymorphicNetSerializer.h"
#include "GameplayEffectContextHandleNetSerializer.generated.h"

USTRUCT()
struct FGameplayEffectContextHandleNetSerializerConfig : public FPolymorphicStructNetSerializerConfig
{
	GENERATED_BODY()
};

void InitGameplayEffectContextHandleNetSerializerTypeCache();

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FGameplayEffectContextHandleNetSerializer, GAMEPLAYABILITIES_API);

}