// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"
#include "GameplayEffectContextNetSerializer.generated.h"

USTRUCT()
struct FGameplayEffectContextNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

#if UE_WITH_REMOTE_OBJECT_HANDLE || (UE_NET_IRIS_NETREFHANDLE_SERIAL_SIZE == 63)
constexpr SIZE_T GetGameplayEffectContextNetSerializerSafeQuantizedSize() { return 528; }
#else
constexpr SIZE_T GetGameplayEffectContextNetSerializerSafeQuantizedSize() { return 496; }
#endif

UE_NET_DECLARE_SERIALIZER(FGameplayEffectContextNetSerializer, GAMEPLAYABILITIES_API);

}
