// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagCountContainerNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagCountContainerNetSerializer)

#include "GameplayEffectTypes.h"
#include "Engine/NetConnection.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/Serialization/PolymorphicNetSerializerImpl.h"

extern TAutoConsoleVariable<bool> CVarReplicateTagCountContainerWithIris;

void FNetGameplayTagCountContainerStateForNetSerialize::CopyFromTagCountContainer(const FGameplayTagCountContainer& Container)
{
	for (auto Item : Container.Items)
	{
		if (Item.ReplicationState == EGameplayTagReplicationState::None)
		{
			continue;
		}
		
		TagStates.Push(FGameplayTagCountItem(Item.Tag, Item.ReplicationState == EGameplayTagReplicationState::TagOnly ? 1 : Item.Count, Item
		.ReplicationState));
	}
}

namespace UE::Net
{

struct FGameplayTagCountContainerNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true;
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bUseSerializerIsEqual = true;
	
public:
	typedef FGameplayTagCountContainer SourceType;
	typedef FNetGameplayTagCountContainerStateForNetSerialize SourceArrayType;
	
	struct FQuantizedType
	{
		alignas(16) uint8 Buffer[16];
	};

	typedef FQuantizedType QuantizedType;
	
	typedef FGameplayTagCountContainerNetSerializerConfig ConfigType;
	static const ConfigType DefaultConfig;

	// 
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		FNetSerializerRegistryDelegates();
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
		virtual void OnPostFreezeNetSerializerRegistry() override;
	};

	static FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	inline static FStructNetSerializerConfig StructNetSerializerConfig;
	static const FNetSerializer* StructNetSerializer;
};

UE_NET_IMPLEMENT_SERIALIZER(FGameplayTagCountContainerNetSerializer);
const FNetSerializer* FGameplayTagCountContainerNetSerializer::StructNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);
	
const FGameplayTagCountContainerNetSerializer::ConfigType FGameplayTagCountContainerNetSerializer::DefaultConfig;
FGameplayTagCountContainerNetSerializer::FNetSerializerRegistryDelegates FGameplayTagCountContainerNetSerializer::NetSerializerRegistryDelegates;
bool bIsPostFreezeCalled = false;

void FGameplayTagCountContainerNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	FNetSerializeArgs TempArgs = Args;
	TempArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->Serialize(Context, TempArgs);
}

void FGameplayTagCountContainerNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetDeserializeArgs TempArgs = Args;
	TempArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->Deserialize(Context, TempArgs);
}
	
void FGameplayTagCountContainerNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	FNetSerializeDeltaArgs TempArgs = Args;
	TempArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->SerializeDelta(Context, TempArgs);
}

void FGameplayTagCountContainerNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetDeserializeDeltaArgs TempArgs = Args;
	TempArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->DeserializeDelta(Context, TempArgs);
}

void FGameplayTagCountContainerNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	// this is to stop initial replication overriding things incorrectly when we're not supposed to be using the iris serializer.
	if (!CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread())
	{
		return;
	}
	
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);

	FNetGameplayTagCountContainerStateForNetSerialize SourceCopy;
	SourceCopy.CopyFromTagCountContainer(Source);

	FNetQuantizeArgs TempArgs = Args;
	TempArgs.Source = NetSerializerValuePointer(&SourceCopy);
	TempArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->Quantize(Context, TempArgs);
}

void FGameplayTagCountContainerNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	// this is to stop initial replication overriding things incorrectly when we're not supposed to be using the iris serializer.
	if (!CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread())
	{
		return;
	}
	
	SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);
	
	FNetGameplayTagCountContainerStateForNetSerialize SourceCopy;

	FNetDequantizeArgs TempArgs = Args;
	TempArgs.Target = NetSerializerValuePointer(&SourceCopy);
	TempArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->Dequantize(Context, TempArgs);

	for (int32 Idx = 0; Idx < SourceCopy.TagStates.Num(); Idx++)
	{
		TargetValue.SetTagCount(SourceCopy.TagStates[Idx].Tag, SourceCopy.TagStates[Idx].Count);
	}
}

bool FGameplayTagCountContainerNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (!CVarReplicateTagCountContainerWithIris.GetValueOnAnyThread())
	{
		return true;
	}
	
	if (Args.bStateIsQuantized)
	{
		FNetIsEqualArgs TempArgs = Args;
		TempArgs.NetSerializerConfig = &StructNetSerializerConfig;
		return StructNetSerializer->IsEqual(Context, TempArgs);
	}
	else
	{
		const SourceType& Source0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Source1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		return Source0 == Source1;
	}
}
	
void FGameplayTagCountContainerNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	FNetCloneDynamicStateArgs TempArgs = Args;
	TempArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->CloneDynamicState(Context, TempArgs);
}
	
void FGameplayTagCountContainerNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	FNetFreeDynamicStateArgs TempArgs = Args;
	TempArgs.NetSerializerConfig = &StructNetSerializerConfig;
	StructNetSerializer->FreeDynamicState(Context, TempArgs);
}

void FGameplayTagCountContainerNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
}

bool FGameplayTagCountContainerNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	FNetGameplayTagCountContainerStateForNetSerialize SourceCopy;
	SourceCopy.CopyFromTagCountContainer(SourceValue);

	FNetValidateArgs ValidateArgs = Args;
	ValidateArgs.NetSerializerConfig = &StructNetSerializerConfig;
	ValidateArgs.Source = NetSerializerValuePointer(&SourceCopy);
	return StructNetSerializer->Validate(Context, ValidateArgs);
}

static const FName PropertyNetSerializerRegistry_NAME_GameplayTagCountContainer("GameplayTagCountContainer");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayTagCountContainer, FGameplayTagCountContainerNetSerializer);

FGameplayTagCountContainerNetSerializer::FNetSerializerRegistryDelegates::FNetSerializerRegistryDelegates()
: UE::Net::FNetSerializerRegistryDelegates(EFlags::ShouldBindLoadedModulesUpdatedDelegate)
{
}
	
FGameplayTagCountContainerNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayTagCountContainer);
}

void FGameplayTagCountContainerNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayTagCountContainer);
}

void FGameplayTagCountContainerNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	const UStruct* TagCountMapStruct = FNetGameplayTagCountContainerStateForNetSerialize::StaticStruct();
	FReplicationStateDescriptorBuilder::FParameters Params;
	StructNetSerializerConfig.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(TagCountMapStruct, Params);

	const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfig.StateDescriptor.GetReference();

	// Validate our assumptions regarding quantized state size and alignment.
	if ((sizeof(QuantizedType) < Descriptor->InternalSize) || (alignof(QuantizedType) < Descriptor->InternalAlignment))
	{
		LowLevelFatalError(TEXT("FGameplayTagCountContainerNetSerializer::FQuantizedType has size %u and alignment %u but requires size %u and alignment %u."), uint32(sizeof(FQuantizedType)), uint32(alignof(FQuantizedType)), Descriptor->InternalSize, Descriptor->InternalAlignment);
	}
}

}
