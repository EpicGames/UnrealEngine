// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"
#include "GameplayEffectTypes.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"

#include "GameplayTagCountContainerNetSerializer.generated.h"

USTRUCT()
struct FGameplayTagCountContainerNetSerializerConfig : public FNetSerializerConfig
{
GENERATED_BODY()
};

USTRUCT()
struct FNetGameplayTagCountContainerStateForNetSerialize
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FGameplayTagCountItem> TagStates;

	void CopyFromTagCountContainer(const FGameplayTagCountContainer& Container);

	bool operator== (const FNetGameplayTagCountContainerStateForNetSerialize& Other) const
	{
		return TagStates == Other.TagStates;
	}
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FGameplayTagCountContainerNetSerializer, GAMEPLAYABILITIES_API);

}
