// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"
#include "MinimalGameplayCueReplicationProxyNetSerializer.generated.h"

USTRUCT()
struct FMinimalGameplayCueReplicationProxyNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FMinimalGameplayCueReplicationProxyNetSerializer, GAMEPLAYABILITIES_API);

}
