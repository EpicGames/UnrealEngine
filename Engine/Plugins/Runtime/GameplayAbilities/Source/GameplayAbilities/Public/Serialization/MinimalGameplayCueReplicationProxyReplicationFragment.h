// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Templates/UniquePtr.h"

namespace UE::Net
{

FReplicationFragment* CreateAndRegisterMinimalGameplayCueReplicationProxyReplicationFragment(UObject* Owner, const FReplicationStateDescriptor* Descriptor, FFragmentRegistrationContext& Context);

/**
 * ReplicationFragment implementation used for Iris backwards compatibility until there's a simple way to markup arbitrary members or declare replicated states in a different manner.
 * That coupled with generic Iris ApplyState funcitonality and not serializing data into gameplay objects directly would make the need for the fragment and the custom serializer go away.
 * Instead the custom logic would be handled in the ApplyState implementation.
 * @note This is NOT what good looks like. Markup your data and avoid custom gameplay logic in serialization code.
 */
class FMinimalGameplayCueReplicationProxyReplicationFragment : public FReplicationFragment
{
public:
	FMinimalGameplayCueReplicationProxyReplicationFragment(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor);

	void Register(FFragmentRegistrationContext& Context);

protected:
	virtual void ApplyReplicatedState(FReplicationStateApplyContext& Context) const override;
	virtual bool PollReplicatedState(EReplicationFragmentPollFlags PollOption) override;
	virtual void CallRepNotifies(FReplicationStateApplyContext& Context) override;
	virtual void CollectOwner(FReplicationStateOwnerCollector* Owners) const override;

private:
	void MimicMinimalGameplayCueReplicationProxyReceiveLogic(FReplicationStateApplyContext& Context) const;
	void CallRepNotify(FReplicationStateApplyContext& Context);

private:
	TRefCountPtr<const FReplicationStateDescriptor> ReplicationStateDescriptor;
	TUniquePtr<FPropertyReplicationState> SrcReplicationState;
	UObject* Owner = nullptr;
};

}

