// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"
#include "MinimalReplicationTagCountNetSerializer.generated.h"

USTRUCT()
struct FMinimalReplicationTagCountMapNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FMinimalReplicationTagCountMapNetSerializer, GAMEPLAYABILITIES_API);

}
