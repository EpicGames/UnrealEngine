// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "TickableAttributeSetInterface.generated.h"

/** Interface for attribute sets that need to tick to update state. These can be expensive */
UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UTickableAttributeSetInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class ITickableAttributeSetInterface
{
	GENERATED_IINTERFACE_BODY()

public:
	/**
	 * Ticks the attribute set by DeltaTime seconds
	 * 
	 * @param DeltaTime Size of the time step in seconds.
	 */
	virtual void Tick(float DeltaTime) = 0;

	/**
	* Does this attribute set need to tick?
	*
	* @return true if this attribute set should currently be ticking, false otherwise.
	*/
	virtual bool ShouldTick() const = 0;
};

