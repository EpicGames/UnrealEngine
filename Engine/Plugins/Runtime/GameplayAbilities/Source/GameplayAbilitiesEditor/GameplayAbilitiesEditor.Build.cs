// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameplayAbilitiesEditor : ModuleRules
	{
		public GameplayAbilitiesEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.Add("GameplayTasks");

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					// ... add private dependencies that you statically link with here ...
					"AssetDefinition",
					"Core",
					"CoreUObject",
					"Engine",
					"EngineAssetDefinitions",
					"AssetTools",
					"ClassViewer",
					"GameplayTags",
					"GameplayTagsEditor",
					"GameplayAbilities",
					"GameplayTasksEditor",
					"InputCore",
					"PropertyEditor",
					"Slate",
					"SlateCore",					
					"BlueprintGraph",
					"Kismet",
					"KismetCompiler",
					"GraphEditor",
					"LevelSequence",
					"MainFrame",
					"EditorFramework",
					"UnrealEd",
					"WorkspaceMenuStructure",
					"ContentBrowser",
					"EditorWidgets",
					"SourceControl",
					"SequencerCore",
					"Sequencer",
					"MovieSceneTools",
					"MovieScene",
					"DataRegistry",
					"DataRegistryEditor",
					"ToolMenus",
					"ApplicationCore",
				}
			);
		}
	}
}
