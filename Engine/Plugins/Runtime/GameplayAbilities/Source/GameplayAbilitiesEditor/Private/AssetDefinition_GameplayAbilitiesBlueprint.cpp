// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_GameplayAbilitiesBlueprint.h"
#include "GameplayAbilitiesBlueprintFactory.h"
#include "GameplayAbilitiesEditor.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_GameplayAbilitiesBlueprint"

EAssetCommandResult UAssetDefinition_GameplayAbilitiesBlueprint::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	const TArray<UObject*>& Objects = OpenArgs.LoadObjects<UObject>();
	for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		UBlueprint* Blueprint = Cast<UBlueprint>(*ObjIt);
		if (Blueprint)
		{
			bool bLetOpen = true;
			if (!Blueprint->ParentClass)
			{
				bLetOpen = EAppReturnType::Yes == FMessageDialog::Open(
					EAppMsgType::YesNo, 
					LOCTEXT("FailedToLoadAbilityBlueprintWithContinue", "Gameplay Ability Blueprint could not be loaded because it derives from an invalid class.  Check to make sure the parent class for this blueprint hasn't been removed! Do you want to continue (it can crash the editor)?")
				);
			}
		
			if (bLetOpen)
			{
				const TSharedRef<FGameplayAbilitiesEditor> NewEditor(new FGameplayAbilitiesEditor());

				TArray<UBlueprint*> Blueprints;
				Blueprints.Add(Blueprint);

				NewEditor->InitGameplayAbilitiesEditor(Mode, OpenArgs.ToolkitHost, Blueprints, ShouldUseDataOnlyEditor(Blueprint));
			}
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("FailedToLoadAbilityBlueprint", "Gameplay Ability Blueprint could not be loaded because it derives from an invalid class.  Check to make sure the parent class for this blueprint hasn't been removed!"));
		}
	}
	return EAssetCommandResult::Handled;
}

UFactory* UAssetDefinition_GameplayAbilitiesBlueprint::GetFactoryForBlueprintType(UBlueprint* InBlueprint) const
{
	UGameplayAbilitiesBlueprintFactory* GameplayAbilitiesBlueprintFactory = NewObject<UGameplayAbilitiesBlueprintFactory>();
	GameplayAbilitiesBlueprintFactory->ParentClass = TSubclassOf<UGameplayAbility>(*InBlueprint->GeneratedClass);
	return GameplayAbilitiesBlueprintFactory;
}

bool UAssetDefinition_GameplayAbilitiesBlueprint::ShouldUseDataOnlyEditor(const UBlueprint* Blueprint) const
{
	return FBlueprintEditorUtils::IsDataOnlyBlueprint(Blueprint)
		&& !FBlueprintEditorUtils::IsLevelScriptBlueprint(Blueprint)
		&& !FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint)
		&& !Blueprint->bForceFullEditor
		&& !Blueprint->bIsNewlyCreated;
}

#undef LOCTEXT_NAMESPACE
