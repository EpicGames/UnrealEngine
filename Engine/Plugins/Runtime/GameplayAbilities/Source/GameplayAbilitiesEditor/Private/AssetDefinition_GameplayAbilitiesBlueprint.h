// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GameplayAbilityBlueprint.h"
#include "Script/AssetDefinition_Blueprint.h"
#include "AssetDefinition_GameplayAbilitiesBlueprint.generated.h"

UCLASS()
class UAssetDefinition_GameplayAbilitiesBlueprint : public UAssetDefinition_Blueprint
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_GameplayAbilitiesBlueprint", "Gameplay Ability Blueprint"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(0, 96, 128); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UGameplayAbilityBlueprint::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Gameplay, NSLOCTEXT("AssetDefinition", "AssetDefinition_GameplayAbilitiesBlueprintSubMenu", "Other"), ECategoryMenuType::Section)
		};

		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual UFactory* GetFactoryForBlueprintType(UBlueprint* InBlueprint) const override;
	// UAssetDefinition End

private:
	/** Returns true if the blueprint is data only */
	bool ShouldUseDataOnlyEditor(const UBlueprint* Blueprint) const;
};
