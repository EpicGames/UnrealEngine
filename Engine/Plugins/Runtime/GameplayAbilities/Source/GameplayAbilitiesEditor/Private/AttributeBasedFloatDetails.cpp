// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttributeBasedFloatDetails.h"
#include "Misc/Attribute.h"
#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "IDetailPropertyRow.h"
#include "GameplayEffect.h"
#include "IDetailChildrenBuilder.h"

#define LOCTEXT_NAMESPACE "AttributeBasedFloatDetails"

TSharedRef<IPropertyTypeCustomization> FAttributeBasedFloatDetails::MakeInstance()
{
	return MakeShareable(new FAttributeBasedFloatDetails());
}

void FAttributeBasedFloatDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];
}

void FAttributeBasedFloatDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildrenProps = 0;
	StructPropertyHandle->GetNumChildren(NumChildrenProps);

	AttributeCalculationTypePropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAttributeBasedFloat, AttributeCalculationType));
	const TSharedPtr<IPropertyHandle> FinalChannelHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAttributeBasedFloat, FinalChannel));
	
	if (ensure(FinalChannelHandle.IsValid()) && ensure(FinalChannelHandle->IsValidHandle()) && ensure(AttributeCalculationTypePropertyHandle.IsValid()) && ensure(AttributeCalculationTypePropertyHandle->IsValidHandle()))
	{
		for (uint32 ChildIdx = 0; ChildIdx < NumChildrenProps; ++ChildIdx)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIdx);
			if (ChildHandle.IsValid() && ChildHandle->IsValidHandle())
			{
				IDetailPropertyRow& NewPropRow = StructBuilder.AddProperty(ChildHandle.ToSharedRef());
				if (ChildHandle->GetProperty() == FinalChannelHandle->GetProperty())
				{
					NewPropRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FAttributeBasedFloatDetails::GetFinalChannelVisibility)));
				}
			}
		}
	}
}

EVisibility FAttributeBasedFloatDetails::GetFinalChannelVisibility() const
{
	bool bVisible = false;

	if (AttributeCalculationTypePropertyHandle.IsValid() && AttributeCalculationTypePropertyHandle->IsValidHandle())
	{
		uint8 EnumVal = 0;
		if (AttributeCalculationTypePropertyHandle->GetValue(EnumVal) == FPropertyAccess::Success)
		{
			bVisible = (EnumVal == static_cast<uint8>(EAttributeBasedFloatCalculationType::AttributeMagnitudeEvaluatedUpToChannel));
		}
	}

	return bVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
