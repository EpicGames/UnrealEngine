// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "IPropertyTypeCustomization.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;

/** Details customization for FAttributeBasedFloat */
class FAttributeBasedFloatDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** Overridden to provide the property name or hide, if necessary */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	
	/** Overridden to allow for possibly being hidden */
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	/** Called via delegate to determine visibility of the final channel property */
	EVisibility GetFinalChannelVisibility() const;

	/** Property handle to the AttributeCalculationType property; Used to determine visibility of final channel property */
	TSharedPtr<IPropertyHandle> AttributeCalculationTypePropertyHandle;
};
