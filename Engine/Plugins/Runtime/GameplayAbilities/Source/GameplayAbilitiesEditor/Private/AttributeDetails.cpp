// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttributeDetails.h"
#include "UObject/UnrealType.h"
#include "SlateOptMacros.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSlider.h"
#include "Engine/CurveTable.h"
#include "PropertyHandle.h"
#include "IPropertyUtilities.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "AttributeSet.h"
#include "AbilitySystemGlobals.h"
#include "Widgets/Input/STextComboBox.h"
#include "AbilitySystemComponent.h"
#include "SGameplayAttributeWidget.h"
#include "DataRegistryEditorModule.h"
#include "DataRegistrySubsystem.h"
#include "PropertyCustomizationHelpers.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "AttributeDetailsCustomization"

DEFINE_LOG_CATEGORY(LogAttributeDetails);

TSharedRef<IPropertyTypeCustomization> FAttributePropertyDetails::MakeInstance()
{
	return MakeShareable(new FAttributePropertyDetails());
}

void FAttributePropertyDetails::CustomizeHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	MyProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayAttribute,Attribute));
	OwnerProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayAttribute,AttributeOwner));
	NameProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayAttribute,AttributeName));

	PropertyOptions.Empty();
	PropertyOptions.Add(MakeShareable(new FString("None")));

	const FString& FilterMetaStr = StructPropertyHandle->GetProperty()->GetMetaData(TEXT("FilterMetaTag"));

	TArray<FProperty*> PropertiesToAdd;
	FGameplayAttribute::GetAllAttributeProperties(PropertiesToAdd, FilterMetaStr);

	for ( auto* Property : PropertiesToAdd )
	{
		PropertyOptions.Add(MakeShareable(new FString(FString::Printf(TEXT("%s.%s"), *Property->GetOwnerVariant().GetName(), *Property->GetName()))));
	}

	FProperty* PropertyValue = nullptr;
	if (MyProperty.IsValid())
	{
		FProperty *ObjPtr = nullptr;
		MyProperty->GetValue(ObjPtr);
		PropertyValue = ObjPtr;
	}

	HeaderRow.
		NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
	.ValueContent()
		.MinDesiredWidth(500)
		.MaxDesiredWidth(4096)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			//.FillWidth(1.0f)
			.HAlign(HAlign_Fill)
			.Padding(0.f, 0.f, 2.f, 0.f)
			[
				SNew(SGameplayAttributeWidget)
				.OnAttributeChanged(this, &FAttributePropertyDetails::OnAttributeChanged)
				.DefaultProperty(PropertyValue)
				.FilterMetaData(FilterMetaStr)
			]
		];
}

void FAttributePropertyDetails::CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{


}

TSharedPtr<FString> FAttributePropertyDetails::GetPropertyType() const
{
	if (MyProperty.IsValid())
	{
		FProperty *PropertyValue = nullptr;
		MyProperty->GetValue(PropertyValue);
		if (PropertyValue)
		{
			const FString FullString = PropertyValue->GetOwnerVariant().GetName() + TEXT(".") + PropertyValue->GetName();
			for (int32 i=0; i < PropertyOptions.Num(); ++i)
			{
				if (PropertyOptions[i].IsValid() && PropertyOptions[i].Get()->Equals(FullString))
				{
					return PropertyOptions[i];
				}
			}
		}
	}

	return PropertyOptions[0]; // This should always be none
}

void FAttributePropertyDetails::OnChangeProperty(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo)
{
	if (ItemSelected.IsValid() && MyProperty.IsValid())
	{		
		FString FullString = *ItemSelected.Get();
		FString ClassName;
		FString PropertyName;

		FullString.Split( TEXT("."), &ClassName, &PropertyName);

		UClass *FoundClass = UClass::TryFindTypeSlow<UClass>(ClassName);
		if (FoundClass)
		{
			FProperty *Property = FindFProperty<FProperty>(FoundClass, *PropertyName);
			if (Property)
			{
				MyProperty->SetValue(Property);
				
				return;
			}
		}

		UObject* nullObj = nullptr;
		MyProperty->SetValue(nullObj);
	}
}

void FAttributePropertyDetails::OnAttributeChanged(FProperty* SelectedAttribute)
{
	if (MyProperty.IsValid())
	{
		MyProperty->SetValue(SelectedAttribute);

		// When we set the attribute we should also set the owner and name info
		if (OwnerProperty.IsValid())
		{
			OwnerProperty->SetValue(SelectedAttribute ? SelectedAttribute->GetOwnerStruct() : nullptr);
		}

		if (NameProperty.IsValid())
		{
			FString AttributeName;
			if (SelectedAttribute)
			{
				SelectedAttribute->GetName(AttributeName);
			}
			NameProperty->SetValue(AttributeName);
		}
	}
}

// ------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------

TSharedRef<IDetailCustomization> FAttributeDetails::MakeInstance()
{
	return MakeShareable(new FAttributeDetails);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FAttributeDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	MyProperty = DetailLayout.GetProperty("PropertyReference", UAttributeSet::StaticClass());

	PropertyOptions.Empty();
	PropertyOptions.Add(MakeShareable(new FString("None")));

	for( TFieldIterator<FProperty> It(UAttributeSet::StaticClass(), EFieldIteratorFlags::ExcludeSuper) ; It ; ++It )
	{
		FProperty *Property = *It;
		PropertyOptions.Add(MakeShareable(new FString(Property->GetName())));
	}

	IDetailCategoryBuilder& Category = DetailLayout.EditCategory("Variable", LOCTEXT("VariableDetailsCategory", "Variable"));
	const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();
	
	Category.AddCustomRow( LOCTEXT("ReplicationLabel", "Replication") )
		//.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::ReplicationVisibility))
		.NameContent()
		[
			SNew(STextBlock)
			.ToolTipText(LOCTEXT("PropertyType_Tooltip", "Which Property To Modify?"))
			.Text( LOCTEXT("PropertyModifierInfo", "Property") )
			.Font( DetailFontInfo )
		]
		.ValueContent()
		[
			SNew(STextComboBox)
			.OptionsSource( &PropertyOptions )
			.InitiallySelectedItem(GetPropertyType())
			.OnSelectionChanged( this, &FAttributeDetails::OnChangeProperty )
		];
	
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


TSharedPtr<FString> FAttributeDetails::GetPropertyType() const
{
	if (!MyProperty.IsValid())
		return PropertyOptions[0];

	FProperty *PropertyValue = nullptr;
	MyProperty->GetValue(PropertyValue);

	if (PropertyValue != nullptr)
	{
		for (int32 i=0; i < PropertyOptions.Num(); ++i)
		{
			if (PropertyOptions[i].IsValid() && PropertyOptions[i].Get()->Equals(PropertyValue->GetName()))
			{
				return PropertyOptions[i];
			}
		}
	}

	return PropertyOptions[0]; // This should always be none
}

void FAttributeDetails::OnChangeProperty(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo)
{
	if (!ItemSelected.IsValid())
	{
		return;
	}

	FString PropertyName = *ItemSelected.Get();

	for( TFieldIterator<FProperty> It(UAttributeSet::StaticClass(), EFieldIteratorFlags::ExcludeSuper) ; It ; ++It )
	{
		FProperty* Property = *It;
		if (PropertyName == Property->GetName())
		{
			MyProperty->SetValue(Property);
			return;
		}
	}
}

// ------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------

TSharedRef<IPropertyTypeCustomization> FScalableFloatDetails::MakeInstance()
{
	return MakeShareable(new FScalableFloatDetails());
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FScalableFloatDetails::CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	bSourceRefreshQueued = false;
	PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();

	ValueProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FScalableFloat,Value));
	CurveTableHandleProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FScalableFloat,Curve));
	RegistryTypeProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FScalableFloat, RegistryType));

	if (ValueProperty.IsValid() && CurveTableHandleProperty.IsValid() && RegistryTypeProperty.IsValid())
	{
		RowNameProperty = CurveTableHandleProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCurveTableRowHandle, RowName));
		CurveTableProperty = CurveTableHandleProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCurveTableRowHandle, CurveTable));

		UpdatePreviewLevels();

		CurveTableProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FScalableFloatDetails::OnCurveSourceChanged));
		RegistryTypeProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FScalableFloatDetails::OnCurveSourceChanged));
		RowNameProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FScalableFloatDetails::OnRowNameChanged));

		HeaderRow
			.NameContent()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth( 600 )
			.MaxDesiredWidth( 4096 )
			[
				SNew(SVerticalBox)
				.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FScalableFloatDetails::IsEditable)))

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(0.2f)
					.HAlign(HAlign_Fill)
					.Padding(1.f, 0.f, 2.f, 0.f)
					[
						ValueProperty->CreatePropertyValueWidget()
					]
		
					+SHorizontalBox::Slot()
					.FillWidth(0.40f)
					.HAlign(HAlign_Fill)
					.Padding(2.f, 0.f, 2.f, 0.f)
					[
						CreateCurveTableWidget()
					]

					+SHorizontalBox::Slot()
					.FillWidth(0.40f)
					.HAlign(HAlign_Fill)
					.Padding(2.f, 0.f, 0.f, 0.f)
					[
						CreateRegistryTypeWidget()
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					[
						SNew(SHorizontalBox)
						.Visibility(this, &FScalableFloatDetails::GetAssetButtonVisiblity)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							PropertyCustomizationHelpers::MakeUseSelectedButton(FSimpleDelegate::CreateSP(this, &FScalableFloatDetails::OnUseSelected))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateSP(this, &FScalableFloatDetails::OnBrowseTo))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							PropertyCustomizationHelpers::MakeClearButton(FSimpleDelegate::CreateSP(this, &FScalableFloatDetails::OnClear))
						]
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 3.f, 0.f, 0.f)
				[
					SNew(SHorizontalBox)
					.Visibility(this, &FScalableFloatDetails::GetRowNameVisibility)
					
					+SHorizontalBox::Slot()
					.FillWidth(0.4f)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						CreateRowNameWidget()
					]

					+SHorizontalBox::Slot()
					.FillWidth(0.3f)
					.HAlign(HAlign_Fill)
					.Padding(2.f, 0.f, 2.f, 0.f)
					[
						SNew(SVerticalBox)
						.Visibility(this, &FScalableFloatDetails::GetPreviewVisibility)
				
						+SVerticalBox::Slot()
						.HAlign(HAlign_Center)
						[
							SNew(STextBlock)
							.Text(this, &FScalableFloatDetails::GetRowValuePreviewLabel)
						]

						+SVerticalBox::Slot()
						.HAlign(HAlign_Center)
						[
							SNew(STextBlock)
							.Text(this, &FScalableFloatDetails::GetRowValuePreviewText)
						]
					]

					+SHorizontalBox::Slot()
					.FillWidth(0.3f)
					.HAlign(HAlign_Fill)
					.Padding(2.f, 0.f, 0.f, 0.f)
					[
						SNew(SSlider)
						.Visibility(this, &FScalableFloatDetails::GetPreviewVisibility)
						.ToolTipText(LOCTEXT("LevelPreviewToolTip", "Adjust the preview level."))
						.Value(this, &FScalableFloatDetails::GetPreviewLevel)
						.OnValueChanged(this, &FScalableFloatDetails::SetPreviewLevel)
					]
				]
			];	
	}
}

TSharedRef<SWidget> FScalableFloatDetails::CreateCurveTableWidget()
{
	return SNew(SComboButton)
		.OnGetMenuContent(this, &FScalableFloatDetails::GetCurveTablePicker)
		.ContentPadding(FMargin(2.0f, 2.0f))
		.Visibility(this, &FScalableFloatDetails::GetCurveTableVisiblity)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &FScalableFloatDetails::GetCurveTableText)
			.ToolTipText(this, &FScalableFloatDetails::GetCurveTableTooltip)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		];
}

TSharedRef<SWidget> FScalableFloatDetails::CreateRegistryTypeWidget()
{
	// Only support curve types
	static FName RealCurveName = FName("RealCurve");

	return SNew(SBox)
		.Padding(0.0f)
		.ToolTipText(this, &FScalableFloatDetails::GetRegistryTypeTooltip)
		.Visibility(this, &FScalableFloatDetails::GetRegistryTypeVisiblity)
		.VAlign(VAlign_Center)
		[
			PropertyCustomizationHelpers::MakePropertyComboBox(RegistryTypeProperty,
			FOnGetPropertyComboBoxStrings::CreateStatic(&FDataRegistryEditorModule::GenerateDataRegistryTypeComboBoxStrings, true, RealCurveName),
			FOnGetPropertyComboBoxValue::CreateSP(this, &FScalableFloatDetails::GetRegistryTypeValueString))
		];
}

TSharedRef<SWidget> FScalableFloatDetails::CreateRowNameWidget()
{
	FPropertyAccess::Result* OutResult = nullptr;

	return SNew(SBox)
		.Padding(0.0f)
		.ToolTipText(this, &FScalableFloatDetails::GetRowNameComboBoxContentTooltip)
		.VAlign(VAlign_Center)
		[
			FDataRegistryEditorModule::MakeDataRegistryItemNameSelector(
				FOnGetDataRegistryDisplayText::CreateSP(this, &FScalableFloatDetails::GetRowNameComboBoxContentText),
				FOnGetDataRegistryId::CreateSP(this, &FScalableFloatDetails::GetRegistryId, OutResult),
				FOnSetDataRegistryId::CreateSP(this, &FScalableFloatDetails::SetRegistryId),
				FOnGetCustomDataRegistryItemNames::CreateSP(this, &FScalableFloatDetails::GetCustomRowNames),
				true)
		];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SWidget> FScalableFloatDetails::GetCurveTablePicker()
{
	const bool bAllowClear = true;
	TArray<const UClass*> AllowedClasses;
	AllowedClasses.Add(UCurveTable::StaticClass());

	FAssetData CurrentAssetData;
	UCurveTable* SelectedTable = GetCurveTable();

	if (SelectedTable)
	{
		CurrentAssetData = FAssetData(SelectedTable);
	}

	return PropertyCustomizationHelpers::MakeAssetPickerWithMenu(CurrentAssetData,
		bAllowClear,
		AllowedClasses,
		PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(AllowedClasses),
		FOnShouldFilterAsset(),
		FOnAssetSelected::CreateSP(this, &FScalableFloatDetails::OnSelectCurveTable),
		FSimpleDelegate::CreateSP(this, &FScalableFloatDetails::OnCloseMenu));
}

void FScalableFloatDetails::OnSelectCurveTable(const FAssetData& AssetData)
{
	UObject* SelectedTable = AssetData.GetAsset();

	CurveTableProperty->SetValue(SelectedTable);
	
	// Also clear type
	RegistryTypeProperty->SetValueFromFormattedString(FString());
}

void FScalableFloatDetails::OnCloseMenu()
{
	FSlateApplication::Get().DismissAllMenus();
}

FText FScalableFloatDetails::GetCurveTableText() const
{
	FPropertyAccess::Result FoundResult;
	UCurveTable* SelectedTable = GetCurveTable(&FoundResult);

	if (SelectedTable)
	{
		return FText::AsCultureInvariant(SelectedTable->GetName());
	}
	else if (FoundResult == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return LOCTEXT("PickCurveTable", "Use CurveTable...");
}

FText FScalableFloatDetails::GetCurveTableTooltip() const
{
	UCurveTable* SelectedTable = GetCurveTable();

	if (SelectedTable)
	{
		return FText::AsCultureInvariant(SelectedTable->GetPathName());
	}

	return LOCTEXT("PickCurveTableTooltip", "Select a CurveTable asset containing Curve to multiply by Value");
}

EVisibility FScalableFloatDetails::GetCurveTableVisiblity() const
{
	FDataRegistryType RegistryType = GetRegistryType();
	return RegistryType.IsValid() ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility FScalableFloatDetails::GetAssetButtonVisiblity() const
{
	UCurveTable* CurveTable = GetCurveTable();
	return CurveTable ? EVisibility::Visible : EVisibility::Collapsed;
}

void FScalableFloatDetails::OnBrowseTo()
{
	UCurveTable* CurveTable = GetCurveTable();
	if (CurveTable)
	{
		TArray<FAssetData> SyncAssets;
		SyncAssets.Add(FAssetData(CurveTable));
		GEditor->SyncBrowserToObjects(SyncAssets);
	}
}

void FScalableFloatDetails::OnClear()
{
	OnSelectCurveTable(FAssetData());
	if (RowNameProperty.IsValid())
	{
		RowNameProperty->ResetToDefault();
	}
}

void FScalableFloatDetails::OnUseSelected()
{
	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);

	for (const FAssetData& AssetData : SelectedAssets)
	{
		UCurveTable* FoundTable = Cast<UCurveTable>(AssetData.GetAsset());

		if (FoundTable)
		{
			OnSelectCurveTable(AssetData);
			return;
		}
	}
}

FString FScalableFloatDetails::GetRegistryTypeValueString() const
{
	FPropertyAccess::Result FoundResult;
	FDataRegistryType RegistryType = GetRegistryType(&FoundResult);

	if (RegistryType.IsValid())
	{
		return RegistryType.ToString();
	}
	else if (FoundResult == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values").ToString();
	}

	return LOCTEXT("PickRegistry", "Use Registry...").ToString();
}

FText FScalableFloatDetails::GetRegistryTypeTooltip() const
{
	return LOCTEXT("PickRegistryTooltip", "Select a DataRegistry containing Curve to multiply by Value");
}

EVisibility FScalableFloatDetails::GetRegistryTypeVisiblity() const
{
	UCurveTable* CurveTable = GetCurveTable();
	FDataRegistryType RegistryType = GetRegistryType();
	bool bIsSystemEnabled = UDataRegistrySubsystem::Get()->IsConfigEnabled();
	return RegistryType.IsValid() || (bIsSystemEnabled && !CurveTable) ? EVisibility::Visible : EVisibility::Collapsed;
}

void FScalableFloatDetails::OnCurveSourceChanged()
{
	// Need a frame deferral to deal with multi edit caching problems
	TSharedPtr<IPropertyUtilities> PinnedUtilities = PropertyUtilities.Pin();
	if (!bSourceRefreshQueued && PinnedUtilities.IsValid())
	{
		PinnedUtilities->EnqueueDeferredAction(FSimpleDelegate::CreateSP(this, &FScalableFloatDetails::RefreshSourceData));
		bSourceRefreshQueued = true;
	}
}

void FScalableFloatDetails::OnRowNameChanged()
{
	UpdatePreviewLevels();
}

void FScalableFloatDetails::RefreshSourceData()
{
	// Set the default value to 1.0 when using a curve source, so the value in the table is used directly. Only do this if the value is currently 0 (default)
	// Set it back to 0 when setting back. Only do this if the value is currently 1 to go back to the default.

	UObject* CurveTable = GetCurveTable();
	FDataRegistryType RegistryType = GetRegistryType();

	float Value = -1.0f;
	FPropertyAccess::Result ValueResult = ValueProperty->GetValue(Value);

	// Only modify if all are the same for multi select
	if (CurveTable || RegistryType.IsValid())
	{
		if (Value == 0.f || ValueResult == FPropertyAccess::MultipleValues)
		{
			ValueProperty->SetValue(1.f);
		}
	}
	else
	{
		if (Value == 1.f || ValueResult == FPropertyAccess::MultipleValues)
		{
			ValueProperty->SetValue(0.f);
		}
	}

	if (RegistryType.IsValid())
	{
		// Registry type has priority over curve table
		UCurveTable* NullTable = nullptr;
		CurveTableProperty->SetValue(NullTable);
	}

	bSourceRefreshQueued = false;
}

UCurveTable* FScalableFloatDetails::GetCurveTable(FPropertyAccess::Result* OutResult) const
{
	FPropertyAccess::Result TempResult;
	if (OutResult == nullptr)
	{
		OutResult = &TempResult;
	}

	UCurveTable* CurveTable = nullptr;
	if (CurveTableProperty.IsValid())
	{
		*OutResult = CurveTableProperty->GetValue((UObject*&)CurveTable);
	}
	else
	{
		*OutResult = FPropertyAccess::Fail;
	}

	return CurveTable;
}

FDataRegistryType FScalableFloatDetails::GetRegistryType(FPropertyAccess::Result* OutResult) const
{
	FPropertyAccess::Result TempResult;
	if (OutResult == nullptr)
	{
		OutResult = &TempResult;
	}
	
	FString RegistryString;

	// Bypassing the struct because GetValueAsFormattedStrings doesn't work well on multi select
	if (RegistryTypeProperty.IsValid())
	{
		*OutResult = RegistryTypeProperty->GetValueAsFormattedString(RegistryString);
	}
	else
	{
		*OutResult = FPropertyAccess::Fail;
	}

	return FDataRegistryType(*RegistryString);
}

EVisibility FScalableFloatDetails::GetRowNameVisibility() const
{
	UCurveTable* CurveTable = GetCurveTable();
	FDataRegistryType RegistryType = GetRegistryType();

	return (CurveTable || RegistryType.IsValid()) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FScalableFloatDetails::GetPreviewVisibility() const
{
	FName CurrentRow = GetRowName();
	return (CurrentRow != NAME_None) ? EVisibility::Visible : EVisibility::Hidden;
}

float FScalableFloatDetails::GetPreviewLevel() const
{
	return (MaxPreviewLevel != MinPreviewLevel) ? (PreviewLevel-MinPreviewLevel) / (MaxPreviewLevel-MinPreviewLevel) : 0;
}

void FScalableFloatDetails::SetPreviewLevel(float NewLevel)
{
	PreviewLevel = FMath::FloorToInt(NewLevel * (MaxPreviewLevel - MinPreviewLevel) + MinPreviewLevel);
}

FText FScalableFloatDetails::GetRowNameComboBoxContentText() const
{
	FPropertyAccess::Result RowResult;
	FName RowName = GetRowName(&RowResult);

	if (RowResult != FPropertyAccess::MultipleValues)
	{
		if (RowName != NAME_None)
		{
			return FText::FromName(RowName);
		}
		else
		{
			return LOCTEXT("SelectCurve", "Select Curve...");
		}
	}
	return LOCTEXT("MultipleValues", "Multiple Values");
}

FText FScalableFloatDetails::GetRowNameComboBoxContentTooltip() const
{
	FName RowName = GetRowName();

	if (RowName != NAME_None)
	{
		return FText::FromName(RowName);
	}

	return LOCTEXT("SelectCurveTooltip", "Select a Curve, this will be scaled using input level and then multiplied by Value");
}

FText FScalableFloatDetails::GetRowValuePreviewLabel() const
{
	FPropertyAccess::Result FoundResult;
	const FRealCurve* FoundCurve = GetRealCurve(&FoundResult);
	if (FoundCurve)
	{
		return FText::Format(LOCTEXT("LevelPreviewLabel", "Preview At {0}"), FText::AsNumber(PreviewLevel));
	}
	else if (FoundResult == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}
	else
	{
		return LOCTEXT("ErrorFindingCurve", "ERROR: Invalid Curve!");
	}
}

FText FScalableFloatDetails::GetRowValuePreviewText() const
{
	const FRealCurve* FoundCurve = GetRealCurve();
	if (FoundCurve)
	{
		float Value;
		if (ValueProperty->GetValue(Value) == FPropertyAccess::Success)
		{
			static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
				.SetMinimumFractionalDigits(3)
				.SetMaximumFractionalDigits(3);
			return FText::AsNumber(Value * FoundCurve->Eval(PreviewLevel), &FormatOptions);
		}
	}

	return FText::GetEmpty();
}

FName FScalableFloatDetails::GetRowName(FPropertyAccess::Result* OutResult) const
{
	FPropertyAccess::Result TempResult;
	if (OutResult == nullptr)
	{
		OutResult = &TempResult;
	}

	FName ReturnName;
	if (RowNameProperty.IsValid())
	{
		*OutResult = RowNameProperty->GetValue(ReturnName);
	}
	else
	{
		*OutResult = FPropertyAccess::Fail;
	}

	return ReturnName;
}

FDataRegistryId FScalableFloatDetails::GetRegistryId(FPropertyAccess::Result* OutResult) const
{
	FPropertyAccess::Result TempResult;
	if (OutResult == nullptr)
	{
		OutResult = &TempResult;
	}

	// Cache name result so we can return multiple values
	FPropertyAccess::Result NameResult;
	FName RowName = GetRowName(&NameResult);

	UCurveTable* CurveTable = GetCurveTable(OutResult);
	if (CurveTable)
	{
		// Curve tables are all valid but names may differ
		*OutResult = NameResult;

		// Use the fake custom type, options will get filled in by GetCustomRowNames
		return FDataRegistryId(FDataRegistryType::CustomContextType, RowName);
	}

	// This is a real registry, or is empty/invalid
	FDataRegistryType RegistryType = GetRegistryType(OutResult);

	if (*OutResult == FPropertyAccess::Success)
	{
		// Names may differ
		*OutResult = NameResult;
	}

	return FDataRegistryId(RegistryType, RowName);
}

void FScalableFloatDetails::SetRegistryId(FDataRegistryId NewId)
{
	// Always set row name, only set type if it's valid and different
	RowNameProperty->SetValue(NewId.ItemName);

	FDataRegistryType CurrentType = GetRegistryType();
	if (NewId.RegistryType != FDataRegistryType::CustomContextType && NewId.RegistryType != CurrentType)
	{
		RegistryTypeProperty->SetValueFromFormattedString(NewId.RegistryType.ToString());
	}
}

void FScalableFloatDetails::GetCustomRowNames(TArray<FName>& OutRows) const
{
	UCurveTable* CurveTable = GetCurveTable();

	if (CurveTable != nullptr)
	{
		for (TMap<FName, FRealCurve*>::TConstIterator Iterator(CurveTable->GetRowMap()); Iterator; ++Iterator)
		{
			OutRows.Add(Iterator.Key());
		}
	}
}

const FRealCurve* FScalableFloatDetails::GetRealCurve(FPropertyAccess::Result* OutResult) const
{
	FPropertyAccess::Result TempResult;
	if (OutResult == nullptr)
	{
		OutResult = &TempResult;
	}

	// First check curve table, abort if values differ
	UCurveTable* CurveTable = GetCurveTable(OutResult);
	if (*OutResult != FPropertyAccess::Success)
	{
		return nullptr;
	}

	FName RowName = GetRowName(OutResult);
	if (*OutResult != FPropertyAccess::Success)
	{
		return nullptr;
	}

	if (CurveTable && !RowName.IsNone())
	{
		return CurveTable->FindCurveUnchecked(RowName);
	}

	FDataRegistryId RegistryId = GetRegistryId(OutResult);
	if (RegistryId.IsValid())
	{
		// Now try registry, we will only get here if there are not multiple values
		UDataRegistry* Registry = UDataRegistrySubsystem::Get()->GetRegistryForType(RegistryId.RegistryType);
		if (Registry)
		{
			const FRealCurve* OutCurve = nullptr;
			if (Registry->GetCachedCurveRaw(OutCurve, RegistryId))
			{
				return OutCurve;
			}
		}
	}

	return nullptr;
}

bool FScalableFloatDetails::IsEditable() const
{
	return true;
}

void FScalableFloatDetails::UpdatePreviewLevels()
{
	if (const FRealCurve* FoundCurve = GetRealCurve())
	{
		FoundCurve->GetTimeRange(MinPreviewLevel, MaxPreviewLevel);
	}
	else
	{
		MinPreviewLevel = DefaultMinPreviewLevel;
		MaxPreviewLevel = DefaultMaxPreviewLevel;
	}

	PreviewLevel = FMath::Clamp(PreviewLevel, MinPreviewLevel, MaxPreviewLevel);
}

void FScalableFloatDetails::CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	
}

//-------------------------------------------------------------------------------------

#undef LOCTEXT_NAMESPACE
