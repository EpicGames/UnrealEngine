// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayAbilitiesBlueprintFactory.h"
#include "InputCoreTypes.h"
#include "UObject/Interface.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Abilities/GameplayAbility.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "EdGraphSchema_K2.h"

#include "ClassViewerModule.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "BlueprintEditorSettings.h"

#include "GameplayAbilityBlueprint.h"
#include "GameplayAbilityGraph.h"
#include "GameplayAbilityGraphSchema.h"

#include "ClassViewerFilter.h"

#include "SlateOptMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilitiesBlueprintFactory)

#define LOCTEXT_NAMESPACE "UGameplayAbilitiesBlueprintFactory"


// ------------------------------------------------------------------------------
// Dialog to configure creation properties
// ------------------------------------------------------------------------------
BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

class SGameplayAbilityBlueprintCreateDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGameplayAbilityBlueprintCreateDialog){}

	SLATE_END_ARGS()

		/** Constructs this widget with InArgs */
		void Construct(const FArguments& InArgs)
	{
			bOkClicked = false;
			ParentClass = UGameplayAbility::StaticClass();

			ChildSlot
				[
					SNew(SBorder)
					.Visibility(EVisibility::Visible)
					.BorderImage(FAppStyle::GetBrush("Menu.Background"))
					[
						SNew(SBox)
						.Visibility(EVisibility::Visible)
						.WidthOverride(500.0f)
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.FillHeight(1)
							[
								SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
								.Content()
								[
									SAssignNew(ParentClassContainer, SVerticalBox)
								]
							]

							// Ok/Cancel buttons
							+ SVerticalBox::Slot()
								.AutoHeight()
								.HAlign(HAlign_Right)
								.VAlign(VAlign_Bottom)
								.Padding(8)
								[
									SNew(SUniformGridPanel)
									.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
									.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
									.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
									+ SUniformGridPanel::Slot(0, 0)
									[
										SNew(SButton)
										.HAlign(HAlign_Center)
										.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
										.OnClicked(this, &SGameplayAbilityBlueprintCreateDialog::OkClicked)
										.Text(LOCTEXT("CreateGameplayAbilityBlueprintOk", "OK"))
									]
									+ SUniformGridPanel::Slot(1, 0)
										[
											SNew(SButton)
											.HAlign(HAlign_Center)
											.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
											.OnClicked(this, &SGameplayAbilityBlueprintCreateDialog::CancelClicked)
											.Text(LOCTEXT("CreateGameplayAbilityBlueprintCancel", "Cancel"))
										]
								]
						]
					]
				];

			MakeParentClassPicker();
		}

	/** Sets properties for the supplied GameplayAbilitiesBlueprintFactory */
	bool ConfigureProperties(TWeakObjectPtr<UGameplayAbilitiesBlueprintFactory> InGameplayAbilitiesBlueprintFactory)
	{
		GameplayAbilitiesBlueprintFactory = InGameplayAbilitiesBlueprintFactory;

		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(LOCTEXT("CreateGameplayAbilityBlueprintOptions", "Create Gameplay Ability Blueprint"))
			.ClientSize(FVector2D(400, 700))
			.SupportsMinimize(false).SupportsMaximize(false)
			[
				AsShared()
			];

		PickerWindow = Window;

		GEditor->EditorAddModalWindow(Window);
		GameplayAbilitiesBlueprintFactory.Reset();

		return bOkClicked;
	}

private:
	class FGameplayAbilityBlueprintParentFilter : public IClassViewerFilter
	{
	public:
		/** All children of these classes will be included unless filtered out by another setting. */
		TSet< const UClass* > AllowedChildrenOfClasses;

		FGameplayAbilityBlueprintParentFilter() {}

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			// If it appears on the allowed child-of classes list (or there is nothing on that list)
			return InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			// If it appears on the allowed child-of classes list (or there is nothing on that list)
			return InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
		}
	};

	/** Creates the combo menu for the parent class */
	void MakeParentClassPicker()
	{
		// Load the classviewer module to display a class picker
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

		// Fill in options
		FClassViewerInitializationOptions Options;
		Options.Mode = EClassViewerMode::ClassPicker;

		// Only allow parenting to base blueprints.
		Options.bIsBlueprintBaseOnly = true;

		TSharedPtr<FGameplayAbilityBlueprintParentFilter> Filter = MakeShareable(new FGameplayAbilityBlueprintParentFilter);

		// All child child classes of UGameplayAbility are valid.
		Filter->AllowedChildrenOfClasses.Add(UGameplayAbility::StaticClass());
		Options.ClassFilters.Add(Filter.ToSharedRef());

		ParentClassContainer->ClearChildren();
		ParentClassContainer->AddSlot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ParentClass", "Parent Class:"))
				.ShadowOffset(FVector2D(1.0f, 1.0f))
			];

		ParentClassContainer->AddSlot()
			[
				ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &SGameplayAbilityBlueprintCreateDialog::OnClassPicked))
			];
	}

	/** Handler for when a parent class is selected */
	void OnClassPicked(UClass* ChosenClass)
	{
		ParentClass = ChosenClass;
	}

	/** Handler for when ok is clicked */
	FReply OkClicked()
	{
		if (GameplayAbilitiesBlueprintFactory.IsValid())
		{
			GameplayAbilitiesBlueprintFactory->BlueprintType = BPTYPE_Normal;
			GameplayAbilitiesBlueprintFactory->ParentClass = ParentClass.Get();
		}

		CloseDialog(true);

		return FReply::Handled();
	}

	void CloseDialog(bool bWasPicked = false)
	{
		bOkClicked = bWasPicked;
		if (PickerWindow.IsValid())
		{
			PickerWindow.Pin()->RequestDestroyWindow();
		}
	}

	/** Handler for when cancel is clicked */
	FReply CancelClicked()
	{
		CloseDialog();
		return FReply::Handled();
	}

	FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			CloseDialog();
			return FReply::Handled();
		}
		return SWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

private:
	/** The factory for which we are setting up properties */
	TWeakObjectPtr<UGameplayAbilitiesBlueprintFactory> GameplayAbilitiesBlueprintFactory;

	/** A pointer to the window that is asking the user to select a parent class */
	TWeakPtr<SWindow> PickerWindow;

	/** The container for the Parent Class picker */
	TSharedPtr<SVerticalBox> ParentClassContainer;

	/** The selected class */
	TWeakObjectPtr<UClass> ParentClass;

	/** True if Ok was clicked */
	bool bOkClicked;
};

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

/*------------------------------------------------------------------------------
	UGameplayAbilitiesBlueprintFactory implementation.
------------------------------------------------------------------------------*/

UGameplayAbilitiesBlueprintFactory::UGameplayAbilitiesBlueprintFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UGameplayAbilityBlueprint::StaticClass();
	ParentClass = UGameplayAbility::StaticClass();
}

bool UGameplayAbilitiesBlueprintFactory::ConfigureProperties()
{
	TSharedRef<SGameplayAbilityBlueprintCreateDialog> Dialog = SNew(SGameplayAbilityBlueprintCreateDialog);
	return Dialog->ConfigureProperties(this);
};

UObject* UGameplayAbilitiesBlueprintFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a gameplay ability blueprint, then create and init one
	check(Class->IsChildOf(UGameplayAbilityBlueprint::StaticClass()));

	// If they selected an interface, force the parent class to be UInterface
	if (BlueprintType == BPTYPE_Interface)
	{
		ParentClass = UInterface::StaticClass();
	}

	if ( ( ParentClass == NULL ) || !FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass) || !ParentClass->IsChildOf(UGameplayAbility::StaticClass()) )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ClassName"), (ParentClass != NULL) ? FText::FromString( ParentClass->GetName() ) : LOCTEXT("Null", "(null)") );
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format( LOCTEXT("CannotCreateGameplayAbilityBlueprint", "Cannot create a Gameplay Ability Blueprint based on the class '{ClassName}'."), Args ) );
		return NULL;
	}
	else
	{
		UGameplayAbilityBlueprint* NewBP = CastChecked<UGameplayAbilityBlueprint>(FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BlueprintType, UGameplayAbilityBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), CallingContext));

		if (NewBP)
		{
			UGameplayAbilityBlueprint* AbilityBP = UGameplayAbilityBlueprint::FindRootGameplayAbilityBlueprint(NewBP);
			if (AbilityBP == NULL)
			{
				const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

				// Only allow a gameplay ability graph if there isn't one in a parent blueprint
				UEdGraph* NewGraph = FBlueprintEditorUtils::CreateNewGraph(NewBP, TEXT("Gameplay Ability Graph"), UGameplayAbilityGraph::StaticClass(), UGameplayAbilityGraphSchema::StaticClass());
#if WITH_EDITORONLY_DATA
				if (NewBP->UbergraphPages.Num())
				{
					FBlueprintEditorUtils::RemoveGraphs(NewBP, NewBP->UbergraphPages);
				}
#endif
				FBlueprintEditorUtils::AddUbergraphPage(NewBP, NewGraph);
				NewBP->LastEditedDocuments.Add(NewGraph);
				NewGraph->bAllowDeletion = false;

				UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
				if(Settings && Settings->bSpawnDefaultBlueprintNodes)
				{
					int32 NodePositionY = 0;
					FKismetEditorUtilities::AddDefaultEventNode(NewBP, NewGraph, FName(TEXT("K2_ActivateAbility")), UGameplayAbility::StaticClass(), NodePositionY);
					FKismetEditorUtilities::AddDefaultEventNode(NewBP, NewGraph, FName(TEXT("K2_OnEndAbility")), UGameplayAbility::StaticClass(), NodePositionY);
				}
			}
		}

		return NewBP;
	}
}

UObject* UGameplayAbilitiesBlueprintFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
}

#undef LOCTEXT_NAMESPACE

