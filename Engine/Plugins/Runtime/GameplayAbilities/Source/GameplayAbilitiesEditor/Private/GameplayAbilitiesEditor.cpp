// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayAbilitiesEditor.h"
#include "EditorReimportHandler.h"

#if WITH_EDITOR
#include "Editor.h"
#endif



#include "GameplayAbilityBlueprint.h"
#include "GameplayAbilityGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "FGameplayAbilitiesEditor"


/////////////////////////////////////////////////////
// FGameplayAbilitiesEditor

FGameplayAbilitiesEditor::FGameplayAbilitiesEditor()
{
	// todo: Do we need to register a callback for when properties are changed?
}

FGameplayAbilitiesEditor::~FGameplayAbilitiesEditor()
{
	// NOTE: Any tabs that we still have hanging out when destroyed will be cleaned up by FBaseToolkit's destructor
}

void FGameplayAbilitiesEditor::InitGameplayAbilitiesEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode)
{
	InitBlueprintEditor(Mode, InitToolkitHost, InBlueprints, bShouldOpenInDefaultsMode);

	for (auto Blueprint : InBlueprints)
	{
		EnsureGameplayAbilityBlueprintIsUpToDate(Blueprint);
	}
}

void FGameplayAbilitiesEditor::EnsureGameplayAbilityBlueprintIsUpToDate(UBlueprint* Blueprint)
{
#if WITH_EDITORONLY_DATA
	for (UEdGraph* Graph : Blueprint->UbergraphPages)
	{
		// remove the default event graph, if it exists, from existing Gameplay Ability Blueprints
		if (Graph->GetName() == "EventGraph" && Graph->Nodes.Num() == 0)
		{
			check(!Graph->Schema->GetClass()->IsChildOf(UGameplayAbilityGraphSchema::StaticClass()));
			FBlueprintEditorUtils::RemoveGraph(Blueprint, Graph);
			break;
		}
	}
#endif
}

// FRED_TODO: don't merge this back
// FName FGameplayAbilitiesEditor::GetToolkitContextFName() const
// {
// 	return GetToolkitFName();
// }

FName FGameplayAbilitiesEditor::GetToolkitFName() const
{
	return FName("GameplayAbilitiesEditor");
}

FText FGameplayAbilitiesEditor::GetBaseToolkitName() const
{
	return LOCTEXT("GameplayAbilitiesEditorAppLabel", "Gameplay Abilities Editor");
}

FText FGameplayAbilitiesEditor::GetToolkitName() const
{
	const auto& EditingObjs = GetEditingObjects();

	check(EditingObjs.Num() > 0);

	FFormatNamedArguments Args;

	const UObject* EditingObject = EditingObjs[0];

	Args.Add(TEXT("ObjectName"), FText::FromString(EditingObject->GetName()));
	return FText::Format(LOCTEXT("GameplayAbilitiesToolkitName", "{ObjectName}"), Args);
}

FText FGameplayAbilitiesEditor::GetToolkitToolTipText() const
{
	const UObject* EditingObject = GetEditingObject();

	check (EditingObject != NULL);

	return FAssetEditorToolkit::GetToolTipTextForObject(EditingObject);
}

FString FGameplayAbilitiesEditor::GetWorldCentricTabPrefix() const
{
	return TEXT("GameplayAbilitiesEditor");
}

FLinearColor FGameplayAbilitiesEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

UBlueprint* FGameplayAbilitiesEditor::GetBlueprintObj() const
{
	const TArray<UObject*>& EditingObjs = GetEditingObjects();
	for (int32 i = 0; i < EditingObjs.Num(); ++i)
	{
		if (EditingObjs[i]->IsA<UGameplayAbilityBlueprint>()) 
		{ 
			return (UBlueprint*)EditingObjs[i]; 
		}
	}
	return nullptr;
}

FString FGameplayAbilitiesEditor::GetDocumentationLink() const
{
	return FBlueprintEditor::GetDocumentationLink(); // todo: point this at the correct documentation
}

#undef LOCTEXT_NAMESPACE
