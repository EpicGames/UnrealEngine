// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayAbilitiesEditorModule.h"
#include "Stats/StatsMisc.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Textures/SlateIcon.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "HAL/IConsoleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

#include "PropertyEditorModule.h"
#include "AttributeDetails.h"
#include "GameplayEffectTypes.h"
#include "GameplayEffect.h"
#include "Misc/FeedbackContext.h"
#include "GameplayAbilitiesModule.h"
#include "GameplayTagsManager.h"
#include "GameplayTagsModule.h"
#include "AbilitySystemGlobals.h"
#include "GameplayEffectDetails.h"
#include "GameplayEffectExecutionScopedModifierInfoDetails.h"
#include "GameplayTagBlueprintPropertyMappingDetails.h"
#include "GameplayEffectExecutionDefinitionDetails.h"
#include "GameplayEffectModifierMagnitudeDetails.h"
#include "GameplayModEvaluationChannelSettingsDetails.h"
#include "AttributeBasedFloatDetails.h"

#include "AssetToolsModule.h"
#include "IAssetTypeActions.h"
#include "EdGraphUtilities.h"
#include "GameplayAbilitiesGraphPanelPinFactory.h"
#include "GameplayCueTagDetails.h"

#include "BlueprintActionDatabase.h"
#include "K2Node_GameplayCueEvent.h"

#include "SGameplayCueEditor.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Widgets/Docking/SDockTab.h"

#include "LevelEditor.h"
#include "Misc/HotReloadInterface.h"
#include "EditorReimportHandler.h"
#include "GameplayEffectCreationMenu.h"
#include "ISettingsModule.h"

#include "ISequencerModule.h"
#include "Sequencer/MovieSceneGameplayCueSections.h"
#include "Sequencer/GameplayCueTrackEditor.h"
#include "SequencerChannelInterface.h"


class FGameplayAbilitiesEditorModule : public IGameplayAbilitiesEditorModule
{
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface

	TSharedRef<SDockTab> SpawnGameplayCueEditorTab(const FSpawnTabArgs& Args);
	TSharedPtr<SWidget> SummonGameplayCueEditorUI();

	FGetGameplayCueNotifyClasses& GetGameplayCueNotifyClassesDelegate() override
	{
		return GetGameplayCueNotifyClasses;
	}

	FGetGameplayCuePath& GetGameplayCueNotifyPathDelegate()
	{
		return GetGameplayCueNotifyPath;
	}

	FGetGameplayCueInterfaceClasses& GetGameplayCueInterfaceClassesDelegate()
	{
		return GetGameplayCueInterfaceClasses;

	}

	FGetGameplayCueEditorStrings& GetGameplayCueEditorStringsDelegate()
	{
		return GetGameplayCueEditorStrings;
	}

protected:
	static void GameplayTagTreeChanged();

private:

	/** Helper function to apply the gameplay mod evaluation channel aliases as display name data to the enum */
	void ApplyGameplayModEvaluationChannelAliasesToEnumMetadata();

	/** Pin factory for abilities graph; Cached so it can be unregistered */
	TSharedPtr<FGameplayAbilitiesGraphPanelPinFactory> GameplayAbilitiesGraphPanelPinFactory;

	/** Handle to the registered GameplayTagTreeChanged delegate */
	FDelegateHandle GameplayTagTreeChangedDelegateHandle;

	FDelegateHandle TrackEditorHandle;

	FGetGameplayCueNotifyClasses GetGameplayCueNotifyClasses;

	FGetGameplayCuePath GetGameplayCueNotifyPath;

	FGetGameplayCueInterfaceClasses GetGameplayCueInterfaceClasses;
	
	FGetGameplayCueEditorStrings GetGameplayCueEditorStrings;

	TWeakPtr<SDockTab> GameplayCueEditorTab;

	TWeakPtr<SGameplayCueEditor> GameplayCueEditor;

public:
	void HandleNotify_OpenAssetInEditor(FString AssetName, int AssetType);
	void HandleNotify_FindAssetInEditor(FString AssetName, int AssetType);
	static void RegisterDebuggingCallbacks();
	


};

/** The style set for the Gameplay Abilities plugin. Any custom icons or color themes can go in here. */
class FGameplayAbilitiesEditorStyleSet final : public FSlateStyleSet
{
public:

	FGameplayAbilitiesEditorStyleSet()
		: FSlateStyleSet("GameplayAbilitiesEditor")
	{
		SetParentStyleName(FAppStyle::GetAppStyleSetName());

		// The icons are located in \Engine\Plugins\Runtime\GameplayAbilities\Content\Editor\Slate\Icons
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Runtime/GameplayAbilities/Content/Editor/Slate"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		// Gameplay Abilities Editor icons
		static const FVector2D Icon16 = FVector2D(16.0, 16.0);
		static const FVector2D Icon64 = FVector2D(64.0, 64.0);

		Set("GameplayCueEditor_Small", new IMAGE_BRUSH_SVG("Icons/GameplayCueEditor_16", Icon16));
		Set("GameplayCueEditor_Large", new IMAGE_BRUSH_SVG("Icons/GameplayCueEditor_64", Icon64));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	~FGameplayAbilitiesEditorStyleSet()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	static FGameplayAbilitiesEditorStyleSet& Get()
	{
		static FGameplayAbilitiesEditorStyleSet Inst;
		return Inst;
	}
};

IMPLEMENT_MODULE(FGameplayAbilitiesEditorModule, GameplayAbilitiesEditor)

void FGameplayAbilitiesEditorModule::StartupModule()
{
	// Register the details customizer
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout( "GameplayAttribute", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FAttributePropertyDetails::MakeInstance ) );
	PropertyModule.RegisterCustomPropertyTypeLayout( "ScalableFloat", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FScalableFloatDetails::MakeInstance ) );
	PropertyModule.RegisterCustomPropertyTypeLayout( "GameplayEffectExecutionScopedModifierInfo", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FGameplayEffectExecutionScopedModifierInfoDetails::MakeInstance ) );
	PropertyModule.RegisterCustomPropertyTypeLayout( "GameplayTagBlueprintPropertyMapping", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FGameplayTagBlueprintPropertyMappingDetails::MakeInstance ) );
	PropertyModule.RegisterCustomPropertyTypeLayout( "GameplayEffectExecutionDefinition", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FGameplayEffectExecutionDefinitionDetails::MakeInstance ) );
	PropertyModule.RegisterCustomPropertyTypeLayout( "GameplayEffectModifierMagnitude", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FGameplayEffectModifierMagnitudeDetails::MakeInstance ) );
	PropertyModule.RegisterCustomPropertyTypeLayout( "GameplayCueTag", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FGameplayCueTagDetails::MakeInstance ) );
	PropertyModule.RegisterCustomPropertyTypeLayout( "GameplayModEvaluationChannelSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FGameplayModEvaluationChannelSettingsDetails::MakeInstance ) );
	PropertyModule.RegisterCustomPropertyTypeLayout( "AttributeBasedFloat", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FAttributeBasedFloatDetails::MakeInstance ) );

	PropertyModule.RegisterCustomClassLayout( "AttributeSet", FOnGetDetailCustomizationInstance::CreateStatic( &FAttributeDetails::MakeInstance ) );
	PropertyModule.RegisterCustomClassLayout( "GameplayEffect", FOnGetDetailCustomizationInstance::CreateStatic( &FGameplayEffectDetails::MakeInstance ) );

	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings("Project", "Project", "Gameplay Effect Parents",
						NSLOCTEXT("GameplayAbilitiesEditorModule", "GameplayEffectParentName", "Gameplay Effect Parents"),
						NSLOCTEXT("GameplayAbilitiesEditorModule", "GameplayEffectParentNameDesc", "Data Driven way of specifying common parent Gameplay Effect classes that are accessible through File menu"),
						GetMutableDefault<UGameplayEffectCreationMenu>()
						);

		GetDefault<UGameplayEffectCreationMenu>()->AddMenuExtensions();
	}

	// Register factories for pins and nodes
	GameplayAbilitiesGraphPanelPinFactory = MakeShareable(new FGameplayAbilitiesGraphPanelPinFactory());
	FEdGraphUtilities::RegisterVisualPinFactory(GameplayAbilitiesGraphPanelPinFactory);

	// Listen for changes to the gameplay tag tree so we can refresh blueprint actions for the GameplayCueEvent node
	UGameplayTagsManager& GameplayTagsManager = UGameplayTagsManager::Get();
	GameplayTagTreeChangedDelegateHandle = IGameplayTagsModule::OnGameplayTagTreeChanged.AddStatic(&FGameplayAbilitiesEditorModule::GameplayTagTreeChanged);
	
	// GameplayCue editor
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner( FName(TEXT("GameplayCueApp")), FOnSpawnTab::CreateRaw(this, &FGameplayAbilitiesEditorModule::SpawnGameplayCueEditorTab))
		.SetDisplayName(NSLOCTEXT("GameplayAbilitiesEditorModule", "GameplayCueTabTitle", "GameplayCue Editor"))
		.SetTooltipText(NSLOCTEXT("GameplayAbilitiesEditorModule", "GameplayCueTooltipText", "Open GameplayCue Editor tab."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetIcon(FSlateIcon(FGameplayAbilitiesEditorStyleSet::Get().GetStyleSetName(), "GameplayCueEditor_Small"));
		
	ApplyGameplayModEvaluationChannelAliasesToEnumMetadata();

#if WITH_RELOAD
	// This code attempts to relaunch the GameplayCueEditor tab when you hotreload this module
	if (IsReloadActive() && FSlateApplication::IsInitialized())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
		LevelEditorTabManager->TryInvokeTab(FName("GameplayCueApp"));
	}
#endif // WITH_RELOAD

	IGameplayAbilitiesModule::Get().CallOrRegister_OnAbilitySystemGlobalsReady(FSimpleMulticastDelegate::FDelegate::CreateLambda([] {
		FGameplayAbilitiesEditorModule::RegisterDebuggingCallbacks();
	}));
	
	// Invalidate all internal cacheing of FRichCurve* in FScalableFlaots when a UCurveTable is reimported
	FReimportManager::Instance()->OnPostReimport().AddLambda([](UObject* InObject, bool b){ UCurveTable::InvalidateAllCachedCurves(); });

	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	TrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(FGameplayCueTrackEditor::CreateTrackEditor));
	SequencerModule.RegisterChannelInterface<FMovieSceneGameplayCueChannel>();
}

void FGameplayAbilitiesEditorModule::HandleNotify_OpenAssetInEditor(FString AssetName, int AssetType)
{
	//Open the GameplayCue editor if it hasn't been opened.
	if (AssetType == 0)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
		LevelEditorTabManager->TryInvokeTab(FName("GameplayCueApp"));
	}

	//UE_LOGF(LogTemp, Display, "HandleNotify_OpenAssetInEditor!!! %ls %d", *AssetName, AssetType);
	if (GameplayCueEditor.IsValid())
	{
		GameplayCueEditor.Pin()->HandleNotify_OpenAssetInEditor(AssetName, AssetType);
	}
}

void FGameplayAbilitiesEditorModule::HandleNotify_FindAssetInEditor(FString AssetName, int AssetType)
{
	//Find the GameplayCue editor if it hasn't been found.
	if (AssetType == 0)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
		LevelEditorTabManager->TryInvokeTab(FName("GameplayCueApp"));
	}

	//UE_LOGF(LogTemp, Display, "HandleNotify_FindAssetInEditor!!! %ls %d", *AssetName, AssetType);
	if (GameplayCueEditor.IsValid())
	{
		GameplayCueEditor.Pin()->HandleNotify_FindAssetInEditor(AssetName, AssetType);
	}
}

void FGameplayAbilitiesEditorModule::RegisterDebuggingCallbacks()
{
	//register callbacks when Assets are requested to open from the game.
	UAbilitySystemGlobals::Get().AbilityOpenAssetInEditorCallbacks.AddLambda([](FString AssetName, int AssetType) {
		((FGameplayAbilitiesEditorModule *)&IGameplayAbilitiesEditorModule::Get())->HandleNotify_OpenAssetInEditor(AssetName, AssetType);
	});

	UAbilitySystemGlobals::Get().AbilityFindAssetInEditorCallbacks.AddLambda([](FString AssetName, int AssetType) {
		((FGameplayAbilitiesEditorModule *)&IGameplayAbilitiesEditorModule::Get())->HandleNotify_FindAssetInEditor(AssetName, AssetType);
	});
}

void FGameplayAbilitiesEditorModule::GameplayTagTreeChanged()
{
	if (FBlueprintActionDatabase* BAD = FBlueprintActionDatabase::TryGet())
	{
		BAD->RefreshClassActions(UK2Node_GameplayCueEvent::StaticClass());
	}
}

void FGameplayAbilitiesEditorModule::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner( FName(TEXT("GameplayCueApp")) );

		if (GameplayCueEditorTab.IsValid())
		{
			GameplayCueEditorTab.Pin()->RequestCloseTab();
		}
	}

	// Unregister customizations
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout("GameplayEffect");
		PropertyModule.UnregisterCustomClassLayout("AttributeSet");

		PropertyModule.UnregisterCustomPropertyTypeLayout("AttributeBasedFloat");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GameplayModEvaluationChannelSettings");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GameplayCueTag");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GameplayEffectModifierMagnitude");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GameplayEffectExecutionDefinition");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GameplayEffectExecutionScopedModifierInfo");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GameplayTagBlueprintPropertyMapping");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ScalableFloat");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GameplayAttribute");
	}

	// Unregister graph factories
	if (GameplayAbilitiesGraphPanelPinFactory.IsValid())
	{
		FEdGraphUtilities::UnregisterVisualPinFactory(GameplayAbilitiesGraphPanelPinFactory);
		GameplayAbilitiesGraphPanelPinFactory.Reset();
	}

	if ( UObjectInitialized() && IGameplayTagsModule::IsAvailable() )
	{
		IGameplayTagsModule::OnGameplayTagTreeChanged.Remove(GameplayTagTreeChangedDelegateHandle);
	}

	if (ISequencerModule* SequencerModulePtr = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer"))
	{
		SequencerModulePtr->UnRegisterTrackEditor(TrackEditorHandle);
	}
}


TSharedRef<SDockTab> FGameplayAbilitiesEditorModule::SpawnGameplayCueEditorTab(const FSpawnTabArgs& Args)
{
	return SAssignNew(GameplayCueEditorTab, SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SummonGameplayCueEditorUI().ToSharedRef()
		];
}


TSharedPtr<SWidget> FGameplayAbilitiesEditorModule::SummonGameplayCueEditorUI()
{
	TSharedPtr<SWidget> ReturnWidget;
	if( IsInGameThread() )
	{
		TSharedPtr<SGameplayCueEditor> SharedPtr = SNew(SGameplayCueEditor);
		ReturnWidget = SharedPtr;
		GameplayCueEditor = SharedPtr;
	}
	return ReturnWidget;

}

void RecompileGameplayAbilitiesEditor(const TArray<FString>& Args)
{
	GWarn->BeginSlowTask( NSLOCTEXT("GameplayAbilities", "BeginRecompileGameplayAbilitiesTask", "Recompiling GameplayAbilitiesEditor Module..."), true);
	
	IHotReloadInterface* HotReload = IHotReloadInterface::GetPtr();
	if(HotReload != nullptr)
	{
		TArray< UPackage* > PackagesToRebind;
		UPackage* Package = FindPackage( NULL, TEXT("/Script/GameplayAbilitiesEditor"));
		if( Package != NULL )
		{
			PackagesToRebind.Add( Package );
		}

		HotReload->RebindPackages(PackagesToRebind, EHotReloadFlags::WaitForCompletion, *GLog);
	}

	GWarn->EndSlowTask();
}

FAutoConsoleCommand RecompileGameplayAbilitiesEditorCommand(
	TEXT("GameplayAbilitiesEditor.HotReload"),
	TEXT("Recompiles the gameplay abilities editor module"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&RecompileGameplayAbilitiesEditor)
	);
	
void FGameplayAbilitiesEditorModule::ApplyGameplayModEvaluationChannelAliasesToEnumMetadata()
{
	UAbilitySystemGlobals* AbilitySystemGlobalsCDO = UAbilitySystemGlobals::StaticClass()->GetDefaultObject<UAbilitySystemGlobals>();
	const UEnum* EvalChannelEnum = StaticEnum<EGameplayModEvaluationChannel>();
	if (ensure(EvalChannelEnum) && ensure(AbilitySystemGlobalsCDO))
	{
		const TCHAR* DisplayNameMeta = TEXT("DisplayName");
		const TCHAR* HiddenMeta = TEXT("Hidden");
		const TCHAR* UnusedMeta = TEXT("Unused");

		const int32 NumEnumValues = EvalChannelEnum->NumEnums();
			
		// First mark all of the enum values hidden and unused
		for (int32 EnumValIdx = 0; EnumValIdx < NumEnumValues; ++EnumValIdx)
		{
			EvalChannelEnum->SetMetaData(HiddenMeta, TEXT(""), EnumValIdx);
			EvalChannelEnum->SetMetaData(DisplayNameMeta, UnusedMeta, EnumValIdx);
		}

		// If allowed to use channels, mark the valid ones with aliases
		if (AbilitySystemGlobalsCDO->ShouldAllowGameplayModEvaluationChannels())
		{
			const int32 MaxChannelVal = static_cast<int32>(EGameplayModEvaluationChannel::Channel_MAX);
			for (int32 AliasIdx = 0; AliasIdx < MaxChannelVal; ++AliasIdx)
			{
				const FName& Alias = AbilitySystemGlobalsCDO->GetGameplayModEvaluationChannelAlias(AliasIdx);
				if (!Alias.IsNone())
				{
					EvalChannelEnum->RemoveMetaData(HiddenMeta, AliasIdx);
					EvalChannelEnum->SetMetaData(DisplayNameMeta, *Alias.ToString(), AliasIdx);
				}
			}
		}
		else
		{
			// If not allowed to use channels, also hide the "Evaluate up to channel" option 
			const UEnum* AttributeBasedFloatCalculationTypeEnum = StaticEnum<EAttributeBasedFloatCalculationType>();
			if (ensure(AttributeBasedFloatCalculationTypeEnum))
			{
				const int32 ChannelBasedCalcIdx = AttributeBasedFloatCalculationTypeEnum->GetIndexByValue(static_cast<int64>(EAttributeBasedFloatCalculationType::AttributeMagnitudeEvaluatedUpToChannel));
				AttributeBasedFloatCalculationTypeEnum->SetMetaData(HiddenMeta, TEXT(""), ChannelBasedCalcIdx);
			}
		}
	}
}
