// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "AttributeSet.h"
#include "EdGraphUtilities.h"
#include "EdGraphSchema_K2.h"
#include "SGraphPin.h"
#include "SGameplayAttributeGraphPin.h"

class FGameplayAbilitiesGraphPanelPinFactory: public FGraphPanelPinFactory
{
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override
	{
		if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && InPin->PinType.PinSubCategoryObject == FGameplayAttribute::StaticStruct())
		{
			return SNew(SGameplayAttributeGraphPin, InPin);
		}
		return NULL;
	}
};
