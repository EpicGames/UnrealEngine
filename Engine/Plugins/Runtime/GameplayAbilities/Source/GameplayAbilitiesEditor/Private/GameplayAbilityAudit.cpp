// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayAbilityAudit.h"

#include "Abilities/GameplayAbility.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "GameplayAbilityBlueprint.h"

#include "K2Node_CallFunction.h"
#include "K2Node_BaseAsyncTask.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_VariableSet.h"

#include "Factories/DataTableFactory.h"

#include "IAssetTools.h"
#include "ContentBrowserMenuContexts.h"
#include "IContentBrowserSingleton.h"
#include "Textures/SlateIcon.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityAudit)

#define LOCTEXT_NAMESPACE "GameplayAbilityAudit"
DEFINE_LOG_CATEGORY_STATIC(LogGameplayAbilityAudit, Log, Log);

/** Public helper functions that may be useful in your own implementation */
namespace GameplayAbilityAudit
{
	/**
	 * Gather all of the Graphs from a Blueprint but include the Macro graphs as well (treat the Macro as an expanded version of the Graph)
	 */
	TArray<UEdGraph*> GatherAllGraphsIncludingMacros(const UBlueprint& LoadedInstance)
	{
		TArray<UEdGraph*> AllGraphs;
		LoadedInstance.GetAllGraphs(AllGraphs);

		// Treat Macros as if they belonged to our class...
		for (int Index = 0; Index < AllGraphs.Num(); ++Index)
		{
			UEdGraph* Graph = AllGraphs[Index];

			TArray<UK2Node_MacroInstance*> Macros;
			Graph->GetNodesOfClass(Macros);
			for (const UK2Node_MacroInstance* Macro : Macros)
			{
				UEdGraph* MacroGraph = Macro->GetMacroGraph();
				if (MacroGraph)
				{
					AllGraphs.AddUnique(MacroGraph);
				}
			}
		}

		return AllGraphs;
	}
} // namespace GameplayAbilityAudit

/** Base implementation for auditing Gameplay Abilities */
void FGameplayAbilityAuditRow::FillDataFromGameplayAbility(const UGameplayAbility& GameplayAbility)
{
	const FName NAME_ShouldAbilityRespondToEvent = FName(TEXT("K2_ShouldAbilityRespondToEvent"));
	const FName NAME_ActivateAbility = FName(TEXT("K2_ActivateAbility"));
	const FName NAME_ActivateAbilityFromEvent = FName(TEXT("K2_ActivateAbilityFromEvent"));
	const FName NAME_CanActivateAbility = FName(TEXT("K2_CanActivateAbility"));

	FGameplayAbilityAuditRow& AuditRow = *this;

	// Get all of the data from the Gameplay Ability
	auto ImplementedInBlueprint = [](const UFunction* Func) -> bool
	{
		return Func && ensure(Func->GetOuter()) && Func->GetOuter()->IsA(UBlueprintGeneratedClass::StaticClass());
	};

	AuditRow.bOverridesShouldAbilityRespondToEvent = ImplementedInBlueprint(GameplayAbility.FindFunction(NAME_ShouldAbilityRespondToEvent));
	AuditRow.bOverridesCanActivate = ImplementedInBlueprint(GameplayAbility.FindFunction(NAME_CanActivateAbility));
	if (ImplementedInBlueprint(GameplayAbility.FindFunction(NAME_ActivateAbility)))
	{
		AuditRow.ActivationPath = EGameplayAbilityActivationPath::Blueprint;
	}
	else if (ImplementedInBlueprint(GameplayAbility.FindFunction(NAME_ActivateAbilityFromEvent)))
	{
		AuditRow.ActivationPath = EGameplayAbilityActivationPath::FromEvent;
	}

	if (const UGameplayEffect* TheCostGE = GameplayAbility.GetCostGameplayEffect())
	{
		AuditRow.CostGE = TheCostGE->GetClass()->GetFName();
	}
	if (const UGameplayEffect* TheCooldownGE = GameplayAbility.GetCooldownGameplayEffect())
	{
		AuditRow.CooldownGE = TheCooldownGE->GetClass()->GetFName();
	}

	// Gather all of the other GameplayTagContainer referenced Tags (will also include AssetTags).
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FProperty* AbilityTagsProperty = UGameplayAbility::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UGameplayAbility, AbilityTags));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	for (TPropertyValueIterator<FStructProperty> It(GameplayAbility.GetClass(), &GameplayAbility); It; ++It)
	{
		const bool bIsTagContainer = It.Key()->SameType(AbilityTagsProperty);
		if (!bIsTagContainer)
		{
			continue;
		}

		if (const FGameplayTagContainer* TagContainer = reinterpret_cast<const FGameplayTagContainer*>(It.Value()))
		{
			for (const FGameplayTag& GameplayTag : *TagContainer)
			{
				AuditRow.ReferencedTags.Emplace(GameplayTag.GetTagName());
			}
		}
	}

	// Asset Tags should go in its own field because of its importance
	for (const FGameplayTag& GameplayTag : GameplayAbility.GetAssetTags())
	{
		AuditRow.AssetTags.Emplace(GameplayTag.GetTagName());
	}

	AuditRow.InstancingPolicy = GameplayAbility.GetInstancingPolicy();
	AuditRow.NetExecutionPolicy = GameplayAbility.GetNetExecutionPolicy();
	AuditRow.NetSecurityPolicy = GameplayAbility.GetNetSecurityPolicy();
	AuditRow.ReplicationPolicy = GameplayAbility.GetReplicationPolicy();
}

/**
 * The implementation that fills a data row with the Gameplay Ability information
 */
void FGameplayAbilityAuditRow::FillDataFromGameplayAbilityBlueprint(const UBlueprint& GameplayAbilityBlueprint)
{
	const FName NAME_CheckCost = FName(TEXT("K2_CheckAbilityCost"));
	const FName NAME_CommitAbility = FName(TEXT("K2_CommitAbility"));
	const FName NAME_EndAbility = FName(TEXT("K2_EndAbility"));
	const FName NAME_EndAbilityLocally = FName(TEXT("K2_EndAbilityLocally"));

	FGameplayAbilityAuditRow& AuditRow = *this;

	// Get all of the graphs that this Gameplay Ability can execute
	TArray<UEdGraph*> AllGraphs = GameplayAbilityAudit::GatherAllGraphsIncludingMacros(GameplayAbilityBlueprint);

	// Now that we have "all of the graphs" (including the macro graphs), let's gather the data
	TArray<UK2Node_CallFunction*> CallFunctionNodes;
	TArray<UK2Node_BaseAsyncTask*> AsyncNodes;
	TArray<UK2Node_VariableSet*> SetVariableNodes;
	for (const UEdGraph* Graph : AllGraphs)
	{
		Graph->GetNodesOfClass<UK2Node_CallFunction>(CallFunctionNodes);
		Graph->GetNodesOfClass<UK2Node_BaseAsyncTask>(AsyncNodes);
		Graph->GetNodesOfClass<UK2Node_VariableSet>(SetVariableNodes);
	}

	// Gather Functions and keep track of some special ones we want to know about
	bool bHasCommitAbility = false;
	bool bHasCheckCost = false;
	TArray<FName> FunctionNames;
	for (const UK2Node_CallFunction* FunctionNode : CallFunctionNodes)
	{
		if (FunctionNode->IsNodePure())
		{
			continue;
		}

		const FName FunctionName = FunctionNode->GetFunctionName();
		AuditRow.bChecksCostManually = AuditRow.bChecksCostManually || (FunctionName == NAME_CheckCost);
		AuditRow.bCommitAbility = AuditRow.bCommitAbility || (FunctionName == NAME_CommitAbility);

		if (FunctionName == NAME_EndAbilityLocally)
		{
			AuditRow.EndAbility |= EGameplayAbilityEndInBlueprints::EndAbilityLocally;
		}
		else if (FunctionName == NAME_EndAbility)
		{
			AuditRow.EndAbility |= EGameplayAbilityEndInBlueprints::EndAbility;
		}
		else
		{
			AuditRow.Functions.AddUnique(FunctionName);
		}
	}

	// List of Async Tasks
	for (const UK2Node_BaseAsyncTask* AsyncNode : AsyncNodes)
	{
		const FText NodeTitle = AsyncNode->GetNodeTitle(ENodeTitleType::ListView);
		AuditRow.AsyncTasks.AddUnique(NodeTitle.ToString());
	}

	// List of mutated variables (means the ability needs to be instanced)
	for (const UK2Node_VariableSet* SetVarNode : SetVariableNodes)
	{
		const FName VarName = SetVarNode->GetVarName();
		AuditRow.MutatedVariables.AddUnique(VarName);
	}
}

/** These are private implementation functions used to hook into the Editorand get the Audit Abilities menu setup */
namespace MenuExtension_GameplayAbilityBlueprintAudit
{
	/** Create a new DataTable Asset for the Gameplay Abilities Audit.  It's the caller's responsibility to give it a RowStruct */
	static UDataTable* CreateAssetForDataTable()
	{
		const FString PackagePathSuggestion = FString(TEXT("/Game/GameplayAbilityAudit"));

		FString PackageName, Name;
		IAssetTools::Get().CreateUniqueAssetName(PackagePathSuggestion, TEXT(""), PackageName, Name);
		const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

		UDataTable* NewTable = Cast<UDataTable>(IAssetTools::Get().CreateAsset(Name, PackagePath, UDataTable::StaticClass(), nullptr));
		if (!NewTable)
		{
			UE_LOGF(LogGameplayAbilityAudit, Error, "Could not create %ls/%ls", *PackageName, *Name);
		}

		return NewTable;
	}

	/**
	 * This is the main function that performs the "audit" logic (gathers data for the DataTable and creates it)
	 */
	void ExecuteActionGameplayAbilityAudit(UScriptStruct& RowStruct, const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		// Create the DataTable to gather the data into
		UDataTable* DataTable = CreateAssetForDataTable();
		DataTable->RowStruct = &RowStruct;

		// Construct the memory for the passed-in row struct.  We know it's derived from FGameplayAbilityAuditRow but not which struct.
		TUniquePtr<uint8[]> NewRawRowData{ new uint8[RowStruct.GetStructureSize()] };
		FGameplayAbilityAuditRow* AuditRow = reinterpret_cast<FGameplayAbilityAuditRow*>(NewRawRowData.Get());
		check(AuditRow);

		// For each selected Blueprint Object
		for (UBlueprint* LoadedInstance : Context->LoadSelectedObjects<UBlueprint>())
		{
			if (!LoadedInstance)
			{
				UE_LOGF(LogGameplayAbilityAudit, Error, "LoadSelectedObject failed on a Selected UBlueprint Instance.  This should not be possible.");
				UE_DEBUG_BREAK();
				continue;
			}

			// Make sure we zero out this struct so none of the old values are present
			RowStruct.InitializeStruct(AuditRow);

			// We should only deal with Gameplay Ability Blueprints (we may have multi-selected other assets)
			if (const UGameplayAbility* GameplayAbility = LoadedInstance->GeneratedClass ? Cast<UGameplayAbility>(LoadedInstance->GeneratedClass->GetDefaultObject()) : nullptr)
			{
				AuditRow->FillDataFromGameplayAbilityBlueprint(*LoadedInstance);
				AuditRow->FillDataFromGameplayAbility(*GameplayAbility);

				DataTable->AddRow(LoadedInstance->GetFName(), *AuditRow);
			}
		}

		// Sync the content browser to the location of the DataTable we created
		IContentBrowserSingleton::Get().SyncBrowserToAssets(TArray<UObject*>{ DataTable });
	}

	/** Go through all structs derived from FGameplayAbilityAuditRow and score them based on what we think we'll need for potential audits (the most derived wins) */
	int GetValidRowMatchScore(const UScriptStruct& Struct)
	{
		const UScriptStruct* AuditRowStruct = FGameplayAbilityAuditRow::StaticStruct();
		int InheritanceDepth = -1;

		// If a child of the table row struct base, but not itself
		const bool bBasedOnAuditRow = AuditRowStruct && Struct.IsChildOf(AuditRowStruct);
		const bool bValidStruct = bBasedOnAuditRow && (Struct.GetOutermost() != GetTransientPackage());
		if (bValidStruct)
		{
			// We are just saying the deeper the inheritance, the better the match
			// This heuristic basically means if you've derived from FGameplayAbilityAuditRow, you're a better match
			// However, two derived classes are a toss-up.
			// The reasoning is we're assuming you're multi-selecting a ton of GameplayAbilities, but the results must all share the same row structure.
			const UStruct* CurrentStruct = &Struct;
			while (CurrentStruct)
			{
				CurrentStruct = CurrentStruct->GetSuperStruct();
				++InheritanceDepth;
			}
		}

		return InheritanceDepth;
	}

	/** Find the best struct derived from FGameplayAbilityAuditRow that we will use to audit all of our Gameplay Ability Blueprints */
	UScriptStruct& FindBestAuditRowStruct()
	{
		UScriptStruct* BestAuditRowStruct = nullptr;
		int BestScore = -1;

		for (TObjectIterator<UScriptStruct> It; It; ++It)
		{
			UScriptStruct* Struct = *It;
			const int Score = Struct ? GetValidRowMatchScore(*Struct) : -1;
			if (Score > BestScore)
			{
				BestAuditRowStruct = Struct;
				BestScore = Score;
			}
		}

		// Couldn't find one?  Super odd, we should at least end up with FGameplayAbilityAuditRow
		if (!BestAuditRowStruct)
		{
			BestAuditRowStruct = FGameplayAbilityAuditRow::StaticStruct();
		}

		UE_LOGF(LogGameplayAbilityAudit, Log, "Selected %ls as the best Gameplay Ability Audit Functionality", *GetNameSafe(BestAuditRowStruct));
		return *BestAuditRowStruct;
	}

	/** This is the way we actually register the audit menu item.  We create this statically and it registers a menu in the editor. */
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []()
		{
			UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
				{
					// Let's figure out the best struct to use for the audit functionality...
					UScriptStruct& BestAuditRowStruct = FindBestAuditRowStruct();
					FNewToolMenuSectionDelegate MenuCreator = FNewToolMenuSectionDelegate::CreateLambda([&BestAuditRowStruct](FToolMenuSection& InSection)
						{
							// Since we're registered to execute on any UBlueprint, we need to ensure we've selected a Gameplay Ability Blueprint
							UContentBrowserAssetContextMenuContext* ContentBrowserContext = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
							if (ContentBrowserContext)
							{
								bool bPassesClassFilter = false;
								for (const FAssetData& AssetData : ContentBrowserContext->GetSelectedAssetsOfType(UBlueprint::StaticClass()))
								{
									if (TSubclassOf<UBlueprint> AssetClass = AssetData.GetClass())
									{
										if (const UClass* BlueprintParentClass = UBlueprint::GetBlueprintParentClassFromAssetTags(AssetData))
										{
											bPassesClassFilter |= BlueprintParentClass->IsChildOf(UGameplayAbility::StaticClass());
										}
									}
								}

								// We aren't a BP that generates a class derived from UGameplayAbility
								if (!bPassesClassFilter)
								{
									return;
								}
							}

							const TAttribute<FText> Label = LOCTEXT("GameplayAbilityBlueprint_ExecuteActionGameplayAbilityAudit", "Audit Gameplay Abilities");
							const TAttribute<FText> ToolTip = LOCTEXT("GameplayAbilityBlueprint_ExecuteActionGameplayAbilityAuditTooltip", "Export data for selected abilities into a DataTable");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Audit");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([&BestAuditRowStruct](const FToolMenuContext& InContext) { ExecuteActionGameplayAbilityAudit(BestAuditRowStruct, InContext); });
							InSection.AddMenuEntry(TEXT("GameplayAbilityBlueprint_ExecuteGameplayAbilityActionAudit"), Label, ToolTip, Icon, UIAction);
						});

					FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

					// Gameplay Ability Assets are Blueprint Assets (and aren't necessarily UGameplayAbilityBlueprints)
					UToolMenu* BPMenu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UBlueprint::StaticClass());
					BPMenu->FindOrAddSection("GetAssetActions").AddDynamicEntry(NAME_None, MenuCreator);
				}));
		});
} // namespace MenuExtension_GameplayAbilityBlueprintAudit

#undef LOCTEXT_NAMESPACE
