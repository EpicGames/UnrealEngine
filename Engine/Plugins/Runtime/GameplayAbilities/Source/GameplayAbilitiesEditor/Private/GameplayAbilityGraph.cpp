// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayAbilityGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityGraph)

#define LOCTEXT_NAMESPACE "GameplayAbilityGraph"

/////////////////////////////////////////////////////
// UGameplayAbilityGraph

UGameplayAbilityGraph::UGameplayAbilityGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#undef LOCTEXT_NAMESPACE

