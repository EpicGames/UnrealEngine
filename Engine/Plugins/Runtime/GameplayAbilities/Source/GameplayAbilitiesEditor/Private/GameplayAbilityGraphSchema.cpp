// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayAbilityGraphSchema.h"
#include "EdGraphSchema_K2_Actions.h"
#include "GameplayEffect.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayAbilityGraphSchema)

UGameplayAbilityGraphSchema::UGameplayAbilityGraphSchema(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

UK2Node_VariableGet* UGameplayAbilityGraphSchema::SpawnVariableGetNode(const FVector2D GraphPosition, class UEdGraph* ParentGraph, FName VariableName, UStruct* Source) const
{
	return Super::SpawnVariableGetNode(GraphPosition, ParentGraph, VariableName, Source);
}

UK2Node_VariableSet* UGameplayAbilityGraphSchema::SpawnVariableSetNode(const FVector2D GraphPosition, class UEdGraph* ParentGraph, FName VariableName, UStruct* Source) const
{
	return Super::SpawnVariableSetNode(GraphPosition, ParentGraph, VariableName, Source);
}

