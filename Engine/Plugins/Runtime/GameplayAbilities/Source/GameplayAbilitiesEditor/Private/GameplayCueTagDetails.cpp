// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueTagDetails.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "PropertyHandle.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "AbilitySystemGlobals.h"
#include "GameplayCueInterface.h"
#include "Widgets/Input/SHyperlink.h"
#include "GameplayCueManager.h"
#include "GameplayCueSet.h"
#include "SGameplayCueEditor.h"
#include "SlateOptMacros.h"

#define LOCTEXT_NAMESPACE "GameplayCueDetailsCustomization"

DEFINE_LOG_CATEGORY(LogGameplayCueDetails);

TSharedRef<IPropertyTypeCustomization> FGameplayCueTagDetails::MakeInstance()
{
	return MakeShareable(new FGameplayCueTagDetails());
}

FText FGameplayCueTagDetails::GetTagText() const
{
	FString Str;
	FGameplayTag* TagPtr = GetTag();
	if (TagPtr && TagPtr->IsValid())
	{
		Str = TagPtr->GetTagName().ToString();
	}

	return FText::FromString(Str);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FGameplayCueTagDetails::CustomizeHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);

	GameplayTagProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayCueTag,GameplayCueTag));

	FSimpleDelegate OnTagChanged = FSimpleDelegate::CreateSP(this, &FGameplayCueTagDetails::OnPropertyValueChanged);
	GameplayTagProperty->SetOnPropertyValueChanged(OnTagChanged);

	UGameplayCueManager* CueManager = UAbilitySystemGlobals::Get().GetGameplayCueManager();
	if (CueManager)
	{
		CueManager->OnGameplayCueNotifyAddOrRemove.AddSP(this, &FGameplayCueTagDetails::OnPropertyValueChanged);
	}

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];
}

void FGameplayCueTagDetails::CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	GameplayTagProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayCueTag,GameplayCueTag));
	if (GameplayTagProperty.IsValid())
	{
		IDetailPropertyRow& PropRow = StructBuilder.AddProperty(GameplayTagProperty.ToSharedRef());
	}

	bool ValidTag = UpdateNotifyList();
	bool HasNotify = (NotifyList.Num() > 0);

	StructBuilder.AddCustomRow( LOCTEXT("NotifyLinkStr", "Notify") )
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget(LOCTEXT("NotifyStr", "Notify"))
	]
	.ValueContent()
	.MaxDesiredWidth(512)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(2.f, 2.f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.0f, 2.0f)
			.AutoWidth()
			[
				SAssignNew(ListView, SListView < TSharedRef<FSoftObjectPath> >)
				.SelectionMode(ESelectionMode::None)
				.ListItemsSource(&NotifyList)
				.OnGenerateRow(this, &FGameplayCueTagDetails::GenerateListRow)
				.Visibility(this, &FGameplayCueTagDetails::GetListViewVisibility)
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight( )
		[
			SNew(SButton)
			.Text(LOCTEXT("AddNew", "Add New"))
			.Visibility(this, &FGameplayCueTagDetails::GetAddNewNotifyVisibliy)
			.OnClicked(this, &FGameplayCueTagDetails::OnAddNewNotifyClicked)
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<ITableRow> FGameplayCueTagDetails::GenerateListRow(TSharedRef<FSoftObjectPath> NotifyName, const TSharedRef<STableViewBase>& OwnerTable)
{	
	FString ShortName = NotifyName->ToString();
	int32 idx;
	if (ShortName.FindLastChar(TEXT('.'), idx))
	{
		ShortName.RightInline(ShortName.Len() - (idx+1), EAllowShrinking::No);
		ShortName.RemoveFromEnd(TEXT("_c"));
	}

	return
	SNew(STableRow< TSharedRef<FSoftObjectPath> >, OwnerTable)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		[
			SNew(SHyperlink)
			.Style(FAppStyle::Get(), "Common.GotoBlueprintHyperlink")
			.Text(FText::FromString(ShortName))
			.OnNavigate(this, &FGameplayCueTagDetails::NavigateToHandler, NotifyName)
		]
	];
}

void FGameplayCueTagDetails::NavigateToHandler(TSharedRef<FSoftObjectPath> AssetRef)
{
	SGameplayCueEditor::OpenEditorForNotify(AssetRef->ToString());
}

FReply FGameplayCueTagDetails::OnAddNewNotifyClicked()
{
	FGameplayTag* TagPtr = GetTag();
	if (TagPtr && TagPtr->IsValid())
	{
		SGameplayCueEditor::CreateNewGameplayCueNotifyDialogue(TagPtr->ToString(), nullptr);
		OnPropertyValueChanged();
	}
	return FReply::Handled();
}

void FGameplayCueTagDetails::OnPropertyValueChanged()
{
	UpdateNotifyList();
	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
	}
}

bool FGameplayCueTagDetails::UpdateNotifyList()
{
	NotifyList.Empty();
	
	FGameplayTag* Tag = GetTag();
	bool ValidTag = Tag && Tag->IsValid();
	if (ValidTag)
	{
		uint8 EnumVal;
		GameplayTagProperty->GetValue(EnumVal);
		if (UGameplayCueManager* CueManager = UAbilitySystemGlobals::Get().GetGameplayCueManager())
		{
			if (UGameplayCueSet* CueSet = CueManager->GetEditorCueSet())
			{
				if (int32* idxPtr = CueSet->GameplayCueDataMap.Find(*Tag))
				{
					int32 idx = *idxPtr;
					if (CueSet->GameplayCueData.IsValidIndex(idx))
					{
						FGameplayCueNotifyData& Data = CueSet->GameplayCueData[*idxPtr];
						TSharedRef<FSoftObjectPath> Item(MakeShareable(new FSoftObjectPath(Data.GameplayCueNotifyObj)));
						NotifyList.Add(Item);
					}
				}
			}
		}
	}

	return ValidTag;
}

FGameplayTag* FGameplayCueTagDetails::GetTag() const
{
	FGameplayTag* Tag = nullptr;
	TArray<void*> RawStructData;
	if (GameplayTagProperty.IsValid())
	{
		GameplayTagProperty->AccessRawData(RawStructData);
		if (RawStructData.Num() > 0)
		{
			Tag = (FGameplayTag*)(RawStructData[0]);
		}
	}
	return Tag;
}

EVisibility FGameplayCueTagDetails::GetAddNewNotifyVisibliy() const
{
	FGameplayTag* Tag = GetTag();
	bool ValidTag = Tag && Tag->IsValid();
	bool HasNotify = NotifyList.Num() > 0;
	return (!HasNotify && ValidTag) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FGameplayCueTagDetails::GetListViewVisibility() const
{
	return NotifyList.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
