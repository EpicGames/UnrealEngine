// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/SoftObjectPath.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class IPropertyHandle;
struct FGameplayTag;

DECLARE_LOG_CATEGORY_EXTERN(LogGameplayCueDetails, Log, All);

class FGameplayCueTagDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();


private:

	FGameplayCueTagDetails()
	{
	}

	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;	

	TSharedPtr<IPropertyHandle> GameplayTagProperty;

	TSharedRef<ITableRow> GenerateListRow(TSharedRef<FSoftObjectPath> NotifyName, const TSharedRef<STableViewBase>& OwnerTable);
	TArray<TSharedRef<FSoftObjectPath> > NotifyList;
	TSharedPtr< SListView < TSharedRef<FSoftObjectPath> > > ListView;

	void NavigateToHandler(TSharedRef<FSoftObjectPath> AssetRef);

	FReply OnAddNewNotifyClicked();

	/** Updates the selected tag*/
	void OnPropertyValueChanged();
	bool UpdateNotifyList();
	FGameplayTag* GetTag() const;
	FText GetTagText() const;

	EVisibility GetListViewVisibility() const;
	EVisibility GetAddNewNotifyVisibliy() const;
};
