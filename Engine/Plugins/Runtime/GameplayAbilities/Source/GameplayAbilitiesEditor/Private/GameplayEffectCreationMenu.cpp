// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectCreationMenu.h"
#include "GameplayAbilitiesEditor.h"


#include "LevelEditor.h"

#include "Modules/ModuleManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserDelegates.h"
#include "Factories/BlueprintFactory.h"
#include "AssetToolsModule.h"
#include "IContentBrowserSingleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayEffectCreationMenu)

// Static
TFunction< FString(FString BaseName, FString Path) > UGameplayEffectCreationMenu::GetDefaultAssetNameFunc;

UGameplayEffectCreationMenu::UGameplayEffectCreationMenu()
{

}

// Helper struct/function. The editable data is flat and we want to build a tree representation of it
struct FGEMenuItem : TSharedFromThis<FGEMenuItem>
{
	TMap<FString, TSharedPtr<FGEMenuItem>> SubItems;

	FString DefaultAssetName;
	TSubclassOf<class UObject> CDO;

	static void BuildMenus_r(TSharedPtr<FGEMenuItem> Item, FMenuBuilder& MenuBuilder, TArray<FString> SelectedPaths)
	{
		for (auto It : Item->SubItems)
		{
			TSharedPtr<FGEMenuItem> SubItem  = It.Value;
			FString CatName = It.Key;

			// Add a submenu if this item has sub items
			if (SubItem->SubItems.Num() > 0)
			{
				MenuBuilder.AddSubMenu(
					FText::FromString(CatName),
					FText::FromString(CatName),
					FNewMenuDelegate::CreateLambda([SubItem, SelectedPaths](FMenuBuilder& SubMenuBuilder)
					{
						BuildMenus_r(SubItem, SubMenuBuilder, SelectedPaths);
					})
				);
			}

			// Add the actual menu item to create the new GE
			if (SubItem->CDO.Get() != nullptr)
			{
				MenuBuilder.AddMenuEntry(
				FText::FromString(CatName), // note: the last category string is what we use for this. Using the "Default Asset Name" is not desirable here. (e.g., Damage|Ability|Instant but "Damage" is default asset name)
				FText(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([=]()
					{
						if (SelectedPaths.Num() > 0)
						{
							FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
							FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

							TArray<FAssetData> SelectedAssets;
							ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);

							// Create the GameplayCue Notify
							UBlueprintFactory* BlueprintFactory = NewObject<UBlueprintFactory>();
							BlueprintFactory->ParentClass = SubItem->CDO;

							// Figure out default name. Projects can set UGameplayEffectCreationMenu::GetDefaultAssetNameFunc to customize this.
							FString PackageName = SelectedPaths[0];
							FString AssetName;
							if (UGameplayEffectCreationMenu::GetDefaultAssetNameFunc)
							{
								AssetName = UGameplayEffectCreationMenu::GetDefaultAssetNameFunc(SubItem->DefaultAssetName, PackageName);
							}
							else
							{
								AssetName = FString::Printf(TEXT("GE_%s"), *SubItem->DefaultAssetName);
							}

							FString DefaultFullPath = PackageName / AssetName;

							AssetToolsModule.Get().CreateUniqueAssetName(DefaultFullPath, TEXT(""), /*out*/ PackageName, /*out*/ AssetName);

							ContentBrowserModule.Get().CreateNewAsset(*AssetName, SelectedPaths[0], UBlueprint::StaticClass(), BlueprintFactory);
						}

					}))
				);
			}
		}
	}
};

void TopMenuBuilderFunc(FMenuBuilder& TopMenuBuilder, const TArray<FString> SelectedPaths, TArray<FGameplayEffectCreationData> Definitions)
{
	if (Definitions.Num() == 0)
	{
		return;
	}

	TopMenuBuilder.AddSubMenu(
	NSLOCTEXT("GameplayAbilitiesEditorModule", "CreateGameplayEffect", "New Gameplay Effect"),
	NSLOCTEXT("GameplayAbilitiesEditorModule", "CreateGameplayEffect_Tooltip", "Create new Gameplay Effect from list of curated parents"),
	FNewMenuDelegate::CreateLambda([SelectedPaths, Definitions](FMenuBuilder& GETopMenuBuilder)
	{
		// Loop through our Definitions and build FGEItems 
		TSharedPtr<FGEMenuItem> RootItem = MakeShareable(new FGEMenuItem);
		for (const FGameplayEffectCreationData& Def : Definitions)
		{
			if (Def.ParentGameplayEffect.Get() == nullptr)
			{
				continue;
			}

			TArray<FString> CategoryStrings;
			Def.MenuPath.ParseIntoArray(CategoryStrings, TEXT("|"), true);

			FGEMenuItem* CurrentItem = RootItem.Get();
			for (int32 idx=0; idx < CategoryStrings.Num(); ++idx)
			{
				FString& Str = CategoryStrings[idx];
				TSharedPtr<FGEMenuItem>& DestItem = CurrentItem->SubItems.FindOrAdd(Str);
				if (!DestItem.IsValid())
				{							
					DestItem = MakeShareable(new FGEMenuItem);
				}
				CurrentItem = DestItem.Get();
			}

			CurrentItem->DefaultAssetName = Def.BaseName;
			CurrentItem->CDO = Def.ParentGameplayEffect.Get();
		}

		// Build menu delegates based on our data
		FGEMenuItem::BuildMenus_r(RootItem, GETopMenuBuilder, SelectedPaths);
	}));
}
	
void UGameplayEffectCreationMenu::AddMenuExtensions() const
{
	TSharedPtr<FUICommandList> CommandBindings = MakeShareable(new FUICommandList());

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>( TEXT("ContentBrowser") );
	TWeakObjectPtr<const UGameplayEffectCreationMenu> WeakThis(this);
	ContentBrowserModule.GetAllAssetContextMenuExtenders().Add(FContentBrowserMenuExtender_SelectedPaths::CreateLambda([WeakThis](const TArray<FString>& SelectedPaths)
	{
		TSharedRef<FExtender> Extender = MakeShared<FExtender>();
		if (WeakThis.IsValid())
		{
			Extender->AddMenuExtension(
				"ContentBrowserNewAdvancedAsset",
				EExtensionHook::After,
				TSharedPtr<FUICommandList>(),
				FMenuExtensionDelegate::CreateStatic(&TopMenuBuilderFunc, SelectedPaths, WeakThis->Definitions)
			);
		}

		return Extender;
	}));
}

