// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectDetails.h"
#include "UObject/UnrealType.h"
#include "DetailLayoutBuilder.h"
#include "GameplayEffectTypes.h"
#include "GameplayEffect.h"

#define LOCTEXT_NAMESPACE "GameplayEffectDetailsCustomization"

DEFINE_LOG_CATEGORY(LogGameplayEffectDetails);

// --------------------------------------------------------FGameplayEffectDetails---------------------------------------

TSharedRef<IDetailCustomization> FGameplayEffectDetails::MakeInstance()
{
	return MakeShareable(new FGameplayEffectDetails);
}

void FGameplayEffectDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	MyDetailLayout = &DetailLayout;

	TArray< TWeakObjectPtr<UObject> > Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);
	if (Objects.Num() != 1)
	{
		// I don't know what to do here or what could be expected. Just return to disable all templating functionality
		return;
	}

	FSimpleDelegate UpdateDurationPolicyDelegate = FSimpleDelegate::CreateSP(this, &FGameplayEffectDetails::OnDurationPolicyChange);

	TSharedPtr<IPropertyHandle> DurationPolicyProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGameplayEffect, DurationPolicy), UGameplayEffect::StaticClass());
	DurationPolicyProperty->SetOnPropertyValueChanged(UpdateDurationPolicyDelegate);

	TSharedPtr<IPropertyHandle> PeriodProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGameplayEffect, Period), UGameplayEffect::StaticClass());
	TSharedPtr<IPropertyHandle> PeriodValue = PeriodProperty ? PeriodProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FScalableFloat, Value)) : nullptr;
	if (PeriodValue)
	{
		PeriodValue->SetOnPropertyValueChanged(UpdateDurationPolicyDelegate);
	}

	// Hide properties where necessary
	UGameplayEffect* Obj = Cast<UGameplayEffect>(Objects[0].Get());
	if (Obj)
	{
		if (Obj->DurationPolicy != EGameplayEffectDurationType::HasDuration)
		{
			TSharedPtr<IPropertyHandle> DurationMagnitudeProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGameplayEffect, DurationMagnitude), UGameplayEffect::StaticClass());
			DetailLayout.HideProperty(DurationMagnitudeProperty);
		}

		if (Obj->DurationPolicy == EGameplayEffectDurationType::Instant || Obj->Period.Value <= 0.0f)
		{
			TSharedPtr<IPropertyHandle> PeriodInhibitionPolicyProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGameplayEffect, PeriodicInhibitionPolicy), UGameplayEffect::StaticClass());
			TSharedPtr<IPropertyHandle> ExecutePeriodicEffectOnApplicationProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGameplayEffect, bExecutePeriodicEffectOnApplication), UGameplayEffect::StaticClass());
			DetailLayout.HideProperty(PeriodInhibitionPolicyProperty);
			DetailLayout.HideProperty(ExecutePeriodicEffectOnApplicationProperty);
		}

		// The modifier array needs to be told to specifically hide evaluation channel settings for instant effects, as they do not factor evaluation channels at all
		// and instead only operate on base values. To that end, mark the instance metadata so that the customization for the evaluation channel is aware it has to hide
		// (see FGameplayModEvaluationChannelSettingsDetails for handling)
		TSharedPtr<IPropertyHandle> ModifierArrayProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGameplayEffect, Modifiers), UGameplayEffect::StaticClass());
		if (ModifierArrayProperty.IsValid() && ModifierArrayProperty->IsValidHandle())
		{
			FString ForceHideMetadataValue;
			if (Obj->DurationPolicy == EGameplayEffectDurationType::Instant)
			{
				ForceHideMetadataValue = FGameplayModEvaluationChannelSettings::ForceHideMetadataEnabledValue;
			}
			ModifierArrayProperty->SetInstanceMetaData(FGameplayModEvaluationChannelSettings::ForceHideMetadataKey, ForceHideMetadataValue);
		}
	}
}

void FGameplayEffectDetails::OnDurationPolicyChange()
{
	MyDetailLayout->ForceRefreshDetails();
}


//-------------------------------------------------------------------------------------

#undef LOCTEXT_NAMESPACE
