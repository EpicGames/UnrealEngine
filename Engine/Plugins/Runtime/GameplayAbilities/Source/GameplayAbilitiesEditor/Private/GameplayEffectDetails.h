// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class IPropertyHandle;

DECLARE_LOG_CATEGORY_EXTERN(LogGameplayEffectDetails, Log, All);

class FGameplayEffectDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

	IDetailLayoutBuilder* MyDetailLayout;

	void OnDurationPolicyChange();
};

