// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectExecutionDefinitionDetails.h"
#include "IDetailChildrenBuilder.h"

#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "GameplayEffectTypes.h"
#include "GameplayEffect.h"
#include "GameplayEffectExecutionCalculation.h"
#include "IPropertyUtilities.h"

#define LOCTEXT_NAMESPACE "GameplayEffectExecutionDefinitionDetailsCustomization"

TSharedRef<IPropertyTypeCustomization> FGameplayEffectExecutionDefinitionDetails::MakeInstance()
{
	return MakeShareable(new FGameplayEffectExecutionDefinitionDetails());
}

void FGameplayEffectExecutionDefinitionDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];
}

void FGameplayEffectExecutionDefinitionDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	bShowCalculationModifiers = false;
	bShowPassedInTags = false;

	// @todo: For now, only allow single editing
	if (StructPropertyHandle->GetNumOuterObjects() == 1)
	{
		CalculationClassPropHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayEffectExecutionDefinition, CalculationClass));
		TSharedPtr<IPropertyHandle> ConditionalEffectsPropHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayEffectExecutionDefinition, ConditionalGameplayEffects));
		TSharedPtr<IPropertyHandle> CalcModPropHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayEffectExecutionDefinition, CalculationModifiers));
		TSharedPtr<IPropertyHandle> PassedInTagsHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayEffectExecutionDefinition, PassedInTags));
		CalculationModifiersArrayPropHandle = CalcModPropHandle.IsValid() ? CalcModPropHandle->AsArray() : nullptr;

		if (CalculationClassPropHandle.IsValid())
		{
			CalculationClassPropHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FGameplayEffectExecutionDefinitionDetails::OnCalculationClassChanged));
			StructBuilder.AddProperty(CalculationClassPropHandle.ToSharedRef());
			StructCustomizationUtils.GetPropertyUtilities()->EnqueueDeferredAction(FSimpleDelegate::CreateSP(this, &FGameplayEffectExecutionDefinitionDetails::UpdateCalculationModifiers));
		}

		if (CalculationModifiersArrayPropHandle.IsValid())
		{
			IDetailPropertyRow& PropRow = StructBuilder.AddProperty(CalcModPropHandle.ToSharedRef());
			PropRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FGameplayEffectExecutionDefinitionDetails::GetCalculationModifierVisibility)));
		}

		if (ConditionalEffectsPropHandle.IsValid())
		{
			StructBuilder.AddProperty(ConditionalEffectsPropHandle.ToSharedRef());
		}

		if (PassedInTagsHandle.IsValid())
		{
			IDetailPropertyRow& PropRow = StructBuilder.AddProperty(PassedInTagsHandle.ToSharedRef());
			PropRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FGameplayEffectExecutionDefinitionDetails::GetPassedInTagsVisibility)));
		}
	}
}

void FGameplayEffectExecutionDefinitionDetails::OnCalculationClassChanged()
{
	UpdateCalculationModifiers();
}

void FGameplayEffectExecutionDefinitionDetails::UpdateCalculationModifiers()
{
	TArray<FGameplayEffectAttributeCaptureDefinition> ValidCaptureDefinitions;
	FGameplayTagContainer ValidTransientAggregatorIdentifiers;

	// Try to extract the collection of valid capture definitions & transient aggregator identifiers from the execution class CDO, if possible
	if (CalculationClassPropHandle.IsValid())
	{
		UObject* ObjValue = nullptr;
		CalculationClassPropHandle->GetValue(ObjValue);

		UClass* ClassObj = Cast<UClass>(ObjValue);
		if (ClassObj)
		{
			const UGameplayEffectExecutionCalculation* ExecutionCDO = ClassObj->GetDefaultObject<UGameplayEffectExecutionCalculation>();
			if (ExecutionCDO)
			{
				ExecutionCDO->GetValidScopedModifierAttributeCaptureDefinitions(ValidCaptureDefinitions);
				ValidTransientAggregatorIdentifiers = ExecutionCDO->GetValidTransientAggregatorIdentifiers();

				// Grab this while we are at it so we know if we should show the 'Passed In Tags' property
				bShowPassedInTags = ExecutionCDO->DoesRequirePassedInTags();
			}
		}
	}

	// Want to hide the calculation modifiers if there are no valid definitions or transient aggregator IDs
	bShowCalculationModifiers = (ValidCaptureDefinitions.Num() > 0) || (ValidTransientAggregatorIdentifiers.Num() > 0);

	// Need to prune out any modifiers that aren't supported by the execution class
	if (CalculationModifiersArrayPropHandle.IsValid())
	{
		uint32 NumChildren = 0;
		CalculationModifiersArrayPropHandle->GetNumElements(NumChildren);

		// If there aren't any valid definitions or transient aggregator IDs, just dump the whole array
		if (ValidCaptureDefinitions.Num() == 0 && ValidTransientAggregatorIdentifiers.Num() == 0 && NumChildren > 0)
		{
			CalculationModifiersArrayPropHandle->EmptyArray();
		}
		// There are some valid definitions/IDs, so verify any existing ones to make sure they are in the valid array
		else
		{
			for (int32 ChildIdx = NumChildren - 1; ChildIdx >= 0; --ChildIdx)
			{
				TSharedRef<IPropertyHandle> ChildPropHandle = CalculationModifiersArrayPropHandle->GetElement(ChildIdx);
				
				TArray<const void*> RawScopedModInfoStructs;
				ChildPropHandle->AccessRawData(RawScopedModInfoStructs);

				// @todo: For now, only allow single editing
				ensure(RawScopedModInfoStructs.Num() == 1);
				const FGameplayEffectExecutionScopedModifierInfo& CurModInfo = *reinterpret_cast<const FGameplayEffectExecutionScopedModifierInfo*>(RawScopedModInfoStructs[0]);
				
				const bool bInvalidAttributeAggregator = (CurModInfo.AggregatorType == EGameplayEffectScopedModifierAggregatorType::CapturedAttributeBacked && !ValidCaptureDefinitions.Contains(CurModInfo.CapturedAttribute));
				const bool bInvalidTransientAggregator = (CurModInfo.AggregatorType == EGameplayEffectScopedModifierAggregatorType::Transient && !ValidTransientAggregatorIdentifiers.HasTagExact(CurModInfo.TransientAggregatorIdentifier));
				if (bInvalidAttributeAggregator || bInvalidTransientAggregator)
				{
					CalculationModifiersArrayPropHandle->DeleteItem(ChildIdx);
				}
			}
		}
	}
}

EVisibility FGameplayEffectExecutionDefinitionDetails::GetCalculationModifierVisibility() const
{
	return (bShowCalculationModifiers ? EVisibility::Visible : EVisibility::Collapsed);
}

EVisibility FGameplayEffectExecutionDefinitionDetails::GetPassedInTagsVisibility() const
{
	return (bShowPassedInTags ? EVisibility::Visible : EVisibility::Collapsed);
}

#undef LOCTEXT_NAMESPACE
