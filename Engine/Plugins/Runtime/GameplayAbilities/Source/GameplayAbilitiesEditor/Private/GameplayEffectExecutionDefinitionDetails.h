// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;

/** Details customization for FGameplayEffectExecutionDefinition */
class FGameplayEffectExecutionDefinitionDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** Overridden to provide the property name */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	
	/** Overridden to allow for hiding/updating of the calculation modifiers array as the calculation class changes */
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	/** Called via delegate when the user changes the calculation class */
	void OnCalculationClassChanged();

	/** Helper function to determine whether to hide/show the modifiers, as well as prune invalid options */
	void UpdateCalculationModifiers();

	/** Visibility delegate for calculation modifiers */
	EVisibility GetCalculationModifierVisibility() const;

	/** Visibility delegate for Passed In Tags field */
	EVisibility GetPassedInTagsVisibility() const;

	/** Property handle for the calculation class */
	TSharedPtr<IPropertyHandle> CalculationClassPropHandle;

	/** Property handle array for the calculation modifiers */
	TSharedPtr<IPropertyHandleArray> CalculationModifiersArrayPropHandle;

	/** If true, the calculation modifiers array should be shown */
	bool bShowCalculationModifiers;

	/** If true, the Passed In Tags field will be shown */
	bool bShowPassedInTags;
};
