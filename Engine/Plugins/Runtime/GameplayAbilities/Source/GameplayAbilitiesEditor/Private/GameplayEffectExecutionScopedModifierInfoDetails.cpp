// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectExecutionScopedModifierInfoDetails.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Input/SComboBox.h"
#include "DetailWidgetRow.h"
#include "GameplayEffect.h"
#include "GameplayEffectExecutionCalculation.h"
#include "DetailLayoutBuilder.h"
#include "SlateOptMacros.h"
#include "GameplayEffectTypes.h"

#define LOCTEXT_NAMESPACE "GameplayEffectExecutionScopedModifierInfoDetailsCustomization"

/** Simple struct tracking scoped mod backing data from the scoped mod info */
struct FAggregatorDetailsBackingData
{
	// Constructors
	FAggregatorDetailsBackingData()
		: AggregatorType(EGameplayEffectScopedModifierAggregatorType::CapturedAttributeBacked)
		, CaptureDefinition()
		, TransientAggregatorIdentifier()
	{
	}

	FAggregatorDetailsBackingData(const FGameplayEffectAttributeCaptureDefinition& InCaptureDef)
		: AggregatorType(EGameplayEffectScopedModifierAggregatorType::CapturedAttributeBacked)
		, CaptureDefinition(InCaptureDef)
		, TransientAggregatorIdentifier()
	{
	}

	FAggregatorDetailsBackingData(const FGameplayTag& InIdentifier)
		: AggregatorType(EGameplayEffectScopedModifierAggregatorType::Transient)
		, CaptureDefinition()
		, TransientAggregatorIdentifier(InIdentifier)
	{
	}

	// Equality operators
	bool operator==(const FAggregatorDetailsBackingData& Other) const
	{
		return (AggregatorType == Other.AggregatorType) && (CaptureDefinition == Other.CaptureDefinition) && (TransientAggregatorIdentifier == Other.TransientAggregatorIdentifier);
	}

	bool operator!=(const FAggregatorDetailsBackingData& Other) const
	{
		return (AggregatorType != Other.AggregatorType) || (CaptureDefinition != Other.CaptureDefinition) || (TransientAggregatorIdentifier != Other.TransientAggregatorIdentifier);
	}

	/** Aggregator type for the backing data */
	EGameplayEffectScopedModifierAggregatorType AggregatorType;

	/** Capture definition used if attribute-backed aggregator type */
	FGameplayEffectAttributeCaptureDefinition CaptureDefinition;

	/** Identifier used if transient/"temporary variable" aggregator type */
	FGameplayTag TransientAggregatorIdentifier;
};

TSharedRef<IPropertyTypeCustomization> FGameplayEffectExecutionScopedModifierInfoDetails::MakeInstance()
{
	return MakeShareable(new FGameplayEffectExecutionScopedModifierInfoDetails());
}

void FGameplayEffectExecutionScopedModifierInfoDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

/** Custom widget class to cleanly represent mod backing data in a combo box */
class SScopedModBackingDataWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SScopedModBackingDataWidget) {}
	SLATE_END_ARGS()

	/** Construct the widget from a grid panel wrapped with a border */
	void Construct(const FArguments& InArgs)
	{
		this->ChildSlot
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(this, &SScopedModBackingDataWidget::GetWidgetSwitcherIdx)
			+SWidgetSwitcher::Slot()
			[
				SNew(SBorder)
				.HAlign(HAlign_Fill)
				[
					SNew(SGridPanel)
					+SGridPanel::Slot(0, 0)
					.HAlign(HAlign_Right)
					.Padding(FMargin(2.f))
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("ScopedModifierDetails", "CapturedAttributeLabel", "Captured Attribute:"))
						.Font(IDetailLayoutBuilder::GetDetailFontBold())
					]
					+SGridPanel::Slot(1, 0)
					.HAlign(HAlign_Left)
					.Padding(FMargin(2.f))
					[
						SNew(STextBlock)
						.Text(this, &SScopedModBackingDataWidget::GetCapturedAttributeText)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					+SGridPanel::Slot(0, 1)
					.HAlign(HAlign_Right)
					.Padding(FMargin(2.f))
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("ScopedModifierDetails", "CapturedAttributeSourceLabel", "Captured Source:"))
						.Font(IDetailLayoutBuilder::GetDetailFontBold())
					]
					+SGridPanel::Slot(1, 1)
					.HAlign(HAlign_Left)
					.Padding(FMargin(2.f))
					[
						SNew(STextBlock)
						.Text(this, &SScopedModBackingDataWidget::GetCapturedAttributeSourceText)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					+SGridPanel::Slot(0, 2)
					.HAlign(HAlign_Right)
					.Padding(FMargin(2.f))
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("ScopedModifierDetails", "CapturedAttributeSnapshotLabel", "Captured Status:"))
						.Font(IDetailLayoutBuilder::GetDetailFontBold())
					]
					+SGridPanel::Slot(1, 2)
					.HAlign(HAlign_Left)
					.Padding(FMargin(2.f))
					[
						SNew(STextBlock)
						.Text(this, &SScopedModBackingDataWidget::GetCapturedAttributeSnapshotText)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
			]
			+SWidgetSwitcher::Slot()
			[
				SNew(SBorder)
				.HAlign(HAlign_Fill)
				[
					SNew(SGridPanel)
					+SGridPanel::Slot(0, 0)
					.HAlign(HAlign_Right)
					.Padding(FMargin(2.f))
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("ScopedModifierDetails", "TempVariableLabel", "Temporary Variable:"))
						.Font(IDetailLayoutBuilder::GetDetailFontBold())
					]
					+SGridPanel::Slot(1, 0)
					.HAlign(HAlign_Left)
					.Padding(FMargin(2.f))
					[
						SNew(STextBlock)
						.Text(this, &SScopedModBackingDataWidget::GetTemporaryVariableText)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					+SGridPanel::Slot(0, 1)
					.HAlign(HAlign_Center)
					.Padding(FMargin(2.f))
					.ColumnSpan(2)
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("ScopedModifierDetails", "TempVariableDesc", "Temporary value exposed by calculation."))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
			]
		];
	}

	/** Set the data that backs the widget; This is used as a way to reduce expensive FText creations constantly */
	void SetBackingData(const FAggregatorDetailsBackingData& InBackingData)
	{
		static const FText SnapshotText(NSLOCTEXT("ScopedModifierDetails", "CapturedAttributeSnapshotted", "Snapshotted"));
		static const FText NotSnapshotText(NSLOCTEXT("ScopedModifierDetails", "CapturedAttributeNotSnapshotted", "Not Snapshotted"));

		if (InBackingData != BackingData)
		{
			BackingData = InBackingData;
			CapturedAttributeText = FText::FromString(BackingData.CaptureDefinition.AttributeToCapture.GetName());
			UEnum::GetDisplayValueAsText(TEXT("GameplayAbilities.EGameplayEffectAttributeCaptureSource"), BackingData.CaptureDefinition.AttributeSource, CapturedAttributeSourceText);
			CapturedAttributeSnapshotText = BackingData.CaptureDefinition.bSnapshot ? SnapshotText : NotSnapshotText;
			TemporaryVariableText = FText::FromString(BackingData.TransientAggregatorIdentifier.ToString());

			WidgetSwitcherIdx = (BackingData.AggregatorType == EGameplayEffectScopedModifierAggregatorType::CapturedAttributeBacked) ? 0 : 1;
		}
	}

private:

	/** Simple accessor to cached widget switcher index */
	int32 GetWidgetSwitcherIdx() const
	{
		return WidgetSwitcherIdx;
	}

	/** Simple accessor to cached captured attribute text */
	FText GetCapturedAttributeText() const
	{
		return CapturedAttributeText;
	}

	/** Simple accessor to cached captured attribute source text */
	FText GetCapturedAttributeSourceText() const
	{
		return CapturedAttributeSourceText;
	}

	/** Simple accessor to cached captured attribute snapshot text */
	FText GetCapturedAttributeSnapshotText() const
	{
		return CapturedAttributeSnapshotText;
	}

	/** Simple accessor to cached temporary variable text */
	FText GetTemporaryVariableText() const
	{
		return TemporaryVariableText;
	}

	/** Aggregator data backing the widget */
	FAggregatorDetailsBackingData BackingData;

	/** Cached attribute text */
	FText CapturedAttributeText;

	/** Cached attribute capture source text */
	FText CapturedAttributeSourceText;

	/** Cached attribute snapshot status text */
	FText CapturedAttributeSnapshotText;

	/** Cached temporary variable status text */
	FText TemporaryVariableText;

	/** Cached widget switcher index */
	int32 WidgetSwitcherIdx;
};

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FGameplayEffectExecutionScopedModifierInfoDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	AvailableBackingData.Empty();

	ScopedModifierStructPropertyHandle = StructPropertyHandle;

	TSharedPtr<IPropertyHandle> ParentArrayHandle = StructPropertyHandle->GetParentHandle();
	const bool bIsExecutionDefAttribute = (ParentArrayHandle.IsValid() && ParentArrayHandle->GetProperty()->GetOwner<UObject>() == FGameplayEffectExecutionDefinition::StaticStruct());
	
	if (bIsExecutionDefAttribute)
	{
		TArray<const void*> StructPtrs;
		StructPropertyHandle->AccessRawData(StructPtrs);

		// Only allow changing the backing definition while single-editing
		if (StructPtrs.Num() == 1)
		{
			TSharedPtr<IPropertyHandle> ExecutionDefinitionHandle = ParentArrayHandle->GetParentHandle();
			if (ExecutionDefinitionHandle.IsValid())
			{
				TArray<const void*> ExecutionDefStructs;
				ExecutionDefinitionHandle->AccessRawData(ExecutionDefStructs);

				if (ExecutionDefStructs.Num() == 1)
				{
					// Extract all of the valid capture definitions off of the capture class
					const FGameplayEffectExecutionDefinition& ExecutionDef = *reinterpret_cast<const FGameplayEffectExecutionDefinition*>(ExecutionDefStructs[0]);
					if (ExecutionDef.CalculationClass)
					{
						const UGameplayEffectExecutionCalculation* ExecCalcCDO = ExecutionDef.CalculationClass->GetDefaultObject<UGameplayEffectExecutionCalculation>();
						if (ensure(ExecCalcCDO))
						{
							TArray<FGameplayEffectAttributeCaptureDefinition> CaptureDefs;
							ExecCalcCDO->GetValidScopedModifierAttributeCaptureDefinitions(CaptureDefs);

							for (const FGameplayEffectAttributeCaptureDefinition& CurDef : CaptureDefs)
							{
								AvailableBackingData.Add(MakeShareable(new FAggregatorDetailsBackingData(CurDef)));
							}

							FGameplayTagContainer ValidTransientAggregatorIdentifiers = ExecCalcCDO->GetValidTransientAggregatorIdentifiers();
							for (const FGameplayTag& CurIdentifier : ValidTransientAggregatorIdentifiers)
							{
								AvailableBackingData.Add(MakeShareable(new FAggregatorDetailsBackingData(CurIdentifier)));
							}
						}
					}
				}
			}
		}

		// Construct a custom combo box widget outlining possible backing data choices
		if (AvailableBackingData.Num() > 0)
		{
			TSharedPtr< SComboBox< TSharedPtr<FAggregatorDetailsBackingData> > > BackingComboBox;

			StructBuilder.AddCustomRow(NSLOCTEXT("ScopedModifierDetails", "BackingDataLabel", "Backing Data"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("ScopedModifierDetails", "BackingDataLabel", "Backing Data"))
				.ToolTipText(NSLOCTEXT("ScopedModifierDetails", "BackingDataTooltip", "The backing data to use to populate the scoped modifier. Only options specified by the execution class are presented here."))
				.Font(StructCustomizationUtils.GetRegularFont())
			]
			.ValueContent()
			.MinDesiredWidth(350.f)
			[
				SAssignNew(BackingComboBox, SComboBox< TSharedPtr<FAggregatorDetailsBackingData> >)
				.OptionsSource(&AvailableBackingData)
				.OnSelectionChanged(this, &FGameplayEffectExecutionScopedModifierInfoDetails::OnBackingDataComboBoxSelectionChanged)
				.OnGenerateWidget(this, &FGameplayEffectExecutionScopedModifierInfoDetails::OnGenerateBackingDataComboWidget)
				.Content()
				[
					SAssignNew(PrimaryBackingDataWidget, SScopedModBackingDataWidget)
				]
			];

			// Set the initial value on the combo box; done this way to intentionally cause a change delegate
			if (BackingComboBox.IsValid())
			{
				BackingComboBox->SetSelectedItem(GetCurrentBackingData());
			}
		}
	}

	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	// Add all of the properties, though skip the original captured attribute if inside an execution, as it is using the custom
	// combo box instead
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		const TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName ChildPropName = ChildHandle->GetProperty()->GetFName();

		const bool bAlwaysShowableProperty = ((ChildPropName != GET_MEMBER_NAME_CHECKED(FGameplayEffectExecutionScopedModifierInfo, CapturedAttribute))
			&& (ChildPropName != GET_MEMBER_NAME_CHECKED(FGameplayEffectExecutionScopedModifierInfo, TransientAggregatorIdentifier))
			&& (ChildPropName != GET_MEMBER_NAME_CHECKED(FGameplayEffectExecutionScopedModifierInfo, AggregatorType)));

		if (!bIsExecutionDefAttribute || bAlwaysShowableProperty)
		{
			StructBuilder.AddProperty(ChildHandle);
		}
	}
}

void FGameplayEffectExecutionScopedModifierInfoDetails::OnBackingDataComboBoxSelectionChanged(TSharedPtr<FAggregatorDetailsBackingData> InSelectedItem, ESelectInfo::Type InSelectInfo)
{
	SetCurrentBackingData(InSelectedItem);

	// Need to update the base backing data widget in the combo box manually due to caching strategy
	if (PrimaryBackingDataWidget.IsValid())
	{
		PrimaryBackingDataWidget->SetBackingData(*InSelectedItem.Get());
	}
}

TSharedRef<SWidget> FGameplayEffectExecutionScopedModifierInfoDetails::OnGenerateBackingDataComboWidget(TSharedPtr<FAggregatorDetailsBackingData> InItem)
{
	TSharedPtr<SScopedModBackingDataWidget> NewBackingDataWidget;
	SAssignNew(NewBackingDataWidget, SScopedModBackingDataWidget);
	NewBackingDataWidget->SetBackingData(*InItem.Get());

	return NewBackingDataWidget.ToSharedRef();
}

TSharedPtr<FAggregatorDetailsBackingData> FGameplayEffectExecutionScopedModifierInfoDetails::GetCurrentBackingData() const
{
	if (ScopedModifierStructPropertyHandle.IsValid() && ScopedModifierStructPropertyHandle->GetProperty())
	{
		TArray<const void*> RawStructPtrs;
		ScopedModifierStructPropertyHandle->AccessRawData(RawStructPtrs);

		// Only showing the combo box for single-editing
		const FGameplayEffectExecutionScopedModifierInfo& ScopedModifierInfo = *reinterpret_cast<const FGameplayEffectExecutionScopedModifierInfo*>(RawStructPtrs[0]);

		for (TSharedPtr<FAggregatorDetailsBackingData> CurBackingData : AvailableBackingData)
		{
			if (CurBackingData.IsValid() && (CurBackingData->AggregatorType == ScopedModifierInfo.AggregatorType))
			{
				if (ScopedModifierInfo.AggregatorType == EGameplayEffectScopedModifierAggregatorType::CapturedAttributeBacked)
				{
					if (ScopedModifierInfo.CapturedAttribute == CurBackingData->CaptureDefinition)
					{
						return CurBackingData;
					}
				}
				else
				{
					if (ScopedModifierInfo.TransientAggregatorIdentifier == CurBackingData->TransientAggregatorIdentifier)
					{
						return CurBackingData;
					}
				}
			}
		}
	}

	return AvailableBackingData[0];
}

void FGameplayEffectExecutionScopedModifierInfoDetails::SetCurrentBackingData(TSharedPtr<FAggregatorDetailsBackingData> InBackingData)
{
	if (ScopedModifierStructPropertyHandle.IsValid() && ScopedModifierStructPropertyHandle->GetProperty() && InBackingData.IsValid())
	{
		TArray<void*> RawStructPtrs;
		ScopedModifierStructPropertyHandle->AccessRawData(RawStructPtrs);

		FGameplayEffectExecutionScopedModifierInfo& ScopedModifierInfo = *reinterpret_cast<FGameplayEffectExecutionScopedModifierInfo*>(RawStructPtrs[0]);
		
		const bool bSameData = (ScopedModifierInfo.AggregatorType == InBackingData->AggregatorType)
			&& (ScopedModifierInfo.CapturedAttribute == InBackingData->CaptureDefinition)
			&& (ScopedModifierInfo.TransientAggregatorIdentifier == InBackingData->TransientAggregatorIdentifier);

		if (!bSameData)
		{
			ScopedModifierStructPropertyHandle->NotifyPreChange();
			ScopedModifierInfo.AggregatorType = InBackingData->AggregatorType;
			if (ScopedModifierInfo.AggregatorType == EGameplayEffectScopedModifierAggregatorType::CapturedAttributeBacked)
			{
				ScopedModifierInfo.CapturedAttribute = InBackingData->CaptureDefinition;
				ScopedModifierInfo.TransientAggregatorIdentifier = FGameplayTag::EmptyTag;
			}
			else
			{
				ScopedModifierInfo.CapturedAttribute = FGameplayEffectAttributeCaptureDefinition();
				ScopedModifierInfo.TransientAggregatorIdentifier = InBackingData->TransientAggregatorIdentifier;
			}
			ScopedModifierStructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
	}
}

#undef LOCTEXT_NAMESPACE
