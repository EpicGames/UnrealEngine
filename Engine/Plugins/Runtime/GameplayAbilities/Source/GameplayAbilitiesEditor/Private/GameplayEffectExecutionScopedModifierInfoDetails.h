// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "IDetailChildrenBuilder.h"

class FDetailWidgetRow;
class SScopedModBackingDataWidget;
struct FAggregatorDetailsBackingData;

/** Details customization for FGameplayEffectExecutionScopedModifierInfo */
class FGameplayEffectExecutionScopedModifierInfoDetails : public IPropertyTypeCustomization
{
public:
	
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** Overridden to provide the property name */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	
	/** Overridden to allow for a custom selection widget for scoped modifiers inside a custom execution */
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	/** Delegate called when combo box selection is changed */
	void OnBackingDataComboBoxSelectionChanged(TSharedPtr<FAggregatorDetailsBackingData> InSelectedItem, ESelectInfo::Type InSelectInfo);
	
	/* Called to generate the widgets for custom combo box entries */
	TSharedRef<SWidget> OnGenerateBackingDataComboWidget(TSharedPtr<FAggregatorDetailsBackingData> InItem);
	
	/** Get the current aggregator backing data, if possible; Otherwise falls back to first available definition from execution class */
	TSharedPtr<FAggregatorDetailsBackingData> GetCurrentBackingData() const;

	/** Set the current capture definition */
	void SetCurrentBackingData(TSharedPtr<FAggregatorDetailsBackingData> InBackingData);

	/** Cached property handle for the overall scoped modifier info struct */
	TSharedPtr<IPropertyHandle> ScopedModifierStructPropertyHandle;

	/** Primary backing data widget shown for the custom combo box */
	TSharedPtr<SScopedModBackingDataWidget> PrimaryBackingDataWidget;

	/** Backing source for the custom combo box; Populated by all valid backing data from the execution class */
	TArray< TSharedPtr<FAggregatorDetailsBackingData> > AvailableBackingData;
};
