// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayEffectModifierMagnitudeDetails.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"

#define LOCTEXT_NAMESPACE "GameplayEffectModifierMagnitudeDetailsCustomization"

TSharedRef<IPropertyTypeCustomization> FGameplayEffectModifierMagnitudeDetails::MakeInstance()
{
	return MakeShareable(new FGameplayEffectModifierMagnitudeDetails());
}

void FGameplayEffectModifierMagnitudeDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];
}

void FGameplayEffectModifierMagnitudeDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Set up acceleration map
	PropertyToCalcEnumMap.Empty();
	PropertyToCalcEnumMap.Add(FindFieldChecked<FProperty>(FGameplayEffectModifierMagnitude::StaticStruct(), GET_MEMBER_NAME_CHECKED(FGameplayEffectModifierMagnitude, ScalableFloatMagnitude)), EGameplayEffectMagnitudeCalculation::ScalableFloat);
	PropertyToCalcEnumMap.Add(FindFieldChecked<FProperty>(FGameplayEffectModifierMagnitude::StaticStruct(), GET_MEMBER_NAME_CHECKED(FGameplayEffectModifierMagnitude, AttributeBasedMagnitude)), EGameplayEffectMagnitudeCalculation::AttributeBased);
	PropertyToCalcEnumMap.Add(FindFieldChecked<FProperty>(FGameplayEffectModifierMagnitude::StaticStruct(), GET_MEMBER_NAME_CHECKED(FGameplayEffectModifierMagnitude, CustomMagnitude)), EGameplayEffectMagnitudeCalculation::CustomCalculationClass);
	PropertyToCalcEnumMap.Add(FindFieldChecked<FProperty>(FGameplayEffectModifierMagnitude::StaticStruct(), GET_MEMBER_NAME_CHECKED(FGameplayEffectModifierMagnitude, SetByCallerMagnitude)), EGameplayEffectMagnitudeCalculation::SetByCaller);

	// Hook into calculation type changes
	MagnitudeCalculationTypePropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayEffectModifierMagnitude, MagnitudeCalculationType));
	if (MagnitudeCalculationTypePropertyHandle.IsValid())
	{
		MagnitudeCalculationTypePropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FGameplayEffectModifierMagnitudeDetails::OnCalculationTypeChanged));
		StructBuilder.AddProperty(MagnitudeCalculationTypePropertyHandle.ToSharedRef());
	}
	OnCalculationTypeChanged();

	// Set up visibility delegates on all of the magnitude calculations
	TSharedPtr<IPropertyHandle> ScalableFloatMagnitudePropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayEffectModifierMagnitude, ScalableFloatMagnitude));
	if (ScalableFloatMagnitudePropertyHandle.IsValid())
	{
		IDetailPropertyRow& PropRow = StructBuilder.AddProperty(ScalableFloatMagnitudePropertyHandle.ToSharedRef());
		PropRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FGameplayEffectModifierMagnitudeDetails::GetMagnitudeCalculationPropertyVisibility, ScalableFloatMagnitudePropertyHandle->GetProperty())));
	}

	TSharedPtr<IPropertyHandle> AttributeBasedMagnitudePropertyHandle  = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayEffectModifierMagnitude, AttributeBasedMagnitude));
	if (AttributeBasedMagnitudePropertyHandle.IsValid())
	{
		IDetailPropertyRow& PropRow = StructBuilder.AddProperty(AttributeBasedMagnitudePropertyHandle.ToSharedRef());
		PropRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FGameplayEffectModifierMagnitudeDetails::GetMagnitudeCalculationPropertyVisibility, AttributeBasedMagnitudePropertyHandle->GetProperty())));
	}

	TSharedPtr<IPropertyHandle> CustomMagnitudePropertyHandle  = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayEffectModifierMagnitude, CustomMagnitude));
	if (CustomMagnitudePropertyHandle.IsValid())
	{
		IDetailPropertyRow& PropRow = StructBuilder.AddProperty(CustomMagnitudePropertyHandle.ToSharedRef());
		PropRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FGameplayEffectModifierMagnitudeDetails::GetMagnitudeCalculationPropertyVisibility, CustomMagnitudePropertyHandle->GetProperty())));
	}

	TSharedPtr<IPropertyHandle> SetByCallerPropertyHandle  = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayEffectModifierMagnitude, SetByCallerMagnitude));
	if (SetByCallerPropertyHandle.IsValid())
	{
		IDetailPropertyRow& PropRow = StructBuilder.AddProperty(SetByCallerPropertyHandle.ToSharedRef());
		PropRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FGameplayEffectModifierMagnitudeDetails::GetMagnitudeCalculationPropertyVisibility, SetByCallerPropertyHandle->GetProperty())));
	}
}

void FGameplayEffectModifierMagnitudeDetails::OnCalculationTypeChanged()
{
	uint8 EnumVal;
	MagnitudeCalculationTypePropertyHandle->GetValue(EnumVal);
	VisibleCalculationType = static_cast<EGameplayEffectMagnitudeCalculation>(EnumVal);
}

EVisibility FGameplayEffectModifierMagnitudeDetails::GetMagnitudeCalculationPropertyVisibility(FProperty* InProperty) const
{
	const EGameplayEffectMagnitudeCalculation* EnumVal = PropertyToCalcEnumMap.Find(InProperty);
	return ((EnumVal && (*EnumVal == VisibleCalculationType)) ? EVisibility::Visible : EVisibility::Collapsed);
}

#undef LOCTEXT_NAMESPACE
