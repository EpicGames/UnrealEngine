// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "IPropertyTypeCustomization.h"
#include "GameplayEffect.h"
#include "PropertyHandle.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;

/** Details customization for FGameplayEffectModifierMagnitude */
class FGameplayEffectModifierMagnitudeDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** Overridden to provide the property name */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	
	/** Overridden to only show the magnitude that is currently being used based on calculation type */
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	/** Called via delegate when the user changes the calculation type */
	void OnCalculationTypeChanged();

	/** Visibility delegate for the various methods of calculating magnitude */
	EVisibility GetMagnitudeCalculationPropertyVisibility(FProperty* InProperty) const;

	/** Property handle of the enumeration of the magnitude calculation type */
	TSharedPtr<IPropertyHandle> MagnitudeCalculationTypePropertyHandle;

	/** Currently visible magnitude calculation type */
	EGameplayEffectMagnitudeCalculation VisibleCalculationType;

	/** Acceleration map for determining whether to show a magnitude property or not */
	TMap<FProperty*, EGameplayEffectMagnitudeCalculation> PropertyToCalcEnumMap;
};
