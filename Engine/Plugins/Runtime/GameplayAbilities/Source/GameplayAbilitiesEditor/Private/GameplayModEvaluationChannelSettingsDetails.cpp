// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayModEvaluationChannelSettingsDetails.h"
#include "DetailWidgetRow.h"
#include "GameplayEffectTypes.h"
#include "AbilitySystemGlobals.h"
#include "IDetailChildrenBuilder.h"

#define LOCTEXT_NAMESPACE "GameplayEffectExecutionScopedModifierInfoDetailsCustomization"

TSharedRef<IPropertyTypeCustomization> FGameplayModEvaluationChannelSettingsDetails::MakeInstance()
{
	return MakeShareable(new FGameplayModEvaluationChannelSettingsDetails());
}

void FGameplayModEvaluationChannelSettingsDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// See if the game even wants to allow gameplay effect eval. channels in the first place
	bShouldBeVisible = UAbilitySystemGlobals::Get().ShouldAllowGameplayModEvaluationChannels();
	
	// Verify that a parent handle hasn't forcibly marked evaluation channels hidden
	if (bShouldBeVisible)
	{
		TSharedPtr<IPropertyHandle> ParentHandle = StructPropertyHandle->GetParentHandle();
		while (ParentHandle.IsValid() && ParentHandle->IsValidHandle())
		{
			const FString* Meta = ParentHandle->GetInstanceMetaData(FGameplayModEvaluationChannelSettings::ForceHideMetadataKey);
			if (Meta && !Meta->IsEmpty())
			{
				bShouldBeVisible = false;
				break;
			}
			ParentHandle = ParentHandle->GetParentHandle();
		}
	}

	if (bShouldBeVisible)
	{
		HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
	}
	else
	{
		StructPropertyHandle->MarkHiddenByCustomization();
	}
}

void FGameplayModEvaluationChannelSettingsDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (bShouldBeVisible && StructPropertyHandle->IsValidHandle())
	{
		TSharedPtr<IPropertyHandle> ChannelHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayModEvaluationChannelSettings, Channel));
		if (ChannelHandle.IsValid() && ChannelHandle->IsValidHandle())
		{
			StructBuilder.AddProperty(ChannelHandle.ToSharedRef());
		}
	}
}

#undef LOCTEXT_NAMESPACE
