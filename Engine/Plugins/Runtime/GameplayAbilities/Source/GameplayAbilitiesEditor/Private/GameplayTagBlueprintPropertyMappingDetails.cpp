// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagBlueprintPropertyMappingDetails.h"
#include "GameplayEffectTypes.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Engine/Blueprint.h"
#include "Widgets/Input/SComboBox.h"


TSharedRef<IPropertyTypeCustomization> FGameplayTagBlueprintPropertyMappingDetails::MakeInstance()
{
	return MakeShareable(new FGameplayTagBlueprintPropertyMappingDetails());
}

void FGameplayTagBlueprintPropertyMappingDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	NamePropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayTagBlueprintPropertyMapping, PropertyName));
	GuidPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayTagBlueprintPropertyMapping, PropertyGuid));

	FString SelectedPropertyName;
	NamePropertyHandle->GetValue(SelectedPropertyName);

	TArray<void*> RawData;
	GuidPropertyHandle->AccessRawData(RawData);

	FGuid SelectedPropertyGuid;
	if (RawData.Num() > 0)
	{
		SelectedPropertyGuid = *static_cast<FGuid*>(RawData[0]);
	}

	FProperty* FoundProperty = nullptr;

	TArray<UObject*> OuterObjects;
	NamePropertyHandle->GetOuterObjects(OuterObjects);

	for (UObject* ParentObject : OuterObjects)
	{
		if (!ParentObject)
		{
			continue;
		}

		for (TFieldIterator<FProperty> PropertyIt(ParentObject->GetClass()); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			if (!Property)
			{
				continue;
			}

			// Only support booleans, floats, and integers.
			const bool bIsValidType = Property->IsA(FBoolProperty::StaticClass()) || Property->IsA(FIntProperty::StaticClass()) || Property->IsA(FFloatProperty::StaticClass());
			if (!bIsValidType)
			{
				continue;
			}

			// Only accept properties from a blueprint.
			if (!UBlueprint::GetBlueprintFromClass(Property->GetOwnerClass()))
			{
				continue;
			}

			// Ignore properties that don't have a GUID since we rely on it to handle property name changes.
			FGuid PropertyToTestGuid = GetPropertyGuid(Property);
			if (!PropertyToTestGuid.IsValid())
			{
				continue;
			}

			// Add the property to the combo box.
			PropertyOptions.AddUnique(Property);

			// Find our current selected property in the list.
			if (SelectedPropertyGuid == PropertyToTestGuid)
			{
				FoundProperty = Property;
			}
		}
	}

	// Sort the options list alphabetically.
	PropertyOptions.StableSort([](const TFieldPath<FProperty>& A, const TFieldPath<FProperty>& B) { return (A->GetName() < B->GetName()); });

	if ((FoundProperty == nullptr) || (FoundProperty != SelectedProperty) || (FoundProperty->GetName() != SelectedPropertyName))
	{
		// The selected property needs to be updated.
		OnChangeProperty(FoundProperty, ESelectInfo::Direct);
	}

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
}

void FGameplayTagBlueprintPropertyMappingDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> TagToMapHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayTagBlueprintPropertyMapping, TagToMap));
	TSharedPtr<IPropertyHandle> PropertyToEditHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayTagBlueprintPropertyMapping, PropertyToEdit));

	// Add the FGameplay Tag first.
	StructBuilder.AddProperty(TagToMapHandle.ToSharedRef());

	// Add the combo box next.
	IDetailPropertyRow& PropertyRow = StructBuilder.AddProperty(StructPropertyHandle);
	PropertyRow.CustomWidget()
		.NameContent()
		[
			PropertyToEditHandle->CreatePropertyNameWidget()
		]

		.ValueContent()
		.HAlign(HAlign_Left)
		.MinDesiredWidth(250.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SComboBox< TFieldPath<FProperty> >)
				.OptionsSource(&PropertyOptions)
				.OnGenerateWidget(this, &FGameplayTagBlueprintPropertyMappingDetails::GeneratePropertyWidget)
				.OnSelectionChanged(this, &FGameplayTagBlueprintPropertyMappingDetails::OnChangeProperty)
				.ContentPadding(FMargin(2.0f, 2.0f))
				.ToolTipText(this, &FGameplayTagBlueprintPropertyMappingDetails::GetSelectedValueText)
				.InitiallySelectedItem(SelectedProperty)
				[
					SNew(STextBlock)
					.Text(this, &FGameplayTagBlueprintPropertyMappingDetails::GetSelectedValueText)
				]
			]
		];
}

void FGameplayTagBlueprintPropertyMappingDetails::OnChangeProperty(TFieldPath<FProperty> ItemSelected, ESelectInfo::Type SelectInfo)
{
	if (NamePropertyHandle.IsValid() && GuidPropertyHandle.IsValid())
	{
		SelectedProperty = ItemSelected;

		NamePropertyHandle->SetValue(GetPropertyName(ItemSelected));

		TArray<void*> RawData;
		GuidPropertyHandle->AccessRawData(RawData);

		if (RawData.Num() > 0)
		{
			FGuid* RawGuid = static_cast<FGuid*>(RawData[0]);
			*RawGuid = GetPropertyGuid(ItemSelected);
		}
	}
}

FGuid FGameplayTagBlueprintPropertyMappingDetails::GetPropertyGuid(TFieldPath<FProperty> Property) const
{
	FGuid Guid;

	if (Property != nullptr)
	{
		UBlueprint::GetGuidFromClassByFieldName<FProperty>(Property->GetOwnerClass(), Property->GetFName(), Guid);
	}

	return Guid;
}

FString FGameplayTagBlueprintPropertyMappingDetails::GetPropertyName(TFieldPath<FProperty> Property) const
{
	return (Property != nullptr ? Property->GetName() : TEXT("None"));
}

TSharedRef<SWidget> FGameplayTagBlueprintPropertyMappingDetails::GeneratePropertyWidget(TFieldPath<FProperty> Property)
{
	return SNew(STextBlock).Text(FText::FromString(GetPropertyName(Property.Get())));
}

FText FGameplayTagBlueprintPropertyMappingDetails::GetSelectedValueText() const
{
	return FText::FromString(GetPropertyName(SelectedProperty));
}
