// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/FieldPath.h"
#include "IPropertyTypeCustomization.h"


class IPropertyHandle;
class FDetailWidgetRow;
class IDetailChildrenBuilder;
class SWidget;


class FGameplayTagBlueprintPropertyMappingDetails : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:

	void OnChangeProperty(TFieldPath<FProperty> ItemSelected, ESelectInfo::Type SelectInfo);

	FGuid GetPropertyGuid(TFieldPath<FProperty> Property) const;
	FString GetPropertyName(TFieldPath<FProperty> Property) const;

	TSharedRef<SWidget> GeneratePropertyWidget(TFieldPath<FProperty> Property);

	FText GetSelectedValueText() const;

protected:

	TSharedPtr<IPropertyHandle> NamePropertyHandle;
	TSharedPtr<IPropertyHandle> GuidPropertyHandle;

	TArray<TFieldPath<FProperty>> PropertyOptions;

	TFieldPath<FProperty> SelectedProperty;
};
