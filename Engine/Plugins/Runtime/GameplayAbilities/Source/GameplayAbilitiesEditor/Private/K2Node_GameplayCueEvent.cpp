// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_GameplayCueEvent.h"
#include "EdGraph/EdGraph.h"
#include "GameplayTagContainer.h"
#include "GameplayTagsManager.h"
#include "GameplayCueInterface.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintEventNodeSpawner.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "BlueprintActionDatabaseRegistrar.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GameplayCueEvent)

#define LOCTEXT_NAMESPACE "K2Node_GameplayCueEvent"

static FName GAMEPLAYABILITIES_BlueprintCustomHandler = TEXT("BlueprintCustomHandler");

UK2Node_GameplayCueEvent::UK2Node_GameplayCueEvent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EventReference.SetExternalMember(GAMEPLAYABILITIES_BlueprintCustomHandler, UGameplayCueInterface::StaticClass());
}

void UK2Node_GameplayCueEvent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if(Ar.IsLoading())
	{
		if(Ar.UEVer() < VER_UE4_K2NODE_EVENT_MEMBER_REFERENCE && EventSignatureName_DEPRECATED.IsNone() && EventSignatureClass_DEPRECATED == nullptr)
		{
			EventReference.SetExternalMember(GAMEPLAYABILITIES_BlueprintCustomHandler, UGameplayCueInterface::StaticClass());
		}
	}
}

FText UK2Node_GameplayCueEvent::GetTooltipText() const
{
	return FText::Format(LOCTEXT("GameplayCueEvent_Tooltip", "Handle GameplayCue Event {0}"), FText::FromName(CustomFunctionName));
}

FText UK2Node_GameplayCueEvent::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromName(CustomFunctionName);
	//return LOCTEXT("HandleGameplayCueEvent", "HandleGameplaCueEvent");
}

bool UK2Node_GameplayCueEvent::IsCompatibleWithGraph(UEdGraph const* TargetGraph) const
{
	bool bIsCompatible = false;
	if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph))
	{
		bIsCompatible = Blueprint->GeneratedClass && Blueprint->GeneratedClass->ImplementsInterface(UGameplayCueInterface::StaticClass());
	}	
	return bIsCompatible && Super::IsCompatibleWithGraph(TargetGraph);
}

void UK2Node_GameplayCueEvent::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (!ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		return;
	}

	auto CustomizeCueNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FName TagName)
	{
		UK2Node_GameplayCueEvent* EventNode = CastChecked<UK2Node_GameplayCueEvent>(NewNode);
		EventNode->CustomFunctionName = TagName;
	};
	
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
	FGameplayTag RootTag = Manager.RequestGameplayTag(FName(TEXT("GameplayCue")), false);
	if (RootTag.IsValid())
	{
		FGameplayTagContainer CueTags = Manager.RequestGameplayTagChildren(RootTag);
		// Add a root GameplayCue function as a default
		CueTags.AddTag(RootTag);
		for (auto TagIt = CueTags.CreateConstIterator(); TagIt; ++TagIt)
		{
			UBlueprintNodeSpawner::FCustomizeNodeDelegate PostSpawnDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeCueNodeLambda, TagIt->GetTagName());
		
			UBlueprintNodeSpawner* NodeSpawner = UBlueprintEventNodeSpawner::Create(GetClass(), TagIt->GetTagName());
			check(NodeSpawner != nullptr);
			NodeSpawner->CustomizeNodeDelegate = PostSpawnDelegate;
		
			ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
		}
	}
}

#undef LOCTEXT_NAMESPACE

