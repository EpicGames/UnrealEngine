// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_LatentAbilityCall.h"
#include "Abilities/GameplayAbility.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Kismet2/CompilerResultsLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_LatentAbilityCall)

static FName FK2Node_LatentAbilityCallHelper_RequiresConnection(TEXT("RequiresConnection"));

/////////////////////////////////////////////////////
// UK2Node_LatentAbilityCall

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_LatentAbilityCall::UK2Node_LatentAbilityCall(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (HasAnyFlags(RF_ClassDefaultObject) == true)
	{
		UK2Node_LatentGameplayTaskCall::RegisterSpecializedTaskNodeClass(GetClass());
	}
}

bool UK2Node_LatentAbilityCall::IsHandling(TSubclassOf<UGameplayTask> TaskClass) const
{
	return TaskClass && TaskClass->IsChildOf(UAbilityTask::StaticClass());
}

bool UK2Node_LatentAbilityCall::IsCompatibleWithGraph(UEdGraph const* TargetGraph) const
{
	bool bIsCompatible = false;
	
	EGraphType GraphType = TargetGraph->GetSchema()->GetGraphType(TargetGraph);
	bool const bAllowLatentFuncs = (GraphType == GT_Ubergraph || GraphType == GT_Macro);
	
	if (bAllowLatentFuncs)
	{
		UBlueprint* MyBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
		if (MyBlueprint && MyBlueprint->GeneratedClass)
		{
			if (MyBlueprint->GeneratedClass->IsChildOf(UGameplayAbility::StaticClass()))
			{
				bIsCompatible = true;
			}
		}
	}
	
	return bIsCompatible;
}

void UK2Node_LatentAbilityCall::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	struct GetMenuActions_Utils
	{
		static void SetNodeFunc(UEdGraphNode* NewNode, bool /*bIsTemplateNode*/, TWeakObjectPtr<UFunction> FunctionPtr)
		{
			UK2Node_LatentAbilityCall* AsyncTaskNode = CastChecked<UK2Node_LatentAbilityCall>(NewNode);
			if (FunctionPtr.IsValid())
			{
				UFunction* Func = FunctionPtr.Get();
				FObjectProperty* ReturnProp = CastFieldChecked<FObjectProperty>(Func->GetReturnProperty());
						
				AsyncTaskNode->ProxyFactoryFunctionName = Func->GetFName();
				AsyncTaskNode->ProxyFactoryClass        = Func->GetOuterUClass();
				AsyncTaskNode->ProxyClass               = ReturnProp->PropertyClass;
			}
		}
	};

	UClass* NodeClass = GetClass();
	ActionRegistrar.RegisterClassFactoryActions<UAbilityTask>( FBlueprintActionDatabaseRegistrar::FMakeFuncSpawnerDelegate::CreateLambda([NodeClass](const UFunction* FactoryFunc)->UBlueprintNodeSpawner*
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintFunctionNodeSpawner::Create(FactoryFunc);
		check(NodeSpawner != nullptr);
		NodeSpawner->NodeClass = NodeClass;

		TWeakObjectPtr<UFunction> FunctionPtr = MakeWeakObjectPtr(const_cast<UFunction*>(FactoryFunc));
		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(GetMenuActions_Utils::SetNodeFunc, FunctionPtr);

		return NodeSpawner;
	}) );
}

void UK2Node_LatentAbilityCall::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	UFunction* DelegateSignatureFunction = NULL;
	for (TFieldIterator<FProperty> PropertyIt(ProxyClass); PropertyIt; ++PropertyIt)
	{
		if (FMulticastDelegateProperty* Property = CastField<FMulticastDelegateProperty>(*PropertyIt))
		{
			if (Property->GetBoolMetaData(FK2Node_LatentAbilityCallHelper_RequiresConnection))
			{
				if (UEdGraphPin* DelegateExecPin = FindPin(Property->GetFName()))
				{
					if (DelegateExecPin->LinkedTo.Num() < 1)
					{
						const FText MessageText = FText::Format(LOCTEXT("NoConnectionToRequiredExecPin", "@@ - Unhandled event.  You need something connected to the '{0}' pin"), FText::FromName(Property->GetFName()));
						MessageLog.Warning(*MessageText.ToString(), this);
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

