// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayAttributeGraphPin.h"
#include "Widgets/SBoxPanel.h"
#include "AttributeSet.h"
#include "SGameplayAttributeWidget.h"
#include "ScopedTransaction.h"
#include "UObject/CoreRedirects.h"
#include "UObject/UObjectIterator.h"
#include "AbilitySystemComponent.h"

#define LOCTEXT_NAMESPACE "K2Node"

void SGameplayAttributeGraphPin::Construct( const FArguments& InArgs, UEdGraphPin* InGraphPinObj )
{
	SGraphPin::Construct( SGraphPin::FArguments(), InGraphPinObj );
	LastSelectedProperty = nullptr;

}

TSharedRef<SWidget>	SGameplayAttributeGraphPin::GetDefaultValueWidget()
{	
	FGameplayAttribute DefaultAttribute;

	// Parse out current default value
	FString DefaultString = GraphPinObj->GetDefaultAsString();
	if (!DefaultString.IsEmpty())
	{
		UScriptStruct* PinLiteralStructType = FGameplayAttribute::StaticStruct();
		PinLiteralStructType->ImportText(*DefaultString, &DefaultAttribute, nullptr, EPropertyPortFlags::PPF_SerializedAsImportText, GError, PinLiteralStructType->GetName(), true);
	}

	// Create widget
	return SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SGameplayAttributeWidget)
			.OnAttributeChanged(this, &SGameplayAttributeGraphPin::OnAttributeChanged)
			.DefaultProperty(DefaultAttribute.GetUProperty())
			.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
			.IsEnabled(this, &SGameplayAttributeGraphPin::GetDefaultValueIsEnabled)
		];
}

void SGameplayAttributeGraphPin::OnAttributeChanged(FProperty* SelectedAttribute)
{
	FString FinalValue;
	FGameplayAttribute NewAttributeStruct;
	NewAttributeStruct.SetUProperty(SelectedAttribute);

	FGameplayAttribute::StaticStruct()->ExportText(FinalValue, &NewAttributeStruct, &NewAttributeStruct, nullptr, EPropertyPortFlags::PPF_SerializedAsImportText, nullptr);

	if (FinalValue != GraphPinObj->GetDefaultAsString())
	{
		const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangePinValue", "Change Pin Value"));
		GraphPinObj->Modify();
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, FinalValue);
	}

	LastSelectedProperty = SelectedAttribute;
}

#undef LOCTEXT_NAMESPACE
