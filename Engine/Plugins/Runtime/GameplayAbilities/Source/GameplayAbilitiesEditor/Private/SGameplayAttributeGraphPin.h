// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SGraphPin.h"

class SGameplayAttributeGraphPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGameplayAttributeGraphPin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	void OnAttributeChanged(FProperty* SelectedAttribute);

	FProperty* LastSelectedProperty;

private:
	bool GetDefaultValueIsEnabled() const
	{
		return !GraphPinObj->bDefaultValueIsReadOnly;
	}
};
