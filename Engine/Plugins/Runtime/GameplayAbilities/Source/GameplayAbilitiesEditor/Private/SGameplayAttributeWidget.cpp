// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayAttributeWidget.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSearchBox.h"

#include "AbilitySystemComponent.h"
#include "AttributeSet.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/TextFilter.h"
#include "Misc/OutputDeviceNull.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "K2Node"

DECLARE_DELEGATE_OneParam(FOnAttributePicked, FProperty*);

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
struct FAttributeViewerNode
{
public:
	FAttributeViewerNode(FProperty* InAttribute, FString InAttributeName)
	{
		Attribute = InAttribute;
		AttributeName = MakeShareable(new FString(InAttributeName));
	}

	/** The displayed name for this node. */
	TSharedPtr<FString> AttributeName;

	FProperty* Attribute;
};

/** The item used for visualizing the attribute in the list. */
class SAttributeItem : public SComboRow< TSharedPtr<FAttributeViewerNode> >
{
public:

	SLATE_BEGIN_ARGS(SAttributeItem)
		: _HighlightText()
		, _TextColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f))
	{}

	/** The text this item should highlight, if any. */
	SLATE_ARGUMENT(FText, HighlightText)
	/** The color text this item will use. */
	SLATE_ARGUMENT(FSlateColor, TextColor)
	/** The node this item is associated with. */
	SLATE_ARGUMENT(TSharedPtr<FAttributeViewerNode>, AssociatedNode)

	SLATE_END_ARGS()

	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		AssociatedNode = InArgs._AssociatedNode;

		this->ChildSlot
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(0.0f, 3.0f, 6.0f, 3.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromString(*AssociatedNode->AttributeName.Get()))
						.HighlightText(InArgs._HighlightText)
						.ColorAndOpacity(this, &SAttributeItem::GetTextColor)
						.IsEnabled(true)
					]
			];

		TextColor = InArgs._TextColor;

		STableRow< TSharedPtr<FAttributeViewerNode> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true),
			InOwnerTableView
			);
	}

	/** Returns the text color for the item based on if it is selected or not. */
	FSlateColor GetTextColor() const
	{
		const TSharedPtr< ITypedTableView< TSharedPtr<FAttributeViewerNode> > > OwnerWidget = OwnerTablePtr.Pin();
		const TSharedPtr<FAttributeViewerNode>* MyItem = OwnerWidget->Private_ItemFromWidget(this);
		const bool bIsSelected = OwnerWidget->Private_IsItemSelected(*MyItem);

		if (bIsSelected)
		{
			return FSlateColor::UseForeground();
		}

		return TextColor;
	}

private:

	/** The text color for this item. */
	FSlateColor TextColor;

	/** The Attribute Viewer Node this item is associated with. */
	TSharedPtr< FAttributeViewerNode > AssociatedNode;
};

class SAttributeListWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAttributeListWidget)
	{
	}

	SLATE_ARGUMENT(FString, FilterMetaData)
	SLATE_ARGUMENT(FOnAttributePicked, OnAttributePickedDelegate)

	SLATE_END_ARGS()

	/**
	* Construct the widget
	*
	* @param	InArgs			A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs);

	virtual ~SAttributeListWidget();

private:
	typedef TTextFilter< const FProperty& > FAttributeTextFilter;

	/** Called by Slate when the filter box changes text. */
	void OnFilterTextChanged(const FText& InFilterText);

	/** Creates the row widget when called by Slate when an item appears on the list. */
	TSharedRef< ITableRow > OnGenerateRowForAttributeViewer(TSharedPtr<FAttributeViewerNode> Item, const TSharedRef< STableViewBase >& OwnerTable);

	/** Called by Slate when an item is selected from the tree/list. */
	void OnAttributeSelectionChanged(TSharedPtr<FAttributeViewerNode> Item, ESelectInfo::Type SelectInfo);

	/** Updates the list of items in the dropdown menu */
	TSharedPtr<FAttributeViewerNode> UpdatePropertyOptions();

	/** Delegate to be called when an attribute is picked from the list */
	FOnAttributePicked OnAttributePicked;

	/** The search box */
	TSharedPtr<SSearchBox> SearchBoxPtr;

	/** Holds the Slate List widget which holds the attributes for the Attribute Viewer. */
	TSharedPtr<SListView<TSharedPtr< FAttributeViewerNode > >> AttributeList;

	/** Array of items that can be selected in the dropdown menu */
	TArray<TSharedPtr<FAttributeViewerNode>> PropertyOptions;

	/** Filters needed for filtering the assets */
	TSharedPtr<FAttributeTextFilter> AttributeTextFilter;

	/** Filter for meta data */
	FString FilterMetaData;
};

SAttributeListWidget::~SAttributeListWidget()
{
	if (OnAttributePicked.IsBound())
	{
		OnAttributePicked.Unbind();
	}
}

void SAttributeListWidget::Construct(const FArguments& InArgs)
{
	struct Local
	{
		static void AttributeToStringArray(const FProperty& Property, OUT TArray< FString >& StringArray)
		{
			UClass* Class = Property.GetOwnerClass();
			if ((Class->IsChildOf(UAttributeSet::StaticClass()) && !Class->ClassGeneratedBy) ||
				(Class->IsChildOf(UAbilitySystemComponent::StaticClass()) && !Class->ClassGeneratedBy))
			{
				StringArray.Add(FString::Printf(TEXT("%s.%s"), *Class->GetName(), *Property.GetName()));
			}
		}
	};

	FilterMetaData = InArgs._FilterMetaData;
	OnAttributePicked = InArgs._OnAttributePickedDelegate;

	// Setup text filtering
	AttributeTextFilter = MakeShareable(new FAttributeTextFilter(FAttributeTextFilter::FItemToStringArray::CreateStatic(&Local::AttributeToStringArray)));

	UpdatePropertyOptions();

	TSharedPtr< SWidget > ClassViewerContent;

	SAssignNew(ClassViewerContent, SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(SearchBoxPtr, SSearchBox)
			.HintText(NSLOCTEXT("Abilities", "SearchBoxHint", "Search Attributes"))
			.OnTextChanged(this, &SAttributeListWidget::OnFilterTextChanged)
			.DelayChangeNotificationsWhileTyping(true)
		]

	+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Visibility(EVisibility::Collapsed)
		]
	
	+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(AttributeList, SListView<TSharedPtr< FAttributeViewerNode > >)
			.Visibility(EVisibility::Visible)
			.SelectionMode(ESelectionMode::Single)
			.ListItemsSource(&PropertyOptions)

 			// Generates the actual widget for a tree item
			.OnGenerateRow(this, &SAttributeListWidget::OnGenerateRowForAttributeViewer)

 			// Find out when the user selects something in the tree
			.OnSelectionChanged(this, &SAttributeListWidget::OnAttributeSelectionChanged)
		];


	ChildSlot
		[
			ClassViewerContent.ToSharedRef()
		];
}

TSharedRef< ITableRow > SAttributeListWidget::OnGenerateRowForAttributeViewer(TSharedPtr<FAttributeViewerNode> Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	TSharedRef< SAttributeItem > ReturnRow = SNew(SAttributeItem, OwnerTable)
		.HighlightText(SearchBoxPtr->GetText())
		.TextColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.f))
		.AssociatedNode(Item);

	return ReturnRow;
}

TSharedPtr<FAttributeViewerNode> SAttributeListWidget::UpdatePropertyOptions()
{
	PropertyOptions.Empty();
	TSharedPtr<FAttributeViewerNode> InitiallySelected = MakeShareable(new FAttributeViewerNode(nullptr, "None"));

	PropertyOptions.Add(InitiallySelected);

	// Gather all UAttribute classes
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass *Class = *ClassIt;
		if (Class->IsChildOf(UAttributeSet::StaticClass()) && !Class->ClassGeneratedBy)
		{
			// Allow entire classes to be filtered globally
			if (Class->HasMetaData(TEXT("HideInDetailsView")))
			{
				continue;
			}

			for (TFieldIterator<FProperty> PropertyIt(Class, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
			{
				FProperty *Property = *PropertyIt;

				// Ignore property types that cannot represent gameplay attributes
				if (!FGameplayAttribute::IsSupportedProperty(Property))
				{
					continue;
				}

				// if we have a search string and this doesn't match, don't show it
				if (AttributeTextFilter.IsValid() && !AttributeTextFilter->PassesFilter(*Property))
				{
					continue;
				}

				// don't show attributes that are filtered by meta data
				if (!FilterMetaData.IsEmpty() && Property->HasMetaData(*FilterMetaData))
				{
					continue;
				}

				// Allow properties to be filtered globally (never show up)
				if (Property->HasMetaData(TEXT("HideInDetailsView")))
				{
					continue;
				}

				TSharedPtr<FAttributeViewerNode> SelectableProperty = MakeShareable(new FAttributeViewerNode(Property, FString::Printf(TEXT("%s.%s"), *Class->GetName(), *Property->GetName())));
				PropertyOptions.Add(SelectableProperty);
			}
		}

		// UAbilitySystemComponent can add 'system' attributes
		if (Class->IsChildOf(UAbilitySystemComponent::StaticClass()) && !Class->ClassGeneratedBy)
		{
			for (TFieldIterator<FProperty> PropertyIt(Class, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
			{
				FProperty* Property = *PropertyIt;

				// SystemAttributes have to be explicitly tagged
				if (Property->HasMetaData(TEXT("SystemGameplayAttribute")) == false)
				{
					continue;
				}

				// if we have a search string and this doesn't match, don't show it
				if (AttributeTextFilter.IsValid() && !AttributeTextFilter->PassesFilter(*Property))
				{
					continue;
				}

				TSharedPtr<FAttributeViewerNode> SelectableProperty = MakeShareable(new FAttributeViewerNode(Property, FString::Printf(TEXT("%s.%s"), *Class->GetName(), *Property->GetName())));
				PropertyOptions.Add(SelectableProperty);
			}
		}
	}

	return InitiallySelected;
}

void SAttributeListWidget::OnFilterTextChanged(const FText& InFilterText)
{
	AttributeTextFilter->SetRawFilterText(InFilterText);
	SearchBoxPtr->SetError(AttributeTextFilter->GetFilterErrorText());

	UpdatePropertyOptions();
}

void SAttributeListWidget::OnAttributeSelectionChanged(TSharedPtr<FAttributeViewerNode> Item, ESelectInfo::Type SelectInfo)
{
	OnAttributePicked.ExecuteIfBound(Item->Attribute);
}

void SGameplayAttributeWidget::Construct(const FArguments& InArgs)
{
	FilterMetaData = InArgs._FilterMetaData;
	OnAttributeChanged = InArgs._OnAttributeChanged;
	SelectedProperty = InArgs._DefaultProperty;

	TWeakPtr<SGameplayAttributeWidget> WeakSelf = StaticCastWeakPtr<SGameplayAttributeWidget>(AsWeak());

	ChildSlot
	[
		SNew(SBorder)
			.OnMouseButtonDown_Lambda([WeakSelf](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
				{
					const TSharedPtr<SGameplayAttributeWidget> Self = WeakSelf.Pin();
					if (Self.IsValid() && MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
					{
						Self->OnMenu(MouseEvent);
						return FReply::Handled();
					}
					return FReply::Unhandled();
				})
			.Padding(0.0f)
			.BorderImage(FStyleDefaults::GetNoBrush())
			[
				// set up the combo button
				SAssignNew(ComboButton, SComboButton)
					.OnGetMenuContent(this, &SGameplayAttributeWidget::GenerateAttributePicker)
					.ContentPadding(FMargin(2.0f, 2.0f))
					.ToolTipText(this, &SGameplayAttributeWidget::GetSelectedValueAsString)
					.ButtonContent()
					[
						SNew(STextBlock)
							.Text(this, &SGameplayAttributeWidget::GetSelectedValueAsString)
					]
			]
	];
}

void SGameplayAttributeWidget::OnCopyAttribute(FProperty* AttributeProperty)
{
	FGameplayAttribute Attribute(AttributeProperty);

	if (Attribute.IsValid())
	{
		if (const TObjectPtr<UObject> OwnerObject = Attribute.GetAttributeSetClass()->GetDefaultObject(false))
		{
			FString ExportedString;

			FGameplayAttribute::StaticStruct()->ExportText(ExportedString, &Attribute, &Attribute, OwnerObject, 0, nullptr);
			FPlatformApplicationMisc::ClipboardCopy(*ExportedString);
		}
	}
}

FGameplayAttribute AttributeTryImportTextFromClipboard()
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	FOutputDeviceNull NullOut;
	FGameplayAttribute Attribute;
	FGameplayAttribute::StaticStruct()->ImportText(*PastedText, &Attribute, /*OwnerObject*/nullptr, 0, &NullOut, FGameplayAttribute::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);

	return Attribute;
}

bool SGameplayAttributeWidget::CanPaste() const
{
	const FGameplayAttribute Attribute = AttributeTryImportTextFromClipboard();

	return Attribute.IsValid();
}

void SGameplayAttributeWidget::OnPasteAttribute()
{
	const FGameplayAttribute Attribute = AttributeTryImportTextFromClipboard();

	if (Attribute.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("GameplayAttributeWidget_PasteAttribute", "Paste Gameplay Attribute"));

		SelectedProperty = Attribute.GetUProperty();
		OnAttributeChanged.ExecuteIfBound(SelectedProperty);
	}
}

void SGameplayAttributeWidget::OnMenu(const FPointerEvent& MouseEvent)
{
	FMenuBuilder MenuBuilder(/*bShouldCloseWindowAfterMenuSelection=*/ true, /*CommandList=*/ nullptr);

	FGameplayAttribute SelectedAttribute(SelectedProperty);

	auto IsValidAttribute = [](const FGameplayAttribute& Attribute)
		{
			return Attribute.IsValid();
		};

	MenuBuilder.AddMenuEntry(
		LOCTEXT("GameplayAttribute_SearchForReferences", "Search For References"),
		FText::Format(LOCTEXT("GameplayAttributeWidget_SearchForReferencesTooltip", "Find references to attribute {0}"), GetSelectedValueAsString()),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Search"),
		FUIAction(FExecuteAction::CreateLambda([&]()
			{
				const FText AttributeName = GetSelectedValueAsString();
				if (FEditorDelegates::OnOpenReferenceViewer.IsBound() && !AttributeName.IsEmpty())
				{
					TArray<FAssetIdentifier> AssetIdentifiers;
					AssetIdentifiers.Emplace(FGameplayAttribute::StaticStruct(), *AttributeName.ToString());
					FEditorDelegates::OnOpenReferenceViewer.Broadcast(AssetIdentifiers, FReferenceViewerParams());
				}
			}))
	);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry(
		NSLOCTEXT("PropertyView", "CopyProperty", "Copy"),
		FText::Format(LOCTEXT("GameplayAttributeWidget_CopyAttributeTooltip", "Copy attribute {0} to clipboard"), GetSelectedValueAsString()),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayAttributeWidget::OnCopyAttribute, SelectedProperty), FCanExecuteAction::CreateLambda(IsValidAttribute, SelectedAttribute)));

	const FGameplayAttribute Attribute = AttributeTryImportTextFromClipboard();
	FText FormattedPastedAttribute;

	if (Attribute.GetUProperty())
	{
		UClass* Class = Attribute.GetUProperty()->GetOwnerClass();
		FString PropertyString = FString::Printf(TEXT("%s.%s"), *Class->GetName(), *Attribute.GetUProperty()->GetName());
		FormattedPastedAttribute = FText::FromString(PropertyString);
	}
	else
	{
		FormattedPastedAttribute = FText::FromString(TEXT("None"));
	}

	MenuBuilder.AddMenuEntry(
		NSLOCTEXT("PropertyView", "PasteProperty", "Paste"),
		FText::Format(LOCTEXT("GameplayAttributeWidget_PasteAttributeTooltip", "Paste attribute ({0}) from clipboard."), FormattedPastedAttribute),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste"),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayAttributeWidget::OnPasteAttribute), FCanExecuteAction::CreateSP(this, &SGameplayAttributeWidget::CanPaste)));

	// Spawn context menu
	FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
	FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

}

void SGameplayAttributeWidget::OnAttributePicked(FProperty* InProperty)
{
	if (OnAttributeChanged.IsBound())
	{
		OnAttributeChanged.Execute(InProperty);
	}

	// Update the selected item for displaying
	SelectedProperty = InProperty;

	// close the list
	ComboButton->SetIsOpen(false);
}

TSharedRef<SWidget> SGameplayAttributeWidget::GenerateAttributePicker()
{
	FOnAttributePicked OnPicked(FOnAttributePicked::CreateRaw(this, &SGameplayAttributeWidget::OnAttributePicked));

	return SNew(SBox)
		.WidthOverride(280)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(500)
			[
				SNew(SAttributeListWidget)
				.OnAttributePickedDelegate(OnPicked)
				.FilterMetaData(FilterMetaData)
			]
		];
}

FText SGameplayAttributeWidget::GetSelectedValueAsString() const
{
	if (SelectedProperty)
	{
		UClass* Class = SelectedProperty->GetOwnerClass();
		FString PropertyString = FString::Printf(TEXT("%s.%s"), *Class->GetName(), *SelectedProperty->GetName());
		return FText::FromString(PropertyString);
	}

	return FText::FromString(TEXT("None"));
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE
