// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"

class SGameplayAttributeWidget : public SCompoundWidget
{
public:

	/** Called when a tag status is changed */
	DECLARE_DELEGATE_OneParam(FOnAttributeChanged, FProperty*)

	SLATE_BEGIN_ARGS(SGameplayAttributeWidget)
	: _FilterMetaData()
	, _DefaultProperty(nullptr)
	{}
	SLATE_ARGUMENT(FString, FilterMetaData)
	SLATE_ARGUMENT(FProperty*, DefaultProperty)
	SLATE_EVENT(FOnAttributeChanged, OnAttributeChanged) // Called when a tag status changes
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);	

private:

	TSharedRef<SWidget> GenerateAttributePicker();

	FText GetSelectedValueAsString() const;

	/**	Called when the widget is clicked using RMB */
	void OnMenu(const FPointerEvent& MouseEvent);

	void OnCopyAttribute(FProperty* AttributeProperty);
	bool CanPaste() const;
	void OnPasteAttribute();

	/** Handles updates when the selected attribute changes */
	void OnAttributePicked(FProperty* InProperty);

	/** Delegate to call when the selected attribute changes */
	FOnAttributeChanged OnAttributeChanged;

	/** The search string being used to filter the attributes */
	FString FilterMetaData;

	/** The currently selected attribute */
	FProperty* SelectedProperty;

	/** Used to display an attribute picker. */
	TSharedPtr<class SComboButton> ComboButton;
};
