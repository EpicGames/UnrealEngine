// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayCueEditor.h"
#include "Modules/ModuleManager.h"
#include "GameplayAbilitiesEditorModule.h"
#include "Misc/Paths.h"
#include "Stats/StatsMisc.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Engine/Blueprint.h"
#include "Factories/BlueprintFactory.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/ObjectLibrary.h"
#include "Widgets/Input/SComboButton.h"
#include "GameplayTagContainer.h"
#include "UObject/UnrealType.h"
#include "AbilitySystemLog.h"
#include "GameplayCueSet.h"
#include "GameplayCueNotify_Actor.h"
#include "GameplayTagsManager.h"
#include "GameplayCueTranslator.h"
#include "GameplayCueManager.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "GameplayCueNotify_Static.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SSearchBox.h"
#include "GameplayTagsModule.h"

#include "AbilitySystemGlobals.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameplayTagsEditorModule.h"
#include "LevelEditor.h"
#include "SAddNewGameplayTagWidget.h"

#include "SGameplayCueEditor_Picker.h"
#include "Subsystems/AssetEditorSubsystem.h"
#define LOCTEXT_NAMESPACE "SGameplayCueEditor"

static const FName CueTagColumnName("GameplayCueTags");
static const FName CueHandlerColumnName("GameplayCueHandlers");
static const FString CueTagPrefix(TEXT("GameplayCue."));

// Whether to show the Hotreload button in the GC editor.
#define GAMEPLAYCUEEDITOR_HOTRELOAD_BUTTON 0

// Whether to enable the "show only leaf tags option", if 0, the option is enabled by default. (This is probably not a useful thing to have, in case it ever is, this can be reenabled)
#define GAMEPLAYCUEEDITOR_SHOW_ONLY_LEAFTAGS_OPTION 0

/** Base class for any item in the the Cue/Handler Tree. */
struct FGCTreeItem : public TSharedFromThis<FGCTreeItem>
{
	FGCTreeItem()
	{
		TranslationUniqueID = 0;
	}
	virtual ~FGCTreeItem(){}


	FName GameplayCueTagName;
	FGameplayTag GameplayCueTag;
	FString Description;

	FSoftObjectPath GameplayCueNotifyObj;
	FSoftObjectPath ParentGameplayCueNotifyObj;
	TWeakObjectPtr<UFunction> FunctionPtr;

	int32 TranslationUniqueID;

	TArray< TSharedPtr< FGCTreeItem > > Children;
};

typedef STreeView< TSharedPtr< FGCTreeItem > > SGameplayCueTreeView;

// -----------------------------------------------------------------

/** Base class for items in the filtering tree (for gameplay cue translator filtering) */
struct FGCFilterTreeItem : public TSharedFromThis<FGCFilterTreeItem>
{
	virtual ~FGCFilterTreeItem() {}
	
	FGameplayCueTranslationEditorOnlyData Data;	
	TArray<FName> ToNames;
	TArray< TSharedPtr< FGCFilterTreeItem > > Children;
};

typedef STreeView< TSharedPtr< FGCFilterTreeItem > > SFilterTreeView;

// -----------------------------------------------------------------

class SGameplayCueEditorImpl : public SGameplayCueEditor
{
public:

	virtual void Construct(const FArguments& InArgs) override;

	/** Helper function to determine the visibility of the Add New Tag widget */
	EVisibility DetermineAddNewTagWidgetVisibility() const;

	/** Helper functions to expand or collapse the add new tag widget */
	EVisibility DetermineExpandableUIVisibility() const;
	ECheckBoxState GetAddTagSectionExpansionState() const;
	void OnAddTagSectionExpansionStateChanged(ECheckBoxState NewState);

private:

	// ---------------------------------------------------------------------

	/* Expand the Add Tag section (if it's visible) */
	bool bAddTagSectionExpanded;

	/** Show all GC Tags, even ones without handlers */
	bool bShowAll;

	/** Show all possible overrides, even ones that don't exist */
	bool bShowAllOverrides;

	/** Show only GC Tags that explicitly exist. If a.b.c is in the dictionary, don't show a.b as a distinct tag */
	bool bShowOnlyLeafTags;

	/** Track when filter state is dirty, so that we only rebuilt view when it has changed, once menu is closed. */
	bool bFilterIDsDirty;

	/** Global flag suppress rebuilding cue tree view. Needed when doing operations that would rebuld it multiple times */
	static bool bSuppressCueViewUpdate;

	/** Main widgets that shows the gameplay cue tree */
	TSharedPtr<SGameplayCueTreeView> GameplayCueTreeView;

	/** Source of GC tree view items */
	TArray< TSharedPtr< FGCTreeItem > > GameplayCueListItems;

	/** Widget for the override/transition filters */
	TSharedPtr< SFilterTreeView > FilterTreeView;

	/** Source of filter items */
	TArray< TSharedPtr< FGCFilterTreeItem > > FilterListItems;

	/** Tracking which filters are selected (by transition UniqueIDs) */
	TArray<int32> FilterIds;

	/** Map for viewing GC blueprint events (only built if user wants to) */
	TMultiMap<FGameplayTag, UFunction*> EventMap;

	/** Last selected tag. Used to keep tag selection in between recreation of GC view */
	FName SelectedTag;

	/** Last selected tag, uniqueID if it came from a translated tag. Used to get the right tag selected (nested vs root) */
	int32 SelectedUniqueID;

	/** Pointer to actual selected item */
	TSharedPtr<FGCTreeItem> SelectedItem;

	/** Search text for highlighting */
	FText SearchText;

	/** The search box widget */
	TSharedPtr< class SSearchBox > SearchBoxPtr;

	/** For tracking expanded tags in between recreation of GC view */
	TSet<FName>	ExpandedTags;

	/** The widget that controls how new gameplay tags are added to the config files */
	TSharedPtr<class SAddNewGameplayTagWidget> AddNewTagWidget;

public:

	virtual void OnNewGameplayCueTagCommited(const FString& TagName, const FString& TagComment, const FName& TagSource) override
	{
		FScopedSlowTask SlowTask(0.f, LOCTEXT("AddingNewGameplaycue", "Adding new GameplayCue Tag"));
		SlowTask.MakeDialog();

		check(!TagName.IsEmpty());

		TSharedPtr<FGameplayTagNode> TagNode = UGameplayTagsManager::Get().FindTagNode(FName(*TagName));
		const FGameplayTag* NewTag = TagNode.IsValid() ? &TagNode->GetCompleteTag() : nullptr;

		UpdateGameplayCueListItems(NewTag);
	}

	bool IsValidTag(const FString& TagName, FText* ErrorMsg) const
	{
		if (!TagName.StartsWith(CueTagPrefix))
		{
			if (ErrorMsg)
			{
				*ErrorMsg = FText::Format(LOCTEXT("BadPrefixError", "Gameplay cue tags must start with {0}"), FText::FromString(CueTagPrefix));
			}
			return false;
		}

		return true;
	}

	virtual void OnSearchTagCommited(const FText& InText, ETextCommit::Type InCommitType) override
	{
		if (InCommitType == ETextCommit::OnEnter || InCommitType == ETextCommit::OnCleared || InCommitType == ETextCommit::OnUserMovedFocus)
		{
			if (SearchText.EqualTo(InText) == false)
			{
				SearchText = InText;
				UpdateGameplayCueListItems();
			}
		}
	}

	FReply DoSearch()
	{
		UpdateGameplayCueListItems();
		return FReply::Handled();
	}


	void OnFilterMenuOpenChanged(bool bOpen)
	{
		if (bOpen == false && bFilterIDsDirty)
		{
			UpdateGameplayCueListItems();
			bFilterIDsDirty = false;
		}
	}

	void HandleShowAllCheckedStateChanged(ECheckBoxState NewValue)
	{
		bShowAll = (NewValue == ECheckBoxState::Unchecked);
		UpdateGameplayCueListItems();
	}

	void HandleShowAllOverridesCheckedStateChanged(ECheckBoxState NewValue)
	{
		bShowAllOverrides = (NewValue == ECheckBoxState::Checked);
		UpdateGameplayCueListItems();
	}

	void HandleShowOnLeafTagsCheckedStateChanged(ECheckBoxState NewValue)
	{
		bShowOnlyLeafTags = (NewValue == ECheckBoxState::Checked);
		UpdateGameplayCueListItems();
	}

	ECheckBoxState HandleShowAllCheckBoxIsChecked() const
	{
		return bShowAll ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
	}	
	
	ECheckBoxState HandleShowAllOverridesCheckBoxIsChecked() const
	{
		return bShowAllOverrides ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	ECheckBoxState HandleShowOnlyLeafTagsCheckBoxIsChecked() const
	{
		return bShowOnlyLeafTags ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void HandleNotify_OpenAssetInEditor(FString AssetName, int AssetType)
	{
		if (AssetType == 0)
		{
			if ( SearchBoxPtr.IsValid() )
			{
				SearchBoxPtr->SetText(FText::FromString(AssetName));
			}

			SearchText = FText::FromString(AssetName);
			UpdateGameplayCueListItems();

			if (GameplayCueListItems.Num() == 1)
			{	//If there is only one element, open it.
				if (GameplayCueListItems[0]->GameplayCueNotifyObj.IsValid())
				{
					SGameplayCueEditor::OpenEditorForNotify(GameplayCueListItems[0]->GameplayCueNotifyObj.ToString());
				}
				else if (GameplayCueListItems[0]->FunctionPtr.IsValid())
				{
					SGameplayCueEditor::OpenEditorForNotify(GameplayCueListItems[0]->FunctionPtr->GetOuter()->GetPathName());
				}
			}
		}
	}

	void HandleNotify_FindAssetInEditor(FString AssetName, int AssetType)
	{
		if (AssetType == 0)
		{
			if ( SearchBoxPtr.IsValid() )
			{
				SearchBoxPtr->SetText(FText::FromString(AssetName));
			}

			SearchText = FText::FromString(AssetName);
			UpdateGameplayCueListItems();
		}
	}


	// -----------------------------------------------------------------

	TSharedRef<SWidget> GetFilterListContent()
	{
		if (FilterTreeView.IsValid() == false)
		{
			SAssignNew(FilterTreeView, SFilterTreeView)
			.TreeItemsSource(&FilterListItems)
			.OnGenerateRow(this, &SGameplayCueEditorImpl::OnGenerateWidgetForFilterListView)
			.OnGetChildren( this, &SGameplayCueEditorImpl::OnGetFilterChildren )
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(CueTagColumnName)
				.DefaultLabel(NSLOCTEXT("GameplayCueEditor", "GameplayCueTagTrans", "Translator"))
			);
		}

		UpdateFilterListItems();
		ExpandFilterItems();
		bFilterIDsDirty = false;

		return 
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			FilterTreeView.ToSharedRef()
		];
	}

	/** Builds widget for rows in the GameplayCue Editor tab */
	TSharedRef<ITableRow> OnGenerateWidgetForGameplayCueListView(TSharedPtr< FGCTreeItem > InItem, const TSharedRef<STableViewBase>& OwnerTable);

	void OnFilterStateChanged(ECheckBoxState NewValue, TSharedPtr< FGCFilterTreeItem > Item)
	{
		if (NewValue == ECheckBoxState::Checked)
		{
			FilterIds.AddUnique(Item->Data.UniqueID);
			bFilterIDsDirty = true;
		}
		else if (NewValue == ECheckBoxState::Unchecked)
		{
			FilterIds.Remove(Item->Data.UniqueID);
			bFilterIDsDirty = true;
		}
	}

	ECheckBoxState IsFilterChecked(TSharedPtr<FGCFilterTreeItem> Item) const
	{
		return FilterIds.Contains(Item->Data.UniqueID) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	TSharedRef<ITableRow> OnGenerateWidgetForFilterListView(TSharedPtr< FGCFilterTreeItem > InItem, const TSharedRef<STableViewBase>& OwnerTable);

	void OnPropertyValueChanged()
	{
		UpdateGameplayCueListItems();
	}

	void OnGetChildren(TSharedPtr<FGCTreeItem> Item, TArray< TSharedPtr<FGCTreeItem> >& Children)
	{
		if (Item.IsValid())
		{
			Children.Append(Item->Children);
		}
	}

	void OnGetFilterChildren(TSharedPtr<FGCFilterTreeItem> Item, TArray< TSharedPtr<FGCFilterTreeItem> >& Children)
	{
		if (Item.IsValid())
		{
			Children.Append(Item->Children);
		}
	}

	bool AddChildTranslatedTags_r(FName ThisGameplayCueTag, UGameplayCueManager* CueManager, TSharedPtr< FGCTreeItem > NewItem)
	{
		bool ChildPassedFilter = false;
		TArray<FGameplayCueTranslationEditorInfo> ChildrenTranslatedTags;
		if (CueManager->TranslationManager.GetTranslatedTags(ThisGameplayCueTag, ChildrenTranslatedTags)) 
		{
			for (FGameplayCueTranslationEditorInfo& ChildInfo : ChildrenTranslatedTags)
			{
				TSharedPtr< FGCTreeItem > NewHandlerItem(new FGCTreeItem());					
				NewHandlerItem->GameplayCueTagName = ChildInfo.GameplayTagName;
				NewHandlerItem->GameplayCueTag = ChildInfo.GameplayTag;
				NewHandlerItem->ParentGameplayCueNotifyObj = NewItem->GameplayCueNotifyObj.IsValid() ?  NewItem->GameplayCueNotifyObj :  NewItem->ParentGameplayCueNotifyObj;

				// Should this be filtered out?
				bool PassedFilter = (FilterIds.Num() == 0 || FilterIds.Contains(ChildInfo.EditorData.UniqueID));
				PassedFilter |= AddChildTranslatedTags_r(ChildInfo.GameplayTagName, CueManager, NewHandlerItem);
				ChildPassedFilter |= PassedFilter;

				if (PassedFilter)
				{
					FindGameplayCueNotifyObj(CueManager, NewHandlerItem);
					NewHandlerItem->Description = ChildInfo.EditorData.EditorDescription.ToString();
					NewHandlerItem->TranslationUniqueID = ChildInfo.EditorData.UniqueID;

					NewItem->Children.Add(NewHandlerItem);

					if (ExpandedTags.Contains(NewHandlerItem->GameplayCueTagName))
					{
						GameplayCueTreeView->SetItemExpansion(NewHandlerItem, true);
					}

					CheckSelectGCItem(NewHandlerItem);
				}				
			}
		}

		return ChildPassedFilter;
	}

	bool FindGameplayCueNotifyObj(UGameplayCueManager* CueManager, TSharedPtr< FGCTreeItem >& Item)
	{
		if (CueManager && Item->GameplayCueTag.IsValid())
		{
			UGameplayCueSet* EditorSet = CueManager->GetEditorCueSet();
			if (EditorSet == nullptr)
			{
				return false;
			}

			if (int32* idxPtr = EditorSet->GameplayCueDataMap.Find(Item->GameplayCueTag))
			{
				int32 idx = *idxPtr;
				if (EditorSet->GameplayCueData.IsValidIndex(idx))
				{
					FGameplayCueNotifyData& Data = EditorSet->GameplayCueData[idx];
					Item->GameplayCueNotifyObj = Data.GameplayCueNotifyObj;
					return true;
				}
			}
		}
		return false;
	}

	void CheckSelectGCItem(TSharedPtr< FGCTreeItem > NewItem)
	{
		if (SelectedTag != NAME_None && !SelectedItem.IsValid() && (SelectedTag == NewItem->GameplayCueTagName && NewItem->TranslationUniqueID == SelectedUniqueID))
		{
			SelectedItem = NewItem;
		}
	}

	void UpdateGameplayCueListItems()
	{
		UpdateGameplayCueListItems(nullptr);
	}

	/** Builds content of the list in the GameplayCue Editor */
	void UpdateGameplayCueListItems(const FGameplayTag* SelectedTagPostUpdate)
	{
		if (bSuppressCueViewUpdate)
		{
			return;
		}

#if STATS
		//FString PerfMessage = FString::Printf(TEXT("SGameplayCueEditor::UpdateGameplayCueListItems"));
		//SCOPE_LOG_TIME_IN_SECONDS(*PerfMessage, nullptr)
#endif
		double FindGameplayCueNotifyObjTime = 0.f;
		double AddTranslationTagsTime = 0.f;
		double AddEventsTime = 0.f;


		UGameplayCueManager* CueManager = UAbilitySystemGlobals::Get().GetGameplayCueManager();
		if (!CueManager)
		{
			return;
		}

		GameplayCueListItems.Reset();
		SelectedItem.Reset();

		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
		FString FullSearchString = SearchText.ToString();
		TArray<FString> SearchStrings;
		FullSearchString.ParseIntoArrayWS(SearchStrings);

		// ------------------------------------------------------
		if (bShowAllOverrides)
		{
			// Compute all possible override tags via _Forward method
			CueManager->TranslationManager.BuildTagTranslationTable_Forward();
		}
		else
		{
			// Compute only the existing override tags
			CueManager->TranslationManager.BuildTagTranslationTable();
		}
		// ------------------------------------------------------

		// Get all GC Tags
		FGameplayTagContainer AllGameplayCueTags;
		{
			FString RequestGameplayTagChildrenPerfMessage = FString::Printf(TEXT(" RequestGameplayTagChildren"));
			//SCOPE_LOG_TIME_IN_SECONDS(*RequestGameplayTagChildrenPerfMessage, nullptr)

			if (bShowOnlyLeafTags)
			{
				AllGameplayCueTags = Manager.RequestGameplayTagChildrenInDictionary(UGameplayCueSet::BaseGameplayCueTag());
			}
			else
			{
				AllGameplayCueTags = Manager.RequestGameplayTagChildren(UGameplayCueSet::BaseGameplayCueTag());
			}
		}

		// Create data structs for widgets
		for (const FGameplayTag& ThisGameplayCueTag : AllGameplayCueTags)
		{
			if (SearchStrings.Num() > 0)
			{
				FString GameplayCueString = ThisGameplayCueTag.ToString();
				if (!SearchStrings.ContainsByPredicate([&](const FString& SStr) { return GameplayCueString.Contains(SStr); }))
				{
					continue;
				}
			}
			
			TSharedPtr< FGCTreeItem > NewItem = TSharedPtr<FGCTreeItem>(new FGCTreeItem());
			NewItem->GameplayCueTag = ThisGameplayCueTag;
			NewItem->GameplayCueTagName = ThisGameplayCueTag.GetTagName();

			bool Handled = false;
			bool FilteredOut = false;

			// Add notifies from the global set
			{
				SCOPE_SECONDS_COUNTER(FindGameplayCueNotifyObjTime);
				Handled = FindGameplayCueNotifyObj(CueManager, NewItem);
			}

			CheckSelectGCItem(NewItem);

			// ----------------------------------------------------------------
			// Add children translated tags
			// ----------------------------------------------------------------

			{
				SCOPE_SECONDS_COUNTER(AddTranslationTagsTime);
				AddChildTranslatedTags_r(ThisGameplayCueTag.GetTagName(), CueManager, NewItem);
			}

			FilteredOut = FilterIds.Num() > 0 && NewItem->Children.Num() == 0;

			// ----------------------------------------------------------------
			// Add events implemented by IGameplayCueInterface blueprints
			// ----------------------------------------------------------------

			{
				SCOPE_SECONDS_COUNTER(AddEventsTime);

				TArray<UFunction*> Funcs;
				EventMap.MultiFind(ThisGameplayCueTag, Funcs);

				for (UFunction* Func : Funcs)
				{
					TSharedRef< FGCTreeItem > NewHandlerItem(new FGCTreeItem());
					NewHandlerItem->FunctionPtr = Func;
					NewHandlerItem->GameplayCueTag = ThisGameplayCueTag;
					NewHandlerItem->GameplayCueTagName = ThisGameplayCueTag.GetTagName();

					if (ensure(NewItem.IsValid()))
					{
						NewItem->Children.Add(NewHandlerItem);
						Handled = true;
					}
				}
			}

			// ----------------------------------------------------------------
			
			// if we were told to specifically select an item we want to force it to show up
			bool bForcedShow = false;
			if (SelectedTagPostUpdate && SelectedTagPostUpdate->MatchesTagExact(NewItem->GameplayCueTag))
			{
				SelectedItem = NewItem;
				bForcedShow = true;
			}

			if ((!FilteredOut && (bShowAll || Handled)) || bForcedShow)
			{
				GameplayCueListItems.Add(NewItem);
			}

			if (ExpandedTags.Contains(NewItem->GameplayCueTagName))
			{
				GameplayCueTreeView->SetItemExpansion(NewItem, true);
			}
		}

		{
			FString RequestTreeRefreshMessage = FString::Printf(TEXT("  RequestTreeRefresh"));
			//SCOPE_LOG_TIME_IN_SECONDS(*RequestTreeRefreshMessage, nullptr)

			if (GameplayCueTreeView.IsValid())
			{
				GameplayCueTreeView->RequestTreeRefresh();
			}

			if (SelectedItem.IsValid())
			{
				GameplayCueTreeView->SetItemSelection(SelectedItem, true);
				GameplayCueTreeView->RequestScrollIntoView(SelectedItem);
			}
		}
	}

	void UpdateFilterListItems(bool UpdateView=true)
	{
		UGameplayCueManager* CueManager = UAbilitySystemGlobals::Get().GetGameplayCueManager();
		if (!CueManager)
		{
			return;
		}

		CueManager->TranslationManager.RefreshNameSwaps();

		const TArray<FNameSwapData>& AllNameSwapData = CueManager->TranslationManager.GetNameSwapData();
		FilterListItems.Reset();

		// Make two passes. In the first pass only add filters to the root if ShouldShowInTopLevelFilterList is true.
		// In the second pass, we only add filters as child nodes. This is to make a nice heirarchy of filters, rather than
		// having "sub" filters appear in the root view.
		for (int32 pass=0; pass < 2; ++pass)
		{
			for (const FNameSwapData& NameSwapGroup : AllNameSwapData)
			{
				for (const FGameplayCueTranslationNameSwap& NameSwapData : NameSwapGroup.NameSwaps)
				{
					bool Added = false;

					TSharedPtr< FGCFilterTreeItem > NewItem = TSharedPtr<FGCFilterTreeItem>(new FGCFilterTreeItem());
					NewItem->Data = NameSwapData.EditorData;
					NewItem->ToNames = NameSwapData.ToNames;

					// Look for existing entries
					for (TSharedPtr<FGCFilterTreeItem>& FilterItem : FilterListItems)
					{
						if (FilterItem->ToNames.Num() == 1 && NameSwapData.FromName == FilterItem->ToNames[0])
						{
							FilterItem->Children.Add(NewItem);
							Added = true;
						}
					}

					// Add to root, otherwise
					if (pass == 0 && NameSwapGroup.ClassCDO->ShouldShowInTopLevelFilterList())
					{
						FilterListItems.Add(NewItem);
					}
				}
			}
		}

		if (UpdateView && FilterTreeView.IsValid())
		{
			FilterTreeView->RequestTreeRefresh();
		}
	}

	void ExpandFilterItems()
	{
		// Expand filter items that are checked. This is to prevent people forgetting they have leaf nodes checked and enabled but not obvious in the UI
		// (E.g., they enable a filter, then collapse its parent. Then close override menu. Everytime they open override menu, we want to show default expansion)
		struct local
		{
			static bool ExpandFilterItems_r(TArray<TSharedPtr<FGCFilterTreeItem> >& Items, TArray<int32>& FilterIds, SFilterTreeView* FilterTreeView )
			{
				bool ShouldBeExpanded = false;
				for (TSharedPtr<FGCFilterTreeItem>& FilterItem : Items)
				{
					ShouldBeExpanded |= FilterIds.Contains(FilterItem->Data.UniqueID);
					if (ExpandFilterItems_r(FilterItem->Children, FilterIds, FilterTreeView))
					{
						FilterTreeView->SetItemExpansion(FilterItem, true);
						ShouldBeExpanded = true;
					}
				}
				return ShouldBeExpanded;
			}

		};

		local::ExpandFilterItems_r(FilterListItems, FilterIds, FilterTreeView.Get());
	}

	/** Slow task: use asset registry to find blueprints, load an inspect them to see what GC events they implement. */
	FReply BuildEventMap()
	{
		FScopedSlowTask SlowTask(100.f, LOCTEXT("BuildEventMap", "Searching Blueprints for GameplayCue events"));
		SlowTask.MakeDialog();
		SlowTask.EnterProgressFrame(10.f);

		EventMap.Empty();

		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		auto del = IGameplayAbilitiesEditorModule::Get().GetGameplayCueInterfaceClassesDelegate();
		if (del.IsBound())
		{
			TArray<UClass*> InterfaceClasses;
			del.ExecuteIfBound(InterfaceClasses);
			float WorkPerClass = InterfaceClasses.Num() > 0 ? 90.f / static_cast<float>(InterfaceClasses.Num()) : 0.f;

			for (UClass* InterfaceClass : InterfaceClasses)
			{
				SlowTask.EnterProgressFrame(WorkPerClass);

				TArray<FAssetData> GameplayCueInterfaceActors;
				{
#if STATS
					FString PerfMessage = FString::Printf(TEXT("Searched asset registry %s "), *InterfaceClass->GetName());
					SCOPE_LOG_TIME_IN_SECONDS(*PerfMessage, nullptr)
#endif

					UObjectLibrary* ObjLibrary = UObjectLibrary::CreateLibrary(InterfaceClass, true, true);
					ObjLibrary->LoadBlueprintAssetDataFromPath(TEXT("/Game/"));
					ObjLibrary->GetAssetDataList(GameplayCueInterfaceActors);
				}

			{
#if STATS
				FString PerfMessage = FString::Printf(TEXT("Fully Loaded GameplayCueNotify actors %s "), *InterfaceClass->GetName());
				SCOPE_LOG_TIME_IN_SECONDS(*PerfMessage, nullptr)
#endif				

					for (const FAssetData& AssetData : GameplayCueInterfaceActors)
					{
						UBlueprint* BP = Cast<UBlueprint>(AssetData.GetAsset());
						if (BP)
						{
							for (TFieldIterator<UFunction> FuncIt(BP->GeneratedClass, EFieldIteratorFlags::ExcludeSuper); FuncIt; ++FuncIt)
							{
								FString FuncName = FuncIt->GetName();
								if (FuncName.Contains("GameplayCue"))
								{
									FuncName.ReplaceInline(TEXT("_"), TEXT("."));
									FGameplayTag FoundTag = Manager.RequestGameplayTag(FName(*FuncName), false);
									if (FoundTag.IsValid())
									{
										EventMap.AddUnique(FoundTag, *FuncIt);
									}
								}

							}
						}
					}
				}
			}

			UpdateGameplayCueListItems();
		}

		return FReply::Handled();
	}

	void OnExpansionChanged( TSharedPtr<FGCTreeItem> InItem, bool bIsExpanded )
	{
		if (bIsExpanded)
		{
			ExpandedTags.Add(InItem->GameplayCueTagName);
		}
		else
		{
			ExpandedTags.Remove(InItem->GameplayCueTagName);
		}
	}

	void OnSelectionChanged( TSharedPtr<FGCTreeItem> InItem, ESelectInfo::Type SelectInfo )
	{
		if (InItem.IsValid())
		{
			SelectedTag = InItem->GameplayCueTagName;
			SelectedUniqueID = InItem->TranslationUniqueID;
		}
		else
		{
			SelectedTag = NAME_None;
			SelectedUniqueID = INDEX_NONE;
		}
	}

	void HandleOverrideTypeChange(bool NewValue)
	{
		bShowAllOverrides = NewValue;
		UpdateGameplayCueListItems();
	}
	
	TSharedRef<SWidget> OnGetShowOverrideTypeMenu()
	{
		FMenuBuilder MenuBuilder( true, NULL );

		FUIAction YesAction( FExecuteAction::CreateSP( this, &SGameplayCueEditorImpl::HandleOverrideTypeChange, true ) );
		MenuBuilder.AddMenuEntry( GetOverrideTypeDropDownText_Explicit(true), LOCTEXT("GameplayCueEditor_ShowAllOverrides", "Show ALL POSSIBLE tags for overrides: including Tags that could exist but currently don't"), FSlateIcon(), YesAction );

		FUIAction NoAction( FExecuteAction::CreateSP( this, &SGameplayCueEditorImpl::HandleOverrideTypeChange, false ) );
		MenuBuilder.AddMenuEntry( GetOverrideTypeDropDownText_Explicit(false), LOCTEXT("GameplayCueEditor_ShowValidOverrides", "ONLY show tags for overrides that exist/have been setup."), FSlateIcon(), NoAction );

		return MenuBuilder.MakeWidget();
	}

	FText GetOverrideTypeDropDownText() const
	{
		return GetOverrideTypeDropDownText_Explicit(bShowAllOverrides);
	}
	
	FText GetOverrideTypeDropDownText_Explicit(bool ShowAll) const
	{
		if (ShowAll)
		{
			return LOCTEXT("ShowAllOverrides_Tooltip_Yes", "Show all possible overrides");
		}

		return LOCTEXT("ShowAllOverrides_Tooltip_No", "Show only existing overrides");
	}

	EVisibility GetWaitingOnAssetRegistryVisiblity() const
	{
		if (UGameplayCueManager* CueManager = UAbilitySystemGlobals::Get().GetGameplayCueManager())
		{
			return CueManager->EditorObjectLibraryFullyInitialized ? EVisibility::Collapsed : EVisibility::Visible;
		}
		return EVisibility::Visible;
	}
};

bool SGameplayCueEditorImpl::bSuppressCueViewUpdate = false;

static FReply RecompileGameplayCueEditor_OnClicked()
{	
	GEngine->DeferredCommands.Add( TEXT( "GameplayAbilitiesEditor.HotReload" ) );
	return FReply::Handled();
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

/** Builds widget for rows in the GameplayCue Editor tab */
TSharedRef<ITableRow> SGameplayCueEditorImpl::OnGenerateWidgetForGameplayCueListView(TSharedPtr< FGCTreeItem > InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	class SCueItemWidget : public SMultiColumnTableRow< TSharedPtr< FGCTreeItem > >
	{
	public:
		SLATE_BEGIN_ARGS(SCueItemWidget) {}
		SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const TSharedRef<SGameplayCueTreeView>& InOwnerTable, TSharedPtr<FGCTreeItem> InListItem, SGameplayCueEditorImpl* InGameplayCueEditor)
		{
			Item = InListItem;
			GameplayCueEditor = InGameplayCueEditor;
			SMultiColumnTableRow< TSharedPtr< FGCTreeItem > >::Construct(
				FSuperRowType::FArguments()
				, InOwnerTable);
		}
	private:

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (ColumnName == CueTagColumnName)
			{
				return
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SExpanderArrow, SharedThis(this))
					]
				+ SHorizontalBox::Slot()
					.FillWidth(1)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.ColorAndOpacity(Item->GameplayCueTag.IsValid() ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground())
					.Text(FText::FromString(Item->Description.IsEmpty() ? Item->GameplayCueTagName.ToString() : FString::Printf(TEXT("%s (%s)"), *Item->Description, *Item->GameplayCueTagName.ToString())))
					];
			}
			else if (ColumnName == CueHandlerColumnName)
			{
				if (Item->GameplayCueNotifyObj.ToString().IsEmpty() == false)
				{
					FString ObjName = Item->GameplayCueNotifyObj.ToString();

					int32 idx;
					if (ObjName.FindLastChar(TEXT('.'), idx))
					{
						ObjName.RightChopInline(idx + 1, EAllowShrinking::No);
						if (ObjName.FindLastChar(TEXT('_'), idx))
						{
							ObjName.LeftInline(idx);
						}
					}

					return
						SNew(SBox)
						.HAlign(HAlign_Left)
						[
							SNew(SHyperlink)
							.Style(FAppStyle::Get(), "Common.GotoBlueprintHyperlink")
						.Text(FText::FromString(ObjName))
						.OnNavigate(this, &SCueItemWidget::NavigateToHandler)
						];
				}
				else if (Item->FunctionPtr.IsValid())
				{
					FString ObjName;
					UFunction* Func = Item->FunctionPtr.Get();
					UClass* OuterClass = Cast<UClass>(Func->GetOuter());
					if (OuterClass)
					{
						ObjName = OuterClass->GetName();
						ObjName.RemoveFromEnd(TEXT("_c"));
					}

					return
						SNew(SBox)
						.HAlign(HAlign_Left)
						[
							SNew(SHyperlink)
							.Text(FText::FromString(ObjName))
						.OnNavigate(this, &SCueItemWidget::NavigateToHandler)
						];
				}
				else
				{
					return
						SNew(SBox)
						.HAlign(HAlign_Left)
						[
							SNew(SHyperlink)
							.Style(FAppStyle::Get(), "Common.GotoNativeCodeHyperlink")
						.Text(LOCTEXT("AddNew", "Add New"))
						.OnNavigate(this, &SCueItemWidget::OnAddNewClicked)
						];
				}
			}
			else
			{
				return SNew(STextBlock).Text(LOCTEXT("UnknownColumn", "Unknown Column"));
			}
		}

		/** Create new GameplayCueNotify: brings up dialog to pick class, then creates it via the content browser. */
		void OnAddNewClicked()
		{
			{
				// Add the tag if its not already. Note that the FGameplayTag may be valid as an implicit tag, and calling this
				// will create it as an explicit tag, which is what we want. If the tag already exists

				TGuardValue<bool> SupressUpdate(SGameplayCueEditorImpl::bSuppressCueViewUpdate, true);

				IGameplayTagsEditorModule::Get().AddNewGameplayTagToINI(Item->GameplayCueTagName.ToString());
			}

			UClass* ParentClass = nullptr;

			// If this is an override, use the parent GC notify class as the base class
			if (Item->ParentGameplayCueNotifyObj.IsValid())
			{
				UObject* Obj = Item->ParentGameplayCueNotifyObj.ResolveObject();
				if (!Obj)
				{
					Obj = Item->ParentGameplayCueNotifyObj.TryLoad();
				}

				ParentClass = Cast<UClass>(Obj);
				if (ParentClass == nullptr)
				{
					ABILITY_LOG(Warning, TEXT("Unable to resolve object for parent GC notify: %s"), *Item->ParentGameplayCueNotifyObj.ToString());
				}
			}

			GameplayCueEditor->OnSelectionChanged(Item, ESelectInfo::Direct);

			SGameplayCueEditor::CreateNewGameplayCueNotifyDialogue(Item->GameplayCueTagName.ToString(), ParentClass);
		}

		void NavigateToHandler()
		{
			if (Item->GameplayCueNotifyObj.IsValid())
			{
				SGameplayCueEditor::OpenEditorForNotify(Item->GameplayCueNotifyObj.ToString());

			}
			else if (Item->FunctionPtr.IsValid())
			{
				SGameplayCueEditor::OpenEditorForNotify(Item->FunctionPtr->GetOuter()->GetPathName());
			}
		}

		TSharedPtr< FGCTreeItem > Item;
		SGameplayCueEditorImpl* GameplayCueEditor;
	};

	if (InItem.IsValid())
	{
		return SNew(SCueItemWidget, GameplayCueTreeView.ToSharedRef(), InItem, this);
	}
	else
	{
		return
			SNew(STableRow< TSharedPtr<FGCTreeItem> >, OwnerTable)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("UnknownItemType", "Unknown Item Type"))
			];
	}
}

TSharedRef<ITableRow> SGameplayCueEditorImpl::OnGenerateWidgetForFilterListView(TSharedPtr< FGCFilterTreeItem > InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	class SFilterItemWidget : public SMultiColumnTableRow< TSharedPtr< FGCFilterTreeItem > >
	{
	public:
		SLATE_BEGIN_ARGS(SFilterItemWidget) {}
		SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const TSharedRef<SFilterTreeView>& InOwnerTable, SGameplayCueEditorImpl* InGameplayCueEditor, TSharedPtr<FGCFilterTreeItem> InListItem)
		{
			Item = InListItem;
			GameplayCueEditor = InGameplayCueEditor;
			SMultiColumnTableRow< TSharedPtr< FGCFilterTreeItem > >::Construct(FSuperRowType::FArguments(), InOwnerTable);
		}
	private:

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (ColumnName == CueTagColumnName)
			{
				return
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SExpanderArrow, SharedThis(this))
					]
				+ SHorizontalBox::Slot()
					.FillWidth(1)
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
						.OnCheckStateChanged(GameplayCueEditor, &SGameplayCueEditorImpl::OnFilterStateChanged, Item)
					.IsChecked(GameplayCueEditor, &SGameplayCueEditorImpl::IsFilterChecked, Item)
					.IsEnabled(Item->Data.Enabled)
					.ToolTipText(FText::FromString(Item->Data.ToolTip))
					[
						SNew(STextBlock)
						.Text(FText::FromString(Item->Data.EditorDescription.ToString()))
					.ToolTipText(FText::FromString(Item->Data.ToolTip))
					]
					];
			}
			else
			{
				return SNew(STextBlock).Text(LOCTEXT("UnknownColumn", "Unknown Column"));
			}
		}

		TSharedPtr< FGCFilterTreeItem > Item;
		SGameplayCueEditorImpl* GameplayCueEditor;
	};

	if (InItem.IsValid())
	{
		return SNew(SFilterItemWidget, FilterTreeView.ToSharedRef(), this, InItem);
	}
	else
	{
		return
			SNew(STableRow< TSharedPtr<FGCTreeItem> >, OwnerTable)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("UnknownFilterType", "Unknown Filter Type"))
			];
	}
}

void SGameplayCueEditorImpl::Construct(const FArguments& InArgs)
{
	UGameplayCueManager* CueManager = UAbilitySystemGlobals::Get().GetGameplayCueManager();
	if (CueManager)
	{
		CueManager->OnGameplayCueNotifyAddOrRemove.AddSP(this, &SGameplayCueEditorImpl::OnPropertyValueChanged);
		CueManager->OnEditorObjectLibraryUpdated.AddSP(this, &SGameplayCueEditorImpl::UpdateGameplayCueListItems);
		CueManager->RequestPeriodicUpdateOfEditorObjectLibraryWhileWaitingOnAssetRegistry();
	}

	bShowAll = true;
	bShowAllOverrides = false;
	bShowOnlyLeafTags = true;
	bFilterIDsDirty = false;

	bool CanAddFromINI = UGameplayTagsManager::Get().ShouldImportTagsFromINI(); // We only support adding new tags to the ini files.
	
	ChildSlot
	[
		SNew(SVerticalBox)

		// -- Hot Reload -------------------------------------------------
#if GAMEPLAYCUEEDITOR_HOTRELOAD_BUTTON
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.0f, 2.0f)
			.AutoWidth()
			[

				SNew(SButton)
				.Text(LOCTEXT("HotReload", "Hot Reload"))
				.OnClicked(FOnClicked::CreateStatic(&RecompileGameplayCueEditor_OnClicked))
			]
		]
#endif
		// -------------------------------------------------------------

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.0f, 2.0f)
			.AutoWidth()
			[

				SNew(SButton)
				.Text(LOCTEXT("SearchBPEvents", "Search BP Events"))
				.OnClicked(this, &SGameplayCueEditorImpl::BuildEventMap)
			]

			+ SHorizontalBox::Slot()
			.Padding(2.0f, 2.0f)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SGameplayCueEditorImpl::HandleShowAllCheckBoxIsChecked)
				.OnCheckStateChanged(this, &SGameplayCueEditorImpl::HandleShowAllCheckedStateChanged)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("HideUnhandled", "Hide Unhandled GameplayCues"))
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SGameplayCueEditorImpl::GetAddTagSectionExpansionState)
				.OnCheckStateChanged(this, &SGameplayCueEditorImpl::OnAddTagSectionExpansionStateChanged)
				.CheckedImage(FAppStyle::GetBrush("TreeArrow_Expanded"))
				.CheckedHoveredImage(FAppStyle::GetBrush("TreeArrow_Expanded_Hovered"))
				.CheckedPressedImage(FAppStyle::GetBrush("TreeArrow_Expanded"))
				.UncheckedImage(FAppStyle::GetBrush("TreeArrow_Collapsed"))
				.UncheckedHoveredImage(FAppStyle::GetBrush("TreeArrow_Collapsed_Hovered"))
				.UncheckedPressedImage(FAppStyle::GetBrush("TreeArrow_Collapsed"))
				.Visibility(this, &SGameplayCueEditorImpl::DetermineExpandableUIVisibility)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AddNewTag", "Add New Gameplay Cue Tag"))
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(16.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(AddNewTagWidget, SAddNewGameplayTagWidget)
				.Visibility(this, &SGameplayCueEditorImpl::DetermineAddNewTagWidgetVisibility)
				.OnGameplayTagAdded(this, &SGameplayCueEditorImpl::OnNewGameplayCueTagCommited)
				.IsValidTag(this, &SGameplayCueEditorImpl::IsValidTag)
				.NewTagName(CueTagPrefix)
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.AutoWidth()
			[
				SAssignNew( SearchBoxPtr, SSearchBox )
				.MinDesiredWidth(210.0f)
				.OnTextCommitted(this, &SGameplayCueEditorImpl::OnSearchTagCommited)
			]
			+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("Search", "Search"))
				.OnClicked(this, &SGameplayCueEditorImpl::DoSearch)
			]
		]

		// ---------------------------------------------------------------
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.AutoWidth()
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &SGameplayCueEditorImpl::GetFilterListContent)
				.OnMenuOpenChanged( this, &SGameplayCueEditorImpl::OnFilterMenuOpenChanged )
				.ContentPadding(FMargin(2.0f, 2.0f))
				//.Visibility(this, &FScalableFloatDetails::GetRowNameVisibility)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("GameplayCueOverrides", "Override Filter" ))//&FScalableFloatDetails::GetRowNameComboBoxContentText)
				]
			]
			
			+ SHorizontalBox::Slot()
			.Padding(2.0f, 2.0f)
			.AutoWidth()
			[
				SNew( SComboButton )
				.OnGetMenuContent( this, &SGameplayCueEditorImpl::OnGetShowOverrideTypeMenu )
				.VAlign( VAlign_Center )
				.ContentPadding(2)
				.ButtonContent()
				[
					SNew( STextBlock )
					.ToolTipText( LOCTEXT("ShowOverrideType", "Toggles how we display overrides. Either show the existing overrides, or show possible overrides") )
					.Text( this, &SGameplayCueEditorImpl::GetOverrideTypeDropDownText )
				]

				/*
				SNew(SCheckBox)
				.IsChecked(this, &SGameplayCueEditorImpl::HandleShowAllOverridesCheckBoxIsChecked)
				.OnCheckStateChanged(this, &SGameplayCueEditorImpl::HandleShowAllOverridesCheckedStateChanged)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShowAllOverrides", "Show all possible overrides"))
				]
				*/


			]

#if GAMEPLAYCUEEDITOR_SHOW_ONLY_LEAFTAGS_OPTION
			+ SHorizontalBox::Slot()
			.Padding(2.0f, 2.0f)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SGameplayCueEditorImpl::HandleShowOnlyLeafTagsCheckBoxIsChecked)
				.OnCheckStateChanged(this, &SGameplayCueEditorImpl::HandleShowOnLeafTagsCheckedStateChanged)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShowLeafTagsOnly", "Show leaf tags only"))
				]
			]
#endif			
		]
		
		// ---------------------------------------------------------------

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("WaitingOnAssetRegister", "Waiting on Asset Registry to finish loading (all tags are present but some GC Notifies may not yet be discovered)"))
				.Visibility(this, &SGameplayCueEditorImpl::GetWaitingOnAssetRegistryVisiblity)
			]
		]
		
		// ---------------------------------------------------------------

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(GameplayCueTreeView, SGameplayCueTreeView)
				.TreeItemsSource(&GameplayCueListItems)
				.OnGenerateRow(this, &SGameplayCueEditorImpl::OnGenerateWidgetForGameplayCueListView)
				.OnGetChildren(this, &SGameplayCueEditorImpl::OnGetChildren )
				.OnExpansionChanged(this, &SGameplayCueEditorImpl::OnExpansionChanged)
				.OnSelectionChanged(this, &SGameplayCueEditorImpl::OnSelectionChanged)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(CueTagColumnName)
					.DefaultLabel(NSLOCTEXT("GameplayCueEditor", "GameplayCueTag", "Tag"))
					.FillWidth(0.50)
					

					+ SHeaderRow::Column(CueHandlerColumnName)
					.DefaultLabel(NSLOCTEXT("GameplayCueEditor", "GameplayCueHandlers", "Handlers"))
					//.FillWidth(1000)
				)
			]
		]
	];

	UpdateGameplayCueListItems();
	UpdateFilterListItems();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

EVisibility SGameplayCueEditorImpl::DetermineAddNewTagWidgetVisibility() const
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	if (!Manager.ShouldImportTagsFromINI() || !bAddTagSectionExpanded)
	{
		// If we can't support adding tags from INI files, we should never see this widget
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SGameplayCueEditorImpl::DetermineExpandableUIVisibility() const
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	if (!Manager.ShouldImportTagsFromINI())
	{
		// If we can't support adding tags from INI files, we should never see this widget
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

ECheckBoxState SGameplayCueEditorImpl::GetAddTagSectionExpansionState() const
{
	return bAddTagSectionExpanded ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SGameplayCueEditorImpl::OnAddTagSectionExpansionStateChanged(ECheckBoxState NewState)
{
	bAddTagSectionExpanded = NewState == ECheckBoxState::Checked;
}

// -----------------------------------------------------------------------------------------------------------
//
// -----------------------------------------------------------------------------------------------------------

TSharedRef<SGameplayCueEditor> SGameplayCueEditor::New()
{
	return MakeShareable(new SGameplayCueEditorImpl());
}

void SGameplayCueEditor::CreateNewGameplayCueNotifyDialogue(FString GameplayCue, UClass* ParentClass)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	// If there already is a parent class, use that. Otherwise the developer must select which class to use.
	UClass* ChosenClass = ParentClass;
	if (ChosenClass == nullptr)
	{

		TArray<UClass*>	NotifyClasses;
		auto del = IGameplayAbilitiesEditorModule::Get().GetGameplayCueNotifyClassesDelegate();
		del.ExecuteIfBound(NotifyClasses);
		if (NotifyClasses.Num() == 0)
		{
			NotifyClasses.Add(UGameplayCueNotify_Static::StaticClass());
			NotifyClasses.Add(AGameplayCueNotify_Actor::StaticClass());
		}

		// --------------------------------------------------

		// Null the parent class to ensure one is selected	

		const FText TitleText = LOCTEXT("CreateBlueprintOptions", "New GameplayCue Handler");
	
		const bool bPressedOk = SGameplayCuePickerDialog::PickGameplayCue(TitleText, NotifyClasses, ChosenClass, GameplayCue);
		if (!bPressedOk)
		{
			return;
		}
	}

	if (ensure(ChosenClass))
	{
		FString NewDefaultPathName = SGameplayCueEditor::GetPathNameForGameplayCueTag(GameplayCue);

		// Make sure the name is unique
		FString AssetName;
		FString PackageName;
		AssetToolsModule.Get().CreateUniqueAssetName(NewDefaultPathName, TEXT(""), /*out*/ PackageName, /*out*/ AssetName);
		const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

		// Create the GameplayCue Notify
		UBlueprintFactory* BlueprintFactory = NewObject<UBlueprintFactory>();
		BlueprintFactory->ParentClass = ChosenClass;
		ContentBrowserModule.Get().CreateNewAsset(AssetName, PackagePath, UBlueprint::StaticClass(), BlueprintFactory);
	}
}

void SGameplayCueEditor::OpenEditorForNotify(FString NotifyFullPath)
{
	// This nonsense is to handle the case where the asset only exists in memory
	// and there for does not have a linker/exist on disk. (The FString version
	// of OpenEditorForAsset does not handle this).
	FSoftObjectPath AssetRef(NotifyFullPath);

	UObject* Obj = AssetRef.ResolveObject();
	if (!Obj)
	{
		Obj = AssetRef.TryLoad();
	}

	if (Obj)
	{
		UPackage* pkg = Cast<UPackage>(Obj->GetOuter());
		if (pkg)
		{
			FString AssetName = FPaths::GetBaseFilename(AssetRef.ToString());
			UObject* AssetObject = FindObject<UObject>(pkg, *AssetName);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AssetObject);
		}
	}
}

FString SGameplayCueEditor::GetPathNameForGameplayCueTag(FString GameplayCueTagName)
{
	FString NewDefaultPathName;
	auto PathDel = IGameplayAbilitiesEditorModule::Get().GetGameplayCueNotifyPathDelegate();
	if (PathDel.IsBound())
	{
		NewDefaultPathName = PathDel.Execute(GameplayCueTagName);
	}
	else
	{
		GameplayCueTagName = GameplayCueTagName.Replace(TEXT("GameplayCue."), TEXT(""), ESearchCase::IgnoreCase);
		NewDefaultPathName = FString::Printf(TEXT("/Game/GameplayCueNotifies/GC_%s"), *GameplayCueTagName);
	}
	NewDefaultPathName.ReplaceInline(TEXT("."), TEXT("_"));
	return NewDefaultPathName;
}

#undef LOCTEXT_NAMESPACE
