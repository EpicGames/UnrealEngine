// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SUserWidget.h"

/** Main Gameplay Cue Editor UI widget */
class SGameplayCueEditor : public SUserWidget
{
public:

	SLATE_USER_ARGS(SGameplayCueEditor) {}
	SLATE_END_ARGS()
	virtual void Construct(const FArguments& InArgs) = 0;

	virtual void OnNewGameplayCueTagCommited(const FString& TagName, const FString& TagComment, const FName& TagSource) = 0;

	virtual void OnSearchTagCommited(const FText& InText, ETextCommit::Type InCommitType) = 0;
	virtual void HandleNotify_OpenAssetInEditor(FString AssetName, int AssetType) = 0;
	virtual void HandleNotify_FindAssetInEditor(FString AssetName, int AssetType) = 0;
	
	static FString GetPathNameForGameplayCueTag(FString Tag);

	static void CreateNewGameplayCueNotifyDialogue(FString GameplayCue, UClass* ParentClass);
	static void OpenEditorForNotify(FString NotifyFullPath);
};
