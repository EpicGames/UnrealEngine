// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayCueEditor_Picker.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SGameplayCuePickerDialog::Construct(const FArguments& InArgs)
{
	WeakParentWindow = InArgs._ParentWindow;
	DefaultClasses = InArgs._DefaultClasses;
	GameplayCueTag = InArgs._GameplayCueTag;

	FLinearColor AssetColor = FLinearColor::White;

	bPressedOk = false;
	ChosenClass = NULL;

	FString PathStr = SGameplayCueEditor::GetPathNameForGameplayCueTag(GameplayCueTag);
	FGameplayCueEditorStrings Strings;
	auto del = IGameplayAbilitiesEditorModule::Get().GetGameplayCueEditorStringsDelegate();
	if (del.IsBound())
	{
		Strings = del.Execute();
	}

	ChildSlot
	[
		SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(SBox)
			.Visibility(EVisibility::Visible)
			.Padding(2.f)
			.WidthOverride(520.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(2.f, 2.f)
				.AutoHeight()
				[
					SNew(SBorder)
					.Visibility(EVisibility::Visible)
					.BorderImage( FAppStyle::GetBrush("AssetThumbnail.AssetBackground") )
					.BorderBackgroundColor(AssetColor.CopyWithNewOpacity(0.3f))
					[
						SNew(SExpandableArea)
						.AreaTitle(NSLOCTEXT("SGameplayCuePickerDialog", "CommonClassesAreaTitle", "GameplayCue Notifies"))
						.BodyContent()
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.Padding(2.f, 2.f)
							.AutoHeight()
							[
								SNew(STextBlock)
								
								.Text(FText::FromString(Strings.GameplayCueNotifyDescription1))
								.AutoWrapText(true)
							]

							+SVerticalBox::Slot()
							.AutoHeight( )
							[
								SNew(SListView < UClass*  >)
								.SelectionMode(ESelectionMode::None)
								.ListItemsSource(&DefaultClasses)
								.OnGenerateRow(this, &SGameplayCuePickerDialog::GenerateListRow)
							]

							+SVerticalBox::Slot()
							.Padding(2.f, 2.f)
							.AutoHeight()
							[
								SNew(STextBlock)
								.Text(FText::FromString(FString::Printf(TEXT("This will create a new GameplayCue Notify here:"))))
								.AutoWrapText(true)
							]
							+SVerticalBox::Slot()
							.Padding(2.f, 2.f)
							.AutoHeight()
							[
								SNew(STextBlock)
								.Text(FText::FromString(PathStr))
								.HighlightText(FText::FromString(PathStr))
								.AutoWrapText(true)
							]
							+SVerticalBox::Slot()
							.Padding(2.f, 2.f)
							.AutoHeight()
							[
								SNew(STextBlock)
								.Text(FText::FromString(Strings.GameplayCueNotifyDescription2))
								.AutoWrapText(true)
							]
						]
					]
				]

				+SVerticalBox::Slot()
				.Padding(2.f, 2.f)
				.AutoHeight()
				[
					SNew(SBorder)
					.Visibility(EVisibility::Visible)
					.BorderImage( FAppStyle::GetBrush("AssetThumbnail.AssetBackground") )
					.BorderBackgroundColor(AssetColor.CopyWithNewOpacity(0.3f))
					[
						SNew(SExpandableArea)
						.AreaTitle(NSLOCTEXT("SGameplayCuePickerDialogEvents", "CommonClassesAreaTitleEvents", "Custom BP Events"))
						.BodyContent()
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.Padding(2.f, 2.f)
							.AutoHeight()
							[
								SNew(STextBlock)
								.Text(FText::FromString(Strings.GameplayCueEventDescription1))
								.AutoWrapText(true)

							]

							+SVerticalBox::Slot()
							.Padding(2.f, 2.f)
							.AutoHeight()
							[
								SNew(STextBlock)
								.Text(FText::FromString(Strings.GameplayCueEventDescription2))
								.AutoWrapText(true)
							]							
						]
					]
				]
			]
		]
	];
}

/** Spawns window for picking new GameplayCue handler/notify */
bool SGameplayCuePickerDialog::PickGameplayCue(const FText& TitleText, const TArray<UClass*>& DefaultClasses, UClass*& OutChosenClass, FString InGameplayCueName)
{
	// Create the window to pick the class
	TSharedRef<SWindow> PickerWindow = SNew(SWindow)
		.Title(TitleText)
		.SizingRule( ESizingRule::Autosized )
		.ClientSize( FVector2D( 0.f, 600.f ))
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	TSharedRef<SGameplayCuePickerDialog> ClassPickerDialog = SNew(SGameplayCuePickerDialog)
		.ParentWindow(PickerWindow)
		.DefaultClasses(DefaultClasses)
		.GameplayCueTag(InGameplayCueName);

	PickerWindow->SetContent(ClassPickerDialog);

	GEditor->EditorAddModalWindow(PickerWindow);

	if (ClassPickerDialog->bPressedOk)
	{
		OutChosenClass = ClassPickerDialog->ChosenClass;
		return true;
	}
	else
	{
		// Ok was not selected, NULL the class
		OutChosenClass = NULL;
		return false;
	}
}

/** Generates rows in the list of GameplayCueNotify classes to pick from */
TSharedRef<ITableRow> SGameplayCuePickerDialog::GenerateListRow(UClass* ItemClass, const TSharedRef<STableViewBase>& OwnerTable)
{	
	const FSlateBrush* ItemBrush = FSlateIconFinder::FindIconBrushForClass(ItemClass);

	return 
	SNew(STableRow< UClass* >, OwnerTable)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.MaxHeight(60.0f)
		.Padding(10.0f, 6.0f, 0.0f, 4.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(0.65f)
			[
				SNew(SButton)
				.OnClicked(this, &SGameplayCuePickerDialog::OnDefaultClassPicked, ItemClass)
				.Content()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.FillWidth(0.12f)
					[
						SNew(SImage)
						.Image(ItemBrush)
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f)
					.FillWidth(0.8f)
					[
						SNew(STextBlock)
						.Text(ItemClass->GetDisplayNameText())
					]

				]
			]
			+SHorizontalBox::Slot()
			.Padding(10.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(ItemClass->GetToolTipText(true))
				.AutoWrapText(true)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				FEditorClassUtils::GetDocumentationLinkWidget(ItemClass)
			]
		]
	];
}

FReply SGameplayCuePickerDialog::OnDefaultClassPicked(UClass* InChosenClass)
{
	ChosenClass = InChosenClass;
	bPressedOk = true;
	if (WeakParentWindow.IsValid())
	{
		WeakParentWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SGameplayCuePickerDialog::OnClassPickerConfirmed()
{
	if (ChosenClass == NULL)
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("EditorFactories", "MustChooseClassWarning", "You must choose a class."));
	}
	else
	{
		bPressedOk = true;

		if (WeakParentWindow.IsValid())
		{
			WeakParentWindow.Pin()->RequestDestroyWindow();
		}
	}
	return FReply::Handled();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
