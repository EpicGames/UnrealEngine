// Copyright Epic Games, Inc. All Rights Reserved.
// Meant to be only by SGameplayCueEditor.h

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "SGameplayCueEditor.h"
#include "GameplayAbilitiesEditorModule.h"
#include "Widgets/SBoxPanel.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "Widgets/SWindow.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Styling/SlateIconFinder.h"
#include "EditorClassUtils.h"
#include "Editor.h"

/** Widget for picking a new GameplayCue Notify class (similar to actor class picker)  */
class SGameplayCuePickerDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SGameplayCuePickerDialog )
	{}

		SLATE_ARGUMENT( TSharedPtr<SWindow>, ParentWindow )
		SLATE_ARGUMENT( TArray<UClass*>, DefaultClasses )
		SLATE_ARGUMENT( FString, GameplayCueTag )

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	static bool PickGameplayCue(const FText& TitleText, const TArray<UClass*>& DefaultClasses, UClass*& OutChosenClass, FString InGameplayCueTag);

private:
	/** Handler for when a class is picked in the class picker */
	void OnClassPicked(UClass* InChosenClass)
	{
		ChosenClass = InChosenClass;
	}

	/** Creates the default class widgets */
	TSharedRef<ITableRow> GenerateListRow(UClass* InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handler for when "Ok" we selected in the class viewer */
	FReply OnClassPickerConfirmed();

	FReply OnDefaultClassPicked(UClass* InChosenClass);

	/** Handler for the custom button to hide/unhide the default class viewer */
	void OnDefaultAreaExpansionChanged(bool bExpanded);

	/** Handler for the custom button to hide/unhide the class viewer */
	void OnCustomAreaExpansionChanged(bool bExpanded);	

private:
	/** A pointer to the window that is asking the user to select a parent class */
	TWeakPtr<SWindow> WeakParentWindow;

	/** The class that was last clicked on */
	UClass* ChosenClass;

	/** A flag indicating that Ok was selected */
	bool bPressedOk;

	/**  An array of default classes used in the dialog **/
	TArray<UClass*> DefaultClasses;

	FString GameplayCueTag;
};