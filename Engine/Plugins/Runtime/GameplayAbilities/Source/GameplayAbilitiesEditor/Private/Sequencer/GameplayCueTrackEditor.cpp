// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/GameplayCueTrackEditor.h"
#include "Sequencer/MovieSceneGameplayCueTrack.h"
#include "Sequencer/MovieSceneGameplayCueSections.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "Styling/AppStyle.h"
#include "UObject/Package.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "ISequencerSection.h"
#include "Widgets/SBoxPanel.h"
#include "MVVM/Views/ViewUtilities.h"
#include "MovieSceneSequenceEditor.h"
#include "LevelSequence.h"

#define LOCTEXT_NAMESPACE "FGameplayCueTrackEditor"



TSharedRef<ISequencerTrackEditor> FGameplayCueTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShared<FGameplayCueTrackEditor>(InSequencer);
}


TSharedRef<ISequencerSection> FGameplayCueTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	return MakeShared<FSequencerSection>(SectionObject);
}


FGameplayCueTrackEditor::FGameplayCueTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{}

void FGameplayCueTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	auto SubMenuHandler = [this, ObjectBindings](FMenuBuilder& SubMenuBuilder)
	{
		this->BuildAddTrackMenuImpl(SubMenuBuilder, ObjectBindings);
	};

	MenuBuilder.AddSubMenu(
		LOCTEXT("GameplayCues_Label", "Gameplay Cues"),
		FText::Format(LOCTEXT("GameplayCues_Tooltip", "Options for triggering gameplay cues on the selected {0}|plural(one=object,other=objects)"), ObjectBindings.Num()),
		FNewMenuDelegate::CreateLambda(SubMenuHandler)
	);
}

void FGameplayCueTrackEditor::BuildAddTrackMenuImpl(FMenuBuilder& MenuBuilder, TArrayView<const FGuid> InObjectBindingIDs)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieScene* FocusedMovieScene = SequencerPtr ? SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene() : nullptr;
	if (FocusedMovieScene == nullptr)
	{
		return;
	}

	const FFrameNumber StartTime       = SequencerPtr->GetLocalTime().Time.FrameNumber;
	const FFrameRate   TickResolution  = FocusedMovieScene->GetTickResolution();
	const int32        DefaultDuration = (1.0 * TickResolution).FrameNumber.Value;

	TArray<FGuid> ObjectBindingCopy(InObjectBindingIDs.GetData(), InObjectBindingIDs.Num());

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddNewRangeTrack", "Gameplay Cue (Range)"),
		LOCTEXT("AddNewRangeTrack_Tooltip", "Adds a new section that triggers a gameplay cue over a given range"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FGameplayCueTrackEditor::AddTracks, TRange<FFrameNumber>(StartTime, StartTime + DefaultDuration), UMovieSceneGameplayCueSection::StaticClass(), ObjectBindingCopy)
		)
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddNewTriggerTrack", "Gameplay Cue (Trigger)"),
		LOCTEXT("AddNewTriggerTrack_Tooltip", "Adds a new section that can trigger gameplay cues"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FGameplayCueTrackEditor::AddTracks, TRange<FFrameNumber>::All(), UMovieSceneGameplayCueTriggerSection::StaticClass(), ObjectBindingCopy)
		)
	);
}

TSharedPtr<SWidget> FGameplayCueTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	check(Track);

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	TWeakObjectPtr<UMovieSceneTrack> WeakTrack = Track;
	const int32 RowIndex = Params.TrackInsertRowIndex;

	auto SubMenuCallback = [this, WeakTrack, RowIndex]
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		UMovieSceneTrack* TrackPtr = WeakTrack.Get();
		if (TrackPtr)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddNewTriggerSection", "Trigger"),
				LOCTEXT("AddNewTriggerSectionTooltip", "Adds a new section that can trigger gameplay cues at specific times"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FGameplayCueTrackEditor::HandleAddSectionToTrack, TrackPtr, UMovieSceneGameplayCueTriggerSection::StaticClass(), RowIndex + 1))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddNewRangeSection", "Range"),
				LOCTEXT("AddNewRangeSectionTooltip", "Adds a new section that executes a gameplay cue over time"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FGameplayCueTrackEditor::HandleAddSectionToTrack, TrackPtr, UMovieSceneGameplayCueSection::StaticClass(), RowIndex + 1))
			);
		}

		return MenuBuilder.MakeWidget();
	};

	return UE::Sequencer::MakeAddButton(LOCTEXT("AddSection", "Section"), FOnGetContent::CreateLambda(SubMenuCallback), Params.ViewModel);
}

void FGameplayCueTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{

}

bool FGameplayCueTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneGameplayCueTrack::StaticClass();
}

bool FGameplayCueTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneGameplayCueTrack::StaticClass()) : ETrackSupport::Default;

	if (TrackSupported == ETrackSupport::NotSupported)
	{
		return false;
	}

	return (InSequence && InSequence->IsA(ULevelSequence::StaticClass())) || TrackSupported == ETrackSupport::Supported;
}

const FSlateBrush* FGameplayCueTrackEditor::GetIconBrush() const
{
	return FAppStyle::GetBrush("Sequencer.Tracks.Event");
}

void FGameplayCueTrackEditor::AddTracks(TRange<FFrameNumber> SectionTickRange, UClass* SectionClass, TArray<FGuid> InObjectBindingIDs)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieScene* FocusedMovieScene = SequencerPtr ? SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene() : nullptr;

	if (FocusedMovieScene == nullptr)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddGameplayCueTrack_Transaction", "Add Gameplay Cue Track"));
	FocusedMovieScene->Modify();

	if (InObjectBindingIDs.Num() == 0)
	{
		UMovieSceneGameplayCueTrack* NewMainTrack = FocusedMovieScene->AddTrack<UMovieSceneGameplayCueTrack>();
		AddSectionToTrack(NewMainTrack, SectionTickRange, SectionClass);

		SequencerPtr->OnAddTrack(NewMainTrack, FGuid());
	}
	else for (const FGuid& ObjectBindingID : InObjectBindingIDs)
	{
		if (ObjectBindingID.IsValid())
		{
			UMovieSceneGameplayCueTrack* NewObjectTrack = FocusedMovieScene->AddTrack<UMovieSceneGameplayCueTrack>(ObjectBindingID);
			AddSectionToTrack(NewObjectTrack, SectionTickRange, SectionClass);

			SequencerPtr->OnAddTrack(NewObjectTrack, ObjectBindingID);
		}
	}

	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

void FGameplayCueTrackEditor::HandleAddSectionToTrack(UMovieSceneTrack* Track, UClass* SectionClass, int32 RowIndex)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return;
	}

	UMovieScene* MovieScene = Track->GetTypedOuter<UMovieScene>();
	if (MovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddGameplayCueSection_Transaction", "Add Gameplay Cue Section"));
	Track->Modify();


	UMovieSceneSection* NewSection = NewObject<UMovieSceneSection>(Track, SectionClass, NAME_None, RF_Transactional);

	if (SectionClass == UMovieSceneGameplayCueSection::StaticClass())
	{
		const FFrameNumber StartTime       = SequencerPtr->GetLocalTime().Time.FrameNumber;
		const int32        DefaultDuration = (1.0 * MovieScene->GetTickResolution()).FrameNumber.Value;

		NewSection->InitialPlacementOnRow(Track->GetAllSections(), StartTime, DefaultDuration, RowIndex);
	}
	else
	{
		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (Section && Section->GetRowIndex() >= RowIndex)
			{
				Section->Modify();
				Section->SetRowIndex(Section->GetRowIndex() + 1);
			}
		}
		NewSection->SetRange(TRange<FFrameNumber>::All());
	}

	Track->AddSection(*NewSection);
	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

void FGameplayCueTrackEditor::AddSectionToTrack(UMovieSceneTrack* Track, const TRange<FFrameNumber>& SectionTickRange, UClass* SectionClass)
{
	UMovieSceneSection* NewSection = NewObject<UMovieSceneSection>(Track, SectionClass, NAME_None, RF_Transactional);
	NewSection->SetRange(SectionTickRange);
	Track->AddSection(*NewSection);
}

#undef LOCTEXT_NAMESPACE
