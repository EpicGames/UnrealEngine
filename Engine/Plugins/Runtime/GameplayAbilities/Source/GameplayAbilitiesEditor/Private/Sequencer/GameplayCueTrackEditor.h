// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "MovieSceneTrackEditor.h"

class ISequencer;

class FMenuBuilder;

/**
* A property track editor for named events.
*/
class FGameplayCueTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/**
	 * Factory function to create an instance of this class (called by a sequencer).
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 * @return The new instance of this class.
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer);

public:

	FGameplayCueTrackEditor(TSharedRef<ISequencer> InSequencer);

public:

	// ISequencerTrackEditor interface

	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual void BuildTrackContextMenu( FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track ) override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;

private:

	void BuildAddTrackMenuImpl(FMenuBuilder& MenuBuilder, TArrayView<const FGuid> InObjectBindingIDs);
	void HandleAddSectionToTrack(UMovieSceneTrack* Track, UClass* SectionClass, int32 RowIndex);

	void AddTracks(TRange<FFrameNumber> SectionTickRange, UClass* SectionClass, TArray<FGuid> InObjectBindingIDs);
	void AddSectionToTrack(UMovieSceneTrack* Track, const TRange<FFrameNumber>& SectionTickRange, UClass* SectionClass);
};
