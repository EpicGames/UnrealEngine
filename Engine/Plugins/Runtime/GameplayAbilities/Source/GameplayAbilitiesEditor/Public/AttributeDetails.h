// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "IDetailCustomization.h"
#include "ScalableFloat.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"

#define UE_API GAMEPLAYABILITIESEDITOR_API

class IDetailLayoutBuilder;
class IPropertyHandle;
class IPropertyUtilities;
class SComboButton;
class SSearchBox;

DECLARE_LOG_CATEGORY_EXTERN(LogAttributeDetails, Log, All);

class FAttributeDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

	TSharedPtr<FString> GetPropertyType() const;

	TArray<TSharedPtr<FString>> PropertyOptions;

	TSharedPtr<IPropertyHandle> MyProperty;

	void OnChangeProperty(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo);	
};


class FAttributePropertyDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;              

private:

	// the attribute property
	TSharedPtr<IPropertyHandle> MyProperty;
	// the owner property
	TSharedPtr<IPropertyHandle> OwnerProperty;
	// the name property
	TSharedPtr<IPropertyHandle> NameProperty;

	TArray<TSharedPtr<FString>> PropertyOptions;

	TSharedPtr<FString>	GetPropertyType() const;

	void OnChangeProperty(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo);
	void OnAttributeChanged(FProperty* SelectedAttribute);
};

class FScalableFloatDetails : public IPropertyTypeCustomization
{
public:
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();
	static constexpr float DefaultMinPreviewLevel = 0.f;
	static constexpr float DefaultMaxPreviewLevel = 30.f;

	FScalableFloatDetails()
		: PreviewLevel(0.f)
		, MinPreviewLevel(DefaultMinPreviewLevel)
		, MaxPreviewLevel(DefaultMaxPreviewLevel) 
	{
	}

protected:

	UE_API virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	UE_API virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;              

	UE_API bool IsEditable() const;
	UE_API void UpdatePreviewLevels();

	// Curve Table selector
	UE_API TSharedRef<SWidget> CreateCurveTableWidget();
	UE_API TSharedRef<SWidget> GetCurveTablePicker();
	UE_API void OnSelectCurveTable(const FAssetData& AssetData);
	UE_API void OnCloseMenu();
	UE_API FText GetCurveTableText() const;
	UE_API FText GetCurveTableTooltip() const;
	UE_API EVisibility GetCurveTableVisiblity() const;
	UE_API EVisibility GetAssetButtonVisiblity() const;
	UE_API void OnBrowseTo();
	UE_API void OnClear();
	UE_API void OnUseSelected();
	
	// Registry Type selector
	UE_API TSharedRef<SWidget> CreateRegistryTypeWidget();
	UE_API FString GetRegistryTypeValueString() const;
	UE_API FText GetRegistryTypeTooltip() const;
	UE_API EVisibility GetRegistryTypeVisiblity() const;

	// Curve source accessors
	UE_API void OnCurveSourceChanged();
	UE_API void RefreshSourceData();
	UE_API class UCurveTable* GetCurveTable(FPropertyAccess::Result* OutResult = nullptr) const;
	UE_API FDataRegistryType GetRegistryType(FPropertyAccess::Result* OutResult = nullptr) const;

	// Row/item name widget
	UE_API TSharedRef<SWidget> CreateRowNameWidget();
	UE_API EVisibility GetRowNameVisibility() const;
	UE_API FText GetRowNameComboBoxContentText() const;
	UE_API FText GetRowNameComboBoxContentTooltip() const;
	UE_API void OnRowNameChanged();

	// Preview widgets
	UE_API EVisibility GetPreviewVisibility() const;
	UE_API float GetPreviewLevel() const;
	UE_API void SetPreviewLevel(float NewLevel);
	UE_API FText GetRowValuePreviewLabel() const;
	UE_API FText GetRowValuePreviewText() const;

	// Row accessors and callbacks
	UE_API FName GetRowName(FPropertyAccess::Result* OutResult = nullptr) const;
	UE_API const FRealCurve* GetRealCurve(FPropertyAccess::Result* OutResult = nullptr) const;
	UE_API FDataRegistryId GetRegistryId(FPropertyAccess::Result* OutResult = nullptr) const;
	UE_API void SetRegistryId(FDataRegistryId NewId);
	UE_API void GetCustomRowNames(TArray<FName>& OutRows) const;

	TSharedPtr<IPropertyHandle> ValueProperty;
	TSharedPtr<IPropertyHandle> CurveTableHandleProperty;
	TSharedPtr<IPropertyHandle> CurveTableProperty;
	TSharedPtr<IPropertyHandle> RowNameProperty;
	TSharedPtr<IPropertyHandle> RegistryTypeProperty;

	TWeakPtr<IPropertyUtilities> PropertyUtilities;

	float PreviewLevel;
	float MinPreviewLevel;
	float MaxPreviewLevel;
	bool bSourceRefreshQueued;
};

#undef UE_API
