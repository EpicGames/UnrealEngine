// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/Blueprint.h"
#include "Factories/Factory.h"
#include "GameplayAbilitiesBlueprintFactory.generated.h"

UCLASS(HideCategories=Object, MinimalAPI)
class UGameplayAbilitiesBlueprintFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// The type of blueprint that will be created
	UPROPERTY(EditAnywhere, Category=GameplayAbilitiesBlueprintFactory)
	TEnumAsByte<enum EBlueprintType> BlueprintType;

	// The parent class of the created blueprint
	UPROPERTY(EditAnywhere, Category=GameplayAbilitiesBlueprintFactory)
	TSubclassOf<class UGameplayAbility> ParentClass;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};
