// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"
#include "CoreMinimal.h"

//////////////////////////////////////////////////////////////////////////
// FGameplayAbilitiesEditor

/**
 * Gameplay abilities asset editor (extends Blueprint editor)
 */
class FGameplayAbilitiesEditor : public FBlueprintEditor
{
public:
	/**
	 * Edits the specified gameplay ability asset(s)
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	InBlueprints			The blueprints to edit
	 * @param	bShouldOpenInDefaultsMode	If true, the editor will open in defaults editing mode
	 */ 

	void InitGameplayAbilitiesEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode);

private:
	/**
	 * Updates existing gameplay ability blueprints to make sure that they are up to date
	 * 
	 * @param	Blueprint	The blueprint to be updated
	 */
	void EnsureGameplayAbilityBlueprintIsUpToDate(UBlueprint* Blueprint);

public:
	FGameplayAbilitiesEditor();

	virtual ~FGameplayAbilitiesEditor();

public:
	// IToolkit interface
	// FRED_TODO: don't merge this back
//	virtual FName GetToolkitContextFName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	// End of IToolkit interface

	/** @return the documentation location for this editor */
	virtual FString GetDocumentationLink() const override;
	
	/** Returns a pointer to the Blueprint object we are currently editing, as long as we are editing exactly one */
	virtual UBlueprint* GetBlueprintObj() const override;
};
