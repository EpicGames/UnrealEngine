// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define UE_API GAMEPLAYABILITIESEDITOR_API

struct FGameplayCueEditorStrings
{
	FString GameplayCueNotifyDescription1;
	FString GameplayCueNotifyDescription2;

	FString GameplayCueEventDescription1;
	FString GameplayCueEventDescription2;

	FGameplayCueEditorStrings()
	{
		GameplayCueNotifyDescription1 = FString(TEXT("GameplayCue Notifies are stand alone handlers, similiar to AnimNotifies. Most GameplyCues can be implemented through these notifies. Notifies excel at handling standardized effects. The classes below provide the most common functionality needed."));
		GameplayCueNotifyDescription2 = FString(TEXT(""));

		GameplayCueEventDescription1 = FString(TEXT("GameplayCues can also be implemented via custom events on character blueprints."));
		GameplayCueEventDescription2 = FString(TEXT("To add a custom BP event, open the blueprint and look for custom events starting with GameplayCue.*"));
	}

	FGameplayCueEditorStrings(FString Notify1, FString Notify2, FString Event1, FString Event2)
	{
		GameplayCueNotifyDescription1 = Notify1;
		GameplayCueNotifyDescription2 = Notify2;
		GameplayCueEventDescription1 = Event1;
		GameplayCueEventDescription2 = Event2;
	}
};

DECLARE_DELEGATE_OneParam(FGetGameplayCueNotifyClasses, TArray<UClass*>&);
DECLARE_DELEGATE_OneParam(FGetGameplayCueInterfaceClasses, TArray<UClass*>&);
DECLARE_DELEGATE_RetVal_OneParam(FString, FGetGameplayCuePath, FString)

DECLARE_DELEGATE_RetVal(FGameplayCueEditorStrings, FGetGameplayCueEditorStrings);

/**
 * The public interface to this module
 */
class IGameplayAbilitiesEditorModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IGameplayAbilitiesEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IGameplayAbilitiesEditorModule >("GameplayAbilitiesEditor");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "GameplayAbilitiesEditor" );
	}

	/** Sets delegate that will be called to retrieve list of gameplay cue notify classes to be presented by GameplayCue Editor when creating a new notify */
	virtual FGetGameplayCueNotifyClasses& GetGameplayCueNotifyClassesDelegate() = 0;

	virtual FGetGameplayCueInterfaceClasses& GetGameplayCueInterfaceClassesDelegate() = 0;

	/** Sets delegate that will be called to get the save path for a gameplay cue notify that is created through the GameplayCue Editor */
	virtual FGetGameplayCuePath& GetGameplayCueNotifyPathDelegate() = 0;

	/** Returns strings used in the GameplayCue Editor widgets. Useful for games to override with game specific information for designers (real examples, etc). */
	virtual FGetGameplayCueEditorStrings& GetGameplayCueEditorStringsDelegate() = 0;
	
};

#undef UE_API
