// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataTable.h"
#include "Abilities/GameplayAbility.h"
#include "GameplayAbilityAudit.generated.h"

#define UE_API GAMEPLAYABILITIESEDITOR_API

class UBlueprint;
class UEdGraph;

/**
 * This file has the implementation of the Gameplay Ability Audit code.  You can right-click on assets and select "Audit" which will produce a DataTable with useful information.
 * The trick to do a full audit is to select all of the GameplayAbility Blueprints in the Content Browser (use filter 'NativeParentClass=GameplayAbility'), then right-click and select the Audit Context Menu Item
 * 
 * You can extend this functionality in your own project in two simple steps:
 *	Step 1: Derive your own audit Data Table Row from FGameplayAbilityAuditRow.
 *  Step 2: Override FillDataFromGameplayAbilityBlueprint in your derived class for any Blueprint code (e.g. node / flow inspection).
 *  Step 3: Override FillDataFromGameplayAbility in your derived class for any data/variable inspection (e.g. you've made your own UGameplayAbility-derived type).
 * 
 * Some magic in the namespace MenuExtension_GameplayAbilityBlueprintAudit should pick up your derived class and use it (assuming one derived row type per project).
 */

/** This enum indicates what flow the Blueprint takes once activated (or native if not Activate is not handled by Blueprints) */
UENUM()
enum class EGameplayAbilityActivationPath : uint8
{
	Native,		/* Ability has a Native Activate function (and therefore uninspectable by Blueprint analysis) */
	Blueprint,	/* Ability has an Activate event implemented in Blueprints and is therefore likely not triggered by a GameplayEvent */
	FromEvent	/* Ability has an ActivateByEvent event implemented in Blueprints and is therefore activated by a GameplayEvent */
};

UENUM()
enum class EGameplayAbilityEndInBlueprints : uint8
{
	Missing				= 0,		/* Ability does not have an EndAbility call in Blueprints and therefore must be ended by external Gameplay code */
	EndAbility			= 1 << 0,	/* EndAbility is called in Blueprints and is an indication the Server can control the flow of the Gameplay Ability */
	EndAbilityLocally	= 1 << 1	/* EndAbilityLocally is called in the Blueprints and is an indication of being predicted locally */
};
ENUM_CLASS_FLAGS(EGameplayAbilityEndInBlueprints)

/** This contains all of the data we gather about a Blueprint during the Audit process */
USTRUCT(BlueprintInternalUseOnlyHierarchical)
struct FGameplayAbilityAuditRow : public FTableRowBase
{
	GENERATED_BODY()
	
	/** Fill this structure with data from a UBlueprint (GameplayAbility asset).  Return true if successful. */
	UE_API virtual void FillDataFromGameplayAbilityBlueprint(const UBlueprint& GameplayAbilityBlueprint);

	/** Fill this structure with data from a UGameplayAbility (likely compiled from a UBlueprint).  Return true if successful. */
	UE_API virtual void FillDataFromGameplayAbility(const UGameplayAbility& GameplayAbility);

protected:

	/** Does the Blueprint override Activate, ActivateByEvent, or is it left up to Native code? */
	UPROPERTY()
	EGameplayAbilityActivationPath ActivationPath = EGameplayAbilityActivationPath::Native;

	/** The InstancingPolicy of the GameplayAbility (Instancing Behavior can affect memory, which functions can be called, what data can be used, and replication) */
	UPROPERTY()
	TEnumAsByte<EGameplayAbilityInstancingPolicy::Type> InstancingPolicy = EGameplayAbilityInstancingPolicy::Type::NonInstanced;

	/** The NetExecutionPolicy of the GameplayAbility */
	UPROPERTY()
	TEnumAsByte<EGameplayAbilityNetExecutionPolicy::Type> NetExecutionPolicy = EGameplayAbilityNetExecutionPolicy::Type::LocalPredicted;

	/** The NetSecurityPolicy of the GameplayAbility */
	UPROPERTY()
	TEnumAsByte<EGameplayAbilityNetSecurityPolicy::Type> NetSecurityPolicy = EGameplayAbilityNetSecurityPolicy::Type::ClientOrServer;

	/** The Replication Policy which controls if the instance itself is replicated (and therefore controls variable replication) */
	UPROPERTY()
	TEnumAsByte<EGameplayAbilityReplicationPolicy::Type> ReplicationPolicy = EGameplayAbilityReplicationPolicy::Type::ReplicateNo;

	/** Gameplay Tags that the Asset itself has (AbilityTags) */
	UPROPERTY()
	TArray<FName>	AssetTags;

	/** If the GameplayAbility has a Cost GameplayEffect, this field will tell us which one it's using. */
	UPROPERTY()
	FName CostGE = NAME_None;

	/** If the GameplayAbility has a Cooldown GameplayEffect, this field will tell us which one it's using. */
	UPROPERTY()
	FName CooldownGE = NAME_None;

	/** Does the Blueprint implement CanActivate? It can be an indication that there are complex rules for triggering. */
	UPROPERTY()
	bool bOverridesCanActivate = false;

	/** Does the Blueprint implement ShouldAbilityRespondToEvent? It can be an indication that there are complex rules for triggering. */
	UPROPERTY()
	bool bOverridesShouldAbilityRespondToEvent = false;

	/** Does the Blueprint CheckCosts? If so it's an indication that this ability is a multi-step ability. */
	UPROPERTY()
	bool bChecksCostManually = false;

	/** Does the Blueprint Commit?  If not (and we are a non-native BP), the costs will never apply and we are breaking our expected flow. */
	UPROPERTY()
	bool bCommitAbility = false;

	/** How does the Blueprint call EndAbility?  If missing, this Ability can be assumed to be a persistent one and removed from Gameplay code (e.g. while tags are active) */
	UPROPERTY()
	EGameplayAbilityEndInBlueprints EndAbility = EGameplayAbilityEndInBlueprints::Missing;

	/** All of the known referenced tags (list may be incomplete -- does not take into account Blueprint Tags) */
	UPROPERTY()
	TArray<FName>	ReferencedTags;

	/** List of functions the Blueprint calls */
	UPROPERTY()
	TArray<FName>	Functions;

	/** List of AsyncTasks the Blueprint uses */
	UPROPERTY()
	TArray<FString>	AsyncTasks;

	/** List of Variables the Blueprint mutates */
	UPROPERTY()
	TArray<FName>	MutatedVariables;
};

/** Helper functions that may be useful in your implementation */
namespace GameplayAbilityAudit
{
	GAMEPLAYABILITIESEDITOR_API TArray<UEdGraph*> GatherAllGraphsIncludingMacros(const UBlueprint& LoadedInstance);
}

#undef UE_API
