// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraph.h"
#include "GameplayAbilityGraph.generated.h"

UCLASS(MinimalAPI)
class UGameplayAbilityGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()
};

