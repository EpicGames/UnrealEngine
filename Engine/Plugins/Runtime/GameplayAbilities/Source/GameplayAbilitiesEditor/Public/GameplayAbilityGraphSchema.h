// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraphSchema_K2.h"
#include "GameplayAbilityGraphSchema.generated.h"

UCLASS(MinimalAPI)
class UGameplayAbilityGraphSchema : public UEdGraphSchema_K2
{
	GENERATED_UCLASS_BODY()

	/**
	* Creates a new variable getter node and adds it to ParentGraph
	*
	* @param	GraphPosition		The location of the new node inside the graph
	* @param	ParentGraph			The graph to spawn the new node in
	* @param	VariableName		The name of the variable
	* @param	Source				The source of the variable
	* @return	A pointer to the newly spawned node
	*/
	virtual class UK2Node_VariableGet* SpawnVariableGetNode(const FVector2D GraphPosition, class UEdGraph* ParentGraph, FName VariableName, UStruct* Source) const override;

	/**
	* Creates a new variable setter node and adds it to ParentGraph
	*
	* @param	GraphPosition		The location of the new node inside the graph
	* @param	ParentGraph			The graph to spawn the new node in
	* @param	VariableName		The name of the variable
	* @param	Source				The source of the variable
	* @return	A pointer to the newly spawned node
	*/
	virtual class UK2Node_VariableSet* SpawnVariableSetNode(const FVector2D GraphPosition, class UEdGraph* ParentGraph, FName VariableName, UStruct* Source) const override;

	virtual bool ShouldAlwaysPurgeOnModification() const override { return true; }
};
