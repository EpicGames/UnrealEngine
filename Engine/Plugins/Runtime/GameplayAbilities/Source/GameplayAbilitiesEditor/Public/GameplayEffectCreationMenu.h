// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SubclassOf.h"
#include "GameplayEffect.h"
#include "GameplayEffectCreationMenu.generated.h"

#define UE_API GAMEPLAYABILITIESEDITOR_API

class UGameplayEffect;

USTRUCT()
struct FGameplayEffectCreationData
{
	GENERATED_BODY()
	
	/** Where to show this in the menu. Use "|" for sub categories. E.g, "Status|Hard|Stun|Root". */
	UPROPERTY(EditAnywhere, Category="Gameplay Effect")
	FString MenuPath;

	/** The default BaseName of the new asset. E.g "Damage" -> GE_Damage or GE_HeroName_AbilityName_Damage */
	UPROPERTY(EditAnywhere, Category="Gameplay Effect")
	FString BaseName;

	UPROPERTY(EditAnywhere, Category="Gameplay Effect")
	TSubclassOf<UGameplayEffect> ParentGameplayEffect;
};

/** Container to hold EventKeywords for PIE testing */
UCLASS(config=Game,defaultconfig, MinimalAPI)
class UGameplayEffectCreationMenu : public UObject
{
	GENERATED_BODY()

public:

	UE_API UGameplayEffectCreationMenu();
	
	// Set this in your project to programatically define default GE names
	static UE_API TFunction< FString(FString BaseName, FString Path) > GetDefaultAssetNameFunc;



	UE_API void AddMenuExtensions() const;

	UPROPERTY(config, EditAnywhere, Category="Gameplay Effect")
	TArray<FGameplayEffectCreationData> Definitions;
};

#undef UE_API
