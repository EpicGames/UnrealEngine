// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "K2Node_Event.h"
#include "K2Node_GameplayCueEvent.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UEdGraph;

UCLASS()
class UK2Node_GameplayCueEvent : public UK2Node_Event
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	//~ End UEdGraphNode Interface
	
	//~ Begin UK2Node Interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	//~ End UK2Node Interface
};
