// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "EdGraph/EdGraph.h"
#include "GameplayTask.h"
#include "K2Node_LatentGameplayTaskCall.h"
#include "K2Node_LatentAbilityCall.generated.h"

class FBlueprintActionDatabaseRegistrar;

UCLASS()
class UK2Node_LatentAbilityCall : public UK2Node_LatentGameplayTaskCall
{
	GENERATED_BODY()

public:
	UK2Node_LatentAbilityCall(const FObjectInitializer& ObjectInitializer);

	// UEdGraphNode interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsCompatibleWithGraph(UEdGraph const* TargetGraph) const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	// End of UEdGraphNode interface
	
protected:
	virtual bool IsHandling(TSubclassOf<UGameplayTask> TaskClass) const override;
};
